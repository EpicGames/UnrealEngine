// Copyright Epic Games, Inc. All Rights Reserved.
#include "AsyncDetailViewDiff.h"

#include "DetailTreeNode.h"
#include "IDetailsViewPrivate.h"
#include "DiffUtils.h"
#include "PropertyHandleImpl.h"

namespace AsyncDetailViewDiffHelpers
{
	TArray<TWeakObjectPtr<UObject>> GetObjects(const TSharedPtr<FDetailTreeNode>& TreeNode)
	{
		TArray<UObject*> Result;
		if (TSharedPtr<const IDetailsViewPrivate> DetailsView = TreeNode->GetDetailsViewSharedPtr())
		{
			return DetailsView->GetSelectedObjects();
		}
		return {};
	}
}


bool TTreeDiffSpecification<TWeakPtr<FDetailTreeNode>>::AreValuesEqual(const TWeakPtr<FDetailTreeNode>& TreeNodeA, const TWeakPtr<FDetailTreeNode>& TreeNodeB, TArray<FPropertySoftPath>*) const
{
	return AreValuesEqual(TreeNodeA, TreeNodeB);
}

bool TTreeDiffSpecification<TWeakPtr<FDetailTreeNode>>::AreValuesEqual(const TWeakPtr<FDetailTreeNode>& TreeNodeA, const TWeakPtr<FDetailTreeNode>& TreeNodeB) const
{
	const TSharedPtr<FDetailTreeNode> PinnedTreeNodeA = TreeNodeA.Pin();
	const TSharedPtr<FDetailTreeNode> PinnedTreeNodeB = TreeNodeB.Pin();
	if (!PinnedTreeNodeA || !PinnedTreeNodeB)
	{
		return PinnedTreeNodeA == PinnedTreeNodeB;
	}

	const TSharedPtr<IPropertyHandle> PropertyHandleA = PinnedTreeNodeA->CreatePropertyHandle();
	const TSharedPtr<IPropertyHandle> PropertyHandleB = PinnedTreeNodeB->CreatePropertyHandle();
	if (!PropertyHandleA || !PropertyHandleB)
	{
		// category nodes
		return PinnedTreeNodeA->GetNodeName() == PinnedTreeNodeB->GetNodeName();
	}
	const TSharedPtr<FPropertyNode>& PropertyNodeA = StaticCastSharedPtr<FPropertyHandleBase>(PropertyHandleA)->GetPropertyNode();
	const TSharedPtr<FPropertyNode>& PropertyNodeB = StaticCastSharedPtr<FPropertyHandleBase>(PropertyHandleB)->GetPropertyNode();
	if (!PropertyNodeA || !PropertyNodeB)
	{
		return PinnedTreeNodeA->GetNodeName() == PinnedTreeNodeB->GetNodeName();
	}

	TArray<void*> DataValuesA;
	TArray<void*> DataValuesB;
	PropertyHandleA->AccessRawData(DataValuesA);
	PropertyHandleB->AccessRawData(DataValuesB);
	
	if(DataValuesA.IsEmpty() || DataValuesB.IsEmpty())
	{
		return true;
	}
	
	const TArray<TWeakObjectPtr<UObject>> OwningObjectsA = AsyncDetailViewDiffHelpers::GetObjects(PinnedTreeNodeA);
	const TArray<TWeakObjectPtr<UObject>> OwningObjectsB = AsyncDetailViewDiffHelpers::GetObjects(PinnedTreeNodeB);

	if (!ensure(OwningObjectsA.Num() == DataValuesA.Num() && OwningObjectsB.Num() == DataValuesB.Num()))
	{
		return true;
	}

	return DiffUtils::Identical(PropertyHandleA, PropertyHandleB, OwningObjectsA, OwningObjectsB);
}

bool TTreeDiffSpecification<TWeakPtr<FDetailTreeNode>>::AreMatching(const TWeakPtr<FDetailTreeNode>& TreeNodeA, const TWeakPtr<FDetailTreeNode>& TreeNodeB, TArray<FPropertySoftPath>*) const
{
	return AreMatching(TreeNodeA, TreeNodeB);
}

bool TTreeDiffSpecification<TWeakPtr<FDetailTreeNode>>::AreMatching(const TWeakPtr<FDetailTreeNode>& TreeNodeA, const TWeakPtr<FDetailTreeNode>& TreeNodeB) const
{
	const TSharedPtr<FDetailTreeNode> PinnedTreeNodeA = TreeNodeA.Pin();
	const TSharedPtr<FDetailTreeNode> PinnedTreeNodeB = TreeNodeB.Pin();
	if (!PinnedTreeNodeA || !PinnedTreeNodeB)
	{
		return PinnedTreeNodeA == PinnedTreeNodeB;
	}

	const TSharedPtr<IPropertyHandle> PropertyHandleA = PinnedTreeNodeA->CreatePropertyHandle();
	const TSharedPtr<IPropertyHandle> PropertyHandleB = PinnedTreeNodeB->CreatePropertyHandle();
	if (!PropertyHandleA || !PropertyHandleB)
	{
		// category nodes
		return PinnedTreeNodeA->GetNodeName() == PinnedTreeNodeB->GetNodeName();
	}
	
	const int32 ArrayIndexA = PropertyHandleA->GetArrayIndex();
	const int32 ArrayIndexB = PropertyHandleB->GetArrayIndex();
	if (ArrayIndexA != INDEX_NONE && ArrayIndexB != INDEX_NONE)
	{
		const TSharedPtr<IPropertyHandle> KeyHandleA = PropertyHandleA->GetKeyHandle();
		const TSharedPtr<IPropertyHandle> KeyHandleB = PropertyHandleB->GetKeyHandle();
		if (PropertyHandleA->GetKeyHandle() && PropertyHandleB->GetKeyHandle())
		{
			const TArray<TWeakObjectPtr<UObject>> OwningObjectsA = AsyncDetailViewDiffHelpers::GetObjects(PinnedTreeNodeA);
			const TArray<TWeakObjectPtr<UObject>> OwningObjectsB = AsyncDetailViewDiffHelpers::GetObjects(PinnedTreeNodeB);

			return DiffUtils::Identical(KeyHandleA, KeyHandleB, OwningObjectsA, OwningObjectsB);
		}

		const TSharedPtr<IPropertyHandleSet> SetHandleA = PropertyHandleA->GetParentHandle()->AsSet();
		const TSharedPtr<IPropertyHandleSet> SetHandleB = PropertyHandleB->GetParentHandle()->AsSet();
		if (SetHandleA && SetHandleB)
		{
			// match set elements by value
			const TArray<TWeakObjectPtr<UObject>> OwningObjectsA = AsyncDetailViewDiffHelpers::GetObjects(PinnedTreeNodeA);
			const TArray<TWeakObjectPtr<UObject>> OwningObjectsB = AsyncDetailViewDiffHelpers::GetObjects(PinnedTreeNodeB);

			return DiffUtils::Identical(PropertyHandleA, PropertyHandleB, OwningObjectsA, OwningObjectsB);
		}
		
		return ArrayIndexA == ArrayIndexB;
	}
	return true;
}

void TTreeDiffSpecification<TWeakPtr<FDetailTreeNode>>::GetChildren(const TWeakPtr<FDetailTreeNode>& InParent, TArray<TWeakPtr<FDetailTreeNode>>& OutChildren) const
{
	const TSharedPtr<FDetailTreeNode> PinnedParent = InParent.Pin();
	if (PinnedParent)
	{
		TArray<TSharedRef<FDetailTreeNode>> Children;
		PinnedParent->GetChildren(Children);
		for (TSharedRef<FDetailTreeNode> Child : Children)
		{
			OutChildren.Add(Child);
		}
	}
}

bool TTreeDiffSpecification<TWeakPtr<FDetailTreeNode>>::ShouldMatchByValue(const TWeakPtr<FDetailTreeNode>& TreeNode) const
{
	const TSharedPtr<FDetailTreeNode> PinnedTreeNode = TreeNode.Pin();
	if (!PinnedTreeNode)
	{
		return false;
	}
	const TSharedPtr<FPropertyNode> PropertyNodeA = PinnedTreeNode->GetPropertyNode();
	if (!PropertyNodeA || !PropertyNodeA->GetParentNode())
	{
		return false;
	}
	
	const int32 ArrayIndex = PropertyNodeA->GetArrayIndex();
	const FArrayProperty* ParentArrayProperty = CastField<FArrayProperty>(PropertyNodeA->GetParentNode()->GetProperty());
	
	// match array elements by value rather than by index
	return ParentArrayProperty && ArrayIndex != INDEX_NONE;
}

bool TTreeDiffSpecification<TWeakPtr<FDetailTreeNode, ESPMode::ThreadSafe>>::ShouldInheritEqualFromChildren(
	const TWeakPtr<FDetailTreeNode>& TreeNodeA, const TWeakPtr<FDetailTreeNode>& TreeNodeB) const
{
	return false; // this theoretically could return true, but leaving it off has helped find false positives and false negatives
}

FAsyncDetailViewDiff::FAsyncDetailViewDiff(TSharedRef<IDetailsView> InLeftView, TSharedRef<IDetailsView> InRightView)
	: TAsyncTreeDifferences(RootNodesAttribute(InLeftView), RootNodesAttribute(InRightView))
	, LeftView(InLeftView)
	, RightView(InRightView)
{}

void FAsyncDetailViewDiff::GetPropertyDifferences(TArray<FSingleObjectDiffEntry>& OutDiffEntries) const
{
	ForEach(ETreeTraverseOrder::PreOrder, [&](const TUniquePtr<DiffNodeType>& Node)->ETreeTraverseControl
	{
		FPropertyPath PropertyPath;
		FPropertyPath RightPropertyPath;
		if (const TSharedPtr<FDetailTreeNode> LeftTreeNode = Node->ValueA.Pin())
		{
			PropertyPath = LeftTreeNode->GetPropertyPath();
		}
		else if (const TSharedPtr<FDetailTreeNode> RightTreeNode = Node->ValueB.Pin())
		{
			PropertyPath = RightTreeNode->GetPropertyPath();
		}

		// only include tree nodes with properties
		if (!PropertyPath.IsValid())
		{
			return ETreeTraverseControl::Continue;
		}
		
		EPropertyDiffType::Type PropertyDiffType;
		switch (Node->DiffResult)
		{
		case ETreeDiffResult::MissingFromTree1:
			PropertyDiffType = EPropertyDiffType::PropertyAddedToB;
			break;
		case ETreeDiffResult::MissingFromTree2:
			PropertyDiffType = EPropertyDiffType::PropertyAddedToA;
			break;
		case ETreeDiffResult::DifferentValues:
			PropertyDiffType = EPropertyDiffType::PropertyValueChanged;
			break;
		default:
			// only include changes
			return ETreeTraverseControl::Continue;
		}

		OutDiffEntries.Add(FSingleObjectDiffEntry(FPropertySoftPath(PropertyPath), PropertyDiffType));

		// only include top-most properties for adds and removes
		return (Node->DiffResult == ETreeDiffResult::DifferentValues) ? ETreeTraverseControl::Continue : ETreeTraverseControl::SkipChildren;
	});
}

TPair<int32, int32> FAsyncDetailViewDiff::ForEachRow(const TFunction<ETreeTraverseControl(const TUniquePtr<DiffNodeType>&, int32, int32)>& Method) const
{
	const TSharedPtr<IDetailsView> LeftDetailsView = LeftView.Pin();
	const TSharedPtr<IDetailsView> RightDetailsView = RightView.Pin();
	if (!LeftDetailsView || !RightDetailsView)
	{
		return {0,0};
	}
	
	int32 LeftRowNum = 0;
	int32 RightRowNum = 0;
	ForEach(
		ETreeTraverseOrder::PreOrder,
		[&LeftDetailsView,&RightDetailsView,&Method,&LeftRowNum,&RightRowNum](const TUniquePtr<DiffNodeType>& DiffNode)->ETreeTraverseControl
		{
			bool bFoundLeftRow = false;
			if (const TSharedPtr<FDetailTreeNode> LeftTreeNode = DiffNode->ValueA.Pin())
			{
				if (!LeftDetailsView->IsAncestorCollapsed(LeftTreeNode.ToSharedRef()))
				{
					bFoundLeftRow = true;
				}
			}
			
			bool bFoundRightRow = false;
			if (const TSharedPtr<FDetailTreeNode> RightTreeNode = DiffNode->ValueB.Pin())
			{
				if (!RightDetailsView->IsAncestorCollapsed(RightTreeNode.ToSharedRef()))
				{
					bFoundRightRow = true;
				}
			}

			ETreeTraverseControl Control = ETreeTraverseControl::SkipChildren;
			if (bFoundRightRow || bFoundLeftRow)
			{
				Control = Method(DiffNode, LeftRowNum, RightRowNum);
			}
			
			if (bFoundLeftRow)
			{
				++LeftRowNum;
			}
			if (bFoundRightRow)
			{
				++RightRowNum;
			}
			
			return Control;
		}
	);
	return {LeftRowNum, RightRowNum};
}

TArray<FVector2f> FAsyncDetailViewDiff::GenerateScrollSyncRate() const
{
	TArray<FIntVector2> MatchingRows;

	// iterate matching rows of both details panels simultaneously
	auto [LeftRowCount, RightRowCount] = ForEachRow(
		[&MatchingRows](const TUniquePtr<DiffNodeType>& DiffNode, int32 LeftRow, int32 RightRow)->ETreeTraverseControl
		{
			// if both trees share this row, sync scrolling here
			if (DiffNode->ValueA.IsValid() && DiffNode->ValueB.IsValid())
			{
				if (MatchingRows.IsEmpty() || (MatchingRows.Last().X != LeftRow && MatchingRows.Last().Y != RightRow))
				{
					MatchingRows.Emplace(LeftRow, RightRow);
				}
			}
			return ETreeTraverseControl::Continue;
		}
	);

	TArray<FVector2f> FixedPoints;
	FixedPoints.Emplace(0.f, 0.f);
	
	// normalize fixed points
	for (FIntVector2& MatchingRow : MatchingRows)
	{
		FixedPoints.Emplace(
			StaticCast<float>(MatchingRow.X) / LeftRowCount,
			StaticCast<float>(MatchingRow.Y) / RightRowCount
		);
	}
	FixedPoints.Emplace(1.f, 1.f);
	
	return FixedPoints;
}

TAttribute<TArray<TWeakPtr<FDetailTreeNode>>> FAsyncDetailViewDiff::RootNodesAttribute(TWeakPtr<IDetailsView> DetailsView)
{
	return TAttribute<TArray<TWeakPtr<FDetailTreeNode>>>::CreateLambda([DetailsView]()
	{
		TArray<TWeakPtr<FDetailTreeNode>> Result;
		if (const TSharedPtr<IDetailsViewPrivate> Details = StaticCastSharedPtr<IDetailsViewPrivate>(DetailsView.Pin()))
		{
			Details->GetHeadNodes(Result);
		}
		return Result;
	});
}

