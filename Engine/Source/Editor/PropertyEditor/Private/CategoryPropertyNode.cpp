// Copyright Epic Games, Inc. All Rights Reserved.


#include "CategoryPropertyNode.h"
#include "Misc/ConfigCacheIni.h"
#include "ItemPropertyNode.h"
#include "ObjectEditorUtils.h"
#include "ObjectPropertyNode.h"
#include "PropertyEditorHelpers.h"

FCategoryPropertyNode::FCategoryPropertyNode(void)
	: FPropertyNode()
{
}

FCategoryPropertyNode::~FCategoryPropertyNode(void)
{
}

bool FCategoryPropertyNode::IsSubcategory() const
{
	return GetParentNode() != nullptr && GetParentNode()->AsCategoryNode() != nullptr;
}

FText FCategoryPropertyNode::GetDisplayName() const 
{
	FText DisplayName = FObjectEditorUtils::GetCategoryText(CategoryName);
	if (IsSubcategory())
	{
		// The category name may actually contain a path of categories.  When displaying this category
		// in the property window, we only want the leaf part of the path
		const FString& CategoryPath = DisplayName.ToString();
		if (int32 LastDelimiterCharIndex = INDEX_NONE;
			CategoryPath.FindLastChar(FPropertyNodeConstants::CategoryDelimiterChar, LastDelimiterCharIndex))
		{
			// Grab the last sub-category from the path
			FStringView SubCategoryName = FStringView(CategoryPath).Mid(LastDelimiterCharIndex + 1);
			SubCategoryName.TrimStartAndEndInline();
			DisplayName = FText::AsCultureInvariant(SubCategoryName);
		}
	}
	return DisplayName;
}

/**
 * Overridden function for special setup
 */
void FCategoryPropertyNode::InitBeforeNodeFlags()
{

}
/**
 * Overridden function for Creating Child Nodes
 */
void FCategoryPropertyNode::InitChildNodes()
{
	const bool bShowHiddenProperties = !!HasNodeFlags( EPropertyNodeFlags::ShouldShowHiddenProperties );
	const bool bShouldShowDisableEditOnInstance = !!HasNodeFlags(EPropertyNodeFlags::ShouldShowDisableEditOnInstance);

	TArray<FProperty*> Properties;
	TSet<FProperty*> SparseProperties;
	// The parent of a category window has to be an object window.
	FComplexPropertyNode* ComplexNode = FindComplexParent();
	if (ComplexNode)
	{
		FObjectPropertyNode* ObjectNode = FindObjectItemParent();
		
		// Get a list of properties that are in the same category
		for (const UStruct* Structure : ComplexNode->GetAllStructures())
		{
			const bool bIsSparseStruct = (ObjectNode && ObjectNode->IsSparseDataStruct(Cast<const UScriptStruct>(Structure)));
			for (TFieldIterator<FProperty> It(Structure); It; ++It)
			{
				bool bMetaDataAllowVisible = true;
				if (!bShowHiddenProperties)
				{
					static const FName Name_bShowOnlyWhenTrue("bShowOnlyWhenTrue");
					const FString& MetaDataVisibilityCheckString = It->GetMetaData(Name_bShowOnlyWhenTrue);
					if (MetaDataVisibilityCheckString.Len())
					{
						//ensure that the metadata visibility string is actually set to true in order to show this property
						// @todo Remove this
						GConfig->GetBool(TEXT("UnrealEd.PropertyFilters"), *MetaDataVisibilityCheckString, bMetaDataAllowVisible, GEditorPerProjectIni);
					}
				}

				if (bMetaDataAllowVisible)
				{
					// Add if we are showing non-editable props and this is the 'None' category, 
					// or if this is the right category, and we are showing non-editable
					if (FObjectEditorUtils::GetCategoryFName(*It) == CategoryName && PropertyEditorHelpers::ShouldBeVisible(*this, *It))
					{
						if (bIsSparseStruct)
						{
							SparseProperties.Add(*It);
						}

						Properties.Add(*It);
					}
				}
			}
		}
	}

	PropertyEditorHelpers::OrderPropertiesFromMetadata(Properties);

	for( int32 PropertyIndex = 0 ; PropertyIndex < Properties.Num() ; ++PropertyIndex )
	{
		TSharedPtr<FItemPropertyNode> NewItemNode( new FItemPropertyNode );

		FPropertyNodeInitParams InitParams;
		InitParams.ParentNode = SharedThis(this);
		InitParams.Property = Properties[PropertyIndex];
		InitParams.ArrayOffset = 0;
		InitParams.ArrayIndex = INDEX_NONE;
		InitParams.bAllowChildren = true;
		InitParams.bForceHiddenPropertyVisibility = bShowHiddenProperties;
		InitParams.bCreateDisableEditOnInstanceNodes = bShouldShowDisableEditOnInstance;
		InitParams.IsSparseProperty = SparseProperties.Contains(Properties[PropertyIndex]) ? FPropertyNodeInitParams::EIsSparseDataProperty::True : FPropertyNodeInitParams::EIsSparseDataProperty::Inherit;

		NewItemNode->InitNode( InitParams );

		AddChildNode(NewItemNode);
	}
}


/**
 * Appends my path, including an array index (where appropriate)
 */
bool FCategoryPropertyNode::GetQualifiedName( FString& PathPlusIndex, const bool bWithArrayIndex, const FPropertyNode* StopParent, bool bIgnoreCategories ) const
{
	bool bAddedAnything = false;

	const TSharedPtr<FPropertyNode> ParentNode = ParentNodeWeakPtr.Pin();
	if (ParentNode && StopParent != ParentNode.Get())
	{
		bAddedAnything = ParentNode->GetQualifiedName(PathPlusIndex, bWithArrayIndex, StopParent, bIgnoreCategories );
	}
	
	if (!bIgnoreCategories)
	{
		if (bAddedAnything)
		{
			PathPlusIndex += TEXT(".");
		}

		GetCategoryName().AppendString(PathPlusIndex);
		bAddedAnything = true;
	}

	return bAddedAnything;
}
