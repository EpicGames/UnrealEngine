// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PropertyNode.h"

class FCategoryPropertyNode : public FPropertyNode
{
public:
	FCategoryPropertyNode();
	virtual ~FCategoryPropertyNode();

	/**
	 * Overridden function to get the derived object node
	 */
	virtual FCategoryPropertyNode* AsCategoryNode() override { return this; }
	virtual const FCategoryPropertyNode* AsCategoryNode() const override { return this; }

	/**
	 * Accessor functions for getting a category name
	 */
	void SetCategoryName(const FName& InCategoryName) { CategoryName = InCategoryName; }
	const FName& GetCategoryName(void) const { return CategoryName; }

	/**
	 * Checks to see if this category is a 'sub-category'
	 *
	 * @return	True if this category node is a sub-category, otherwise false
	 */
	bool IsSubcategory() const;

	virtual FText GetDisplayName() const override;

protected:
	/**
	 * Overridden function for special setup
	 */
	virtual void InitBeforeNodeFlags() override;
	/**
	 * Overridden function for Creating Child Nodes
	 */
	virtual void InitChildNodes() override;

	/**
	 * Appends my path, including an array index (where appropriate)
	 */
	virtual bool GetQualifiedName( FString& PathPlusIndex, const bool bWithArrayIndex, const FPropertyNode* StopParent = nullptr, bool bIgnoreCategories = false ) const override;

	/**
	 * Stored category name for the window
	 */
	FName CategoryName;
};
