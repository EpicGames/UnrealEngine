// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomChildBuilder.h"
#include "Modules/ModuleManager.h"
#include "DetailGroup.h"
#include "PropertyHandleImpl.h"
#include "DetailPropertyRow.h"
#include "ObjectPropertyNode.h"
#include "SStandaloneCustomizedValueWidget.h"

IDetailChildrenBuilder& FCustomChildrenBuilder::AddCustomBuilder( TSharedRef<class IDetailCustomNodeBuilder> InCustomBuilder )
{
	FDetailLayoutCustomization NewCustomization;
	NewCustomization.CustomBuilderRow = MakeShareable( new FDetailCustomBuilderRow( InCustomBuilder ) );

	ChildCustomizations.Add( NewCustomization );
	return *this;
}

IDetailGroup& FCustomChildrenBuilder::AddGroup( FName GroupName, const FText& LocalizedDisplayName, const bool bStartExpanded)
{
	FDetailLayoutCustomization NewCustomization;
	NewCustomization.DetailGroup = MakeShareable( new FDetailGroup( GroupName, ParentCategory.Pin().ToSharedRef(), LocalizedDisplayName, bStartExpanded ) );

	ChildCustomizations.Add( NewCustomization );

	return *NewCustomization.DetailGroup;
}

IDetailGroup* FCustomChildrenBuilder::GetGroup(FName GroupName)
{
	for (const FDetailLayoutCustomization& ChildCustomization : ChildCustomizations)
	{
		if (FDetailGroup* DetailGroup = ChildCustomization.DetailGroup.Get())
		{
			if (DetailGroup->GetGroupName() == GroupName)
			{
				return ChildCustomization.DetailGroup.Get();	
			}
		}
	}

	return nullptr;
}

FDetailWidgetRow& FCustomChildrenBuilder::AddCustomRow( const FText& SearchString )
{
	const TSharedRef<FDetailWidgetRow> NewRow = MakeShared<FDetailWidgetRow>();
	FDetailLayoutCustomization NewCustomization;

	NewRow->FilterString( SearchString );

	// Bind to PasteFromText if specified
	if (const TSharedPtr<FOnPasteFromText> PasteFromTextDelegate = GetParentCategory().OnPasteFromText())
	{
		NewRow->OnPasteFromTextDelegate = PasteFromTextDelegate;
	}

	if (const TSharedPtr<FOnCopyToText> CopyToTextDelegate = GetParentCategory().OnCopyToText())
	{
		NewRow->OnCopyToTextDelegate = CopyToTextDelegate;
	}
	
	NewCustomization.WidgetDecl = NewRow;

	ChildCustomizations.Add( NewCustomization );
	return *NewRow;
}

IDetailPropertyRow& FCustomChildrenBuilder::AddProperty( TSharedRef<IPropertyHandle> PropertyHandle )
{
	check( PropertyHandle->IsValidHandle() )

	FDetailLayoutCustomization NewCustomization;
	NewCustomization.PropertyRow = MakeShareable( new FDetailPropertyRow( StaticCastSharedRef<FPropertyHandleBase>( PropertyHandle )->GetPropertyNode(), ParentCategory.Pin().ToSharedRef() ) );

	if (CustomResetChildToDefault.IsSet())
	{
		NewCustomization.PropertyRow->OverrideResetToDefault(CustomResetChildToDefault.GetValue());
	}

	ChildCustomizations.Add( NewCustomization );

	return *NewCustomization.PropertyRow;
}

IDetailPropertyRow* FCustomChildrenBuilder::AddExternalStructure(TSharedRef<FStructOnScope> ChildStructure, FName UniqueIdName)
{
	return AddExternalStructureProperty(ChildStructure, NAME_None, FAddPropertyParams().UniqueId(UniqueIdName));
}

IDetailPropertyRow* FCustomChildrenBuilder::AddExternalStructureProperty(TSharedRef<FStructOnScope> ChildStructure, FName PropertyName, const FAddPropertyParams& Params)
{
	return AddExternalStructureProperty<>(ChildStructure, PropertyName, Params);
}

IDetailPropertyRow* FCustomChildrenBuilder::AddExternalStructure(TSharedPtr<IStructureDataProvider> ChildStructure, FName UniqueIdName)
{
	return AddExternalStructureProperty(ChildStructure, NAME_None, FAddPropertyParams().UniqueId(UniqueIdName));
}

IDetailPropertyRow* FCustomChildrenBuilder::AddChildStructure(TSharedRef<IPropertyHandle> PropertyHandle, TSharedPtr<IStructureDataProvider> ChildStructure, FName UniqueIdName, const FText& DisplayNameOverride)
{
	return AddChildStructureProperty(PropertyHandle, ChildStructure, NAME_None, FAddPropertyParams().UniqueId(UniqueIdName), DisplayNameOverride);
}

IDetailPropertyRow* FCustomChildrenBuilder::AddExternalStructureProperty(TSharedPtr<IStructureDataProvider> ChildStructure, FName PropertyName, const FAddPropertyParams& Params)
{
	return AddExternalStructureProperty<>(ChildStructure, PropertyName, Params);
}

IDetailPropertyRow* FCustomChildrenBuilder::AddStructureProperty(const FAddPropertyParams& Params, TFunctionRef<void(FDetailLayoutCustomization&)> MakePropertyRowCustomization)
{
	FDetailLayoutCustomization NewCustomization;

	MakePropertyRowCustomization(NewCustomization);

	if (Params.ShouldHideRootObjectNode() && NewCustomization.HasPropertyNode() && NewCustomization.GetPropertyNode()->AsComplexNode())
	{
		NewCustomization.PropertyRow->SetForceShowOnlyChildren(true);
	}

	TSharedPtr<FDetailPropertyRow> NewRow = NewCustomization.PropertyRow;

	if (NewRow.IsValid())
	{
		NewRow->SetCustomExpansionId(Params.GetUniqueId());

		TSharedPtr<FPropertyNode> PropertyNode = NewRow->GetPropertyNode();
		TSharedPtr<FComplexPropertyNode> RootNode = StaticCastSharedRef<FComplexPropertyNode>(PropertyNode->FindComplexParent()->AsShared());

		ChildCustomizations.Add(NewCustomization);
	}

	return NewRow.Get();
}

template<class T>
IDetailPropertyRow* FCustomChildrenBuilder::AddExternalStructureProperty(const T& ChildStructure, FName PropertyName, const FAddPropertyParams& Params)
{
	return AddStructureProperty(Params, [&](FDetailLayoutCustomization& NewCustomization)
	{
		FDetailPropertyRow::MakeExternalPropertyRowCustomization(
			ChildStructure, PropertyName, ParentCategory.Pin().ToSharedRef(), NewCustomization, Params
		);
	});
}

IDetailPropertyRow* FCustomChildrenBuilder::AddChildStructureProperty(TSharedRef<IPropertyHandle> PropertyHandle,
	TSharedPtr<IStructureDataProvider> ChildStructure, FName PropertyName, const FAddPropertyParams& Params, const FText& DisplayNameOverride)
{
	return AddStructureProperty(Params, [&](FDetailLayoutCustomization& NewCustomization)
	{
		FDetailPropertyRow::MakeChildPropertyRowCustomization(
			PropertyHandle, ChildStructure, PropertyName, ParentCategory.Pin().ToSharedRef(), NewCustomization, Params, DisplayNameOverride
		);
	});
}

TArray<TSharedPtr<IPropertyHandle>> FCustomChildrenBuilder::AddAllExternalStructureProperties(TSharedRef<FStructOnScope> ChildStructure)
{
	const TSharedPtr<FDetailCategoryImpl> ParentCategoryPinned = ParentCategory.Pin();
	return ParentCategoryPinned ? ParentCategoryPinned->AddAllExternalStructureProperties(ChildStructure) : TArray<TSharedPtr<IPropertyHandle>>();
}

TArray<TSharedPtr<IPropertyHandle>> FCustomChildrenBuilder::AddAllExternalStructureProperties(TSharedPtr<IStructureDataProvider> ChildStructure)
{
	const TSharedPtr<FDetailCategoryImpl> ParentCategoryPinned = ParentCategory.Pin();
	return ParentCategoryPinned ? ParentCategoryPinned->AddAllExternalStructureProperties(ChildStructure) : TArray<TSharedPtr<IPropertyHandle>>();
}

IDetailPropertyRow* FCustomChildrenBuilder::AddExternalObjects(const TArray<UObject*>& Objects, FName UniqueIdName)
{
	FAddPropertyParams Params = FAddPropertyParams()
		.UniqueId(UniqueIdName)
		.AllowChildren(true);

	return AddExternalObjects(Objects, Params);
}

IDetailPropertyRow* FCustomChildrenBuilder::AddExternalObjects(const TArray<UObject*>& Objects, const FAddPropertyParams& Params)
{
	return AddExternalObjectProperty(Objects, NAME_None, Params);
}

IDetailPropertyRow* FCustomChildrenBuilder::AddExternalObjectProperty(const TArray<UObject*>& Objects, FName PropertyName, const FAddPropertyParams& Params)
{
	TSharedRef<FDetailCategoryImpl> ParentCategoryRef = ParentCategory.Pin().ToSharedRef();

	FDetailLayoutCustomization NewCustomization;
	FDetailPropertyRow::MakeExternalPropertyRowCustomization(Objects, PropertyName, ParentCategoryRef, NewCustomization, Params);

	if (Params.ShouldHideRootObjectNode() && NewCustomization.HasPropertyNode() && NewCustomization.GetPropertyNode()->AsObjectNode())
	{
		NewCustomization.PropertyRow->SetForceShowOnlyChildren(true);
	}

	TSharedPtr<FDetailPropertyRow> NewRow = NewCustomization.PropertyRow;
	if (NewRow.IsValid())
	{
		NewRow->SetCustomExpansionId(Params.GetUniqueId());

		ChildCustomizations.Add(NewCustomization);
	}

	return NewRow.Get();
}

TSharedRef<SWidget> FCustomChildrenBuilder::GenerateStructValueWidget( TSharedRef<IPropertyHandle> StructPropertyHandle )
{
	FStructProperty* StructProperty = CastFieldChecked<FStructProperty>( StructPropertyHandle->GetProperty() );

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	
	TSharedPtr<IDetailsViewPrivate> DetailsView = ParentCategory.Pin()->GetDetailsViewSharedPtr();

	FPropertyTypeLayoutCallback LayoutCallback = PropertyEditorModule.GetPropertyTypeCustomization(StructProperty, *StructPropertyHandle, DetailsView ? DetailsView->GetCustomPropertyTypeLayoutMap() : FCustomPropertyTypeLayoutMap() );
	if (LayoutCallback.IsValid())
	{
		TSharedRef<IPropertyTypeCustomization> CustomStructInterface = LayoutCallback.GetCustomizationInstance();

		return SNew( SStandaloneCustomizedValueWidget, CustomStructInterface, StructPropertyHandle).ParentCategory(ParentCategory.Pin().ToSharedRef());
	}
	else
	{
		// Uncustomized structs have nothing for their value content
		return SNullWidget::NullWidget;
	}
}

IDetailCategoryBuilder& FCustomChildrenBuilder::GetParentCategory() const
{
	return *ParentCategory.Pin();
}

FCustomChildrenBuilder& FCustomChildrenBuilder::OverrideResetChildrenToDefault(const FResetToDefaultOverride& ResetToDefault)
{
	CustomResetChildToDefault = ResetToDefault;
	return *this;
}

IDetailGroup* FCustomChildrenBuilder::GetParentGroup() const
{
	return ParentGroup.IsValid() ? ParentGroup.Pin().Get() : nullptr;
}
