// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "UObject/StructOnScope.h"
#include "PropertyHandle.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilderImpl.h"

class IDetailGroup;

class FCustomChildrenBuilder : public IDetailChildrenBuilder
{
public:
	FCustomChildrenBuilder(TSharedRef<FDetailCategoryImpl> InParentCategory, TSharedPtr<IDetailGroup> InParentGroup = nullptr)
		: ParentCategory(InParentCategory)
		, ParentGroup(InParentGroup)
	{}

	virtual IDetailChildrenBuilder& AddCustomBuilder(TSharedRef<class IDetailCustomNodeBuilder> InCustomBuilder) override;
	virtual IDetailGroup& AddGroup(FName GroupName, const FText& LocalizedDisplayName, const bool bStartExpanded = false) override;
	virtual IDetailGroup* GetGroup(FName GroupName) override;
	virtual FDetailWidgetRow& AddCustomRow(const FText& SearchString) override;
	virtual IDetailPropertyRow& AddProperty(TSharedRef<IPropertyHandle> PropertyHandle) override;
	UE_DEPRECATED(5.1, "Please use the overload that takes an FAddPropertyParams instead.")
	virtual IDetailPropertyRow* AddExternalObjects(const TArray<UObject*>& Objects, FName UniqueIdName) override;
	virtual IDetailPropertyRow* AddExternalObjects(const TArray<UObject*>& Objects, const FAddPropertyParams& Params) override;
	virtual IDetailPropertyRow* AddExternalObjectProperty(const TArray<UObject*>& Objects, FName PropertyName, const FAddPropertyParams& Params) override;
	virtual IDetailPropertyRow* AddExternalStructure(TSharedRef<FStructOnScope> ChildStructure, FName UniqueIdName) override;
	virtual IDetailPropertyRow* AddExternalStructure(TSharedPtr<IStructureDataProvider> ChildStructure, FName UniqueIdName) override;
	virtual IDetailPropertyRow* AddChildStructure(TSharedRef<IPropertyHandle> PropertyHandle, TSharedPtr<IStructureDataProvider> ChildStructure, FName UniqueIdName, const FText& DisplayNameOverride = FText()) override;
	virtual IDetailPropertyRow* AddExternalStructureProperty(TSharedRef<FStructOnScope> ChildStructure, FName PropertyName, const FAddPropertyParams& Params) override;
	virtual IDetailPropertyRow* AddExternalStructureProperty(TSharedPtr<IStructureDataProvider> ChildStructure, FName PropertyName, const FAddPropertyParams& Params) override;
	virtual IDetailPropertyRow* AddChildStructureProperty(TSharedRef<IPropertyHandle> PropertyHandle, TSharedPtr<IStructureDataProvider> ChildStructure, FName PropertyName, const FAddPropertyParams& Params = FAddPropertyParams(), const FText& DisplayNameOverride = FText()) override;
	virtual TArray<TSharedPtr<IPropertyHandle>> AddAllExternalStructureProperties(TSharedRef<FStructOnScope> ChildStructure) override;
	virtual TArray<TSharedPtr<IPropertyHandle>> AddAllExternalStructureProperties(TSharedPtr<IStructureDataProvider> ChildStructure) override;
	virtual TSharedRef<SWidget> GenerateStructValueWidget(TSharedRef<IPropertyHandle> StructPropertyHandle) override;
	virtual IDetailCategoryBuilder& GetParentCategory() const override;
	virtual IDetailGroup* GetParentGroup() const override;

	const TArray< FDetailLayoutCustomization >& GetChildCustomizations() const { return ChildCustomizations; }

	/** Set the user customized reset to default for the children of this builder */
	FCustomChildrenBuilder& OverrideResetChildrenToDefault(const FResetToDefaultOverride& ResetToDefault);

private:
	TArray< FDetailLayoutCustomization > ChildCustomizations;
	TWeakPtr<FDetailCategoryImpl> ParentCategory;
	TWeakPtr<IDetailGroup> ParentGroup;

	/** User customized reset to default on children */
	TOptional<FResetToDefaultOverride> CustomResetChildToDefault;

	IDetailPropertyRow* AddStructureProperty(const FAddPropertyParams& Params, TFunctionRef<void(FDetailLayoutCustomization&)> MakePropertyRowCustomization);
	template<class T> IDetailPropertyRow* AddExternalStructureProperty(const T& ChildStructure, FName PropertyName, const FAddPropertyParams& Params);
};
