// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailAdvancedDropdownNode.h"

#include "PropertyCustomizationHelpers.h"
#include "SDetailRowIndent.h"
#include "SDetailTableRowBase.h"
#include "Styling/StyleColors.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "DetailsViewStyle.h"
#include "SDetailsView.h"

class SAdvancedDropdownRow : public SDetailTableRowBase
{
public:
	SLATE_BEGIN_ARGS(SAdvancedDropdownRow)
		: _IsExpanded(false)
		, _IsButtonEnabled(true)
		, _IsVisible(true)
	{}
		SLATE_ATTRIBUTE(bool, IsExpanded)
		SLATE_ATTRIBUTE(bool, IsButtonEnabled)
		SLATE_ATTRIBUTE(bool, IsVisible)
		SLATE_EVENT(FOnClicked, OnClicked)
	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 *
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct( const FArguments& InArgs, IDetailsViewPrivate* InDetailsView, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		IsExpanded = InArgs._IsExpanded;
		OnClicked = InArgs._OnClicked;

		TAttribute<bool> IsVisibleBool = InArgs._IsVisible;
		TAttribute<EVisibility> IsVisible = TAttribute<EVisibility>::CreateLambda([IsVisibleBool]() { return IsVisibleBool.Get(true) ? EVisibility::Visible : EVisibility::Collapsed; });

		TSharedPtr<SWidget> ContentWidget = SNew(SHorizontalBox)
		.Visibility(IsVisible)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Fill)
		.AutoWidth()
		[
			SNew(SDetailRowIndent, SharedThis(this))
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(2.0f, 0.0f, 0.0f, 0.0f)
		.AutoWidth()
		[
			SAssignNew(ExpanderButton, SButton)
			.ButtonStyle(FCoreStyle::Get(), "NoBorder")
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.ClickMethod(EButtonClickMethod::MouseDown)
			.OnClicked(InArgs._OnClicked)
			.IsEnabled(InArgs._IsButtonEnabled)
			.ContentPadding(0.0f)
			.IsFocusable(false)
			.ToolTipText(this, &SAdvancedDropdownRow::GetAdvancedPulldownToolTipText )
			[
				SNew(SImage)
				.Image(this, &SAdvancedDropdownRow::GetExpanderImage)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			]
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Fill)
		.Padding(4.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SBox)
			.VAlign(VAlign_Center)
			.HeightOverride(PropertyEditorConstants::PropertyRowHeight)
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("PropertyEditor", "Advanced", "Advanced"))
				.Font(FAppStyle::Get().GetFontStyle(PropertyEditorConstants::PropertyFontStyle))
				.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
			]
		];

		OwnerTableViewWeak = InOwnerTableView;
		auto GetScrollbarWellBrush = [this]()
		{
			return SDetailTableRowBase::IsScrollBarVisible(OwnerTableViewWeak) ?
				FAppStyle::Get().GetBrush("DetailsView.GridLine") : 
				FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle");
		};

		auto GetScrollbarWellTint = [this]()
		{
			return IsScrollBarVisible(OwnerTableViewWeak) ?
				FStyleColors::White : 
				this->GetRowBackgroundColor();
		};

		FDetailsViewStyle ViewStyle = InDetailsView ? InDetailsView->GetStyleKey() : SDetailsView::GetPrimaryDetailsViewStyleKey();

		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("DetailsView.GridLine"))
			.Padding( 0.0f, 0.0f, 0.0f, 1.0f )
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle"))
					.BorderBackgroundColor(this, &SAdvancedDropdownRow::GetRowBackgroundColor)
					.Padding(0.0f)
					[
						ContentWidget.ToSharedRef()
					]
				]
			]
		];

		STableRow< TSharedPtr< FDetailTreeNode > >::ConstructInternal(
			STableRow::FArguments()
				.Style(FAppStyle::Get(), "DetailsView.TreeView.TableRow")
				.ShowSelection(false),
			InOwnerTableView
		);	
	}

private:

	FText GetAdvancedPulldownToolTipText() const
	{
		return IsExpanded.Get() ? NSLOCTEXT("DetailsView", "HideAdvanced", "Hide Advanced") : NSLOCTEXT("DetailsView", "ShowAdvanced", "Show Advanced");
	}

	const FSlateBrush* GetExpanderImage() const
	{
		const bool bIsItemExpanded = IsExpanded.Get();

		FName ResourceName;
		if (bIsItemExpanded)
		{
			if (ExpanderButton->IsHovered())
			{
				static const FName ExpandedHoveredName = "TreeArrow_Expanded_Hovered";
				ResourceName = ExpandedHoveredName;
			}
			else
			{
				static const FName ExpandedName = "TreeArrow_Expanded";
				ResourceName = ExpandedName;
			}
		}
		else
		{
			if (ExpanderButton->IsHovered())
			{
				static const FName CollapsedHoveredName = "TreeArrow_Collapsed_Hovered";
				ResourceName = CollapsedHoveredName;
			}
			else
			{
				static const FName CollapsedName = "TreeArrow_Collapsed";
				ResourceName = CollapsedName;
			}
		}

		return FAppStyle::Get().GetBrush(ResourceName);
	}

	FSlateColor GetRowBackgroundColor() const
	{
		return PropertyEditorConstants::GetRowBackgroundColor(0, this->IsHovered());
	}

	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			if (OnClicked.IsBound())
			{
				return OnClicked.Execute();
			}
		}
		
		return FReply::Unhandled();
	}

private:
	TAttribute<bool> IsExpanded;
	TSharedPtr<SButton> ExpanderButton;
	FOnClicked OnClicked;
	bool bDisplayShowAdvancedMessage;
};

TSharedRef< ITableRow > FAdvancedDropdownNode::GenerateWidgetForTableView( const TSharedRef<STableViewBase>& OwnerTable, bool bAllowFavoriteSystem)
{
	return SNew(SAdvancedDropdownRow, ParentCategory.GetDetailsViewSharedPtr().Get(), OwnerTable)
		.OnClicked(this, &FAdvancedDropdownNode::OnAdvancedDropDownClicked)
		.IsButtonEnabled(IsEnabled)
		.IsExpanded(IsExpanded)
		.IsVisible(IsVisible);
}

bool FAdvancedDropdownNode::GenerateStandaloneWidget(FDetailWidgetRow& OutRow) const
{
	// Not supported
	return false;
}

void FAdvancedDropdownNode::OnItemExpansionChanged(bool bIsExpanded, bool bShouldSaveState)
{
	// If state does not match, request expansion update to category
	if (bIsExpanded != IsExpanded.Get())
	{
		OnAdvancedDropDownClicked();
	}
}

FReply FAdvancedDropdownNode::OnAdvancedDropDownClicked()
{
	ParentCategory.OnAdvancedDropdownClicked();

	return FReply::Handled();
}

ENodeVisibility FAdvancedDropdownNode::GetVisibility() const
{
	TSharedPtr<IDetailsViewPrivate> DetailsView = GetDetailsViewSharedPtr();
	if (!DetailsView || DetailsView->IsCustomRowVisible(FName(NAME_None), FName(*GetParentCategory()->GetDisplayName().ToString())))
	{
		return ENodeVisibility::Visible;
	}
	return ENodeVisibility::ForcedHidden;
}