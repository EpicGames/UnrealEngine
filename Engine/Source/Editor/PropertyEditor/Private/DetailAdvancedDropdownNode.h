// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "IPropertyUtilities.h"
#include "DetailTreeNode.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "DetailCategoryBuilderImpl.h"
#include "PropertyCustomizationHelpers.h"

class FAdvancedDropdownNode : public FDetailTreeNode, public TSharedFromThis<FAdvancedDropdownNode>
{
public:
	FAdvancedDropdownNode(TSharedRef<FDetailCategoryImpl> InParentCategory, const TAttribute<bool>& InExpanded, const TAttribute<bool>& InEnabled, TAttribute<bool> InVisible)
		: ParentCategory(InParentCategory.Get())
		, IsEnabled(InEnabled)
		, IsExpanded(InExpanded)
		, IsVisible(InVisible)
	{
		SetParentNode(InParentCategory);
	}

private:
	//~ Begin FDetailTreeNode
	virtual TSharedPtr<IDetailsView> GetNodeDetailsViewSharedPtr() const override { return ParentCategory.GetNodeDetailsViewSharedPtr(); }
	virtual TSharedPtr<IDetailsViewPrivate> GetDetailsViewSharedPtr() const override { return ParentCategory.GetDetailsViewSharedPtr(); }
	virtual TSharedRef< ITableRow > GenerateWidgetForTableView( const TSharedRef<STableViewBase>& OwnerTable, bool bAllowFavoriteSystem) override;
	virtual bool GenerateStandaloneWidget(FDetailWidgetRow& OutRow) const override;
	virtual void GetChildren(FDetailNodeList& OutChildren, const bool& bInIgnoreVisibility) override {}
	virtual void OnItemExpansionChanged( bool bIsExpanded, bool bShouldSaveState) override;
	virtual bool ShouldBeExpanded() const override { return false; }
	virtual ENodeVisibility GetVisibility() const override;
	virtual void FilterNode( const FDetailFilter& InFilter ) override {}
	virtual void Tick( float DeltaTime ) override {}
	virtual bool ShouldShowOnlyChildren() const override { return false; }
	virtual FName GetNodeName() const override { return NAME_None; }
	virtual EDetailNodeType GetNodeType() const override { return EDetailNodeType::Advanced; }
	virtual TSharedPtr<IPropertyHandle> CreatePropertyHandle() const override { return nullptr; }
	//~ End FDetailTreeNode

	/** Called when the advanced drop down arrow is clicked */
	FReply OnAdvancedDropDownClicked();

	FDetailCategoryImpl& ParentCategory;
	TAttribute<bool> IsEnabled;
	TAttribute<bool> IsExpanded;
	TAttribute<bool> IsVisible;
};
