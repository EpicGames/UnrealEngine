// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailCategoryBuilderImpl.h"

#include "Components/ActorComponent.h"
#include "DetailAdvancedDropdownNode.h"
#include "DetailBuilderTypes.h"
#include "DetailCategoryGroupNode.h"
#include "DetailGroup.h"
#include "DetailItemNode.h"
#include "DetailPropertyRow.h"
#include "IPropertyGenerationUtilities.h"
#include "ObjectEditorUtils.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "ObjectPropertyNode.h"
#include "PropertyEditorModule.h"
#include "PropertyPermissionList.h"
#include "SDetailCategoryTableRow.h"
#include "StructurePropertyNode.h"
#include "Styling/StyleColors.h"

FString FDetailCategoryImpl::DetailCategories = TEXT("DetailCategories");
FString FDetailCategoryImpl::TransientPerViewDetailCategories = TEXT("TransientPerViewDetailCategories");
FString FDetailCategoryImpl::PersistentPerViewDetailCategories = TEXT("PersistentPerViewDetailCategories");

static void AddLayoutToList(const FDetailLayoutCustomization& Layout, TArray<FDetailLayoutCustomization>& List)
{
	if (Layout.bCustom)
	{
		int32 Index = 0;
		for (; Index < List.Num(); ++Index)
		{
			if (!List[Index].bCustom)
			{
				break;
			}
		}

		List.Insert(Layout, Index);
	}
	else
	{
		List.Add(Layout);
	}
}

void FDetailLayout::AddLayout(const FDetailLayoutCustomization& Layout)
{
	if (Layout.bAdvanced)
	{
		AddLayoutToList(Layout, AdvancedLayouts);
	}
	else
	{
		AddLayoutToList(Layout, SimpleLayouts);
	}
}

FDetailLayoutCustomization* FDetailLayout::GetDefaultLayout(const TSharedRef<FPropertyNode>& PropertyNode)
{
	FDetailLayoutCustomization* Customization = 
		SimpleLayouts.FindByPredicate([&PropertyNode](const FDetailLayoutCustomization& TestCustomization)
		{
			return TestCustomization.GetPropertyNode() == PropertyNode;
		});

	// Didn't find it in the simple layouts, look in advanced layouts
	if (Customization == nullptr)
	{
		Customization = 
			AdvancedLayouts.FindByPredicate([&PropertyNode](const FDetailLayoutCustomization& TestCustomization)
			{
				return TestCustomization.GetPropertyNode() == PropertyNode;
			});
	}

	if (Customization != nullptr && Customization->bCustom)
	{
		return nullptr;
	}

	return Customization;
}

FDetailLayoutCustomization::FDetailLayoutCustomization()
	: PropertyRow(nullptr)
	, WidgetDecl(nullptr)
	, CustomBuilderRow(nullptr)
{

}

bool FDetailLayoutCustomization::HasExternalPropertyRow() const
{
	return HasPropertyNode() && PropertyRow->HasExternalProperty();
}

bool FDetailLayoutCustomization::IsHidden() const
{
	return !IsValidCustomization()
		|| (HasCustomWidget() && WidgetDecl->VisibilityAttr.Get() != EVisibility::Visible)
		|| (HasPropertyNode() && PropertyRow->GetPropertyVisibility() != EVisibility::Visible);
}

TSharedPtr<FPropertyNode> FDetailLayoutCustomization::GetPropertyNode() const
{
	return PropertyRow.IsValid() ? PropertyRow->GetPropertyNode() : nullptr;
}

FDetailWidgetRow FDetailLayoutCustomization::GetWidgetRow() const
{
	if (HasCustomWidget())
	{
		return *WidgetDecl;
	}
	else if (HasCustomBuilder())
	{
		return *CustomBuilderRow->GetWidgetRow();
	}
	else if (HasPropertyNode())
	{
		return PropertyRow->GetWidgetRow();
	}
	else
	{
		return DetailGroup->GetWidgetRow();
	}
}

TArrayView<TSharedPtr<IPropertyHandle>> FDetailLayoutCustomization::GetPropertyHandles() const
{
	if (HasCustomWidget())
	{
		return WidgetDecl->PropertyHandles;
	}
	else if (HasCustomBuilder())
	{
		return CustomBuilderRow->GetWidgetRow()->PropertyHandles;
	}
	else if (HasPropertyNode())
	{
		return PropertyRow->GetPropertyHandles();
	}
	else if (DetailGroup->GetHeaderPropertyRow())
	{
		return DetailGroup->GetHeaderPropertyRow()->GetPropertyHandles();
	}
	return TArrayView<TSharedPtr<IPropertyHandle>>();
}

FText FDetailLayoutCustomization::GetFilterTextString() const
{
	if (HasCustomWidget())
	{
		return WidgetDecl->FilterTextString;
	}
	else if (HasCustomBuilder() && CustomBuilderRow->GetWidgetRow())
	{
		return CustomBuilderRow->GetWidgetRow()->FilterTextString;
	}
	else if (HasPropertyNode())
	{
		return PropertyRow->GetFilterTextString();
	}
	else if (DetailGroup->GetHeaderPropertyRow())
	{
		return DetailGroup->GetHeaderPropertyRow()->GetFilterTextString();
	}
	return {};
}

const IDetailLayoutRow* FDetailLayoutCustomization::GetDetailLayoutRow() const
{
	if (HasCustomWidget())
	{
		return WidgetDecl.Get();
	}
	else if (HasCustomBuilder())
	{
		return CustomBuilderRow.Get();
	}
	else if (HasGroup())
	{
		return DetailGroup.Get();
	}
	else if (PropertyRow.IsValid())
	{
		return PropertyRow.Get();
	}
	return nullptr;
}

FName FDetailLayoutCustomization::GetName() const
{
	if (const IDetailLayoutRow* LayoutRow = GetDetailLayoutRow())
	{
		return LayoutRow->GetRowName();
	}
	return NAME_None;
}

TOptional<FResetToDefaultOverride> FDetailLayoutCustomization::GetCustomResetToDefault() const
{
	if (const IDetailLayoutRow* LayoutRow = GetDetailLayoutRow())
	{
		return LayoutRow->GetCustomResetToDefault();
	}
	return TOptional<FResetToDefaultOverride>();
}

FDetailCategoryImpl::FDetailCategoryImpl(FName InCategoryName, TSharedRef<FDetailLayoutBuilderImpl> InDetailLayout)
	: HeaderContentWidget(nullptr)
	, DetailLayoutBuilder(InDetailLayout)
	, PasteFromTextDelegate(MakeShared<FOnPasteFromText>())
	, CopyToTextDelegate(MakeShared<FOnCopyToText>())
	, CategoryName(InCategoryName)
	, SortOrder(0)
	, bRestoreExpansionState(!ContainsOnlyAdvanced())
	, bShouldBeInitiallyCollapsed(false)
	, bUserShowAdvanced(false)
	, bForceAdvanced(false)
	, bHasFilterStrings(false)
	, bHasVisibleDetails(true)
	, bIsCategoryVisible(true)
	, bFavoriteCategory(false)
	, bShowOnlyChildren(false)
	, bHasVisibleAdvanced(false)
	, bPendingRefresh(false)
	, bPendingRefreshNeedsRefilter(false)
	, bIsEmpty(false)
{
	const UStruct* BaseStruct = InDetailLayout->GetRootNode()->GetBaseStructure();

	static const FName NoCategoryName = TEXT("NoCategory");
	bShowOnlyChildren = (InDetailLayout->IsLayoutForExternalRoot() && !InDetailLayout->GetRootNode()->HasNodeFlags(EPropertyNodeFlags::ShowCategories)) || CategoryName == NoCategoryName;

	// Use the base class name if there is one otherwise this is a generic category not specific to a class
	FName BaseStructName = BaseStruct ? BaseStruct->GetFName() : FName("Generic");

	//Path is separate by '.' so convert category delimiter from '|' to '.'
	FString CategoryDelimiterString;
	CategoryDelimiterString.AppendChar(FPropertyNodeConstants::CategoryDelimiterChar);
	FString CategoryPathDelimiterString;
	CategoryPathDelimiterString.AppendChar(TCHAR('.'));
	CategoryPathName = BaseStructName.ToString() + TEXT(".") + CategoryName.ToString().Replace(*CategoryDelimiterString, *CategoryPathDelimiterString);
	bool bUserShowAdvancedConfigValue = false;
	GConfig->GetBool(TEXT("DetailCategoriesAdvanced"), *CategoryPathName, bUserShowAdvancedConfigValue, GEditorPerProjectIni);

	bUserShowAdvanced = bUserShowAdvancedConfigValue;
}

FDetailCategoryImpl::~FDetailCategoryImpl()
{
	// The children won't be able to pin the weak pointer of their parent (this object) to remove themselves from the list of tickables
	// because at this point the parent's shared pointer reference counter has already reached 0 (the execution has reached this destructor).
	// Therefore the tickable nodes need to be manually removed from that list to prevent these dead node pointers remaining in the layout object.
	for (TSharedRef<FDetailTreeNode>& SimpleChildNode : SimpleChildNodes)
	{
		RemoveTickableNode(*SimpleChildNode);
	}
	for (TSharedRef<FDetailTreeNode>& AdvancedChildNode : AdvancedChildNodes)
	{
		RemoveTickableNode(*AdvancedChildNode);
	}
	RemoveTickableNode(*this);
}


FDetailWidgetRow& FDetailCategoryImpl::AddCustomRow(const FText& FilterString, bool bForAdvanced)
{
	FDetailLayoutCustomization NewCustomization;
	NewCustomization.bCustom = true;
	NewCustomization.bAdvanced = bForAdvanced;
	NewCustomization.WidgetDecl = MakeShareable(new FDetailWidgetRow);
	NewCustomization.WidgetDecl->FilterString(FilterString);

	AddCustomLayout(NewCustomization);

	return *NewCustomization.WidgetDecl;
}


void FDetailCategoryImpl::AddCustomBuilder(TSharedRef<IDetailCustomNodeBuilder> InCustomBuilder, bool bForAdvanced)
{
	FDetailLayoutCustomization NewCustomization;
	NewCustomization.bCustom = true;
	NewCustomization.bAdvanced = bForAdvanced;
	NewCustomization.CustomBuilderRow = MakeShareable(new FDetailCustomBuilderRow(InCustomBuilder));

	if (!InCustomBuilder->GetName().IsNone())
	{
		TStringBuilder<256> PathToNode;
		PathToNode.Append(GetCategoryPathName());
		PathToNode.Append(TEXT("."));
		PathToNode.Append(InCustomBuilder->GetName().ToString());
		NewCustomization.CustomBuilderRow->SetOriginalPath(PathToNode.ToString());
	}

	if (!bFavoriteCategory)
	{
		if (TSharedPtr<IDetailsViewPrivate> DetailsView = GetDetailsViewSharedPtr())
		{
			if (DetailsView->IsCustomBuilderFavorite(NewCustomization.CustomBuilderRow->GetOriginalPath()))
			{
				TSharedPtr<FDetailLayoutBuilderImpl> ParentLayout = GetParentLayoutImpl();
				if (ParentLayout.IsValid())
				{
					static const FName FavoritesCategoryName(TEXT("Favorites"));
					FDetailCategoryImpl& FavoritesCategory = ParentLayout->DefaultCategory(FavoritesCategoryName);

					FDetailLayoutCustomization FavoritesCustomization;
					FavoritesCustomization.bCustom = true;
					FavoritesCustomization.bAdvanced = false;
					FavoritesCustomization.CustomBuilderRow = MakeShareable(new FDetailCustomBuilderRow(InCustomBuilder));
					FavoritesCustomization.CustomBuilderRow->SetOriginalPath(NewCustomization.CustomBuilderRow->GetOriginalPath());
					FavoritesCategory.AddCustomLayout(FavoritesCustomization);
				}
			}
		}
	}

	AddCustomLayout(NewCustomization);
}

IDetailGroup& FDetailCategoryImpl::AddGroup(FName GroupName, const FText& LocalizedDisplayName, bool bForAdvanced, bool bStartExpanded)
{
	FDetailLayoutCustomization NewCustomization;
	NewCustomization.bCustom = true;
	NewCustomization.bAdvanced = bForAdvanced;
	NewCustomization.DetailGroup = MakeShareable(new FDetailGroup(GroupName, AsShared(), LocalizedDisplayName, bStartExpanded));

	AddCustomLayout(NewCustomization);

	return *NewCustomization.DetailGroup;
}

int32 FDetailCategoryImpl::GetNumCustomizations() const
{
	int32 NumCustomizations = 0;

	for (const FDetailLayout& Layout : LayoutMap)
	{
		NumCustomizations += Layout.GetSimpleLayouts().Num();
		NumCustomizations += Layout.GetAdvancedLayouts().Num();
	}

	return NumCustomizations;
}

void FDetailCategoryImpl::GetDefaultProperties(TArray<TSharedRef<IPropertyHandle> >& OutDefaultProperties, bool bSimpleProperties, bool bAdvancedProperties)
{
	TSharedPtr<FDetailLayoutBuilderImpl> ParentLayout = GetParentLayoutImpl();
	for (const FDetailLayout& Layout : LayoutMap)
	{
		if (bSimpleProperties)
		{
			for (const FDetailLayoutCustomization& Customization : Layout.GetSimpleLayouts())
			{
				if (Customization.HasPropertyNode())
				{
					const TSharedPtr<FPropertyNode>& Node = Customization.GetPropertyNode();

					TSharedRef<IPropertyHandle> PropertyHandle = ParentLayout->GetPropertyHandle(Node);
					if (PropertyHandle->IsValidHandle())
					{
						OutDefaultProperties.Add(PropertyHandle);
					}
				}
			}
		}

		if (bAdvancedProperties)
		{
			for (const FDetailLayoutCustomization& Customization : Layout.GetAdvancedLayouts())
			{
				if (Customization.HasPropertyNode())
				{
					const TSharedPtr<FPropertyNode>& Node = Customization.GetPropertyNode();
					TSharedRef<IPropertyHandle> PropertyHandle = ParentLayout->GetPropertyHandle(Node);
					if (PropertyHandle->IsValidHandle())
					{
						OutDefaultProperties.Add(PropertyHandle);
					}
				}
			}
		}
	}
}

void FDetailCategoryImpl::SetCategoryVisibility(bool bIsVisible)
{
	if (bIsVisible != bIsCategoryVisible)
	{
		bIsCategoryVisible = bIsVisible;

		if (TSharedPtr<IDetailsViewPrivate> DetailsView = GetDetailsViewSharedPtr())
		{
			DetailsView->RerunCurrentFilter();
		}

		TSharedPtr<FDetailLayoutBuilderImpl> ParentLayout = GetParentLayoutImpl();
		if (ParentLayout.IsValid())
		{
			ParentLayout->NotifyNodeVisibilityChanged();
		}
	}
}

IDetailCategoryBuilder& FDetailCategoryImpl::InitiallyCollapsed(bool bInShouldBeInitiallyCollapsed)
{
	this->bShouldBeInitiallyCollapsed = bInShouldBeInitiallyCollapsed;
	return *this;
}

IDetailCategoryBuilder& FDetailCategoryImpl::OnExpansionChanged(FOnBooleanValueChanged InOnExpansionChanged)
{
	this->OnExpansionChangedDelegate = InOnExpansionChanged;
	return *this;
}

IDetailCategoryBuilder& FDetailCategoryImpl::RestoreExpansionState(bool bRestore)
{
	this->bRestoreExpansionState = bRestore;
	return *this;
}

IDetailCategoryBuilder& FDetailCategoryImpl::HeaderContent(TSharedRef<SWidget> InHeaderContent, bool bWholeRowContent)
{
	ensureMsgf(!this->HeaderContentWidget.IsValid(), TEXT("Category already has a header content widget defined!"));
	this->HeaderContentWidget = InHeaderContent;
	this->bHeaderContentWholeRowContent = bWholeRowContent;
	return *this;
}

IDetailPropertyRow& FDetailCategoryImpl::AddProperty(FName PropertyPath, UClass* ClassOutermost, FName InstanceName, EPropertyLocation::Type Location)
{
	FDetailLayoutCustomization NewCustomization;
	NewCustomization.bCustom = true;

	TSharedPtr<FDetailLayoutBuilderImpl> ParentLayout = GetParentLayoutImpl();
	TSharedPtr<FPropertyNode> PropertyNode = ParentLayout->GetPropertyNode(PropertyPath, ClassOutermost, InstanceName);
	if (PropertyNode.IsValid())
	{
		ParentLayout->SetCustomProperty(PropertyNode);
	}

	NewCustomization.PropertyRow = MakeShareable(new FDetailPropertyRow(PropertyNode, AsShared()));
	NewCustomization.bAdvanced = (Location == EPropertyLocation::Default) ? IsAdvancedLayout(NewCustomization) : (Location == EPropertyLocation::Advanced);

	AddCustomLayout(NewCustomization);

	return *NewCustomization.PropertyRow;
}

IDetailPropertyRow& FDetailCategoryImpl::AddProperty(TSharedPtr<IPropertyHandle> PropertyHandle, EPropertyLocation::Type Location)
{
	FDetailLayoutCustomization NewCustomization;
	NewCustomization.bCustom = true;

	TSharedPtr<FDetailLayoutBuilderImpl> ParentLayout = GetParentLayoutImpl();
	TSharedPtr<FPropertyNode> PropertyNode = ParentLayout->GetPropertyNode(PropertyHandle);
	if (PropertyNode.IsValid())
	{
		ParentLayout->SetCustomProperty(PropertyNode);
	}

	NewCustomization.PropertyRow = MakeShareable(new FDetailPropertyRow(PropertyNode, AsShared()));
	NewCustomization.bAdvanced = (Location == EPropertyLocation::Default) ? IsAdvancedLayout(NewCustomization) : (Location == EPropertyLocation::Advanced);

	AddCustomLayout(NewCustomization);

	return *NewCustomization.PropertyRow;
}

IDetailPropertyRow* FDetailCategoryImpl::AddExternalObjects(const TArray<UObject*>& Objects, EPropertyLocation::Type Location /*= EPropertyLocation::Default*/, const FAddPropertyParams& Params /*= FAddPropertyParams()*/)
{
	FDetailLayoutCustomization NewCustomization;
	NewCustomization.bCustom = true;
	NewCustomization.bAdvanced = Location == EPropertyLocation::Advanced;

	FAddPropertyParams AddPropertyParams = Params;
	AddPropertyParams.AllowChildren(true);

	FDetailPropertyRow::MakeExternalPropertyRowCustomization(Objects, NAME_None, AsShared(), NewCustomization, AddPropertyParams);

	if (Params.ShouldHideRootObjectNode() && NewCustomization.HasPropertyNode() && NewCustomization.GetPropertyNode()->AsObjectNode())
	{
		NewCustomization.PropertyRow->SetForceShowOnlyChildren(true);
	}

	TSharedPtr<FDetailPropertyRow> NewRow = NewCustomization.PropertyRow;

	if (NewRow.IsValid())
	{
		AddCustomLayout(NewCustomization);
	}

	return NewRow.Get();
}

IDetailPropertyRow* FDetailCategoryImpl::AddExternalObjectProperty(const TArray<UObject*>& Objects, FName PropertyName, EPropertyLocation::Type Location, const FAddPropertyParams& Params)
{
	FDetailLayoutCustomization NewCustomization;
	NewCustomization.bCustom = true;
	NewCustomization.bAdvanced = Location == EPropertyLocation::Advanced;

	FDetailPropertyRow::MakeExternalPropertyRowCustomization(Objects, PropertyName, AsShared(), NewCustomization, Params);

	TSharedPtr<FDetailPropertyRow> NewRow = NewCustomization.PropertyRow;

	if (NewRow.IsValid())
	{
		AddCustomLayout(NewCustomization);

	}

	return NewRow.Get();
}

IDetailPropertyRow* FDetailCategoryImpl::AddExternalStructure(TSharedPtr<FStructOnScope> StructData, EPropertyLocation::Type Location /*= EPropertyLocation::Default*/)
{
	return AddExternalStructureProperty(StructData, NAME_None, Location);
}

IDetailPropertyRow* FDetailCategoryImpl::AddExternalStructureProperty(TSharedPtr<FStructOnScope> StructData, FName PropertyName, EPropertyLocation::Type Location/* = EPropertyLocation::Default*/, const FAddPropertyParams& Params)
{
	FDetailLayoutCustomization NewCustomization;
	NewCustomization.bCustom = true;
	NewCustomization.bAdvanced = Location == EPropertyLocation::Advanced;

	FDetailPropertyRow::MakeExternalPropertyRowCustomization(StructData, PropertyName, AsShared(), NewCustomization, Params);

	if (Params.ShouldHideRootObjectNode() && NewCustomization.HasPropertyNode() && NewCustomization.GetPropertyNode()->AsComplexNode())
	{
		NewCustomization.PropertyRow->SetForceShowOnlyChildren(true);
	}

	TSharedPtr<FDetailPropertyRow> NewRow = NewCustomization.PropertyRow;

	if (NewRow.IsValid())
	{
		TSharedPtr<FPropertyNode> PropertyNode = NewRow->GetPropertyNode();
		TSharedPtr<FComplexPropertyNode> RootNode = StaticCastSharedRef<FComplexPropertyNode>(PropertyNode->FindComplexParent()->AsShared());

		AddCustomLayout(NewCustomization);
	}

	return NewRow.Get();
}

IDetailPropertyRow* FDetailCategoryImpl::AddExternalStructureProperty(TSharedPtr<IStructureDataProvider> StructDataProvider, FName PropertyName, EPropertyLocation::Type Location, const FAddPropertyParams& Params)
{
	FDetailLayoutCustomization NewCustomization;
	NewCustomization.bCustom = true;
	NewCustomization.bAdvanced = Location == EPropertyLocation::Advanced;

	FDetailPropertyRow::MakeExternalPropertyRowCustomization(StructDataProvider, PropertyName, AsShared(), NewCustomization, Params);

	if (Params.ShouldHideRootObjectNode() && NewCustomization.HasPropertyNode() && NewCustomization.GetPropertyNode()->AsComplexNode())
	{
		NewCustomization.PropertyRow->SetForceShowOnlyChildren(true);
	}

	TSharedPtr<FDetailPropertyRow> NewRow = NewCustomization.PropertyRow;

	if (NewRow.IsValid())
	{
		TSharedPtr<FPropertyNode> PropertyNode = NewRow->GetPropertyNode();
		TSharedPtr<FComplexPropertyNode> RootNode = StaticCastSharedRef<FComplexPropertyNode>(PropertyNode->FindComplexParent()->AsShared());

		AddCustomLayout(NewCustomization);
	}

	return NewRow.Get();
}

TArray<TSharedPtr<IPropertyHandle>> FDetailCategoryImpl::AddAllExternalStructureProperties(TSharedRef<FStructOnScope> StructData, EPropertyLocation::Type Location, TArray<IDetailPropertyRow*>* OutPropertiesRow)
{
	return AddAllExternalStructureProperties(MakeShared<FStructOnScopeStructureDataProvider>(StructData), Location, OutPropertiesRow);
}

TArray<TSharedPtr<IPropertyHandle>> FDetailCategoryImpl::AddAllExternalStructureProperties(
	TSharedPtr<IStructureDataProvider> StructProvider, EPropertyLocation::Type Location,
	TArray<IDetailPropertyRow*>* OutPropertiesRow)
{
	TSharedPtr<FStructurePropertyNode> RootPropertyNode(new FStructurePropertyNode);
	RootPropertyNode->SetStructure(StructProvider);

	FPropertyNodeInitParams InitParams;
	InitParams.ParentNode = nullptr;
	InitParams.Property = nullptr;
	InitParams.ArrayOffset = 0;
	InitParams.ArrayIndex = INDEX_NONE;
	InitParams.bAllowChildren = false;
	InitParams.bForceHiddenPropertyVisibility = FPropertySettings::Get().ShowHiddenProperties();
	InitParams.bCreateCategoryNodes = false;

	RootPropertyNode->InitNode(InitParams);

	TArray<TSharedPtr<IPropertyHandle>> Handles;


	if (RootPropertyNode.IsValid())
	{
		RootPropertyNode->RebuildChildren();

		TSharedPtr<FDetailLayoutBuilderImpl> ParentLayout = GetParentLayoutImpl();
		ParentLayout->AddExternalRootPropertyNode(RootPropertyNode.ToSharedRef());

		for (int32 ChildIdx = 0; ChildIdx < RootPropertyNode->GetNumChildNodes(); ++ChildIdx)
		{
			TSharedPtr< FPropertyNode > PropertyNode = RootPropertyNode->GetChildNode(ChildIdx);
			if (FProperty* Property = PropertyNode->GetProperty())
			{
				FDetailLayoutCustomization NewCustomization;
				NewCustomization.PropertyRow = MakeShared<FDetailPropertyRow>(PropertyNode, AsShared(), RootPropertyNode);
				NewCustomization.bAdvanced = Location == EPropertyLocation::Advanced;
				AddDefaultLayout(NewCustomization, NAME_None);

				Handles.Add(ParentLayout->GetPropertyHandle(PropertyNode));
				if (OutPropertiesRow)
				{
					OutPropertiesRow->Add(NewCustomization.PropertyRow.Get());
				}
			}
		}
	}

	return Handles;
}

void FDetailCategoryImpl::AddPropertyNode(TSharedRef<FPropertyNode> PropertyNode, FName InstanceName)
{
	FDetailLayoutCustomization NewCustomization;
	NewCustomization.PropertyRow = MakeShared<FDetailPropertyRow>(PropertyNode, AsShared());
	NewCustomization.bAdvanced = IsAdvancedLayout(NewCustomization);
	AddDefaultLayout(NewCustomization, InstanceName);
}

bool FDetailCategoryImpl::IsAdvancedLayout(const FDetailLayoutCustomization& LayoutInfo)
{
	TSharedPtr<FPropertyNode> PropertyNode = LayoutInfo.GetPropertyNode();
	if (PropertyNode.IsValid() && PropertyNode->HasNodeFlags(EPropertyNodeFlags::IsAdvanced))
	{
		return true;
	}

	return false;
}

void FDetailCategoryImpl::AddCustomLayout(const FDetailLayoutCustomization& LayoutInfo)
{
	ensure(LayoutInfo.bCustom == true);
	GetLayoutForInstance(GetParentLayoutImpl()->GetCurrentCustomizationVariableName()).AddLayout(LayoutInfo);
}

void FDetailCategoryImpl::AddDefaultLayout(const FDetailLayoutCustomization& LayoutInfo, FName InstanceName)
{
	ensure(LayoutInfo.bCustom == false);
	GetLayoutForInstance(InstanceName).AddLayout(LayoutInfo);
}

FDetailLayout& FDetailCategoryImpl::GetLayoutForInstance(FName InstanceName)
{
	return LayoutMap.FindOrAdd(InstanceName);
}

void FDetailCategoryImpl::OnAdvancedDropdownClicked()
{
	bUserShowAdvanced = !bUserShowAdvanced;

	GConfig->SetBool(TEXT("DetailCategoriesAdvanced"), *CategoryPathName, bUserShowAdvanced, GEditorPerProjectIni);

	const bool bRefilterCategory = true;
	RefreshTree(bRefilterCategory);
}

FDetailLayoutCustomization* FDetailCategoryImpl::GetDefaultCustomization(TSharedRef<FPropertyNode> PropertyNode)
{
	FDetailLayout& Layout = GetLayoutForInstance(GetParentLayoutImpl()->GetCurrentCustomizationVariableName());
	
	FDetailLayoutCustomization* Customization = Layout.GetDefaultLayout(PropertyNode);
	return Customization;
}

bool FDetailCategoryImpl::IsEmpty() const
{
	return bIsEmpty;
}

void FDetailCategoryImpl::SetIsEmpty(bool bInIsEmpty)
{
	bIsEmpty = bInIsEmpty;
}

TOptional<FLinearColor> FDetailCategoryImpl::GetColor() const
{
	return Color;
}

void FDetailCategoryImpl::SetColor(FLinearColor InColor)
{
	Color = InColor;
}

void FDetailCategoryImpl::ClearColor()
{
	Color = TOptional<FLinearColor>();
}
 
bool FDetailCategoryImpl::ShouldAdvancedBeExpanded() const
{
	return bUserShowAdvanced || bForceAdvanced;
}

void FDetailCategoryImpl::SetShowAdvanced(bool bShowAdvanced)
{
	bUserShowAdvanced = bShowAdvanced;
}

int32 FDetailCategoryImpl::GetSortOrder() const
{
	return SortOrder;
}

void FDetailCategoryImpl::SetSortOrder(int32 InSortOrder)
{
	SortOrder = InSortOrder;
}

void FDetailCategoryImpl::AddPropertyDisableInstancedReference(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	FDetailLayoutCustomization NewCustomization;
	NewCustomization.bCustom = true;

	TSharedPtr<FDetailLayoutBuilderImpl> ParentLayout = GetParentLayoutImpl();
	TSharedPtr<FPropertyNode> PropertyNode = ParentLayout->GetPropertyNode(PropertyHandle);
	if (PropertyNode.IsValid())
	{
		ParentLayout->SetCustomProperty(PropertyNode);
		PropertyNode->SetIgnoreInstancedReference();
	}

	NewCustomization.PropertyRow = MakeShareable(new FDetailPropertyRow(PropertyNode, AsShared()));
	NewCustomization.bAdvanced = IsAdvancedLayout(NewCustomization);

	AddCustomLayout(NewCustomization);
}

bool FDetailCategoryImpl::IsAdvancedDropdownEnabled() const
{
	return !bForceAdvanced;
}

bool FDetailCategoryImpl::ShouldAdvancedBeVisible() const
{
	return bHasVisibleAdvanced;
}

void FDetailCategoryImpl::RequestItemExpanded(TSharedRef<FDetailTreeNode> TreeNode, bool bShouldBeExpanded)
{
	if (TSharedPtr<IDetailsViewPrivate> DetailsView = GetDetailsViewSharedPtr())
	{		
		DetailsView->RequestItemExpanded(TreeNode, bShouldBeExpanded);
	}
}

void FDetailCategoryImpl::Tick(float DeltaTime)
{
	ensure(bPendingRefresh);
	RefreshTreeInternal(bPendingRefreshNeedsRefilter);
	bPendingRefresh = false;
	RemoveTickableNode(*this);
}

void FDetailCategoryImpl::RefreshTree(bool bRefilterCategory)
{
	TSharedPtr<FDetailLayoutBuilderImpl> ParentLayout = GetParentLayoutImpl();

	// If this is an external property node, refresh it directly since external objects' detail layout builder don't get ticked.
	if (ParentLayout.IsValid() && ParentLayout->IsLayoutForExternalRoot())
	{
		RefreshTreeInternal(bRefilterCategory);
	}
	else
	{
		bPendingRefresh = true;
		bPendingRefreshNeedsRefilter = bRefilterCategory;
		AddTickableNode(*this);
	}
}

void FDetailCategoryImpl::RefreshTreeInternal(bool bRefilterCategory)
{
	if (bRefilterCategory)
	{
		TSharedPtr<FDetailLayoutBuilderImpl> ParentLayout = GetParentLayoutImpl();
		if (ParentLayout.IsValid())
		{
			ParentLayout->RefreshNodeVisbility();
			FilterNode(ParentLayout->GetCurrentFilter());
			ParentLayout->GetPropertyGenerationUtilities().RebuildTreeNodes();
		}
	}
	else
	{
		if (TSharedPtr<IDetailsViewPrivate> DetailsView = GetDetailsViewSharedPtr())
		{
			DetailsView->RefreshTree();
		}
	}
}

void FDetailCategoryImpl::AddTickableNode(FDetailTreeNode& TickableNode)
{
	TSharedPtr<FDetailLayoutBuilderImpl> ParentLayout = GetParentLayoutImpl();
	if (ParentLayout.IsValid())
	{
		ParentLayout->AddTickableNode(TickableNode);
	}
}

void FDetailCategoryImpl::RemoveTickableNode(FDetailTreeNode& TickableNode)
{
	TSharedPtr<FDetailLayoutBuilderImpl> ParentLayout = GetParentLayoutImpl();
	if (ParentLayout.IsValid())
	{
		ParentLayout->RemoveTickableNode(TickableNode);
	}
}

void FDetailCategoryImpl::SaveExpansionState(FDetailTreeNode& InTreeNode)
{
	TSharedPtr<FDetailLayoutBuilderImpl> ParentLayout = GetParentLayoutImpl();
	if (ParentLayout.IsValid())
	{
		bool bIsExpanded = InTreeNode.ShouldBeExpanded();

		FString Key = CategoryPathName;
		Key += TEXT(".");
		Key += InTreeNode.GetNodeName().ToString();

		ParentLayout->SaveExpansionState(Key, bIsExpanded);
	}
}

bool FDetailCategoryImpl::GetSavedExpansionState(FDetailTreeNode& InTreeNode) const
{
	TSharedPtr<FDetailLayoutBuilderImpl> ParentLayout = GetParentLayoutImpl();
	if (ParentLayout.IsValid())
	{
		FString Key = CategoryPathName;
		Key += TEXT(".");
		Key += InTreeNode.GetNodeName().ToString();

		return ParentLayout->GetSavedExpansionState(Key);
	}

	return false;
}

bool FDetailCategoryImpl::ContainsOnlyAdvanced() const
{
	return !bFavoriteCategory && SimpleChildNodes.Num() == 0 && AdvancedChildNodes.Num() > 0;
}

void FDetailCategoryImpl::SetDisplayName(const FText& InDisplayName)
{
	SetDisplayName(CategoryName, InDisplayName);
}

const TOptional<FText>& FDetailCategoryImpl::GetToolTip() const
{
	return ToolTip;
}

void FDetailCategoryImpl::SetToolTip(const FText& InToolTip)
{
	ToolTip = InToolTip;
}

void FDetailCategoryImpl::SetDisplayName(FName InCategoryName, const FText& LocalizedNameOverride)
{
	if (!LocalizedNameOverride.IsEmpty())
	{
		DisplayName = LocalizedNameOverride;
	}
	else if (InCategoryName != NAME_None)
	{
		DisplayName = FObjectEditorUtils::GetCategoryText(InCategoryName);
	}
	else
	{
		// Use the base class name if there is one otherwise this is a generic category not specific to a class
		const UStruct* BaseStruct = GetParentLayoutImpl()->GetRootNode()->GetBaseStructure();
		if (BaseStruct)
		{
			DisplayName = BaseStruct->GetDisplayNameText();
		}
		else
		{
			DisplayName = NSLOCTEXT("DetailCategory", "GenericCategory", "Generic");
		}
	}
}

TSharedPtr<IDetailsViewPrivate> FDetailCategoryImpl::GetDetailsViewSharedPtr() const
{
	TSharedPtr<FDetailLayoutBuilderImpl> ParentLayout = GetParentLayoutImpl();
	if (ParentLayout.IsValid())
	{
		return StaticCastSharedPtr<IDetailsViewPrivate>(ParentLayout->GetDetailsViewSharedPtr());
	}

	return nullptr;
}

TSharedRef<ITableRow> FDetailCategoryImpl::GenerateWidgetForTableView(const TSharedRef<STableViewBase>& OwnerTable, bool bAllowFavoriteSystem)
{
	TSharedPtr<SWidget> HeaderContent = HeaderContentWidget;
	if (InlinePropertyNode.IsValid())
	{
		FDetailWidgetRow Row;
		InlinePropertyNode->GenerateStandaloneWidget(Row);
		HeaderContent = Row.ValueWidget.Widget;
	}

	TSharedPtr<FDetailLayoutBuilderImpl> ParentLayout = GetParentLayoutImpl();
	
	TArray<TWeakObjectPtr<UObject>> Objects;
	TSharedPtr<FComplexPropertyNode> Node = DetailLayoutBuilder.Pin()->GetRootNode();
	if (Node.IsValid())
	{
		if (FObjectPropertyNode* Object = Node->AsObjectNode())
		{
			Objects.Reserve(Object->GetNumObjects());
			for(int32 Index = 0, End = Object->GetNumObjects(); Index < End; ++Index)
			{
				if (UObject* CategoryObject = Object->GetUObject(Index))
				{
					Objects.Add(CategoryObject);
				}
			}			
		}
	}

	TSharedRef<SDetailCategoryTableRow> RowWidget =
		SNew(SDetailCategoryTableRow, AsShared(), OwnerTable)
		.PasteFromText(OnPasteFromText())
		.CopyToText(OnCopyToText())
		.Objects(&Objects)
		.IsEmpty( bIsEmpty )
		.InnerCategory(ParentLayout.IsValid() ? ParentLayout->IsLayoutForExternalRoot() : false)
		.DisplayName(GetDisplayName())
		.HeaderContent(HeaderContent)
		.WholeRowHeaderContent(bHeaderContentWholeRowContent)
		.ShowHeader(ParentLayout.IsValid() ? ParentLayout->GetCurrentFilter().bShowHeaders : true);

	if (ToolTip.IsSet() && !ToolTip->IsEmpty())
	{
		RowWidget->SetToolTipText(ToolTip.GetValue());
	}

	return RowWidget;
}

bool FDetailCategoryImpl::GenerateStandaloneWidget(FDetailWidgetRow& OutRow) const
{
	TSharedPtr<SWidget> HeaderContent = HeaderContentWidget;
	if (InlinePropertyNode.IsValid())
	{
		FDetailWidgetRow Row;
		InlinePropertyNode->GenerateStandaloneWidget(Row);
		HeaderContent = Row.ValueWidget.Widget;
	}

	const bool bIsInnerCategory = GetParentLayoutImpl()->IsLayoutForExternalRoot();
	FTextBlockStyle NameStyle = bIsInnerCategory ? FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText") : FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("DetailsView.CategoryTextStyle");

	TSharedRef<STextBlock> TextBlock =
    	SNew(STextBlock)
    	.Text(GetDisplayName())
    	.TextStyle(&NameStyle)
    	.ShadowOffset(FVector2D::ZeroVector);

    if (ToolTip.IsSet() && !ToolTip->IsEmpty())
    {
    	TextBlock->SetToolTipText(ToolTip.GetValue());
    }

	OutRow.NameContent()
	[
		TextBlock
	];

	if(HeaderContentWidget.IsValid())
	{
		OutRow
			.ValueContent()
			.HAlign(HAlign_Fill)
			[
				HeaderContent.ToSharedRef()
			];
	}

	return true;
}

void FDetailCategoryImpl::GetFilterStrings(TArray<FString>& OutFilterStrings) const
{
	OutFilterStrings.Add(GetDisplayName().ToString());
}

bool FDetailCategoryImpl::GetInitiallyCollapsed() const
{
	return bShouldBeInitiallyCollapsed;
}

void FDetailCategoryImpl::OnItemExpansionChanged(bool bIsExpanded, bool bShouldSaveState)
{
	if (bRestoreExpansionState && bShouldSaveState)
	{
		
		// Save the collapsed state of this section
		GConfig->SetBool(*GetDetailCategoriesExpansionSection(), *GetCategoryPathConfigKey(CategoryPathName), bIsExpanded, GEditorPerProjectIni);
	}

	OnExpansionChangedDelegate.ExecuteIfBound(bIsExpanded);
}

const FString& FDetailCategoryImpl::GetDetailCategoriesExpansionSection() const
{
	if (const TSharedPtr<IDetailsViewPrivate> DetailsView = GetDetailsViewSharedPtr())
	{
		if (DetailsView->GetExpansionPersistenceMethod() == FDetailsViewArgs::EExpansionPersistenceMethod::TransientPerViewIdentifier)
		{
			return TransientPerViewDetailCategories;
		}
		else if (DetailsView->GetExpansionPersistenceMethod() == FDetailsViewArgs::EExpansionPersistenceMethod::PersistentPerViewIdentifier)
		{
			return PersistentPerViewDetailCategories;
		}
	}

	return DetailCategories;
}

FString FDetailCategoryImpl::GetCategoryPathConfigKey(const FString& InPath) const
{
	if (const TSharedPtr<IDetailsViewPrivate> DetailsView = GetDetailsViewSharedPtr())
	{
		if (DetailsView->GetExpansionPersistenceMethod() == FDetailsViewArgs::EExpansionPersistenceMethod::TransientPerViewIdentifier
			|| DetailsView->GetExpansionPersistenceMethod() == FDetailsViewArgs::EExpansionPersistenceMethod::PersistentPerViewIdentifier)
		{
			if (ensure(!DetailsView->GetViewIdentifier().IsNone()))
			{
				TStringBuilder<128> ViewIdentifierString;
				ViewIdentifierString.Append(DetailsView->GetViewIdentifier().ToString());
				ViewIdentifierString.Append(".");
				ViewIdentifierString.Append(InPath);
				return ViewIdentifierString.ToString();
			}
		}

	}

	return InPath;
}

bool FDetailCategoryImpl::ShouldBeExpanded() const
{
	if (bHasFilterStrings)
	{
		return true;
	}
	else if (bRestoreExpansionState)
	{
		// Collapse by default if there are no simple child nodes
		bool bShouldBeExpanded = !ContainsOnlyAdvanced() && !bShouldBeInitiallyCollapsed;
		// Save the collapsed state of this section
		GConfig->GetBool(*GetDetailCategoriesExpansionSection(), *GetCategoryPathConfigKey(CategoryPathName), bShouldBeExpanded, GEditorPerProjectIni);
		return bShouldBeExpanded;
	}
	else
	{
		return !bShouldBeInitiallyCollapsed;
	}
}

ENodeVisibility FDetailCategoryImpl::GetVisibility() const
{
	return bHasVisibleDetails && bIsCategoryVisible ? 
		ENodeVisibility::Visible : ENodeVisibility::ForcedHidden;
}

static bool IsCustomProperty(const TSharedPtr<FPropertyNode>& PropertyNode)
{
	// The property node is custom if it has a custom layout or if its a struct and any of its children have a custom layout
	bool bIsCustom = !PropertyNode.IsValid() || PropertyNode->HasNodeFlags(EPropertyNodeFlags::IsCustomized) != 0;

	return bIsCustom;
}

static bool ShouldBeInlineNode(const TSharedRef<FDetailItemNode>& Node)
{
	TSharedPtr<FPropertyNode> PropertyNode = Node->GetPropertyNode();
	if (PropertyNode.IsValid())
	{
		const FProperty* Property = PropertyNode->GetProperty();
		if (Property != nullptr)
		{
			const FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property);
			const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property);
			const FByteProperty* ByteProperty = CastField<FByteProperty>(Property);

			// Only allow bools and enums as inline nodes.
			if (BoolProperty != nullptr || EnumProperty != nullptr ||
				(ByteProperty != nullptr && ByteProperty->IsEnum()))
			{
				static const FName Name_InlineCategoryProperty("InlineCategoryProperty");
				if (Property->HasMetaData(Name_InlineCategoryProperty))
				{
					return true;
				}
			}
		}
	}

	return false;
}

bool PassesInlineFilters(FDetailItemNode& Node)
{
	if (TSharedPtr<FPropertyNode> PropertyNode = Node.GetPropertyNode())
	{
		if (FPropertyNode* ParentNode = PropertyNode->GetParentNode())
		{
			if (FProperty* ParentProperty = ParentNode->GetProperty())
			{
				// If the DetailParentNode check below ends up having issues, removing it and checking ShowOnlyInnerProperties instead is safer,
				// but may not catch some edge cases with certain customizations (eg PrimaryActorTick).
				if (ParentProperty->GetClass() == FStructProperty::StaticClass()/* && ParentProperty->HasMetaData("ShowOnlyInnerProperties")*/)
				{
					if (TSharedPtr<FDetailTreeNode> DetailNodeParent = Node.GetParentNode().Pin())
					{
						// If the parent detail node's property node doesn't match the property node's parent node, there is a mismatch meaning
						// the struct's properties are inlined. This means we must check separately to see if the struct property itself
						// passes the filter before deciding whether to include its child properties.
						if (ParentNode != DetailNodeParent->GetPropertyNode().Get())
						{
							return FPropertyEditorPermissionList::Get().DoesPropertyPassFilter(FDetailTreeNode::GetPropertyNodeBaseStructure(ParentNode->GetParentNode()), ParentProperty->GetFName());
						}
					}
				}
			}
			else if (FPropertyNode* GrandparentNode = ParentNode->GetParentNode())
			{
				// VisibleAnywhere EditInline properties have to be special-cased as they don't have a proper detail node created, and instead
				// their children are inlined one level higher than they'd normally be. On top of this, object property nodes have an extra property node
				// in them before the child properties, so the grandparent node must be queried instead of the parent.
				if (FObjectPropertyNode* GrandparentAsObjectNode = GrandparentNode->AsObjectNode())
				{
					// It's not reasonable to expect an Actor to allow all possible components that could be added to it,
					// so just always allow inlined Components. This means a Component's property will always show up on
					// an Actor if the property passes the Component's filter rather than both the Component's and the Actor's.
					if (!GrandparentAsObjectNode->GetBaseStructure()->IsChildOf(UActorComponent::StaticClass()))
					{
						if (FProperty* GrandparentProperty = GrandparentAsObjectNode->GetStoredProperty())
						{
							if (GrandparentProperty->HasMetaData("EditInline"))
							{
								return FPropertyEditorPermissionList::Get().DoesPropertyPassFilter(GrandparentProperty->GetOwnerClass(), GrandparentProperty->GetFName());
							}
						}
					}
				}
			}
		}
	}
	return true;
}

void FDetailCategoryImpl::GenerateNodesFromCustomizations(const TArray<FDetailLayoutCustomization>& InCustomizationList, FDetailNodeList& OutNodeList)
{
	TAttribute<bool> IsParentEnabled(this, &FDetailCategoryImpl::IsParentEnabled);
	TArray<FDetailLayoutCustomization> CustomizationList = InCustomizationList;
	for (const FDetailLayoutCustomization& Customization : CustomizationList)
	{
		if (Customization.IsValidCustomization())
		{
			// if a property is customized, skip the default customization
			if (!IsCustomProperty(Customization.GetPropertyNode()) || Customization.bCustom || bFavoriteCategory)
			{
				TSharedRef<FDetailItemNode> NewNode = MakeShareable(new FDetailItemNode(Customization, AsShared(), IsParentEnabled));
				// Discard nodes that don't pass the property permission test. There is a special check here for properties in structs that do not have a
				// parent struct node (eg ShowOnlyInnerProperties). Both the node and it's container must pass the filter.
				if (FPropertyEditorPermissionList::Get().IsEnabled() && (NewNode->GetNodeType() == EDetailNodeType::Object || NewNode->GetNodeType() == EDetailNodeType::Item))
				{
					if (!FPropertyEditorPermissionList::Get().DoesDetailTreeNodePassFilter(NewNode->GetParentBaseStructure(), NewNode) || !PassesInlineFilters(NewNode.Get()))
					{
						continue;
					}
				}
				NewNode->Initialize();

				if (ShouldBeInlineNode(NewNode))
				{
					ensureMsgf(!InlinePropertyNode.IsValid(), TEXT("Multiple properties marked InlineCategoryProperty detected in category %s."), *DisplayName.ToString());
					InlinePropertyNode = NewNode;
					continue;
				}

				// Add the node unless only its children should be visible or it didn't generate any children or if it is a custom builder which can generate children at any point
				if (!NewNode->ShouldShowOnlyChildren() || NewNode->HasGeneratedChildren() || Customization.HasCustomBuilder())
				{
					OutNodeList.Add(NewNode);
				}
			}
		}
	}
}

void FDetailCategoryImpl::GenerateChildrenForSingleLayout(const FDetailLayout& Layout, const TArray<FDetailLayoutCustomization>& Customizations, FDetailNodeList& OutChildren)
{
	FDetailNodeList GeneratedChildren;
	GenerateNodesFromCustomizations(Customizations, GeneratedChildren);

	// If this is the favorite category, create a group and put all properties based on their original category instead of showing them all flat
	const FName InstanceName = Layout.GetInstanceName();
	if (bFavoriteCategory || LayoutMap.ShouldShowGroup(InstanceName))
	{
		TSharedRef<FDetailCategoryGroupNode> GroupNode = MakeShareable(new FDetailCategoryGroupNode(InstanceName, AsShared()));
		GroupNode->SetChildren(GeneratedChildren);
		OutChildren.Add(GroupNode);
	}
	else
	{
		OutChildren.Append(GeneratedChildren);
	}
}

void FDetailCategoryImpl::GenerateChildrenForLayouts()
{
	if (bFavoriteCategory)
	{
		// Merge advanced and simple layout customizations into simple nodes
		// Favorite category should not have advanced nodes
		for (const FDetailLayout& Layout : LayoutMap)
		{
			TArray<FDetailLayoutCustomization> Customizations;
			Customizations.Append(Layout.GetSimpleLayouts());
			Customizations.Append(Layout.GetAdvancedLayouts());
			GenerateChildrenForSingleLayout(Layout, Customizations, SimpleChildNodes);
		}
		return;
	}

	// note: these can't be ranged-for, because the map may have items added to it by customizations during iteration
	for (int32 LayoutIndex = 0; LayoutIndex < LayoutMap.Num(); ++LayoutIndex)
	{
		const FDetailLayout& Layout = LayoutMap[LayoutIndex];
		GenerateChildrenForSingleLayout(Layout, Layout.GetSimpleLayouts(), SimpleChildNodes);
	}

	for (int32 LayoutIndex = 0; LayoutIndex < LayoutMap.Num(); ++LayoutIndex)
	{
		const FDetailLayout& Layout = LayoutMap[LayoutIndex];
		GenerateChildrenForSingleLayout(Layout, Layout.GetAdvancedLayouts(), AdvancedChildNodes);
	}

	// Generate nodes for advanced dropdowns
	if (AdvancedChildNodes.Num() > 0)
	{
		TAttribute<bool> IsExpanded(this, &FDetailCategoryImpl::ShouldAdvancedBeExpanded);
		TAttribute<bool> IsEnabled(this, &FDetailCategoryImpl::IsAdvancedDropdownEnabled);
		TAttribute<bool> IsVisible(this, &FDetailCategoryImpl::ShouldAdvancedBeVisible);

		AdvancedDropdownNode = MakeShared<FAdvancedDropdownNode>(AsShared(), IsExpanded, IsEnabled, IsVisible);
	}
}

void FDetailCategoryImpl::GetChildren(FDetailNodeList& OutChildren, const bool& bInIgnoreVisibility)
{
	GetGeneratedChildren(OutChildren, bInIgnoreVisibility, false);
}

void FDetailCategoryImpl::GetGeneratedChildren(FDetailNodeList& OutChildren, bool bIgnoreVisibility, bool bIgnoreAdvanced)
{
	for (TSharedRef<FDetailTreeNode>& Child : SimpleChildNodes)
	{
		if (bIgnoreVisibility || Child->GetVisibility() == ENodeVisibility::Visible)
		{
			if (Child->ShouldShowOnlyChildren())
			{
				Child->GetChildren(OutChildren, bIgnoreVisibility);
			}
			else
			{
				OutChildren.Add(Child);
			}
		}
	}

	if (!bIgnoreAdvanced)
	{
		if (AdvancedChildNodes.Num() > 0 && AdvancedDropdownNode.IsValid())
		{
			OutChildren.Add(AdvancedDropdownNode.ToSharedRef());
		}

		// bIgnoreVisibility treats advanced as expanded
		if (bIgnoreVisibility || ShouldAdvancedBeExpanded())
		{
			for (TSharedRef<FDetailTreeNode>& Child : AdvancedChildNodes)
			{
				if (bIgnoreVisibility || Child->GetVisibility() == ENodeVisibility::Visible)
				{
					if (Child->ShouldShowOnlyChildren())
					{
						Child->GetChildren(OutChildren, bIgnoreVisibility);
					}
					else
					{
						OutChildren.Add(Child);
					}
				}
			}
		}
	}
}

void FDetailCategoryImpl::FilterNode(const FDetailFilter& InFilter)
{
	bHasFilterStrings = InFilter.FilterStrings.Num() > 0;
	bForceAdvanced = bFavoriteCategory || InFilter.bShowAllAdvanced == true || bHasFilterStrings || ContainsOnlyAdvanced();

	bHasVisibleDetails = false;
	bHasVisibleAdvanced = false;

	if (bFavoriteCategory && !InFilter.bShowFavoritesCategory)
	{
		return;
	}

	// only apply the section filter if the user hasn't typed anything and this isn't the favorites category
	if (InFilter.FilterStrings.IsEmpty() && !InFilter.VisibleSections.IsEmpty() && !bFavoriteCategory)
	{
		const UStruct* BaseStruct = GetParentBaseStructure();
		if (BaseStruct != nullptr)
		{
			static FName PropertyEditor("PropertyEditor");
			const FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);

			TArray<TSharedPtr<FPropertySection>> PropertySections = PropertyModule.FindSectionsForCategory(BaseStruct, CategoryName);
			if (PropertySections.IsEmpty())
			{
				// property has no sections, must be filtered
				return;
			}

			// if this property is not in any visible section, hide it
			bool bFound = false;
			for (const TSharedPtr<FPropertySection>& Section : PropertySections)
			{
				if (Section->HasAddedCategory(CategoryName) &&
					InFilter.VisibleSections.Contains(Section->GetName()))
				{
					bFound = true; 
					break;
				}
			}

			if (!bFound)
			{
				return;
			}
		}
	}

	if (InlinePropertyNode.IsValid())
	{
		bHasVisibleDetails = true;
	}

	for (TSharedRef<FDetailTreeNode>& Child : SimpleChildNodes)
	{
		Child->FilterNode(InFilter);

		if (Child->GetVisibility() == ENodeVisibility::Visible)
		{
			bHasVisibleDetails = true;
			RequestItemExpanded(Child, Child->ShouldBeExpanded());
		}
	}

	for (TSharedRef<FDetailTreeNode>& Child : AdvancedChildNodes)
	{
		Child->FilterNode(InFilter);

		if (Child->GetVisibility() == ENodeVisibility::Visible)
		{
			bHasVisibleDetails = true;
			bHasVisibleAdvanced = true;
			RequestItemExpanded(Child, Child->ShouldBeExpanded());
		}
	}
}

FCustomPropertyTypeLayoutMap FDetailCategoryImpl::GetCustomPropertyTypeLayoutMap() const
{
	TSharedPtr<FDetailLayoutBuilderImpl> ParentLayout = GetParentLayoutImpl();
	if (ParentLayout.IsValid())
	{
		return ParentLayout->GetInstancedPropertyTypeLayoutMap();
	}

	return FCustomPropertyTypeLayoutMap();
}

void FDetailCategoryImpl::GenerateLayout()
{
	// Reset all children
	SimpleChildNodes.Empty();
	AdvancedChildNodes.Empty();
	AdvancedDropdownNode.Reset();
	InlinePropertyNode.Reset();

	GenerateChildrenForLayouts();

	bHasVisibleDetails = SimpleChildNodes.Num() + AdvancedChildNodes.Num() > 0;
}

bool FDetailCategoryImpl::IsParentEnabled() const
{
	TSharedPtr<IDetailsViewPrivate> DetailsView = GetDetailsViewSharedPtr();
	return !DetailsView || DetailsView->IsPropertyEditingEnabled();
}

bool FDetailLayoutMap::ShouldShowGroup(FName RequiredGroupName) const
{
	auto HasInnerPropertiesOnly = [](const TArray<FDetailLayoutCustomization>& CustomizationLayouts) -> bool
		{
			static const FName Name_ShowOnlyInnerProperties("ShowOnlyInnerProperties");

			for (const FDetailLayoutCustomization& Customization : CustomizationLayouts)
			{
				if (const TSharedPtr<FPropertyNode> PropertyNode = Customization.GetPropertyNode())
				{
					if (const TSharedPtr<FPropertyNode> ParentNode = PropertyNode->GetParentNodeSharedPtr())
					{
						if (const FProperty* ParentNodeProperty = ParentNode->GetProperty())
						{
							if (ParentNodeProperty->HasMetaData(Name_ShowOnlyInnerProperties))
							{
								return true;
							}
						}
					}
				}
			}
			return false;
		};

	bool bInnerPropertiesOnly = false;

	if (const FDetailLayout* FoundLayout = Layouts.FindByPredicate([RequiredGroupName](const FDetailLayout& Layout) { return Layout.GetInstanceName() == RequiredGroupName; }))
	{
		bInnerPropertiesOnly = HasInnerPropertiesOnly(FoundLayout->GetSimpleLayouts()) || HasInnerPropertiesOnly(FoundLayout->GetAdvancedLayouts());
	}

	// Should show the group if the group name is not empty and there are more than two entries in the list where one of them is not the default "none" entry (represents the base object)
	return !bInnerPropertiesOnly && RequiredGroupName != NAME_None && Layouts.Num() > 1 && (Layouts.Num() > 2 || !bContainsBaseInstance);
}
