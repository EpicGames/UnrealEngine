// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "IPropertyUtilities.h"
#include "DetailTreeNode.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "PropertyNode.h"
#include "UObject/StructOnScope.h"
#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
#include "DetailCustomBuilderRow.h"
#include "DetailLayoutBuilderImpl.h"
#include "IDetailCustomNodeBuilder.h"
#include "DetailCategoryBuilder.h"
#include "PropertyCustomizationHelpers.h"

class IDetailGroup;
class FDetailGroup;
class IDetailPropertyRow;
class FDetailPropertyRow;

/**
 * Defines a customization for a specific detail
 */
struct FDetailLayoutCustomization
{
	FDetailLayoutCustomization();
	/** The property node for the property detail */
	TSharedPtr<FDetailPropertyRow> PropertyRow;
	/** A group of customizations */
	TSharedPtr<FDetailGroup> DetailGroup;
	/** Custom Widget for displaying the detail */
	TSharedPtr<FDetailWidgetRow> WidgetDecl;
	/** Custom builder for more complicated widgets */
	TSharedPtr<FDetailCustomBuilderRow> CustomBuilderRow;
	/** @return true if this customization has a property node */
	bool HasPropertyNode() const { return GetPropertyNode().IsValid(); }
	/** @return true if this customization has a custom widget */
	bool HasCustomWidget() const { return WidgetDecl.IsValid(); }
	/** @return true if this customization has a custom builder (custom builders will set the custom widget) */
	bool HasCustomBuilder() const { return CustomBuilderRow.IsValid(); }
	/** @return true if this customization has a group */
	bool HasGroup() const { return DetailGroup.IsValid(); }
	/** @return true if this has a customization for an external property row */
	bool HasExternalPropertyRow() const;
	/** @return true if this customization is valid */
	bool IsValidCustomization() const { return HasPropertyNode() || HasCustomWidget() || HasCustomBuilder() || HasGroup(); }
	/** @return true if the customized item is hidden */
	bool IsHidden() const;
	/** @return the property node for this customization (if any ) */
	TSharedPtr<FPropertyNode> GetPropertyNode() const;
	/** @return The row to display from this customization */
	FDetailWidgetRow GetWidgetRow() const;
	/** @return properties being customized */
	TArrayView<TSharedPtr<IPropertyHandle>> GetPropertyHandles() const;
	/** @return the filter text associated with this row, if any */
	FText GetFilterTextString() const;
	/** Whether or not this customization is considered an advanced property. */
	bool bAdvanced { false };
	/** Whether or not this customization is custom or a default one. */
	bool bCustom { false };
	/** @return The current layout row that this customization represents. May be null if IsValidCustomization() is false. */
	const IDetailLayoutRow* GetDetailLayoutRow() const;
	/** @return The name of the row depending on which type of customization this is, then the name of the property node, then NAME_None. */
	FName GetName() const;
	/** @return The custom reset to default if one is available. */
	TOptional<FResetToDefaultOverride> GetCustomResetToDefault() const;
};

class FDetailLayout
{
public:
	FDetailLayout(FName InInstanceName)
		: InstanceName(InInstanceName)
	{}

	void AddLayout(const FDetailLayoutCustomization& Layout);

	const TArray<FDetailLayoutCustomization>& GetSimpleLayouts() const { return SimpleLayouts; }
	const TArray<FDetailLayoutCustomization>& GetAdvancedLayouts() const { return AdvancedLayouts; }

	FDetailLayoutCustomization* GetDefaultLayout(const TSharedRef<FPropertyNode>& PropertyNode);

	bool HasAdvancedLayouts() const { return AdvancedLayouts.Num() > 0; }

	/**
	 * Get the instance name - this is usually the UObject's name when in a multi-selection.
	 * Used to display a group beneath the category if multiple objects share some of the same-named properties.
	 */
	FName GetInstanceName() const { return InstanceName; }

private:
	/** Layouts that appear in the simple (visible by default) area of a category */
	TArray<FDetailLayoutCustomization> SimpleLayouts;
	/** Layouts that appear in the advanced (hidden by default) details area of a category */
	TArray<FDetailLayoutCustomization> AdvancedLayouts;
	/** The sort order in which this layout is displayed (lower numbers are displayed first) */
	FName InstanceName;
};


class FDetailLayoutMap
{
public:
	FDetailLayoutMap()
		: bContainsBaseInstance(false)
	{}

	FDetailLayout& FindOrAdd(FName InstanceName)
	{
		for (int32 LayoutIndex = 0; LayoutIndex < Layouts.Num(); ++LayoutIndex)
		{
			FDetailLayout& Layout = Layouts[LayoutIndex];
			if (Layout.GetInstanceName() == InstanceName)
			{
				return Layout;
			}
		}

		bContainsBaseInstance |= (InstanceName == NAME_None);

		int32 Index = Layouts.Add(FDetailLayout(InstanceName));

		return Layouts[Index];
	}

	using RangedForIteratorType = TArray<FDetailLayout>::RangedForIteratorType;
	using RangedForConstIteratorType = TArray<FDetailLayout>::RangedForConstIteratorType;

	FORCEINLINE RangedForIteratorType      begin()       { return Layouts.begin(); }
	FORCEINLINE RangedForConstIteratorType begin() const { return Layouts.begin(); }
	FORCEINLINE RangedForIteratorType      end()         { return Layouts.end(); }
	FORCEINLINE RangedForConstIteratorType end()   const { return Layouts.end(); }

	FDetailLayout& operator[](int32 Index) { return Layouts[Index]; }
	const FDetailLayout& operator[](int32 Index) const { return Layouts[Index]; }

	int32 Num() const { return Layouts.Num(); }

	/**
	 * @return Whether or not we need to display a group border around a list of details.
	 */
	bool ShouldShowGroup(FName RequiredGroupName) const;

private:
	TArray<FDetailLayout> Layouts;
	bool bContainsBaseInstance;
};


/**
 * Detail category implementation
 */
class FDetailCategoryImpl : public IDetailCategoryBuilder, public FDetailTreeNode, public TSharedFromThis<FDetailCategoryImpl>
{
public:

	static FString DetailCategories;
	static FString TransientPerViewDetailCategories;
	static FString PersistentPerViewDetailCategories;

public:
	FDetailCategoryImpl(FName InCategoryName, TSharedRef<FDetailLayoutBuilderImpl> InDetailLayout);
	~FDetailCategoryImpl();

	/** IDetailCategoryBuilder interface */
	virtual IDetailCategoryBuilder& InitiallyCollapsed(bool bShouldBeInitiallyCollapsed) override;
	virtual IDetailCategoryBuilder& OnExpansionChanged(FOnBooleanValueChanged InOnExpansionChanged) override;
	virtual IDetailCategoryBuilder& RestoreExpansionState(bool bRestore) override;
	virtual IDetailCategoryBuilder& HeaderContent(TSharedRef<SWidget> InHeaderContent, bool bWholeRowContent = false) override;
	virtual IDetailPropertyRow& AddProperty(FName PropertyPath, UClass* ClassOuter = nullptr, FName InstanceName = NAME_None, EPropertyLocation::Type Location = EPropertyLocation::Default) override;
	virtual IDetailPropertyRow& AddProperty(TSharedPtr<IPropertyHandle> PropertyHandle, EPropertyLocation::Type Location = EPropertyLocation::Default) override;
	virtual IDetailPropertyRow* AddExternalObjects(const TArray<UObject*>& Objects, EPropertyLocation::Type Location = EPropertyLocation::Default, const FAddPropertyParams& Params = FAddPropertyParams()) override;
	virtual IDetailPropertyRow* AddExternalObjectProperty(const TArray<UObject*>& Objects, FName PropertyName, EPropertyLocation::Type Location = EPropertyLocation::Default, const FAddPropertyParams& Params = FAddPropertyParams()) override;
	virtual IDetailPropertyRow* AddExternalStructure(TSharedPtr<FStructOnScope> StructData, EPropertyLocation::Type Location = EPropertyLocation::Default) override;
	virtual IDetailPropertyRow* AddExternalStructureProperty(TSharedPtr<FStructOnScope> StructData, FName PropertyName, EPropertyLocation::Type Location = EPropertyLocation::Default, const FAddPropertyParams& Params = FAddPropertyParams()) override;
	virtual IDetailPropertyRow* AddExternalStructureProperty(TSharedPtr<IStructureDataProvider> StructData, FName PropertyName, EPropertyLocation::Type Location = EPropertyLocation::Default, const FAddPropertyParams& Params = FAddPropertyParams()) override;
	virtual TArray<TSharedPtr<IPropertyHandle>> AddAllExternalStructureProperties(TSharedRef<FStructOnScope> StructData, EPropertyLocation::Type Location = EPropertyLocation::Default, TArray<IDetailPropertyRow*>* OutPropertiesRow = nullptr) override;
	virtual TArray<TSharedPtr<IPropertyHandle>> AddAllExternalStructureProperties(TSharedPtr<IStructureDataProvider> StructProvider, EPropertyLocation::Type Location = EPropertyLocation::Default, TArray<IDetailPropertyRow*>* OutPropertiesRow = nullptr) override;
	virtual bool IsParentLayoutValid() const override { return DetailLayoutBuilder.IsValid(); }
	virtual IDetailLayoutBuilder& GetParentLayout() const override { return *DetailLayoutBuilder.Pin(); }
	virtual FDetailWidgetRow& AddCustomRow(const FText& FilterString, bool bForAdvanced = false) override;
	virtual void AddCustomBuilder(TSharedRef<IDetailCustomNodeBuilder> InCustomBuilder, bool bForAdvanced = false) override;
	virtual IDetailGroup& AddGroup(FName GroupName, const FText& LocalizedDisplayName, bool bForAdvanced = false, bool bStartExpanded = false) override;
	virtual void GetDefaultProperties(TArray<TSharedRef<IPropertyHandle> >& OutAllProperties, bool bSimpleProperties = true, bool bAdvancedProperties = true) override;
	virtual const FText& GetDisplayName() const override { return DisplayName; }
	virtual void SetDisplayName(const FText& InDisplayName) override;
	virtual const TOptional<FText>& GetToolTip() const override;
	virtual void SetToolTip(const FText& ToolTip) override;
	virtual void SetCategoryVisibility(bool bIsVisible) override;
	virtual void SetShowAdvanced(bool bShowAdvanced) override;
	virtual int32 GetSortOrder() const override;
	virtual void SetSortOrder(int32 InSortOrder) override;
	virtual void AddPropertyDisableInstancedReference(TSharedPtr<IPropertyHandle> PropertyHandle) override;
	
	/** Delegate handling pasting an optionally tagged text snippet */
	virtual TSharedPtr<FOnPasteFromText> OnPasteFromText() const override { return PasteFromTextDelegate; }

	/** Delegate handling copying a property handle to text. The default property handle impl for copying will be used if the delegate is not bound
	 * or returns false for a property */
	virtual TSharedPtr<FOnCopyToText> OnCopyToText() const override { return CopyToTextDelegate; }

	/** FDetailTreeNode interface */
	virtual TSharedPtr<IDetailsView> GetNodeDetailsViewSharedPtr() const override { return GetDetailsViewSharedPtr(); }
	virtual TSharedPtr<IDetailsViewPrivate> GetDetailsViewSharedPtr() const override;
	virtual TSharedRef< ITableRow > GenerateWidgetForTableView(const TSharedRef<STableViewBase>& OwnerTable, bool bAllowFavoriteSystem) override;
	virtual bool GenerateStandaloneWidget(FDetailWidgetRow& OutRow) const override;

	/** IDetailTreeNode interface */
	virtual EDetailNodeType GetNodeType() const override { return EDetailNodeType::Category; }
	virtual TSharedPtr<IPropertyHandle> CreatePropertyHandle() const override { return nullptr; }
	virtual void GetFilterStrings(TArray<FString>& OutFilterStrings) const override;
	virtual bool GetInitiallyCollapsed() const override;

	virtual void GetChildren(FDetailNodeList& OutChildren, const bool& bInIgnoreVisibility = false) override;
	virtual bool ShouldBeExpanded() const override;
	virtual ENodeVisibility GetVisibility() const override;
	virtual void FilterNode(const FDetailFilter& DetailFilter) override;
	virtual void Tick(float DeltaTime) override;
	virtual bool ShouldShowOnlyChildren() const override { return bShowOnlyChildren; }
	virtual FName GetNodeName() const override { return GetCategoryName(); }

	/**
	 * Gets all generated children with options for ignoring current child visibility or advanced dropdowns
	 */
	void GetGeneratedChildren(FDetailNodeList& OutChildren, bool bIgnoreVisibility, bool bIgnoreAdvancedDropdown);

	FCustomPropertyTypeLayoutMap GetCustomPropertyTypeLayoutMap() const;

	/**
	 * @return The name of the category
	 */
	FName GetCategoryName() const { return CategoryName; }

	/**
	 * @return The parent detail layout builder for this category
	 */
	TSharedPtr<FDetailLayoutBuilderImpl> GetParentLayoutImpl() const { return DetailLayoutBuilder.Pin(); }

	/**
	 * Generates the children for this category
	 */
	void GenerateLayout();

	/**
	 * Adds a property node to the default category layout
	 *
	 * @param PropertyNode			The property node to add
	 * @param InstanceName			The name of the property instance (for duplicate properties of the same type)
	 */
	void AddPropertyNode(TSharedRef<FPropertyNode> PropertyNode, FName InstanceName);

	/**
	 * Sets the display name of the category string
	 *
	 * @param CategoryName			Base category name to use if no localized override is set
	 * @param LocalizedNameOverride	The localized name override to use (can be empty)
	 */
	void SetDisplayName(FName CategoryName, const FText& LocalizedNameOverride);

	/**
	 * Request that a child node of this category be expanded or collapsed
	 *
	 * @param TreeNode				The node to expand or collapse
	 * @param bShouldBeExpanded		True if the node should be expanded, false to collapse it
	 */
	void RequestItemExpanded(TSharedRef<FDetailTreeNode> TreeNode, bool bShouldBeExpanded);

	/**
	 * Notifies the tree view that it needs to be refreshed
	 *
	 * @param bRefilterCategory True if the category should be refiltered
	 */
	void RefreshTree(bool bRefilterCategory);

	/**
	 * Adds a node that needs to be ticked
	 *
	 * @param TickableNode	The node that needs to be ticked
	 */
	void AddTickableNode(FDetailTreeNode& TickableNode);

	/**
	 * Removes a node that no longer needs to be ticked
	 *
	 * @param TickableNode	The node that no longer needs to be ticked
	 */
	void RemoveTickableNode(FDetailTreeNode& TickableNode);

	/** @return The category path for this category */
	const FString& GetCategoryPathName() const { return CategoryPathName; }

	/**
	 * Saves the expansion state of a tree node in this category
	 *
	 * @param InTreeNode	The node to save expansion state from
	 */
	void SaveExpansionState(FDetailTreeNode& InTreeNode);

	/**
	 * Gets the saved expansion state of a tree node in this category
	 *
	 * @param InTreeNode	The node to get expansion state for
	 * @return true if the node should be expanded, false otherwise
	 */
	bool GetSavedExpansionState(FDetailTreeNode& InTreeNode) const;

	/** @return true if this category only contains advanced properties */
	bool ContainsOnlyAdvanced() const;

	/**
	 * Get the number of customizations in this category.
	 */
	int32 GetNumCustomizations() const;

	/**
	 * Called when the advanced dropdown button is clicked
	 */
	void OnAdvancedDropdownClicked();

	/*
	 * Call this function to make the category behave like favorite category
	 */
	void SetCategoryAsSpecialFavorite() { bFavoriteCategory = true; bForceAdvanced = true; }

	/** Is this the Favorites category? */
	bool IsFavoriteCategory() const { return bFavoriteCategory; }

	/** Is this category initially collapsed? */
	bool GetShouldBeInitiallyCollapsed() const { return bShouldBeInitiallyCollapsed; }

	FDetailLayoutCustomization* GetDefaultCustomization(TSharedRef<FPropertyNode> PropertyNode);
	
	/**
	 * If true, this Category should have no UProperty data associated with it, and will be shown as an empty stub
	 * with no expansion arrow
	 */
	virtual bool IsEmpty() const override;
	
	/**
	 * Sets whether this Category is "Empty" ~ that is, should have no UProperty data associated with it, and will be shown
	 * as an empty stub with no expansion arrow
	 */
	virtual void SetIsEmpty(bool bInIsEmpty) override;

	virtual TOptional<FLinearColor> GetColor() const override;

	virtual void SetColor(FLinearColor InColor) override;

	virtual void ClearColor() override;


private:
	virtual void OnItemExpansionChanged(bool bIsExpanded, bool bShouldSaveState) override;

	/** Return string key for the given struct in config .ini for this details view's categories */
	const FString& GetDetailCategoriesExpansionSection() const;

	/** Return string key for the given struct in config .ini for this details view */
	FString GetCategoryPathConfigKey(const FString& InPath) const;

	/**
	 * Generates children for all layouts.
	 */
	void GenerateChildrenForLayouts();

	/**
	 * Generate children for the single given layout and append them to OutChildren.
	 */
	void GenerateChildrenForSingleLayout(const FDetailLayout& Layout, const TArray<FDetailLayoutCustomization>& Customizations, FDetailNodeList& OutChildren);

	/**
	 * Generates nodes from a list of customization in a single layout
	 *
	 * @param InCustomizationList	The list of customizations to generate nodes from
	 * @param OutNodeList			The generated nodes
	 */
	void GenerateNodesFromCustomizations(const TArray<FDetailLayoutCustomization>& InCustomizationList, FDetailNodeList& OutNodeList);

	/**
	 * @return Whether or not a customization should appear in the advanced section of the category by default
	 */
	bool IsAdvancedLayout(const FDetailLayoutCustomization& LayoutInfo);

	/**
	 * Adds a custom layout to this category
	 *
	 * @param LayoutInfo	The custom layout information
	 * @param bForAdvanced	Whether or not the custom layout should appear in the advanced section of the category
	 */
	void AddCustomLayout(const FDetailLayoutCustomization& LayoutInfo);

	/**
	 * Adds a default layout to this category
	 *
	 * @param DefaultLayoutInfo		The layout information
	 * @param bForAdvanced			Whether or not the layout should appear in the advanced section of the category
	 */
	void AddDefaultLayout(const FDetailLayoutCustomization& DefaultLayoutInfo, FName InstanceName);

	/**
	 * Returns the layout for a given object instance name
	 *
	 * @param InstanceName the name of the instance to get the layout for
	 */
	FDetailLayout& GetLayoutForInstance(FName InstanceName);

	/**
	 * @return True of we should show the advanced button
	 */
	bool ShouldAdvancedBeExpanded() const;

	/**
	 * @return true if the advaned dropdown button is enabled
	 */
	bool IsAdvancedDropdownEnabled() const;

	/**
	 * @return the visibility of the advanced help text drop down (it is visible in a category if there are no simple properties)
	 */
	bool ShouldAdvancedBeVisible() const;

	/**
	 * @return true if the parent that hosts us is enabled
	 */
	bool IsParentEnabled() const;

	/**
	 * Does the work of refreshing the tree (this is triggered from RefreshTree)
	 *
	 * @param bRefilterCategory True if the category should be refiltered
	 */
	void RefreshTreeInternal(bool bRefilterCategory);

private:
	/** Layouts that appear in this category category */
	FDetailLayoutMap LayoutMap;
	/** All Simple child nodes */
	FDetailNodeList SimpleChildNodes;
	/** All Advanced child nodes */
	FDetailNodeList AdvancedChildNodes;
	/** Advanced dropdown node. */
	TSharedPtr<FDetailTreeNode> AdvancedDropdownNode;
	/** Delegate called when expansion of the category changes */
	FOnBooleanValueChanged OnExpansionChangedDelegate;
	/** The display name of the category */
	FText DisplayName;
	/** ToolTip for this category */
    TOptional<FText> ToolTip;
	/** The path name of the category */
	FString CategoryPathName;
	/** Custom header content displayed to the right of the category name */
	TSharedPtr<SWidget> HeaderContentWidget;
	/** True if the HeaderContentWidget should span the whole row (hides the category label)  */
	bool bHeaderContentWholeRowContent = false;
	/** A property node that is displayed in the header row to the right of the category name. */
	TSharedPtr<FDetailTreeNode> InlinePropertyNode;
	/** The parent detail builder */
	TWeakPtr<FDetailLayoutBuilderImpl> DetailLayoutBuilder;
	/** Delegate handling pasting an optionally tagged text snippet */
	TSharedPtr<FOnPasteFromText> PasteFromTextDelegate;
	/** Delegate handling copying a property to text. The default property handle copy will be used if not provided */
	TSharedPtr<FOnCopyToText> CopyToTextDelegate;
	
	/** The category identifier */
	FName CategoryName;
	/** The sort order of this category (amongst all categories) */
	int32 SortOrder;
	/** The color that represents the category. */
	TOptional<FLinearColor> Color;
	/** Whether or not to restore the expansion state between sessions */
	bool bRestoreExpansionState : 1;
	/** Whether or not the category should be initially collapsed */
	bool bShouldBeInitiallyCollapsed : 1;
	/** Whether or not advanced properties should be shown (as specified by the user) */
	bool bUserShowAdvanced : 1;
	/** Whether or not advanced properties are forced to be shown (this is an independent toggle from bShowAdvanced which is user driven)*/
	bool bForceAdvanced : 1;
	/** Whether or not the content in the category is being filtered */
	bool bHasFilterStrings : 1;
	/** true if anything is visible in the category */
	bool bHasVisibleDetails : 1;
	/** true if the category is visible at all */
	bool bIsCategoryVisible : 1;
	/*true if this category is the special favorite category, all property in the layout will be display when we generate the root tree */
	bool bFavoriteCategory : 1;
	bool bShowOnlyChildren : 1;
	bool bHasVisibleAdvanced : 1;
	bool bPendingRefresh : 1;
	bool bPendingRefreshNeedsRefilter : 1;

	/**
	* If true, this Category should have no UProperty data associated with it, and will be shown as an empty stub
	* with no expansion arrow
	*/
	bool bIsEmpty 	: 1;
};
