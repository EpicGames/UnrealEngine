// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailCategoryGroupNode.h"
#include "PropertyCustomizationHelpers.h"
#include "SDetailCategoryTableRow.h"

FDetailCategoryGroupNode::FDetailCategoryGroupNode(FName InGroupName, TSharedRef<FDetailCategoryImpl> InParentCategory)
	: ParentCategory(InParentCategory.Get())
	, GroupName( InGroupName )
	, bShouldBeVisible( false )
	, bShowBorder(true)
	, bHasSplitter(false)
{
	SetParentNode(InParentCategory);
}

void FDetailCategoryGroupNode::SetChildren(const FDetailNodeList& InChildNodes)
{
	ChildNodes = InChildNodes;
	for (TSharedRef<FDetailTreeNode> Child : ChildNodes)
	{
		Child->SetParentNode(AsShared());
	}
}

TSharedRef< ITableRow > FDetailCategoryGroupNode::GenerateWidgetForTableView( const TSharedRef<STableViewBase>& OwnerTable, bool bAllowFavoriteSystem)
{
	return SNew( SDetailCategoryTableRow, AsShared(), OwnerTable )
		.DisplayName( FText::FromName(GroupName) )
		.InnerCategory( true )
		.ShowBorder( bShowBorder );
}

bool FDetailCategoryGroupNode::GenerateStandaloneWidget(FDetailWidgetRow& OutRow) const
{
	OutRow.NameContent()
	[
		SNew(STextBlock)
		.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
		.Text(FText::FromName(GroupName))
	];

	return true;
}

void FDetailCategoryGroupNode::GetChildren(FDetailNodeList& OutChildren, const bool& bInIgnoreVisibility)
{
	for( int32 ChildIndex = 0; ChildIndex < ChildNodes.Num(); ++ChildIndex )
	{
		TSharedRef<FDetailTreeNode>& Child = ChildNodes[ChildIndex];
		if( Child->GetVisibility() == ENodeVisibility::Visible )
		{
			if( Child->ShouldShowOnlyChildren() )
			{
				Child->GetChildren( OutChildren, bInIgnoreVisibility );
			}
			else
			{
				OutChildren.Add( Child );
			}
		}
	}
}

void FDetailCategoryGroupNode::FilterNode( const FDetailFilter& InFilter )
{
	bShouldBeVisible = false;
	for( int32 ChildIndex = 0; ChildIndex < ChildNodes.Num(); ++ChildIndex )
	{
		TSharedRef<FDetailTreeNode>& Child = ChildNodes[ChildIndex];

		Child->FilterNode( InFilter );

		if( Child->GetVisibility() == ENodeVisibility::Visible )
		{
			bShouldBeVisible = true;

			ParentCategory.RequestItemExpanded( Child, Child->ShouldBeExpanded() );
		}
	}
}
