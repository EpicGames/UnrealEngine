// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailCategoryBuilderImpl.h"
#include "DetailTreeNode.h"
#include "IDetailsView.h"
#include "IPropertyUtilities.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "UObject/NameTypes.h"

class FDetailCategoryGroupNode : public FDetailTreeNode, public TSharedFromThis<FDetailCategoryGroupNode>
{
public:
	FDetailCategoryGroupNode(FName InGroupName, TSharedRef<FDetailCategoryImpl> InParentCategory);

public:
	void SetChildren(const FDetailNodeList& InChildNodes);

	void SetShowBorder(bool bInShowBorder) { bShowBorder = bInShowBorder; }
	bool GetShowBorder() const { return bShowBorder; }

	void SetHasSplitter(bool bInHasSplitter) { bHasSplitter = bInHasSplitter; }
	bool GetHasSplitter() const { return bHasSplitter; }

private:
	virtual TSharedPtr<IDetailsView> GetNodeDetailsViewSharedPtr() const override { return ParentCategory.GetNodeDetailsViewSharedPtr(); }
	virtual TSharedPtr<IDetailsViewPrivate> GetDetailsViewSharedPtr() const override { return ParentCategory.GetDetailsViewSharedPtr(); }
	virtual void OnItemExpansionChanged( bool bIsExpanded, bool bShouldSaveState) override {}
	virtual bool ShouldBeExpanded() const override { return true; }
	virtual ENodeVisibility GetVisibility() const override { return bShouldBeVisible ? ENodeVisibility::Visible : ENodeVisibility::HiddenDueToFiltering; }
	virtual TSharedRef< ITableRow > GenerateWidgetForTableView( const TSharedRef<STableViewBase>& OwnerTable, bool bAllowFavoriteSystem) override;
	virtual bool GenerateStandaloneWidget(FDetailWidgetRow& OutRow) const override;

	virtual EDetailNodeType GetNodeType() const override { return EDetailNodeType::Category; }
	virtual TSharedPtr<IPropertyHandle> CreatePropertyHandle() const override { return nullptr; }
	virtual void GetFilterStrings(TArray<FString>& OutFilterStrings) const override { OutFilterStrings.Add(GroupName.ToString()); };

	virtual void GetChildren(FDetailNodeList& OutChildren, const bool& bInIgnoreVisibility) override;
	virtual void FilterNode( const FDetailFilter& InFilter ) override;
	virtual void Tick( float DeltaTime ) override {}
	virtual bool ShouldShowOnlyChildren() const override { return false; }
	virtual FName GetNodeName() const override { return GroupName; }

private:
	FDetailNodeList ChildNodes;
	FDetailCategoryImpl& ParentCategory;
	FName GroupName;
	bool bShouldBeVisible;

	bool bShowBorder;
	bool bHasSplitter;
};
