// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailCustomBuilderRow.h"
#include "IDetailCustomNodeBuilder.h"
#include "DetailCategoryBuilderImpl.h"
#include "DetailItemNode.h"
#include "CustomChildBuilder.h"

FDetailCustomBuilderRow::FDetailCustomBuilderRow( TSharedRef<IDetailCustomNodeBuilder> CustomBuilder )
	: CustomNodeBuilder( CustomBuilder )
{
}

TOptional<FResetToDefaultOverride> FDetailCustomBuilderRow::GetCustomResetToDefault() const
{
	if (HeaderRow.IsValid())
	{
		return HeaderRow->GetCustomResetToDefault();
	}
	return TOptional<FResetToDefaultOverride>();
}

void FDetailCustomBuilderRow::Tick( float DeltaTime ) 
{
	return CustomNodeBuilder->Tick( DeltaTime );
}

bool FDetailCustomBuilderRow::RequiresTick() const
{
	return CustomNodeBuilder->RequiresTick();
}

bool FDetailCustomBuilderRow::HasColumns() const
{
	return HeaderRow->HasColumns();
}

bool FDetailCustomBuilderRow::ShowOnlyChildren() const
{
	return !HeaderRow->HasAnyContent();
}

void FDetailCustomBuilderRow::OnItemNodeInitialized( TSharedRef<FDetailItemNode> InTreeNode, TSharedRef<FDetailCategoryImpl> InParentCategory, const TAttribute<bool>& InIsParentEnabled )
{
	ParentCategory = InParentCategory;
	IsParentEnabled = InIsParentEnabled;

	const bool bUpdateFilteredNodes = true;

	// Set a delegate on the interface that it will call to rebuild this nodes children
	CustomNodeBuilder->SetOnRebuildChildren(FSimpleDelegate::CreateSP(InTreeNode, &FDetailItemNode::GenerateChildren, bUpdateFilteredNodes));

	CustomNodeBuilder->SetOnToggleExpansion(FOnToggleNodeExpansion::CreateSP(InTreeNode, &FDetailItemNode::SetExpansionState));

	HeaderRow = MakeShared<FDetailWidgetRow>();

	CustomNodeBuilder->GenerateHeaderRowContent(*HeaderRow);
}

FName FDetailCustomBuilderRow::GetCustomBuilderName() const
{
	return CustomNodeBuilder->GetName();
}

TSharedPtr<IPropertyHandle> FDetailCustomBuilderRow::GetPropertyHandle() const
{
	return CustomNodeBuilder->GetPropertyHandle();
}

void FDetailCustomBuilderRow::OnGenerateChildren( FDetailNodeList& OutChildren )
{
	ChildrenBuilder = MakeShared<FCustomChildrenBuilder>(ParentCategory.Pin().ToSharedRef());

	CustomNodeBuilder->GenerateChildContent( *ChildrenBuilder );
		
	const TArray< FDetailLayoutCustomization >& ChildRows = ChildrenBuilder->GetChildCustomizations();

	for( int32 ChildIndex = 0; ChildIndex < ChildRows.Num(); ++ChildIndex )
	{
		TSharedRef<FDetailItemNode> ChildNodeItem = MakeShareable( new FDetailItemNode( ChildRows[ChildIndex], ParentCategory.Pin().ToSharedRef(), IsParentEnabled ) );
		ChildNodeItem->Initialize();
		OutChildren.Add( ChildNodeItem );
	}
}

bool FDetailCustomBuilderRow::IsInitiallyCollapsed() const
{
	return CustomNodeBuilder->InitiallyCollapsed();
}

TSharedPtr<FDetailWidgetRow> FDetailCustomBuilderRow::GetWidgetRow() const
{
	return HeaderRow;
}

bool FDetailCustomBuilderRow::AreChildCustomizationsHidden() const
{
	if (ChildrenBuilder && ChildrenBuilder->GetChildCustomizations().Num() > 0)
	{
		for (const FDetailLayoutCustomization& ChildCustomizations : ChildrenBuilder->GetChildCustomizations())
		{
			if (!ChildCustomizations.IsHidden())
			{
				return false;
			}
		}
		return true;
	}
	return false;
}