// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "DetailWidgetRow.h"
#include "SDetailsViewBase.h"
#include "DetailCategoryBuilder.h"

class FCustomChildrenBuilder;
class FDetailCategoryImpl;
class FDetailItemNode;
class IDetailCustomNodeBuilder;

class FDetailCustomBuilderRow : public IDetailLayoutRow, public TSharedFromThis<FDetailCustomBuilderRow>
{
public:
	FDetailCustomBuilderRow( TSharedRef<IDetailCustomNodeBuilder> CustomBuilder );
	virtual ~FDetailCustomBuilderRow() = default;

	/** IDetailLayoutRow interface */
	virtual FName GetRowName() const override { return GetCustomBuilderName(); }
	virtual TOptional<FResetToDefaultOverride> GetCustomResetToDefault() const override;

	void Tick( float DeltaTime );
	bool RequiresTick() const;
	bool HasColumns() const;
	bool ShowOnlyChildren() const;
	void OnItemNodeInitialized( TSharedRef<FDetailItemNode> InTreeNode, TSharedRef<FDetailCategoryImpl> InParentCategory, const TAttribute<bool>& InIsParentEnabled );
	TSharedRef<IDetailCustomNodeBuilder> GetCustomBuilder() const { return CustomNodeBuilder; }
	FName GetCustomBuilderName() const;
	TSharedPtr<IPropertyHandle> GetPropertyHandle() const;
	void OnGenerateChildren( FDetailNodeList& OutChildren );
	bool IsInitiallyCollapsed() const;
	TSharedPtr<FDetailWidgetRow> GetWidgetRow() const;
	bool AreChildCustomizationsHidden() const;
	void SetOriginalPath(FStringView Path) { OriginalPath = Path; }
	const FString& GetOriginalPath() const { return OriginalPath; }

private:
	/** Whether or not our parent is enabled */
	TAttribute<bool> IsParentEnabled;
	TSharedPtr<FDetailWidgetRow> HeaderRow;
	TSharedRef<class IDetailCustomNodeBuilder> CustomNodeBuilder;
	TSharedPtr<class FCustomChildrenBuilder> ChildrenBuilder;
	TWeakPtr<FDetailCategoryImpl> ParentCategory;
	FString OriginalPath;
};
