// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PropertyPath.h"

/** Represents a filter which controls the visibility of items in the details view */
struct FDetailFilter
{
	FDetailFilter() :
		bShowOnlyModified(false)
		, bShowAllAdvanced(false)
		, bShowAllChildrenIfCategoryMatches(true)
		, bShowOnlyKeyable(false)
		, bShowOnlyAnimated(false)
		, bShowFavoritesCategory(false)
		, bShowOnlyAllowed(false)
		, bShowLooseProperties(false)
		, bShowHeaders(true)
	{}

	bool IsEmptyFilter() const
	{
		return FilterStrings.Num() == 0
			&& VisibleSections.Num() == 0
			&& VisibleCategories.Num() == 0
			&& bShowOnlyModified == false
			&& bShowAllAdvanced == false
			&& bShowOnlyAllowed == false
			&& bShowAllChildrenIfCategoryMatches == false
			&& bShowOnlyKeyable == false
			&& bShowOnlyAnimated == false
			&& bShowLooseProperties == false
			&& bShowHeaders == true;
	}

	/** Any user search terms that items must match */
	TArray<FString> FilterStrings;
	/** If we should only show modified properties */
	bool bShowOnlyModified;
	/** If we should show all advanced properties */
	bool bShowAllAdvanced;
	/** If we should show all the children if their category name matches the search */
	bool bShowAllChildrenIfCategoryMatches;
	/** If we should only show keyable properties */
	bool bShowOnlyKeyable;
	/** If we should only show animated properties */
	bool bShowOnlyAnimated;
	/** If we should show the favorites category. */
	bool bShowFavoritesCategory;
	/** If we should only show properties that match PropertyAllowList */
	bool bShowOnlyAllowed;
	/* If true, will also show loose properties */
	bool bShowLooseProperties;
	/* If true, category headers */
	bool bShowHeaders;
	/** The set of allowed properties to show. */
	TSet<FPropertyPath> PropertyAllowList;
	/** The set of selected sections to show. If empty, all sections are shown.*/
	TSet<FName> VisibleSections;
	/** The set of selected categories to show. If empty, all categories are shown.*/
	TSet<FName> VisibleCategories;

	/** Delegate that determines whether a property should be forced hidden - evaluated on panel refresh */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FShouldForceHideProperty, const TSharedRef<class IPropertyHandle>&);
	FShouldForceHideProperty ShouldForceHideProperty;
};
