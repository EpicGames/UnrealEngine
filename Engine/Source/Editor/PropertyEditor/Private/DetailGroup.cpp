// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailGroup.h"
#include "Widgets/Input/SButton.h"
#include "PropertyHandleImpl.h"
#include "DetailPropertyRow.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "FDetailGroup"

FDetailGroup::FDetailGroup( const FName InGroupName, TSharedRef<FDetailCategoryImpl> InParentCategory, const FText& InLocalizedDisplayName, const bool bInStartExpanded )
	: ParentCategory( InParentCategory )
	, LocalizedDisplayName( InLocalizedDisplayName )
	, GroupName( InGroupName )
	, bStartExpanded( bInStartExpanded )
	, bResetEnabled( false )
	, DisplayMode( EDetailGroupDisplayMode::Group )
	, PasteFromTextDelegate(MakeShared<FOnPasteFromText>())
	, CopyToTextDelegate(MakeShared<FOnCopyToText>())
{
}

FDetailWidgetRow& FDetailGroup::HeaderRow()
{
	HeaderCustomization = MakeShareable( new FDetailLayoutCustomization );
	HeaderCustomization->WidgetDecl = MakeShareable( new FDetailWidgetRow );

	return *HeaderCustomization->WidgetDecl;
}

IDetailPropertyRow& FDetailGroup::HeaderProperty( TSharedRef<IPropertyHandle> PropertyHandle )
{
	check( PropertyHandle->IsValidHandle() );

	PropertyHandle->MarkHiddenByCustomization();

	HeaderCustomization = MakeShareable( new FDetailLayoutCustomization );
	HeaderCustomization->PropertyRow = MakeShareable( new FDetailPropertyRow( StaticCastSharedRef<FPropertyHandleBase>( PropertyHandle )->GetPropertyNode(), ParentCategory.Pin().ToSharedRef() ) );

	return *HeaderCustomization->PropertyRow;
}

FDetailWidgetRow& FDetailGroup::AddWidgetRow()
{
	FDetailLayoutCustomization NewCustomization;
	NewCustomization.WidgetDecl = MakeShareable( new FDetailWidgetRow );

	GroupChildren.Add( NewCustomization );

	return *NewCustomization.WidgetDecl;
}

IDetailPropertyRow& FDetailGroup::AddPropertyRow( TSharedRef<IPropertyHandle> PropertyHandle )
{
	check( PropertyHandle->IsValidHandle() );

	PropertyHandle->MarkHiddenByCustomization();

	FDetailLayoutCustomization NewCustomization;
	NewCustomization.PropertyRow = MakeShareable( new FDetailPropertyRow( StaticCastSharedRef<FPropertyHandleBase>( PropertyHandle )->GetPropertyNode(), ParentCategory.Pin().ToSharedRef() ) );

	GroupChildren.Add( NewCustomization );

	return *NewCustomization.PropertyRow;
}

IDetailPropertyRow& FDetailGroup::AddExternalStructureProperty(TSharedPtr<IStructureDataProvider> StructDataProvider, FName PropertyName, EPropertyLocation::Type Location, const FAddPropertyParams& Params)
{
	FDetailLayoutCustomization NewCustomization;
	NewCustomization.bCustom = true;
	NewCustomization.bAdvanced = Location == EPropertyLocation::Advanced;

	FDetailPropertyRow::MakeExternalPropertyRowCustomization(StructDataProvider, PropertyName, ParentCategory.Pin().ToSharedRef(), NewCustomization, Params);

	GroupChildren.Add( NewCustomization );

	return *NewCustomization.PropertyRow;
}

IDetailPropertyRow& FDetailGroup::AddExternalObjectProperty(const TArray<UObject*>& Objects, FName PropertyName, EPropertyLocation::Type Location, const FAddPropertyParams& Params)
{
	FDetailLayoutCustomization NewCustomization;
	NewCustomization.bCustom = true;
	NewCustomization.bAdvanced = Location == EPropertyLocation::Advanced;

	FDetailPropertyRow::MakeExternalPropertyRowCustomization(Objects, PropertyName, ParentCategory.Pin().ToSharedRef(), NewCustomization, Params);

	GroupChildren.Add( NewCustomization );

	return *NewCustomization.PropertyRow;
}

TSharedPtr<IDetailPropertyRow> FDetailGroup::FindPropertyRow(TSharedRef<IPropertyHandle> PropertyHandle) const
{
	for (const FDetailLayoutCustomization& Customization : GroupChildren)
	{
		if (Customization.PropertyRow.IsValid() && Customization.PropertyRow->GetPropertyHandle() == PropertyHandle)
		{
			return Customization.PropertyRow;
		}
	}

	return nullptr;
}

IDetailGroup& FDetailGroup::AddGroup(FName NewGroupName, const FText& InLocalizedDisplayName, bool bInStartExpanded)
{
	FDetailLayoutCustomization NewCustomization;
	NewCustomization.DetailGroup = MakeShareable(new FDetailGroup(NewGroupName, ParentCategory.Pin().ToSharedRef(), InLocalizedDisplayName, bStartExpanded));

	GroupChildren.Add(NewCustomization);

	return *NewCustomization.DetailGroup;
}

const TOptional<FText>& FDetailGroup::GetToolTip() const
{
	return LocalizedToolTip;
}

void FDetailGroup::SetToolTip(const FText& ToolTip)
{
	LocalizedToolTip = ToolTip;
}

void FDetailGroup::ToggleExpansion( bool bExpand )
{
	if( ParentCategory.IsValid() && OwnerTreeNode.IsValid() )
	{
		ParentCategory.Pin()->RequestItemExpanded( OwnerTreeNode.Pin().ToSharedRef(), bExpand );
	}
}

bool FDetailGroup::GetExpansionState() const
{
	if (ParentCategory.IsValid() && OwnerTreeNode.IsValid())
	{
		return ParentCategory.Pin()->GetSavedExpansionState(*OwnerTreeNode.Pin().Get());
	}
	return false;
}

TOptional<FResetToDefaultOverride> FDetailGroup::GetCustomResetToDefault() const
{
	if (HeaderCustomization.IsValid())
	{
		return HeaderCustomization->PropertyRow->GetCustomResetToDefault();
	}
	return TOptional<FResetToDefaultOverride>();
}

void FDetailGroup::SetDisplayMode(EDetailGroupDisplayMode Mode)
{
	DisplayMode = Mode;
}

TSharedPtr<FDetailPropertyRow> FDetailGroup::GetHeaderPropertyRow() const
{
	return HeaderCustomization.IsValid() ? HeaderCustomization->PropertyRow : nullptr;
}

TSharedPtr<FPropertyNode> FDetailGroup::GetHeaderPropertyNode() const
{
	return HeaderCustomization.IsValid() ? HeaderCustomization->GetPropertyNode() : nullptr;
}

bool FDetailGroup::HasColumns() const
{
	if( HeaderCustomization.IsValid() && HeaderCustomization->HasPropertyNode() && HeaderCustomization->PropertyRow->HasColumns() )
	{
		return HeaderCustomization->PropertyRow->HasColumns();
	}
	else if( HeaderCustomization.IsValid() && HeaderCustomization->HasCustomWidget() )
	{
		return HeaderCustomization->WidgetDecl->HasColumns();
	}
	
	return true;
}

bool FDetailGroup::RequiresTick() const
{
	bool bRequiresTick = false;
	if( HeaderCustomization.IsValid() && HeaderCustomization->HasPropertyNode() )
	{
		bRequiresTick = HeaderCustomization->PropertyRow->RequiresTick();
	}
	else if ( HeaderCustomization.IsValid() && HeaderCustomization->HasCustomWidget() )
	{
		bRequiresTick = HeaderCustomization->WidgetDecl->VisibilityAttr.IsBound();
	}

	return bRequiresTick;
}

EVisibility FDetailGroup::GetGroupVisibility() const
{
	EVisibility Visibility = EVisibility::Visible;
	if( HeaderCustomization.IsValid() && HeaderCustomization->HasPropertyNode() )
	{
		Visibility = HeaderCustomization->PropertyRow->GetPropertyVisibility();
	}
	else if ( HeaderCustomization.IsValid() && HeaderCustomization->HasCustomWidget() )
	{
		Visibility = HeaderCustomization->WidgetDecl->VisibilityAttr.Get();
	}

	return Visibility;
}

FDetailWidgetRow FDetailGroup::GetWidgetRow()
{
	if( HeaderCustomization.IsValid() && HeaderCustomization->HasPropertyNode() )
	{
		return HeaderCustomization->PropertyRow->GetWidgetRow();
	}
	else if ( HeaderCustomization.IsValid() && HeaderCustomization->HasCustomWidget() )
	{
		return *HeaderCustomization->WidgetDecl;
	}
	else
	{
		FDetailWidgetRow Row;

		Row.NameContent()
		[
			MakeNameWidget()
		];

		Row.OverrideResetToDefault(FResetToDefaultOverride::Create(TAttribute<bool>(this, &FDetailGroup::IsResetVisible), FSimpleDelegate::CreateSP(this, &FDetailGroup::OnResetClicked)));

		return Row;
	}
}

void FDetailGroup::OnItemNodeInitialized( TSharedRef<FDetailItemNode> InTreeNode, TSharedRef<FDetailCategoryImpl> InParentCategory, const TAttribute<bool>& InIsParentEnabled )
{
	OwnerTreeNode = InTreeNode;
	ParentCategory = InParentCategory;
	IsParentEnabled = InIsParentEnabled;

	if( HeaderCustomization.IsValid() && HeaderCustomization->HasPropertyNode() )
	{
		HeaderCustomization->PropertyRow->OnItemNodeInitialized( InParentCategory, InIsParentEnabled, AsShared() );
	}
}

void FDetailGroup::OnGenerateChildren( FDetailNodeList& OutChildren )
{
	for( int32 ChildIndex = 0; ChildIndex < GroupChildren.Num(); ++ChildIndex )
	{
		TSharedRef<FDetailItemNode> NewNode = MakeShareable( new FDetailItemNode( GroupChildren[ChildIndex], ParentCategory.Pin().ToSharedRef(), IsParentEnabled, AsShared()) );
		NewNode->Initialize();
		OutChildren.Add( NewNode );
	}
}

FReply FDetailGroup::OnNameClicked()
{
	OwnerTreeNode.Pin()->ToggleExpansion();

	return FReply::Handled();
}

TSharedRef<SWidget> FDetailGroup::MakeNameWidget()
{
	TSharedRef<STextBlock> TextBlock = SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LocalizedDisplayName);

	if (LocalizedToolTip.IsSet() && !LocalizedToolTip->IsEmpty())
	{
		TextBlock->SetToolTipText(LocalizedToolTip.GetValue());
	}

	return
		SNew( SButton )
		.ButtonStyle( FAppStyle::Get(), "NoBorder" )
		.ContentPadding(FMargin(0,2,0,2))
		.OnClicked( this, &FDetailGroup::OnNameClicked )				
		.ForegroundColor( FSlateColor::UseForeground() )
		.Content()
		[
			TextBlock
		];
}

void FDetailGroup::OnResetClicked()
{
	if (bResetEnabled)
	{
		TArray<TSharedPtr<IPropertyHandle>> PropertyHandles;

		if (GetAllChildrenPropertyHandles(PropertyHandles))
		{
			for (TSharedPtr<IPropertyHandle> PropertyHandle : PropertyHandles)
			{
				PropertyHandle->ResetToDefault();
			}

			OnDetailGroupReset.Broadcast();
		}
	}
}

bool FDetailGroup::IsResetVisible() const
{
	if (bResetEnabled)
	{
		TArray<TSharedPtr<IPropertyHandle>> PropertyHandles;

		if (GetAllChildrenPropertyHandles(PropertyHandles))
		{
			for (TSharedPtr<IPropertyHandle> PropertyHandle : PropertyHandles)
			{
				if (PropertyHandle->DiffersFromDefault())
				{
					return true;
				}
			}
		}
	}

	return false;
}

void FDetailGroup::EnableReset(bool InValue)
{
	bResetEnabled = InValue;
}

bool FDetailGroup::GetAllChildrenPropertyHandles(TArray<TSharedPtr<IPropertyHandle>>& PropertyHandles) const
{
	PropertyHandles.Reserve(GroupChildren.Num());
	return GetAllChildrenPropertyHandlesRecursive(this, PropertyHandles);
}

bool FDetailGroup::GetAllChildrenPropertyHandlesRecursive(const FDetailGroup* CurrentDetailGroup, TArray<TSharedPtr<IPropertyHandle>>& PropertyHandles) const
{
	bool Result = false;

	for (const FDetailLayoutCustomization& Customization : CurrentDetailGroup->GroupChildren)
	{
		if (Customization.HasPropertyNode())
		{
			PropertyHandles.Add(Customization.PropertyRow->GetPropertyHandle());
			Result = true;
		}
		else if (Customization.HasGroup())
		{
			Result &= GetAllChildrenPropertyHandlesRecursive(Customization.DetailGroup.Get(), PropertyHandles);
		}
		else if (Customization.HasCustomWidget())
		{
			PropertyHandles.Append(Customization.WidgetDecl->GetPropertyHandles());
			Result = true;
		}
	}

	return Result;
}

#undef LOCTEXT_NAMESPACE
