// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilderImpl.h"
#include "DetailItemNode.h"
#include "IDetailGroup.h"

class IDetailPropertyRow;

class FDetailGroup : public IDetailGroup, public TSharedFromThis<FDetailGroup>
{
public:
	FDetailGroup( const FName InGroupName, TSharedRef<FDetailCategoryImpl> InParentCategory, const FText& InLocalizedDisplayName, const bool bInStartExpanded = false );

	/** IDetailGroup interface */
	virtual FDetailWidgetRow& HeaderRow() override;
	virtual IDetailPropertyRow& HeaderProperty( TSharedRef<IPropertyHandle> PropertyHandle ) override;
	virtual FDetailWidgetRow& AddWidgetRow() override;
	virtual IDetailPropertyRow& AddPropertyRow( TSharedRef<IPropertyHandle> PropertyHandle ) override;
	virtual IDetailPropertyRow& AddExternalStructureProperty(TSharedPtr<IStructureDataProvider> StructDataProvider, FName PropertyName, EPropertyLocation::Type Location, const FAddPropertyParams& Params) override;
	virtual IDetailPropertyRow& AddExternalObjectProperty(const TArray<UObject*>& Objects, FName PropertyName, EPropertyLocation::Type Location, const FAddPropertyParams& Params) override;
	virtual IDetailGroup& AddGroup(FName NewGroupName, const FText& InLocalizedDisplayName, bool bInStartExpanded = false) override;
	virtual const TOptional<FText>& GetToolTip() const override;
	virtual void SetToolTip(const FText& ToolTip) override;
	virtual void ToggleExpansion( bool bExpand ) override;
	virtual bool GetExpansionState() const override;
	virtual void SetDisplayMode(EDetailGroupDisplayMode Mode) override;

	/** IDetailLayoutRow interface */
	virtual FName GetRowName() const override { return GroupName; }
	virtual TOptional<FResetToDefaultOverride> GetCustomResetToDefault() const override;

	TSharedPtr<FDetailPropertyRow> GetHeaderPropertyRow() const;
	TSharedPtr<FPropertyNode> GetHeaderPropertyNode() const;

	/** @return The name of the group */
	virtual FName GetGroupName() const override { return GetRowName(); }

	/** @return The number of child rows under this group */
	virtual int32 GetNumChildren() const override { return GroupChildren.Num(); }

	/** @return The localized display name of the group */
	const FText& GetGroupDisplayName() const { return LocalizedDisplayName; }
	
	/** Whether or not the group has columns */
	bool HasColumns() const;

	/** @return true if this row should be ticked */
	bool RequiresTick() const;

	/** @return true if this row should start expanded */
	bool ShouldStartExpanded() const { return bStartExpanded; }

	/** @return the display mode that this group should use */
	EDetailGroupDisplayMode GetDisplayMode() const { return DisplayMode; }
	
	/** 
	 * @return The visibility of this group
	 */
	EVisibility GetGroupVisibility() const;

	/**
	 * Called by the owning item node when it has been initialized
	 *
	 * @param InTreeNode		The node that owns this group
	 * @param InParentCategory	The category this group is located in
	 * @param InIsParentEnabled	Whether or not the parent node is enabled
	 */
	void OnItemNodeInitialized( TSharedRef<class FDetailItemNode> InTreeNode, TSharedRef<FDetailCategoryImpl> InParentCategory, const TAttribute<bool>& InIsParentEnabled );

	/** @return the row which should be displayed for this group */
	FDetailWidgetRow GetWidgetRow();

	/**
	 * Called to generate children of this group
	 *
	 * @param OutChildren	The list of children to add to
	 */
	void OnGenerateChildren( FDetailNodeList& OutChildren );


	/**
	* Permit resetting all the properties in the group
	*/
	virtual void EnableReset(bool InValue);

	/**
	* Return the property row associated with the specified property handle
	*/
	virtual TSharedPtr<IDetailPropertyRow> FindPropertyRow(TSharedRef<IPropertyHandle> PropertyHandle) const override;

	/**
	* Return the delegate called when user press the Group Reset ui
	*/
	virtual FDetailGroupReset& GetOnDetailGroupReset() override { return OnDetailGroupReset; }

	/**
	 * Return an optional PasteFromText delegate
	 */
	virtual TSharedPtr<FOnPasteFromText> OnPasteFromText() const override { return PasteFromTextDelegate; }

	/**
	* Return an optional PasteFromText delegate
	*/
	virtual TSharedPtr<FOnCopyToText> OnCopyToText() const override { return CopyToTextDelegate; }


private:
	/**
	 * Called when the name of the group is clicked to expand the group
	 */
	FReply OnNameClicked();

	/** 
	 * Makes a name widget for this group 
	 */
	TSharedRef<SWidget> MakeNameWidget();

	/** Called when the "Reset to Default" button for the location has been clicked */
	void OnResetClicked();
	bool IsResetVisible() const;
	bool GetAllChildrenPropertyHandles(TArray<TSharedPtr<IPropertyHandle>>& PropertyHandles) const;
	bool GetAllChildrenPropertyHandlesRecursive(const FDetailGroup* CurrentDetailGroup, TArray<TSharedPtr<IPropertyHandle>>& PropertyHandles) const;

private:
	/** Customized group children */
	TArray<FDetailLayoutCustomization> GroupChildren;
	/** User customized header row */
	TSharedPtr<FDetailLayoutCustomization> HeaderCustomization;
	/** Owner node of this group */
	TWeakPtr<class FDetailItemNode> OwnerTreeNode;
	/** Parent category of this group */
	TWeakPtr<FDetailCategoryImpl> ParentCategory;
	/** Whether or not our parent is enabled */
	TAttribute<bool> IsParentEnabled;
	/** Display name of this group */
	FText LocalizedDisplayName;
	/** ToolTip for this group */
	TOptional<FText> LocalizedToolTip;
	/** Name identifier of this group */
	FName GroupName;
	/** Whether the detail group should start expanded or not */
	bool bStartExpanded : 1;
	/** Permit resetting all the properties in the group */
	bool bResetEnabled : 1;
	/** Whether the detail group should appear like it's a subcategory or not */
	EDetailGroupDisplayMode DisplayMode;
	/**	Delegate called when user press the Group Reset ui */
	FDetailGroupReset OnDetailGroupReset;
	/** Delegate handling pasting an optionally tagged text snippet */
	TSharedPtr<FOnPasteFromText> PasteFromTextDelegate;
	/** Optional Delegate handling copying to a text snippet */
	TSharedPtr<FOnCopyToText> CopyToTextDelegate;

};
