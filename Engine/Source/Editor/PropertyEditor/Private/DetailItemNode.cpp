// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailItemNode.h"

#include "CategoryPropertyNode.h"
#include "DetailCategoryGroupNode.h"
#include "DetailGroup.h"
#include "DetailPropertyRow.h"
#include "IDetailKeyframeHandler.h"
#include "ObjectPropertyNode.h"
#include "PropertyHandleImpl.h"
#include "PropertyPermissionList.h"
#include "SConstrainedBox.h"
#include "SDetailCategoryTableRow.h"
#include "SDetailSingleItemRow.h"
#include "Subsystems/PropertyVisibilityOverrideSubsystem.h"
#include "UObject/PropertyNames.h"
#include "UObject/PropertyOptional.h"

static const FName NAME_IsLooseMetadata = TEXT("IsLoose");

FDetailItemNode::FDetailItemNode(const FDetailLayoutCustomization& InCustomization, TSharedRef<FDetailCategoryImpl> InParentCategory, TAttribute<bool> InIsParentEnabled, TSharedPtr<IDetailGroup> InParentGroup)
	: Customization( InCustomization )
	, ParentCategory( InParentCategory )
	, ParentGroup(InParentGroup)
	, IsParentEnabled( InIsParentEnabled )
	, CachedItemVisibility( EVisibility::Visible )
	, bForceHidden( false )
	, bShouldBeVisibleDueToFiltering( false )
	, bShouldBeVisibleDueToChildFiltering( false )
	, bTickable( false )
	, bIsExpanded( InCustomization.HasCustomBuilder() ? !InCustomization.CustomBuilderRow->IsInitiallyCollapsed() : false )
	, bIsHighlighted( false )
{
	SetParentNode(InParentCategory);
}

void FDetailItemNode::Initialize()
{
	bool bHasCustomPropertyRowWidget = Customization.PropertyRow 
			                            && ( Customization.PropertyRow->CustomNameWidget()
			                                || Customization.PropertyRow->CustomValueWidget());
	
	if( bHasCustomPropertyRowWidget
	    || ( Customization.HasCustomWidget() && Customization.WidgetDecl->VisibilityAttr.IsBound() )
		|| ( Customization.HasCustomBuilder() && Customization.CustomBuilderRow->RequiresTick() )
		|| ( Customization.HasPropertyNode() && Customization.PropertyRow->RequiresTick() )
		|| ( Customization.HasGroup() && Customization.DetailGroup->RequiresTick() ) )
	{
		// The node needs to be ticked because it has widgets that can dynamically come and go
		bTickable = true;
		if (const TSharedPtr<FDetailCategoryImpl> ParentCategoryPtr = ParentCategory.Pin())
		{
			ParentCategoryPtr->AddTickableNode(*this);
		}
	}

	if( Customization.HasPropertyNode() )
	{
		InitPropertyEditor();
	}
	else if( Customization.HasCustomBuilder() )
	{
		InitCustomBuilder();
	}
	else if( Customization.HasGroup() )
	{
		InitGroup();
	}

	if (Customization.PropertyRow.IsValid() && Customization.PropertyRow->GetForceAutoExpansion())
	{
		const bool bShouldExpand = true;
		const bool bSaveState = false;
		SetExpansionState(bShouldExpand, bSaveState);
	}

	RefreshCachedVisibility();

	const bool bUpdateFilteredNodes = false;
	GenerateChildren( bUpdateFilteredNodes );
}

FDetailItemNode::~FDetailItemNode()
{
	if( bTickable && ParentCategory.IsValid() )
	{
		ParentCategory.Pin()->RemoveTickableNode( *this );
	}
}

EDetailNodeType FDetailItemNode::GetNodeType() const
{
	if (Customization.HasPropertyNode() && Customization.GetPropertyNode()->AsCategoryNode())
	{
		return EDetailNodeType::Category;
	}
	else
	{
		return EDetailNodeType::Item;
	}
}

TSharedPtr<IPropertyHandle> FDetailItemNode::CreatePropertyHandle() const
{
	TSharedPtr<FDetailCategoryImpl> ParentCategoryPtr = ParentCategory.Pin();
	if (Customization.HasPropertyNode() && ParentCategoryPtr.IsValid())
	{
		TSharedPtr<FDetailLayoutBuilderImpl> ParentLayout = ParentCategoryPtr->GetParentLayoutImpl();
		if (ParentLayout.IsValid())
		{
			return ParentLayout->GetPropertyHandle(Customization.GetPropertyNode());
		}
	}
	else if (Customization.HasCustomWidget())
	{
		const TArray<TSharedPtr<IPropertyHandle>>& Handles = Customization.WidgetDecl->GetPropertyHandles();
		if (Handles.Num() > 0)
		{
			return Handles[0];
		}
	}
	else if (Customization.HasCustomBuilder())
	{
		return Customization.CustomBuilderRow->GetPropertyHandle();
	}

	return nullptr;
}

void FDetailItemNode::GetFilterStrings(TArray<FString>& OutFilterStrings) const
{
	if (!Customization.GetFilterTextString().IsEmpty())
	{
		OutFilterStrings.Add(Customization.GetFilterTextString().ToString());
	}

	if (Customization.HasPropertyNode())
	{
		TSharedPtr<FPropertyNode> PropertyNode = Customization.GetPropertyNode();
		if (PropertyNode.IsValid())
		{
			OutFilterStrings.Add(GetPropertyNode()->GetDisplayName().ToString());
			if (PropertyNode->GetDisplayName().ToString() != PropertyNode->GetProperty()->GetName())
			{
				OutFilterStrings.Add(PropertyNode->GetProperty()->GetName());
			}
		}
	}
}

bool FDetailItemNode::GetInitiallyCollapsed() const
{
	if (Customization.IsValidCustomization() && Customization.PropertyRow.IsValid())
	{
		return Customization.PropertyRow->GetForceAutoExpansion() == false;
	}
	return true;
}

void FDetailItemNode::InitPropertyEditor()
{
	FProperty* NodeProperty = Customization.GetPropertyNode()->GetProperty();

	if( NodeProperty && (NodeProperty->IsA<FArrayProperty>() || NodeProperty->IsA<FSetProperty>() || NodeProperty->IsA<FMapProperty>() || NodeProperty->IsA<FOptionalProperty>()))
	{
		const bool bUpdateFilteredNodes = false;
		FSimpleDelegate OnRegenerateChildren = FSimpleDelegate::CreateSP( this, &FDetailItemNode::GenerateChildren, bUpdateFilteredNodes );

		Customization.GetPropertyNode()->SetOnRebuildChildren( OnRegenerateChildren );
	}

	Customization.PropertyRow->OnItemNodeInitialized( ParentCategory.Pin().ToSharedRef(), IsParentEnabled, ParentGroup.IsValid() ? ParentGroup.Pin() : nullptr );

	if (Customization.HasExternalPropertyRow())
	{
		const bool bSaveState = false;
		SetExpansionState(ParentCategory.Pin()->GetSavedExpansionState(*this), bSaveState);
	}
	else if (CastField<FStructProperty>(NodeProperty))
	{
		const bool bSaveState = false;

		// Prefer the property node's own Expanded flag, which is restored per-element
		// (with array indices) by RestoreAllExpandedItems() and avoids key collisions.
		bool bExpanded = Customization.GetPropertyNode()->HasNodeFlags(EPropertyNodeFlags::Expanded);

		// For non-array-element struct properties, fall back to ExpandedDetailNodes if the
		// property node flag is not set. This is needed because some editor contexts (e.g. the
		// Workspace Editor) may not have fully restored property node flags at this point.
		// We skip this fallback for array elements because ExpandedDetailNodes uses
		// Property->GetFName() as key, which is shared across all elements of a struct array,
		// causing a key collision that corrupts individual element expansion states.
		if (!bExpanded && Customization.GetPropertyNode()->GetArrayIndex() == INDEX_NONE)
		{
			bExpanded = ParentCategory.Pin()->GetSavedExpansionState(*this);
		}

		SetExpansionState(bExpanded, bSaveState);
	}
}

void FDetailItemNode::InitCustomBuilder()
{
	Customization.CustomBuilderRow->OnItemNodeInitialized( AsShared(), ParentCategory.Pin().ToSharedRef(), IsParentEnabled );

	// Restore saved expansion state
	FName BuilderName = Customization.CustomBuilderRow->GetCustomBuilderName();
	if( BuilderName != NAME_None )
	{
		const bool bSaveState = false;
		SetExpansionState(ParentCategory.Pin()->GetSavedExpansionState(*this), bSaveState);
	}
}

void FDetailItemNode::InitGroup()
{
	Customization.DetailGroup->OnItemNodeInitialized( AsShared(), ParentCategory.Pin().ToSharedRef(), IsParentEnabled );

	if (Customization.DetailGroup->ShouldStartExpanded())
	{
		bIsExpanded = true;
	}
	else
	{
		// Restore saved expansion state
		FName GroupName = Customization.DetailGroup->GetGroupName();
		if (GroupName != NAME_None)
		{
			const bool bSaveState = false;
			SetExpansionState(ParentCategory.Pin()->GetSavedExpansionState(*this), bSaveState);
		}
	}
}

bool FDetailItemNode::HasMultiColumnWidget() const
{
	return ( Customization.HasCustomWidget() && Customization.WidgetDecl->HasColumns() )
		|| ( Customization.HasCustomBuilder() && Customization.CustomBuilderRow->HasColumns() )
		|| ( Customization.HasGroup() && Customization.DetailGroup->HasColumns() )
		|| ( Customization.HasPropertyNode() && Customization.PropertyRow->HasColumns());
}

void FDetailItemNode::ToggleExpansion()
{
	const bool bSaveState = true;
	SetExpansionState( !bIsExpanded, bSaveState );
}

void FDetailItemNode::SetExpansionState(bool bWantsExpanded, bool bSaveState)
{
	bIsExpanded = bWantsExpanded;

	// Expand the child after filtering if it wants to be expanded
	ParentCategory.Pin()->RequestItemExpanded(AsShared(), bIsExpanded);

	OnItemExpansionChanged(bIsExpanded, bSaveState);
}

void FDetailItemNode::SetExpansionState(bool bWantsExpanded)
{
	const bool bSaveState = true;
	SetExpansionState(bWantsExpanded, bSaveState);
}

TSharedRef< ITableRow > FDetailItemNode::GenerateWidgetForTableView( const TSharedRef<STableViewBase>& OwnerTable, bool bAllowFavoriteSystem)
{
	FTagMetaData TagMeta(TEXT("DetailRowItem"));
	if (ParentCategory.IsValid())
	{
		if (Customization.IsValidCustomization() && Customization.GetPropertyNode().IsValid())
		{
			TagMeta.Tag = *FString::Printf(TEXT("DetailRowItem.%s"), *Customization.GetPropertyNode()->GetDisplayName().ToString());
		}
		else if (Customization.HasCustomWidget() )
		{
			TagMeta.Tag = Customization.GetWidgetRow().RowTagName;
		}
	}

	if( Customization.HasPropertyNode() && Customization.GetPropertyNode()->AsCategoryNode() )
	{
		return
			SNew(SDetailCategoryTableRow, AsShared(), OwnerTable)
			.DisplayName(Customization.GetPropertyNode()->GetDisplayName())
			.AddMetaData<FTagMetaData>(TagMeta)
			.InnerCategory(true);
	}
	else if (Customization.HasGroup() && Customization.DetailGroup->GetDisplayMode() == EDetailGroupDisplayMode::Category)
	{
		return
			SNew(SDetailCategoryTableRow, AsShared(), OwnerTable)
			.DisplayName(Customization.DetailGroup->GetGroupDisplayName())
			.AddMetaData<FTagMetaData>(TagMeta)
			.InnerCategory(true);
	}
	else
	{
		return
			SNew(SDetailSingleItemRow, &Customization, HasMultiColumnWidget(), AsShared(), OwnerTable )
			.AddMetaData<FTagMetaData>(TagMeta)
			.AllowFavoriteSystem(bAllowFavoriteSystem);
	}
}

bool FDetailItemNode::GenerateStandaloneWidget(FDetailWidgetRow& OutRow) const
{
	bool bResult = false;

	if (Customization.HasPropertyNode() && Customization.GetPropertyNode()->AsCategoryNode())
	{
		const bool bIsInnerCategory = true;

		OutRow.NameContent()
		[
			SNew(STextBlock)
			.Text(Customization.GetPropertyNode()->GetDisplayName())
			.Font(FAppStyle::GetFontStyle(bIsInnerCategory ? "PropertyWindow.NormalFont" : "DetailsView.CategoryFontStyle"))
			.ShadowOffset(bIsInnerCategory ? FVector2D::ZeroVector : FVector2D(1.0f, 1.0f))
		];

		bResult = true;
	}
	else if (Customization.IsValidCustomization())
	{
		FDetailWidgetRow Row = Customization.GetWidgetRow();

		// We make some slight modifications to the row here before giving it to OutRow
		if (HasMultiColumnWidget())
		{
			TSharedPtr<SWidget> NameWidget = Row.NameWidget.Widget;
			TSharedPtr<SWidget> ValueWidget =
				SNew(SConstrainedBox)
				.MinWidth(Row.ValueWidget.MinWidth)
				.MaxWidth(Row.ValueWidget.MaxWidth)
				[
					Row.ValueWidget.Widget
				];

			if (Row.IsEnabledAttr.IsSet() || Row.IsValueEnabledAttr.IsSet() || Row.EditConditionValue.IsSet())
			{
				// copies of attributes for lambda captures
				TAttribute<bool> PropertyEnabledAttribute = IsPropertyEditingEnabled();
				TAttribute<bool> RowIsEnabledAttribute = Row.IsEnabledAttr;
				TAttribute<bool> RowEditConditionAttribute = Row.EditConditionValue;

				TAttribute<bool> IsEnabledAttribute = TAttribute<bool>::CreateLambda(
					[PropertyEnabledAttribute, RowIsEnabledAttribute, RowEditConditionAttribute]()
					{
						return PropertyEnabledAttribute.Get() && RowIsEnabledAttribute.Get(true) && RowEditConditionAttribute.Get(true);
					});

				// there's an unavoidable conflict here if the user customizes the widget to have a custom IsEnabled,
				// and a custom EditCondition/IsEnabled on the widget row - we choose to favor the row in this case
				NameWidget->SetEnabled(IsEnabledAttribute);

				if (Row.IsValueEnabledAttr.IsSet())
				{
					TAttribute<bool> RowIsValueEnabledAttribute = Row.IsValueEnabledAttr;
					TAttribute<bool> IsValueWidgetEnabledAttribute = TAttribute<bool>::CreateLambda(
						[IsEnabledAttribute, RowIsValueEnabledAttribute]()
						{
							return IsEnabledAttribute.Get() && RowIsValueEnabledAttribute.Get(true);
						});

					ValueWidget->SetEnabled(IsValueWidgetEnabledAttribute);
				}
				else
				{
					ValueWidget->SetEnabled(IsEnabledAttribute);
				}
			}

			OutRow.NameContent()
			[
				NameWidget.ToSharedRef()
			];

			OutRow.ValueContent()
			[
				ValueWidget.ToSharedRef()
			];
		}
		else
		{
			OutRow.WholeRowContent()
			[
				Row.WholeRowWidget.Widget
			];
		}

		OutRow.CustomResetToDefault = Row.CustomResetToDefault;
		OutRow.IsEnabledAttr = Row.IsEnabledAttr;
		OutRow.VisibilityAttr = Row.VisibilityAttr;
		OutRow.EditConditionValue = Row.EditConditionValue;
		OutRow.OnEditConditionValueChanged = Row.OnEditConditionValueChanged;

		OutRow.CopyMenuAction = Row.CopyMenuAction;
		OutRow.PasteMenuAction = Row.PasteMenuAction;
		OutRow.CustomMenuItems = Row.CustomMenuItems;

		OutRow.FilterTextString = Row.FilterTextString;

		bResult = true;
	}

	return bResult;
}

void FDetailItemNode::GetChildren(FDetailNodeList& OutChildren, const bool& bInIgnoreVisibility)
{
	OutChildren.Reserve(Children.Num());

	for (const TSharedRef<FDetailTreeNode>& Child : Children)
	{
		ENodeVisibility ChildVisibility = Child->GetVisibility();

		// Report the child if the child is visible or we are visible due to filtering and there were no filtered children.  
		// If we are visible due to filtering and so is a child, we only show that child.  
		// If we are visible due to filtering and no child is visible, we show all children

		if( ChildVisibility == ENodeVisibility::Visible
			|| bInIgnoreVisibility
			|| ( !bShouldBeVisibleDueToChildFiltering && bShouldBeVisibleDueToFiltering && ChildVisibility != ENodeVisibility::ForcedHidden ) )
		{
			if( Child->ShouldShowOnlyChildren() )
			{
				Child->GetChildren( OutChildren, bInIgnoreVisibility );
			}
			else
			{
				OutChildren.Add( Child );
			}
		}
	}
}

void FDetailItemNode::GenerateChildren( bool bUpdateFilteredNodes )
{
	FDetailNodeList OldChildren = Children;
	Children.Empty();

	TSharedPtr<FDetailCategoryImpl> ParentCategoryPinned = ParentCategory.Pin();
	if (!ParentCategoryPinned.IsValid())
	{
		return;
	}

	TSharedPtr<FDetailLayoutBuilderImpl> ParentLayout = ParentCategoryPinned->GetParentLayoutImpl();
	if (!ParentLayout.IsValid())
	{
		return;
	}

	// Make sure to remove the root properties referenced by the old children, otherwise they will leak.
	for (TSharedRef<FDetailTreeNode> OldChild : OldChildren)
	{
		TSharedPtr<FComplexPropertyNode> OldChildExternalRootPropertyNode = OldChild->GetExternalRootPropertyNode();
		if (OldChildExternalRootPropertyNode.IsValid())
		{
			ParentLayout->RemoveExternalRootPropertyNode(OldChildExternalRootPropertyNode.ToSharedRef());
		}
	}

	if( Customization.HasPropertyNode() )
	{
		Customization.PropertyRow->OnGenerateChildren( Children );
	}
	else if( Customization.HasCustomBuilder() )
	{
		Customization.CustomBuilderRow->OnGenerateChildren( Children );

		// Need to refresh the tree for custom builders as we could be regenerating children at any point
		ParentCategory.Pin()->RefreshTree( bUpdateFilteredNodes );
	}
	else if( Customization.HasGroup() )
	{
		Customization.DetailGroup->OnGenerateChildren( Children );
	}

	// Discard generated nodes that don't pass the property allow list, as well as generated categories who no longer contain any children
	// Searching backwards guarantees that a category's children will be culled before the category itself.
	for (int32 Index = Children.Num() - 1; Index >= 0; --Index)
	{
		const TSharedRef<FDetailTreeNode>& Child = Children[Index];

		Child->SetParentNode(AsShared());
		if (Child->GetNodeType() == EDetailNodeType::Object || Child->GetNodeType() == EDetailNodeType::Item)
		{
			if (!FPropertyEditorPermissionList::Get().DoesDetailTreeNodePassFilter(Child->GetParentBaseStructure(), Child))
			{
				Children.RemoveAt(Index);
			}
		}
		else if (Child->GetNodeType() == EDetailNodeType::Category)
		{
			// Nodes default to hidden until the filter runs the first time - categories return no children if they're hidden, so force an empty filter to initialize properly
			Child->FilterNode(FDetailFilter());
			FDetailNodeList Subchildren;
			Child->GetChildren(Subchildren);
			if (Subchildren.Num() == 0)
			{
				Children.RemoveAt(Index);
			}
		}
	}
}


void FDetailItemNode::OnItemExpansionChanged( bool bInIsExpanded, bool bShouldSaveState )
{
	bIsExpanded = bInIsExpanded;
	if( Customization.HasPropertyNode() )
	{
		Customization.GetPropertyNode()->SetNodeFlags( EPropertyNodeFlags::Expanded, bInIsExpanded );
	}

	if (ParentCategory.IsValid() && bShouldSaveState &&
			(  (Customization.HasCustomBuilder() && Customization.CustomBuilderRow->GetCustomBuilderName() != NAME_None)
			|| (Customization.HasGroup() && Customization.DetailGroup->GetGroupName() != NAME_None)
			|| (Customization.HasExternalPropertyRow())
			|| (Customization.HasPropertyNode() && CastField<FStructProperty>(Customization.GetPropertyNode()->GetProperty()))))
	{
		ParentCategory.Pin()->SaveExpansionState(*this);
	}
}

bool FDetailItemNode::ShouldBeExpanded() const
{
	bool bShouldBeExpanded = bIsExpanded || bShouldBeVisibleDueToChildFiltering;
	if( Customization.HasPropertyNode() )
	{
		FPropertyNode* PropertyNode = Customization.GetPropertyNode().Get();
		bShouldBeExpanded = PropertyNode->HasNodeFlags( EPropertyNodeFlags::Expanded ) != 0;
		bShouldBeExpanded |= PropertyNode->HasNodeFlags( EPropertyNodeFlags::IsSeenDueToChildFiltering ) != 0;
	}
	return bShouldBeExpanded;
}

ENodeVisibility FDetailItemNode::GetVisibility() const
{
	ENodeVisibility Visibility = CachedItemVisibility == EVisibility::Collapsed ? ENodeVisibility::ForcedHidden : ENodeVisibility::Visible;
	if(Customization.IsHidden() || bForceHidden)
	{
		Visibility = ENodeVisibility::ForcedHidden;
	}
	else
	{
		Visibility = (bShouldBeVisibleDueToFiltering || bShouldBeVisibleDueToChildFiltering) ? Visibility : ENodeVisibility::HiddenDueToFiltering;
	}
	
	if (Visibility == ENodeVisibility::Visible && GetNodeType() == EDetailNodeType::Category)
	{
		Visibility = ENodeVisibility::ForcedHidden;
		for (const TSharedRef<FDetailTreeNode>& Child : Children)
		{
			if (Child->GetVisibility() != ENodeVisibility::ForcedHidden)
			{
				Visibility = ENodeVisibility::Visible;
				break;
			}
		}
	}
	
	return Visibility;
}

static bool PassesAllFilters( FDetailItemNode* ItemNode, const FDetailLayoutCustomization& InCustomization, const FDetailFilter& InFilter, const FString& InCategoryName )
{
	struct Local
	{
		static bool StringPassesFilter(const FDetailFilter& InDetailFilter, const FString& InString)
		{
			// Make sure the passed string matches all filter strings
			if( InString.Len() > 0 )
			{
				for (int32 TestNameIndex = 0; TestNameIndex < InDetailFilter.FilterStrings.Num(); ++TestNameIndex)
				{
					const FString& TestName = InDetailFilter.FilterStrings[TestNameIndex];
					if ( !InString.Contains(TestName) ) 
					{
						return false;
					}
				}
				return true;
			}
			return false;
		}
		static bool ItemIsKeyable( FDetailItemNode *InItemNode, UClass *ObjectClass, TSharedPtr<FPropertyNode> PropertyNode)
		{
			TSharedPtr<IDetailsViewPrivate> DetailsView = InItemNode->GetDetailsViewSharedPtr();
			TSharedPtr<IDetailKeyframeHandler> KeyframeHandler = DetailsView->GetKeyframeHandler();
			if (KeyframeHandler.IsValid())
			{
				TSharedPtr<IPropertyHandle> PropertyHandle = PropertyEditorHelpers::GetPropertyHandle(PropertyNode.ToSharedRef(), nullptr, nullptr);

				return
					KeyframeHandler->IsPropertyKeyingEnabled() &&
					KeyframeHandler->IsPropertyKeyable(ObjectClass, *PropertyHandle);
			}
			return false;
		}

		static bool ItemIsAnimated(FDetailItemNode *InItemNode, TSharedPtr<FPropertyNode> PropertyNode)
		{
			TSharedPtr<IDetailsViewPrivate> DetailsView = InItemNode->GetDetailsViewSharedPtr();
			TSharedPtr<IDetailKeyframeHandler> KeyframeHandler = DetailsView->GetKeyframeHandler();
			if (KeyframeHandler.IsValid())
			{
				TSharedPtr<IPropertyHandle> PropertyHandle = PropertyEditorHelpers::GetPropertyHandle(PropertyNode.ToSharedRef(), nullptr, nullptr);
				FObjectPropertyNode *ParentPropertyNode = PropertyNode->FindObjectItemParent();
				// Get an iterator for the enclosing objects.
				for (int32 ObjIndex = 0; ObjIndex < ParentPropertyNode->GetNumObjects(); ++ObjIndex)
				{
					UObject* ParentObject = ParentPropertyNode->GetUObject(ObjIndex);

					if (KeyframeHandler->IsPropertyAnimated(*PropertyHandle, ParentObject))
					{
						return true;
					}
				}
			}
			return false;
		}

		static FString GetPropertyNodeValueFilterString(const FDetailLayoutCustomization& InCustomization, TSharedPtr<FPropertyNode> PropertyNode)
		{
			if (PropertyNode.IsValid())
			{
				// Is it a container (array, map, set?) - if so, ignore it, we don't care about these, only their inner nodes.
				if (CastField<FArrayProperty>(PropertyNode->GetProperty()) || CastField<FMapProperty>(PropertyNode->GetProperty()) || CastField<FSetProperty>(PropertyNode->GetProperty()) || CastField<FOptionalProperty>(PropertyNode->GetProperty()))
				{
					return FString();
				}

				// Is it a struct?  If so, some structs are useful, like FGameplayTag, or FGameplayTags, but if it's a user struct for the game
				// like FMyGameplayStruct, with a bunch of other sub nodes, that will individually be matched and filtered, there's no reason
				// to filter on the struct as a whole, so essentially what we're doing here is only checking structs that are leaf nodes.
				if (CastField<FStructProperty>(PropertyNode->GetProperty()))
				{
					if (PropertyNode->GetNumChildNodes() > 0)
					{
						return FString();
					}
				}

				// TODO Will have to do something special for EditInlineNew UObjects, rather than just a simple object path.
				if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(PropertyNode->GetProperty()))
				{
					uint8* ValueAddress = nullptr;
					FPropertyAccess::Result Result = PropertyNode->GetSingleReadAddress(ValueAddress);
					if (ValueAddress != nullptr)
					{
						if (UObject* ObjectValue = ObjectProperty->GetObjectPropertyValue(ValueAddress))
						{
							return ObjectValue->GetName();
						}
					}
				}
				else
				{
					// PPF_SimpleObjectText, seems to get the most reasonable string for searching.
					FString OutString;
					PropertyNode->GetPropertyValueString(OutString, true, PPF_SimpleObjectText);

					return OutString;
				}
			}

			return FString();
		}

		static FString GetPropertyNodeKeyFilterString(const FDetailLayoutCustomization& InCustomization, TSharedPtr<FPropertyNode> PropertyNode)
		{
			if (PropertyNode.IsValid())
			{
				// Is it a container (array, map, set?) - if so, ignore it, we don't care about these, only their inner nodes.
				if (CastField<FArrayProperty>(PropertyNode->GetProperty()) || CastField<FMapProperty>(PropertyNode->GetProperty()) || CastField<FSetProperty>(PropertyNode->GetProperty()) || CastField<FOptionalProperty>(PropertyNode->GetProperty()))
				{
					return FString();
				}

				// Need to know if parent is a Map though...
				const FProperty* Property = PropertyNode->GetProperty();
				FPropertyNode* Parent = PropertyNode->GetParentNode();
				if (Parent && Property)
				{
					const FMapProperty* OuterMapProp = Property->GetOwner<FMapProperty>();
					if (OuterMapProp)
					{
						const FProperty* KeyProperty = OuterMapProp->GetKeyProperty();

						uint8* MapValueAddress = nullptr;
						FPropertyAccess::Result Result = Parent->GetSingleReadAddress(MapValueAddress);
						if (Result != FPropertyAccess::Success)
						{
							return FString();
						}
						FScriptMapHelper MapHelper(OuterMapProp, MapValueAddress);
						FScriptMapHelper::FIterator Iterator = MapHelper.CreateIterator(PropertyNode->GetArrayIndex());
						const uint8* PairPtr = MapHelper.GetKeyPtr(Iterator);
					
						FString OutString;
					
						KeyProperty->ExportText_Direct(OutString, PairPtr, PairPtr, nullptr, PPF_SimpleObjectText);
						return OutString;
					}
				}
			}
			return FString();
		}
		
	};
	
	auto IsCustomResetToDefaultVisible = [ItemNode, &InCustomization]()
	{
		TOptional<FResetToDefaultOverride> CustomResetToDefault = InCustomization.GetCustomResetToDefault();
		return CustomResetToDefault.IsSet() && CustomResetToDefault.GetValue().IsResetToDefaultVisible(ItemNode->CreatePropertyHandle());
	};

	bool bPassesAllFilters = true;
	
	TSharedPtr<FPropertyNode> PropertyNodePin = InCustomization.GetPropertyNode();

	if( InFilter.FilterStrings.Num() > 0 || 
		InFilter.bShowOnlyModified == true || 
		InFilter.bShowOnlyAllowed == true ||
		InFilter.bShowOnlyKeyable == true || 
		InFilter.bShowOnlyAnimated == true)
	{
		const bool bSearchFilterIsEmpty = InFilter.FilterStrings.Num() == 0;
		
		const bool bPassesCategoryFilter = !bSearchFilterIsEmpty && InFilter.bShowAllChildrenIfCategoryMatches ? Local::StringPassesFilter(InFilter, InCategoryName) : false;
		const bool bPassesValueFilter = !bSearchFilterIsEmpty && Local::StringPassesFilter(InFilter, Local::GetPropertyNodeValueFilterString(InCustomization, PropertyNodePin));

		const FString KeyValue = Local::GetPropertyNodeKeyFilterString(InCustomization, PropertyNodePin);
		const bool bPassesKeyFilter = !bSearchFilterIsEmpty && Local::StringPassesFilter(InFilter, KeyValue);

		bPassesAllFilters = false;
		if( PropertyNodePin.IsValid() && !PropertyNodePin->AsCategoryNode())
		{
			const bool bIsNotBeingFiltered = PropertyNodePin->HasNodeFlags(EPropertyNodeFlags::IsBeingFiltered) == 0;
			const bool bIsSeenDueToFiltering = PropertyNodePin->HasNodeFlags(EPropertyNodeFlags::IsSeenDueToFiltering) != 0;
			const bool bIsParentSeenDueToFiltering = PropertyNodePin->HasNodeFlags(EPropertyNodeFlags::IsParentSeenDueToFiltering) != 0;

			const bool bPassesTextFilter = bPassesCategoryFilter || bPassesValueFilter || bPassesKeyFilter || Local::StringPassesFilter(InFilter, InCustomization.GetFilterTextString().ToString());
			const bool bPassesSearchFilter = bPassesTextFilter || bSearchFilterIsEmpty || ( bIsNotBeingFiltered || bIsSeenDueToFiltering || bIsParentSeenDueToFiltering );

			bool bPassesModifiedFilter = true;
			if (bPassesSearchFilter && InFilter.bShowOnlyModified)
			{
				bPassesModifiedFilter = PropertyNodePin->GetDiffersFromDefault() || IsCustomResetToDefaultVisible();
			}

			const bool bPassesAllowListFilter = InFilter.bShowOnlyAllowed ? InFilter.PropertyAllowList.Contains(*FPropertyNode::CreatePropertyPath(PropertyNodePin.ToSharedRef())) : true;

			bool bPassesKeyableFilter = true;
			if (InFilter.bShowOnlyKeyable)
			{
				FObjectPropertyNode* ParentPropertyNode = PropertyNodePin->FindObjectItemParent();
				if (ParentPropertyNode != nullptr)
				{
					UClass* ObjectClass = ParentPropertyNode->GetObjectBaseClass();
					bPassesKeyableFilter = Local::ItemIsKeyable(ItemNode, ObjectClass, PropertyNodePin);
				}
				else
				{
					bPassesKeyableFilter = false;
				}
			}
			const bool bPassesAnimatedFilter = (InFilter.bShowOnlyAnimated == false || Local::ItemIsAnimated(ItemNode, PropertyNodePin));

			// The property node is visible (note categories are never visible unless they have a child that is visible )
			bPassesAllFilters = bPassesSearchFilter && bPassesModifiedFilter && bPassesAllowListFilter && bPassesKeyableFilter && bPassesAnimatedFilter;
		}
		else if (InCustomization.HasCustomWidget())
		{
			const bool bPassesTextFilter = bPassesCategoryFilter || bPassesValueFilter || Local::StringPassesFilter(InFilter, InCustomization.WidgetDecl->FilterTextString.ToString());
			//@todo we need to support custom widgets for keyable, animated, in particular for transforms(ComponentTransformDetails).
			const bool bPassesModifiedFilter = (InFilter.bShowOnlyModified == false || InCustomization.WidgetDecl->EditConditionValue.Get(false) || IsCustomResetToDefaultVisible());
			const bool bPassesKeyableFilter = (InFilter.bShowOnlyKeyable == false);
			const bool bPassesAnimatedFilter = (InFilter.bShowOnlyAnimated == false);
			bPassesAllFilters = bPassesTextFilter && bPassesModifiedFilter && bPassesKeyableFilter && bPassesAnimatedFilter;
		}
		else if (InCustomization.HasCustomBuilder())
		{
			const bool bPassesTextFilter = bPassesCategoryFilter || bPassesValueFilter || Local::StringPassesFilter(InFilter, InCustomization.CustomBuilderRow->GetWidgetRow()->FilterTextString.ToString());
			//@todo we need to support custom builders for modified, keyable, animated, in particular for transforms(ComponentTransformDetails).
			const bool bPassesModifiedFilter = (InFilter.bShowOnlyModified == false || IsCustomResetToDefaultVisible());
			const bool bPassesKeyableFilter = (InFilter.bShowOnlyKeyable == false);
			const bool bPassesAnimatedFilter = (InFilter.bShowOnlyAnimated == false);
			bPassesAllFilters = bPassesTextFilter && bPassesModifiedFilter && bPassesKeyableFilter && bPassesAnimatedFilter;
		}
	}

	return bPassesAllFilters;
}

void FDetailItemNode::Tick( float DeltaTime )
{
	if( ensure( bTickable ) )
	{
		if( Customization.HasCustomBuilder() && Customization.CustomBuilderRow->RequiresTick() ) 
		{
			Customization.CustomBuilderRow->Tick( DeltaTime );
		}

		RefreshCachedVisibility(true);
	}
}

EVisibility FDetailItemNode::ComputeItemVisibility() const
{
	EVisibility NewVisibility = EVisibility::Visible;
	if (Customization.HasPropertyNode())
	{
		NewVisibility = Customization.PropertyRow->GetPropertyVisibility();

		if (NewVisibility != EVisibility::Collapsed)
		{
			TSharedPtr<FDetailCategoryImpl> ParentCategoryPtr = GetParentCategory();
			if (ParentCategoryPtr.IsValid())
			{
				TSharedPtr<FDetailLayoutBuilderImpl> ParentLayout = ParentCategoryPtr->GetParentLayoutImpl();
				if (ParentLayout.IsValid())
				{
					TSharedPtr<IPropertyHandle> PropertyHandle = ParentLayout->GetPropertyHandle(Customization.GetPropertyNode());
					if (!ParentLayout->IsPropertyVisible(PropertyHandle.ToSharedRef()))
					{
						NewVisibility = EVisibility::Collapsed;
					}
				}
			}
		}
	}
	else if (Customization.HasCustomWidget())
	{	
		NewVisibility = Customization.WidgetDecl->VisibilityAttr.Get();
	}
	else if (Customization.HasGroup())
	{
		NewVisibility = Customization.DetailGroup->GetGroupVisibility();
	}
	else if (Customization.HasCustomBuilder() && Children.Num() > 0)
	{
		NewVisibility = EVisibility::Collapsed;

		for (TSharedRef<FDetailTreeNode> Child : Children)
		{
			if (Child->GetVisibility() == ENodeVisibility::Visible)
			{
				NewVisibility = EVisibility::Visible;
				break;
			}
		}
	}

	TSharedPtr<const IDetailsViewPrivate> DetailsView = GetDetailsViewSharedPtr();
	if (DetailsView != nullptr)
	{
		// check the details view's IsCustomRowVisible delegate if this isn't a property row
		// properties are handled by the IsPropertyVisible delegate
		if (NewVisibility != EVisibility::Collapsed && !Customization.HasPropertyNode())
		{
			if (!DetailsView->IsCustomRowVisible(Customization.GetName(), GetParentCategory()->GetCategoryName()))
			{
				NewVisibility = EVisibility::Collapsed;
			}
		}
	}

	return NewVisibility;
}

void FDetailItemNode::RefreshVisibility()
{
	RefreshCachedVisibility();
}

void FDetailItemNode::RefreshCachedVisibility(bool bCallChangeDelegate)
{
	// Recache visibility
	EVisibility NewVisibility = ComputeItemVisibility();
	
	if( CachedItemVisibility != NewVisibility )
	{
		// The visibility of a node in the tree has changed.  We must refresh the tree to remove the widget
		CachedItemVisibility = NewVisibility;
		if (bCallChangeDelegate)
		{
			const bool bRefilterCategory = true;
			ParentCategory.Pin()->RefreshTree( bRefilterCategory );
		}
	}
}

bool FDetailItemNode::ShouldShowOnlyChildren() const
{
	// Show only children of this node if there is no content for custom details or the property node requests that only children be shown
	return ( Customization.HasCustomBuilder() && Customization.CustomBuilderRow->ShowOnlyChildren() )
		 || (Customization.HasPropertyNode() && Customization.PropertyRow->ShowOnlyChildren() );
}

FPropertyPath FDetailItemNode::GetPropertyPath() const
{
	FPropertyPath Ret;
	TSharedPtr<FPropertyNode> PropertyNode = Customization.GetPropertyNode();
	if( PropertyNode.IsValid() )
	{
		Ret = *FPropertyNode::CreatePropertyPath( PropertyNode.ToSharedRef() );
	}

	// add properties used by custom widgets
	if (Customization.WidgetDecl)
	{
		for (const TSharedPtr<IPropertyHandle> &ItemPropHandle : Customization.WidgetDecl->PropertyHandles)
		{
			if (ItemPropHandle)
			{
				if (ItemPropHandle->GetIndexInArray() != INDEX_NONE)
				{
					Ret.AddProperty(FPropertyInfo(ItemPropHandle->GetParentHandle()->GetProperty(), INDEX_NONE));
					Ret.AddProperty(FPropertyInfo(ItemPropHandle->GetProperty(), ItemPropHandle->GetIndexInArray()));
				}
				else
				{
					Ret.AddProperty(FPropertyInfo(ItemPropHandle->GetProperty(), INDEX_NONE));
				}
			}
		}
	}

	if (const TSharedPtr<IPropertyHandle> PropertyHandle = CreatePropertyHandle())
	{
		return *PropertyHandle->CreateFPropertyPath();
	}
	return Ret;
}

TAttribute<bool> FDetailItemNode::IsPropertyEditingEnabled() const
{
	return MakeAttributeSP(this, &FDetailItemNode::IsPropertyEditingEnabledImpl);
}

bool FDetailItemNode::IsPropertyEditingEnabledImpl() const
{
	bool bIsEnabled = IsParentEnabled.Get(true);

	TSharedPtr<IDetailsViewPrivate> DetailsView = GetDetailsViewSharedPtr();
	if (DetailsView)
	{
		if (Customization.HasPropertyNode())
		{
			TSharedPtr<FPropertyNode> PropertyNode = Customization.GetPropertyNode();
			if (PropertyNode->GetProperty() != nullptr)
			{
				bIsEnabled &= !DetailsView->IsPropertyReadOnly(FPropertyAndParent(PropertyNode.ToSharedRef()));
			}
		}
		else if (Customization.HasCustomWidget())
		{
			bIsEnabled &= !DetailsView->IsCustomRowReadOnly(Customization.GetName(), GetParentCategory()->GetCategoryName());
		}
	}
	
	return bIsEnabled;
}

TSharedPtr<FPropertyNode> FDetailItemNode::GetPropertyNode() const
{
	return Customization.GetPropertyNode();
}

void FDetailItemNode::GetAllPropertyNodes(TArray<TSharedRef<FPropertyNode>>& OutNodes) const
{
	TSet<TSharedRef<FPropertyNode>> SeenNodes; // make's sure there aren't duplicates
	if (const TSharedPtr<FPropertyNode> Node = GetPropertyNode())
	{
		SeenNodes.Add(Node.ToSharedRef());
		OutNodes.Add(Node.ToSharedRef());
	}

	for (const TSharedPtr<IPropertyHandle>& CurPropertyHandle : Customization.GetPropertyHandles())
	{
		const TSharedPtr<FPropertyHandleBase>& Handle = StaticCastSharedPtr<FPropertyHandleBase>(CurPropertyHandle);
		if (const TSharedPtr<FPropertyNode> Node = Handle->GetPropertyNode())
		{
			if (!SeenNodes.Contains(Node.ToSharedRef()))
			{
				SeenNodes.Add(Node.ToSharedRef());
				OutNodes.Add(Node.ToSharedRef());
			}
		}
	}
}

TSharedPtr<IDetailPropertyRow> FDetailItemNode::GetRow() const
{
	if (Customization.IsValidCustomization() && Customization.PropertyRow.IsValid())
	{
		return Customization.PropertyRow;
	}
	return nullptr;
}

TSharedPtr<FComplexPropertyNode> FDetailItemNode::GetExternalRootPropertyNode() const
{
	if (Customization.IsValidCustomization() && Customization.PropertyRow.IsValid())
	{
		return Customization.PropertyRow->GetExternalRootNode();
	}
	return nullptr;
}

void FDetailItemNode::FilterNode(const FDetailFilter& InFilter)
{
	bShouldBeVisibleDueToFiltering = PassesAllFilters(this, Customization, InFilter, ParentCategory.Pin()->GetDisplayName().ToString());
	if (!bShouldBeVisibleDueToFiltering && ParentGroup.IsValid() && !ParentGroup.Pin()->GetGroupName().IsNone())
	{
		bShouldBeVisibleDueToFiltering = PassesAllFilters(this, Customization, InFilter, ParentGroup.Pin()->GetGroupName().ToString());
	}

	// set bForceHidden if this node is loose and loose properties are hidden
	if (TSharedPtr<FPropertyNode> PropertyNodePin = Customization.GetPropertyNode())
	{
		if (LIKELY(!InFilter.bShowLooseProperties))
		{
			if (FProperty* Property = PropertyNodePin->GetProperty())
			{
				if (Property->GetBoolMetaData(NAME_IsLooseMetadata))
				{
					bForceHidden = true;
				}
			}
		}

		if (!bForceHidden && InFilter.ShouldForceHideProperty.IsBound())
		{
			if (Customization.PropertyRow.IsValid())
			{
				TSharedPtr<IPropertyHandle> PropertyHandle = Customization.PropertyRow->GetPropertyHandle();
				if (PropertyHandle.IsValid() && InFilter.ShouldForceHideProperty.Execute(PropertyHandle.ToSharedRef()))
				{
					bForceHidden = true;
				}
			}
			
		}

		if (!bForceHidden)
		{
			if (FProperty* Property = PropertyNodePin->GetProperty())
			{
				if (UPropertyVisibilityOverrideSubsystem* PropertyVisibilityOverrideSubsystem = UPropertyVisibilityOverrideSubsystem::Get())
				{
					if (Customization.PropertyRow.IsValid())
					{
						if (TSharedPtr<IPropertyHandle> PropertyHandle = Customization.PropertyRow->GetPropertyHandle())
						{
							bForceHidden = PropertyVisibilityOverrideSubsystem->ShouldHidePropertyForDetailsPanel(*PropertyHandle);
						}
					}
					
					if (Property->GetBoolMetaData(PropertyNames::PropertyVisibilityOverrideName))
					{
						// legacy path
						if (!bForceHidden)
						{
							bForceHidden = PropertyVisibilityOverrideSubsystem->ShouldHideProperty(Property);
						}
					}
				}
			}
		}
	}

	bShouldBeVisibleDueToChildFiltering = false;

	// Filter each child
	for( int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex )
	{
		TSharedRef<FDetailTreeNode>& Child = Children[ChildIndex];

		// If the parent is visible, we pass an empty filter to all children so that they resume their
		// default expansion.  This is a lot safer method, otherwise customized details panels tend to be
		// filtered incorrectly because they have no means of discovering if their parents were filtered.
		if ( bShouldBeVisibleDueToFiltering )
		{
			FDetailFilter ChildFilter;
			ChildFilter.bShowLooseProperties = InFilter.bShowLooseProperties; // bShowLooseProperties is inherited from parent regardless
			ChildFilter.ShouldForceHideProperty = InFilter.ShouldForceHideProperty; // ShouldForceHideProperty is inherited from parent regardless
			Child->FilterNode(ChildFilter);

			// The child should be visible, but maybe something else has it hidden, check if it's
			// visible just for safety reasons.
			if ( Child->GetVisibility() == ENodeVisibility::Visible )
			{
				// Expand the child after filtering if it wants to be expanded
				ParentCategory.Pin()->RequestItemExpanded(Child, Child->ShouldBeExpanded());
			}
		}
		else
		{
			Child->FilterNode(InFilter);

			if ( Child->GetVisibility() == ENodeVisibility::Visible )
			{
				if ( !InFilter.IsEmptyFilter() )
				{
					// The child is visible due to filtering so we must also be visible
					bShouldBeVisibleDueToChildFiltering = true;
				}

				// Expand the child after filtering if it wants to be expanded
				ParentCategory.Pin()->RequestItemExpanded(Child, Child->ShouldBeExpanded());
			}
		}
	}

	// if this is a subcategory, it should only be visible if one or more of its children is visible
	if (Customization.HasPropertyNode() &&
		Customization.GetPropertyNode()->AsCategoryNode() && 
		bShouldBeVisibleDueToFiltering)
	{
		bool bAnyChildVisible = false;
		for (const TSharedRef<FDetailTreeNode>& Child : Children)
		{
			if (Child->GetVisibility() == ENodeVisibility::Visible)
			{
				bAnyChildVisible = true;
				break;
			}
		}
		if (!bAnyChildVisible)
		{
			bShouldBeVisibleDueToFiltering = false;
		}
	}
}
