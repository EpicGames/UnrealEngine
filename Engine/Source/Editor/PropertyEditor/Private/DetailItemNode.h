// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "PropertyPath.h"
#include "IPropertyUtilities.h"
#include "DetailTreeNode.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "DetailCategoryBuilderImpl.h"
#include "PropertyCustomizationHelpers.h"

/**
 * A single item in a detail tree                                                              
 */
class FDetailItemNode : public FDetailTreeNode, public TSharedFromThis<FDetailItemNode>
{
public:
	FDetailItemNode( const FDetailLayoutCustomization& InCustomization, TSharedRef<FDetailCategoryImpl> InParentCategory, TAttribute<bool> InIsParentEnabled, TSharedPtr<IDetailGroup> InParentGroup = nullptr);
	~FDetailItemNode();

	/** IDetailTreeNode interface */
	virtual EDetailNodeType GetNodeType() const override;
	virtual TSharedPtr<IPropertyHandle> CreatePropertyHandle() const override;
	virtual void GetFilterStrings(TArray<FString>& OutFilterStrings) const override;
	virtual bool GetInitiallyCollapsed() const override;
	virtual void RefreshVisibility() override;

	/**
	 * Initializes this node                                                              
	 */
	void Initialize();

	void ToggleExpansion();

	void SetExpansionState(bool bWantsExpanded, bool bSaveState);
	void SetExpansionState(bool bWantsExpanded);
	/**
	 * Generates children for this node
	 *
	 * @param bUpdateFilteredNodes If true, details panel will re-filter to account for new nodes being added
	 */
	void GenerateChildren( bool bUpdateFilteredNodes );

	/**
	 * @return TRUE if this node has a widget with multiple columns                                                              
	 */
	bool HasMultiColumnWidget() const;

	/**
	 * @return true if this node has any children (regardless of child visibility)
	 */
	bool HasGeneratedChildren() const { return Children.Num() > 0;}

	/**
	 * @return The new, uncached visibility of this item.
	 */
	EVisibility ComputeItemVisibility() const;

	/**
	 * updates the cached node visibility and optionally calls a tree refresh if it changed
	 */
	void RefreshCachedVisibility(bool bCallChangeDelegate = false);

	/** FDetailTreeNode interface */
	virtual TSharedPtr<IDetailsView> GetNodeDetailsViewSharedPtr() const override { TSharedPtr<FDetailCategoryImpl> PC = GetParentCategory(); return PC.IsValid() ? PC->GetNodeDetailsViewSharedPtr() : nullptr; }
	virtual TSharedPtr<IDetailsViewPrivate> GetDetailsViewSharedPtr() const override{ TSharedPtr<FDetailCategoryImpl> PC = GetParentCategory(); return PC.IsValid() ? PC->GetDetailsViewSharedPtr() : nullptr; }
	virtual TSharedRef< ITableRow > GenerateWidgetForTableView( const TSharedRef<STableViewBase>& OwnerTable, bool bAllowFavoriteSystem) override;
	virtual bool GenerateStandaloneWidget(FDetailWidgetRow& OutRow) const override;
	virtual void GetChildren(FDetailNodeList& OutChildren, const bool& bInIgnoreVisibility = false) override;
	virtual void OnItemExpansionChanged( bool bInIsExpanded, bool bShouldSaveState) override;
	virtual bool ShouldBeExpanded() const override;
	virtual ENodeVisibility GetVisibility() const override;
	virtual void FilterNode( const FDetailFilter& InFilter ) override;
	virtual void Tick( float DeltaTime ) override;
	virtual bool ShouldShowOnlyChildren() const override;
	virtual FName GetNodeName() const override { return Customization.GetName(); }
	virtual TSharedPtr<FDetailCategoryImpl> GetParentCategory() const override { return ParentCategory.Pin(); }
	virtual FPropertyPath GetPropertyPath() const override;
	virtual void SetIsHighlighted(bool bInIsHighlighted) override { bIsHighlighted = bInIsHighlighted; }
	virtual bool IsHighlighted() const override { return bIsHighlighted; }
	virtual bool IsLeaf() override { return Children.Num() == 0;  }
	virtual TAttribute<bool> IsPropertyEditingEnabled() const override;
	virtual TSharedPtr<FPropertyNode> GetPropertyNode() const override;
	virtual void GetAllPropertyNodes(TArray<TSharedRef<FPropertyNode>>& OutNodes) const override;
	virtual TSharedPtr<FComplexPropertyNode> GetExternalRootPropertyNode() const override;
	virtual TSharedPtr<IDetailPropertyRow> GetRow() const override;

private:

	/**
	 * Initializes the property editor on this node                                                              
	 */
	void InitPropertyEditor();

	/**
	 * Initializes the custom builder on this node                                                              
	 */
	void InitCustomBuilder();

	/**
	 * Initializes the detail group on this node                                                              
	 */
	void InitGroup();

	/**
	 * Implementation of IsPropertyEditingEnabled
	 */
	bool IsPropertyEditingEnabledImpl() const;

private:
	/** Customization on this node */
	FDetailLayoutCustomization Customization;
	/** Child nodes of this node */
	FDetailNodeList Children;
	/** Parent categories on this node */
	TWeakPtr<FDetailCategoryImpl> ParentCategory;
	/** Parent group on this node */
	TWeakPtr<IDetailGroup> ParentGroup;
	/** Attribute for checking if our parent is enabled */
	TAttribute<bool> IsParentEnabled;
	/** Cached visibility of this node */
	EVisibility CachedItemVisibility;
	/** If true, this node will be hidden regardless of if it's parent or children would've otherwise overridden the filter result */
	bool bForceHidden;
	/** True if this node passes filtering */
	bool bShouldBeVisibleDueToFiltering;
	/** True if this node is visible because its children are filtered successfully */
	bool bShouldBeVisibleDueToChildFiltering;
	/** True if this node should be ticked */
	bool bTickable;
	/** True if this node is expanded */
	bool bIsExpanded;
	/** True if this node is highlighted */
	bool bIsHighlighted;
};
