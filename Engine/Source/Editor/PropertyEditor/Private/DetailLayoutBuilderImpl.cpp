// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailLayoutBuilderImpl.h"
#include "CategoryPropertyNode.h"
#include "DetailCategoryBuilderImpl.h"
#include "DetailMultiTopLevelObjectRootNode.h"
#include "DetailPropertyRow.h"
#include "IPropertyGenerationUtilities.h"
#include "ObjectEditorUtils.h"
#include "ObjectPropertyNode.h"
#include "PropertyEditorHelpers.h"
#include "PropertyHandleImpl.h"
#include "StructurePropertyNode.h"

FDetailLayoutBuilderImpl::FDetailLayoutBuilderImpl(TSharedPtr<FComplexPropertyNode>& InRootNode, FClassToPropertyMap& InPropertyMap, const TSharedRef<IPropertyUtilities>& InPropertyUtilities, const TSharedRef<IPropertyGenerationUtilities>& InPropertyGenerationUtilities, const TSharedPtr< IDetailsViewPrivate >& InDetailsView, bool bIsExternal)
	: RootNode( InRootNode )
	, PropertyMap( InPropertyMap )
	, PropertyDetailsUtilities( InPropertyUtilities )
	, PropertyGenerationUtilities( InPropertyGenerationUtilities )
	, DetailsView( InDetailsView )
	, CurrentCustomizationClass( nullptr )
	, bLayoutForExternalRoot(bIsExternal)
{
}


FDetailLayoutBuilderImpl::~FDetailLayoutBuilderImpl()
{
	ClearExternalRootPropertyNodes();
}

IDetailCategoryBuilder& FDetailLayoutBuilderImpl::EditCategory(FName CategoryName, const FText& NewLocalizedDisplayName, ECategoryPriority::Type CategoryType)
{
	FText LocalizedDisplayName = NewLocalizedDisplayName;

	// Use a generic name if one was not specified
	if (CategoryName == NAME_None)
	{
		static const FText GeneralString = NSLOCTEXT("DetailLayoutBuilderImpl", "General", "General");
		static const FName GeneralName = TEXT("General");
	
		CategoryName = GeneralName;
		LocalizedDisplayName = GeneralString;
	}

	return EditCategoryAllowNone(CategoryName, LocalizedDisplayName, CategoryType);
}

IDetailCategoryBuilder& FDetailLayoutBuilderImpl::EditCategoryAllowNone(FName CategoryName, const FText& NewLocalizedDisplayName, ECategoryPriority::Type CategoryType)
{
	TSharedPtr<FDetailCategoryImpl> CategoryImpl;
	// If the default category map had a category by the provided name, remove it from the map as it is now customized
	if (!DefaultCategoryMap.RemoveAndCopyValue(CategoryName, CategoryImpl))
	{
		// Default category map did not have a category by the requested name. Find or add it to the custom map
		TSharedPtr<FDetailCategoryImpl>& NewCategoryImpl = CustomCategoryMap.FindOrAdd(CategoryName);

		if (!NewCategoryImpl.IsValid())
		{
			NewCategoryImpl = MakeShareable(new FDetailCategoryImpl(CategoryName, SharedThis(this)));
			// Categories within a type should display in the order they were added but sorting is unstable so numbers are made unique
			const int32 SortOrder = CategoryType * 1000 + (CustomCategoryMap.Num() - 1);
			NewCategoryImpl->SetSortOrder( SortOrder );
		}
		CategoryImpl = NewCategoryImpl;
	}
	else
	{
		// Custom category should not exist yet as it was in the default category map
		checkSlow(!CustomCategoryMap.Contains(CategoryName) && CategoryImpl.IsValid());
		CustomCategoryMap.Add(CategoryName, CategoryImpl);

		// Categories within a type should display in the order they were added but sorting is unstable so numbers are made unique
		const int32 SortOrder = CategoryType * 1000 + (CustomCategoryMap.Num() - 1);
		CategoryImpl->SetSortOrder( SortOrder );
	}
	CategoryImpl->SetDisplayName(CategoryName, NewLocalizedDisplayName);

	return *CategoryImpl;
}

void FDetailLayoutBuilderImpl::GetCategoryNames(TArray<FName>& OutCategoryNames) const
{
	OutCategoryNames.Reserve(OutCategoryNames.Num() + DefaultCategoryMap.Num() + CustomCategoryMap.Num());

	TArray<FName> TempCategoryNames;
	DefaultCategoryMap.GenerateKeyArray(TempCategoryNames);
	OutCategoryNames.Append(TempCategoryNames);
	CustomCategoryMap.GenerateKeyArray(TempCategoryNames);
	OutCategoryNames.Append(TempCategoryNames);
}

IDetailPropertyRow& FDetailLayoutBuilderImpl::AddPropertyToCategory(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	// Get the FProperty itself
	FProperty* Property = InPropertyHandle->GetProperty();

	// Get the property's category name
	FName CategoryFName = FObjectEditorUtils::GetCategoryFName(Property);

	// Get the layout builder's category builder
	IDetailCategoryBuilder& MyCategory = EditCategory(CategoryFName);

	// Add the property to the category
	return MyCategory.AddProperty(InPropertyHandle);
}

FDetailWidgetRow& FDetailLayoutBuilderImpl::AddCustomRowToCategory(TSharedPtr<IPropertyHandle> InPropertyHandle, const FText& CustomSearchString, bool bForAdvanced)
{
	// Get the FProperty itself
	FProperty* Property = InPropertyHandle->GetProperty();

	// Get the property's category name
	FName CategoryFName = FObjectEditorUtils::GetCategoryFName(Property);

	// Get the layout builder's category builder
	IDetailCategoryBuilder& MyCategory = EditCategory(CategoryFName);

	// Add the property to the category
	return MyCategory.AddCustomRow(CustomSearchString, bForAdvanced);
}

TSharedPtr<IPropertyHandle> FDetailLayoutBuilderImpl::AddObjectPropertyData(TConstArrayView<UObject*> Objects, FName PropertyName)
{
	TSharedPtr<IPropertyHandle> Handle;

	if (PropertyName != NAME_None)
	{
		TSharedPtr<FObjectPropertyNode> RootPropertyNode = MakeShared<FObjectPropertyNode>();

		for (UObject* Obj : Objects)
		{
			RootPropertyNode->AddObject(Obj);
		}
		FPropertyNodeInitParams Params;
		Params.bAllowChildren = false;

		RootPropertyNode->InitNode(Params);

		if (TSharedPtr<FPropertyNode> PropertyNode = RootPropertyNode->GenerateSingleChild(PropertyName))
		{
			// This is useless as PropertyNode should already be in the child nodes
			RootPropertyNode->AddChildNode(PropertyNode);
			PropertyNode->RebuildChildren();
			Handle = GetPropertyHandle(PropertyNode);
			AddExternalRootPropertyNode(RootPropertyNode.ToSharedRef());
			
			FClassInstanceToPropertyMap& ClassInstanceToPropertyMap = PropertyMap.FindOrAdd(PropertyNode->GetProperty()->GetOwnerStruct()->GetFName());
			FPropertyNodeMap& PropertyNodeMap = ClassInstanceToPropertyMap.FindOrAdd(NAME_None);
			PropertyNodeMap.Add(PropertyName, PropertyNode);
		}
	}

	return Handle;
}

TSharedPtr<IPropertyHandle> FDetailLayoutBuilderImpl::AddStructurePropertyData(const TSharedPtr<FStructOnScope>& StructData, FName PropertyName)
{
	TSharedPtr<IPropertyHandle> Handle;

	if (PropertyName != NAME_None && StructData && StructData->IsValid())
	{
		TSharedPtr<FStructurePropertyNode> RootPropertyNode = MakeShared<FStructurePropertyNode>();
		
		RootPropertyNode->SetStructure(StructData);
		RootPropertyNode->InitNode(FPropertyNodeInitParams());

		for (int32 ChildIdx = 0; ChildIdx < RootPropertyNode->GetNumChildNodes(); ++ChildIdx)
		{
			TSharedPtr< FPropertyNode > PropertyNode = RootPropertyNode->GetChildNode(ChildIdx);
			if (FProperty* Property = PropertyNode->GetProperty())
			{
				if (Property->GetFName() == PropertyName)
				{
					AddExternalRootPropertyNode(RootPropertyNode.ToSharedRef());
					FClassInstanceToPropertyMap& ClassInstanceToPropertyMap = PropertyMap.FindOrAdd(PropertyNode->GetProperty()->GetOwnerStruct()->GetFName());
					FPropertyNodeMap& PropertyNodeMap = ClassInstanceToPropertyMap.FindOrAdd(NAME_None);
					PropertyNodeMap.Add(PropertyName, PropertyNode);

					RootPropertyNode->AddChildNode(PropertyNode);
					PropertyNode->RebuildChildren();
					Handle = GetPropertyHandle(PropertyNode);
					break;
				}
			}
		}
	}

	return Handle;
}

IDetailPropertyRow* FDetailLayoutBuilderImpl::EditDefaultProperty(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	if (InPropertyHandle.IsValid() && InPropertyHandle->IsValidHandle())
	{
		TSharedPtr<FPropertyNode> PropertyNode = GetPropertyNode(InPropertyHandle);
		if (PropertyNode.IsValid())
		{
			FProperty* Property = InPropertyHandle->GetProperty();

			// Get the property's category name
			FName CategoryFName = FObjectEditorUtils::GetCategoryFName(Property);

			// Get the layout builder's category builder
			TSharedPtr<FDetailCategoryImpl> Category = DefaultCategoryMap.FindRef(CategoryFName);
			if (!Category.IsValid())
			{
				Category = CustomCategoryMap.FindRef(CategoryFName);
			}
			if (!Category.IsValid())
			{
				Category = SubCategoryMap.FindRef(CategoryFName);
			}

			if(Category.IsValid())
			{
				FDetailLayoutCustomization* Customization = Category->GetDefaultCustomization(PropertyNode.ToSharedRef());
				if (Customization)
				{
					return Customization->PropertyRow.Get();
				}
			}
		}
	}
	return nullptr;
}

IDetailPropertyRow* FDetailLayoutBuilderImpl::EditPropertyFromRoot(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	for (const TSharedRef<FDetailTreeNode>& RootTreeNode : AllRootTreeNodes)
	{
		FDetailNodeList ChildNodes;
		RootTreeNode->GetChildren(ChildNodes, true/*bIgnoreVisibility*/);
		for (const TSharedRef<FDetailTreeNode>& ChildNode : ChildNodes)
		{
			if (TSharedPtr<IDetailPropertyRow> PropertyRow = ChildNode->GetRow())
			{
				if (PropertyRow->GetPropertyHandle() == InPropertyHandle)
				{
					return PropertyRow.Get();
				}
			}
		}
	}
	return nullptr;
}

bool FDetailLayoutBuilderImpl::DoesCategoryHaveGeneratedChildren(FName CategoryName)
{
	FDetailNodeList Children;

	FDetailCategoryImpl* Category = nullptr;
	for (const TSharedRef<FDetailTreeNode>& RootTreeNode : AllRootTreeNodes)
	{
		if (RootTreeNode->GetNodeType() == EDetailNodeType::Category && 
			CategoryName == RootTreeNode->GetNodeName())
		{
			Category = (FDetailCategoryImpl*)&RootTreeNode.Get();
		}
	}

	if (Category)
	{
		Category->GetGeneratedChildren(Children, /*bIgnoreVisibility*/true, /*bIgnoreAdvancedDropdown*/false);
	}

	return Children.Num() > 0;
}


TSharedRef<IPropertyHandle> FDetailLayoutBuilderImpl::GetProperty( const FName PropertyPath, const UStruct* ClassOutermost, FName InInstanceName ) const
{	
	TSharedPtr<FPropertyHandleBase> PropertyHandle; 

	TSharedPtr<FPropertyNode> PropertyNodePtr = GetPropertyNode( PropertyPath, ClassOutermost, InInstanceName );
	
	return GetPropertyHandle( PropertyNodePtr );

}

FName FDetailLayoutBuilderImpl::GetTopLevelProperty()
{
	if (!PropertyMap.IsEmpty())
	{
		return PropertyMap.CreateConstIterator().Key();
	}
	return NAME_None;
}

void FDetailLayoutBuilderImpl::HideProperty( const TSharedPtr<IPropertyHandle> PropertyHandle )
{
	if( PropertyHandle.IsValid() && PropertyHandle->IsValidHandle() )
	{
		// Mark the property as customized so it wont show up in the default location
		TSharedPtr<FPropertyNode> PropertyNode = GetPropertyNode( PropertyHandle );
		if( PropertyNode.IsValid() )
		{
			SetCustomProperty( PropertyNode );
		}
	}
}

void FDetailLayoutBuilderImpl::HideProperty( FName PropertyPath, const UStruct* ClassOutermost, FName InstanceName )
{
	TSharedPtr<FPropertyNode> PropertyNode = GetPropertyNode( PropertyPath, ClassOutermost, InstanceName );
	if( PropertyNode.IsValid() )
	{
		SetCustomProperty( PropertyNode );
	}
}

void FDetailLayoutBuilderImpl::ForceRefreshDetails()
{
	PropertyDetailsUtilities.Pin()->ForceRefresh();
}

FDetailCategoryImpl& FDetailLayoutBuilderImpl::DefaultCategory(FName CategoryName)
{
	for (const TSharedRef<FDetailTreeNode>& RootTreeNode : AllRootTreeNodes)
	{
		if (RootTreeNode->GetNodeType() == EDetailNodeType::Category && 
			CategoryName == RootTreeNode->GetNodeName())
		{
			return (FDetailCategoryImpl&) RootTreeNode.Get();
		}
	}

	TSharedPtr<FDetailCategoryImpl>& CategoryImpl = DefaultCategoryMap.FindOrAdd(CategoryName);
	if (!CategoryImpl.IsValid())
	{
		CategoryImpl = MakeShareable(new FDetailCategoryImpl(CategoryName, SharedThis(this)));

		// We want categories within a type to display in the order they were added but sorting is unstable so we make unique numbers 
		uint32 SortOrder = (uint32) ECategoryPriority::Default * 1000 + (DefaultCategoryMap.Num() - 1);
		CategoryImpl->SetSortOrder(SortOrder);

		CategoryImpl->SetDisplayName(CategoryName, FText::GetEmpty());
	}

	return *CategoryImpl;
}

TSharedPtr<FDetailCategoryImpl> FDetailLayoutBuilderImpl::GetSubCategoryImpl(FName CategoryName) const
{
	return SubCategoryMap.FindRef(CategoryName);
}

bool FDetailLayoutBuilderImpl::HasCategory(FName CategoryName) const
{
	return DefaultCategoryMap.Contains(CategoryName);
}

void FDetailLayoutBuilderImpl::BuildCategories( const FCategoryMap& CategoryMap, TArray< TSharedRef<FDetailCategoryImpl> >& OutSimpleCategories, TArray< TSharedRef<FDetailCategoryImpl> >& OutAdvancedCategories )
{		
	for( FCategoryMap::TConstIterator It(CategoryMap); It; ++It )
	{
		TSharedRef<FDetailCategoryImpl> DetailCategory = It.Value().ToSharedRef();

		TSharedPtr<FComplexPropertyNode> RootPropertyNode = GetRootNode();
		const bool bCategoryHidden = PropertyEditorHelpers::IsCategoryHiddenByClass(RootPropertyNode, DetailCategory->GetCategoryName())
			|| ForceHiddenCategories.Contains(DetailCategory->GetCategoryName());

		if( !bCategoryHidden )
		{
			DetailCategory->GenerateLayout();

			if( DetailCategory->ContainsOnlyAdvanced() )
			{
				OutAdvancedCategories.Add( DetailCategory );
			}
			else
			{
				OutSimpleCategories.Add( DetailCategory );
			}
		}
	}
}

void FDetailLayoutBuilderImpl::GenerateDetailLayout()
{
	AllRootTreeNodes.Empty();

	// Sort by the order in which categories were edited
	struct FCompareFDetailCategoryImpl
	{
		FORCEINLINE bool operator()( TSharedPtr<FDetailCategoryImpl> A, TSharedPtr<FDetailCategoryImpl> B ) const
		{
			return A->GetSortOrder() < B->GetSortOrder();
		}
	};

	TArray<TSharedRef<FDetailCategoryImpl>> SimpleCategories;
	TArray<TSharedRef<FDetailCategoryImpl>> AdvancedOnlyCategories;

	// Remove all subcategories
	{
		const TSharedRef<FDetailLayoutBuilderImpl> This = SharedThis(this);

		// The map of Default Categories added after removing Sub categories
		// Only used when there is no support for sub-categories
		FCategoryMap DefaultCategoryMapToAppend;

		// Store the Default Category Map Num as it will be decreasing as sub category entries are removed
		const int32 DefaultCategoryMapCount = DefaultCategoryMap.Num();

		FName ParentStructPropertyName = NAME_None;
		bool bSupportsSubCategory = false;

		if (TSharedPtr<FComplexPropertyNode> RootNodePtr = RootNode.Pin())
		{
			if (FStructProperty* const ParentStructProperty = CastField<FStructProperty>(RootNodePtr->GetProperty()))
			{
				ParentStructPropertyName = ParentStructProperty->GetFName();
			}

			// Currently only Object Nodes with Show Categories support Sub-categories
			bSupportsSubCategory = RootNodePtr->AsObjectNode() && RootNodePtr->HasNodeFlags(EPropertyNodeFlags::ShowCategories);
		}

		SubCategoryMap.Empty();

		for (FCategoryMap::TIterator It(DefaultCategoryMap); It; ++It)
		{
			TSharedPtr<FDetailCategoryImpl> DetailCategory = It.Value();

			int32 Index = INDEX_NONE;
			const FString CategoryNameStr = DetailCategory->GetCategoryName().ToString();
			if (!CategoryNameStr.FindChar(FPropertyNodeConstants::CategoryDelimiterChar, Index))
			{
				// No category delimiter found
				continue;
			}

			// Note: Sub-categories are added later if supported
			if (bSupportsSubCategory)
			{
				SubCategoryMap.Add(It.Key(), DetailCategory);
			}
			// When Sub category isn't supported, generate properties and move them to parent category
			else
			{
				const FName ParentCategoryName = *CategoryNameStr.Left(Index);

				TSharedPtr<FDetailCategoryImpl>* const ExistingParentDetailCategory = DefaultCategoryMap.Find(ParentCategoryName);

				TSharedPtr<FDetailCategoryImpl>& ParentDetailCategory = ExistingParentDetailCategory
					? *ExistingParentDetailCategory
					: DefaultCategoryMapToAppend.FindOrAdd(ParentCategoryName);

				if (!ParentDetailCategory.IsValid())
				{
					ParentDetailCategory = MakeShared<FDetailCategoryImpl>(ParentCategoryName, This);
					ParentDetailCategory->SetSortOrder(DetailCategory->GetSortOrder());
					ParentDetailCategory->SetDisplayName(ParentCategoryName, FText::GetEmpty());
				}

				// Move the Property Nodes from the sub-category to the parent category
				// To do this, generate a layout for the sub category here as they're unsupported and won't have an opportunity to do it later
				FDetailNodeList ChildNodes;
				DetailCategory->GenerateLayout();
				DetailCategory->GetGeneratedChildren(ChildNodes, /*bIgnoreVisibility*/true, /*bIgnoreAdvancedDropdown*/true);
				TArray<TSharedPtr<FPropertyNode>> AddedNodes;
				for (const TSharedRef<FDetailTreeNode>& ChildNode : ChildNodes)
				{
					TSharedPtr<FPropertyNode> PropertyNode = ChildNode->GetPropertyNode();
					if (!PropertyNode.IsValid())
					{
						// If we don't have a property node, then we're likely a custom generated node from a parent node. 
						// If so, add the parent node to the parent category if it has a valid property node, but ensure we don't add more than once, 
						// as there may be more than one custom row added.
						TWeakPtr<FDetailTreeNode> ParentNode = ChildNode->GetParentNode();
						if (ParentNode.IsValid())
						{
							PropertyNode = ParentNode.Pin()->GetPropertyNode();
							if (AddedNodes.Contains(PropertyNode))
							{
								continue;
							}
						}
						if (!PropertyNode.IsValid())
						{
							continue;
						}
					}
					else if (ChildNode->GetExternalRootPropertyNode()) 
					{
						// Also skip children that have been added externally, as those are likely generated from one of the other
						// child nodes, and will be generated again upon moving that node to the outer category
						continue;
					}

					// If there is no outer object then the class is the object root and there is only one instance
					FName InstanceName = ParentStructPropertyName;
					FPropertyNode* const ParentNode = PropertyNode->GetParentNode();
					if (ParentNode && ParentNode->GetProperty())
					{
						InstanceName = ParentNode->GetProperty()->GetFName();
					}
					ParentDetailCategory->AddPropertyNode(PropertyNode.ToSharedRef(), InstanceName);
					AddedNodes.Add(PropertyNode);
				}
			}

			It.RemoveCurrent();
		}

		DefaultCategoryMap.Append(MoveTemp(DefaultCategoryMapToAppend));
	}

	// Build default categories
	while (DefaultCategoryMap.Num() > 0)
	{
		FCategoryMap DefaultCategoryMapCopy = DefaultCategoryMap;

		DefaultCategoryMap.Empty();

		BuildCategories(DefaultCategoryMapCopy, SimpleCategories, AdvancedOnlyCategories);
	}

	// Customizations can add more categories while customizing so just keep doing this until the maps are empty
	while (CustomCategoryMap.Num() > 0)
	{
		FCategoryMap CustomCategoryMapCopy = CustomCategoryMap;

		CustomCategoryMap.Empty();

		BuildCategories(CustomCategoryMapCopy, SimpleCategories, AdvancedOnlyCategories);
	}

	FDetailNodeList CategoryNodes;

	// Run initial sort
	SimpleCategories.Sort(FCompareFDetailCategoryImpl());
	AdvancedOnlyCategories.Sort(FCompareFDetailCategoryImpl());

	if (CategorySortOrderFunctions.Num() > 0)
	{
		TMap<FName, IDetailCategoryBuilder*> AllCategoryMap;
		TArray<TSharedRef<FDetailCategoryImpl>> AllCategories;

		for (TSharedRef<FDetailCategoryImpl>& CategoryImpl : SimpleCategories)
		{
			const FName CategoryName = CategoryImpl->GetCategoryName();
			AllCategories.Add(CategoryImpl);
			IDetailCategoryBuilder* Builder = static_cast<IDetailCategoryBuilder*>(&CategoryImpl.Get());
			AllCategoryMap.Add(CategoryName, Builder);
		}

		for (TSharedRef<FDetailCategoryImpl>& CategoryImpl : AdvancedOnlyCategories)
		{
			const FName CategoryName = CategoryImpl->GetCategoryName();
			AllCategories.Add(CategoryImpl);
			IDetailCategoryBuilder* Builder = static_cast<IDetailCategoryBuilder*>(&CategoryImpl.Get());
			AllCategoryMap.Add(CategoryName, Builder);
		}

		// Run second override function sort
		for (FOnCategorySortOrderFunction& SortFunction : CategorySortOrderFunctions)
		{
			SortFunction(AllCategoryMap);
		}
		AllCategories.Sort(FCompareFDetailCategoryImpl());

		// Merge the two category lists in sorted order
		for (int32 CategoryIndex = 0; CategoryIndex < AllCategories.Num(); ++CategoryIndex)
		{
			CategoryNodes.AddUnique(AllCategories[CategoryIndex]);
		}
	}
	else
	{
		// Merge the two category lists in sorted order
		for (int32 CategoryIndex = 0; CategoryIndex < SimpleCategories.Num(); ++CategoryIndex)
		{
			CategoryNodes.AddUnique(SimpleCategories[CategoryIndex]);
		}

		for (int32 CategoryIndex = 0; CategoryIndex < AdvancedOnlyCategories.Num(); ++CategoryIndex)
		{
			CategoryNodes.AddUnique(AdvancedOnlyCategories[CategoryIndex]);
		}
	}

	TSharedPtr<FComplexPropertyNode> RootNodePinned = RootNode.Pin();
	TSharedPtr<IDetailsViewPrivate> DetailsViewPinned = DetailsView.Pin();
	if (DetailsViewPinned && DetailsViewPinned->GetRootObjectCustomization() && RootNodePinned->GetInstancesNum() && !bLayoutForExternalRoot)
	{
		FObjectPropertyNode* ObjectNode = RootNodePinned->AsObjectNode();

		TSharedPtr<IDetailRootObjectCustomization> RootObjectCustomization = DetailsViewPinned->GetRootObjectCustomization();

		// there are multiple objects in the details panel.  Separate each one with a unique object name node to differentiate them
		TSharedRef<FDetailMultiTopLevelObjectRootNode> NewRootNode = MakeShared<FDetailMultiTopLevelObjectRootNode>(RootObjectCustomization, DetailsViewPinned, ObjectNode);
		NewRootNode->SetChildren(CategoryNodes);

		AllRootTreeNodes.Add(NewRootNode);
	}
	else
	{
		// The categories are the root in this case
		AllRootTreeNodes = CategoryNodes;
	}
}

void FDetailLayoutBuilderImpl::FilterDetailLayout( const FDetailFilter& InFilter )
{
	CurrentFilter = InFilter;
	FilteredRootTreeNodes.Reset();

	for(int32 RootNodeIndex = 0; RootNodeIndex < AllRootTreeNodes.Num(); ++RootNodeIndex)
	{
		TSharedRef<FDetailTreeNode>& RootTreeNode = AllRootTreeNodes[RootNodeIndex];

		RootTreeNode->FilterNode(InFilter);

		if(RootTreeNode->GetVisibility() == ENodeVisibility::Visible)
		{
			if (InFilter.VisibleCategories.IsEmpty() || InFilter.VisibleCategories.Contains(RootTreeNode->GetNodeName()))
			{
				FilteredRootTreeNodes.Add(RootTreeNode);
				if (TSharedPtr<IDetailsViewPrivate> DetailsViewPinned = DetailsView.Pin())
				{
					DetailsViewPinned->RequestItemExpanded(RootTreeNode, RootTreeNode->ShouldBeExpanded());
				}
			}
		}
	}
}

void FDetailLayoutBuilderImpl::SetCurrentCustomizationClass( UStruct* CurrentClass, FName VariableName )
{
	CurrentCustomizationClass = CurrentClass;
	CurrentCustomizationVariableName = VariableName;
}

TSharedPtr<FPropertyNode> FDetailLayoutBuilderImpl::GetPropertyNode( const FName PropertyName, const UStruct* ClassOutermost, FName InstanceName ) const
{
	TSharedPtr<FPropertyNode> PropertyNode = GetPropertyNodeInternal( PropertyName, ClassOutermost, InstanceName );
	return PropertyNode;
}

/**
 * Parses a path node string into a property and index  The string should be in the format "Property[Index]" for arrays or "Property" for non arrays
 * 
 * @param OutProperty	The property name parsed from the string
 * @param OutIndex		The index of the property in an array (INDEX_NONE if not found)
 */
static void GetPropertyAndIndex( const FString& PathNode, FString& OutProperty, int32& OutIndex )
{
	OutIndex = INDEX_NONE;
	FString FoundIndexStr;
	// Split the text into the property (left of the brackets) and index Right of the open bracket
	if( PathNode.Split( TEXT("["), &OutProperty, &FoundIndexStr, ESearchCase::IgnoreCase, ESearchDir::FromEnd ) )
	{
		// Convert the index string into a number 
		OutIndex = FCString::Atoi( *FoundIndexStr );
	}
	else
	{
		// No index was found, the path node is just the property
		OutProperty = PathNode;
	}
}
/**
 * Finds a child property node from the provided parent node (does not recurse into grandchildren)
 *
 * @param InParentNode	The parent node to locate the child from
 * @param PropertyName	The property name to find
 * @param Index			The index of the property if its in an array
 */
static TSharedPtr<FPropertyNode> FindChildPropertyNode( FPropertyNode& InParentNode, const FString& PropertyName, int32 Index )
{
	TSharedPtr<FPropertyNode> FoundNode(nullptr);

	// search each child for a property with the provided name
	for( int32 ChildIndex = 0; ChildIndex < InParentNode.GetNumChildNodes(); ++ChildIndex )
	{
		TSharedPtr<FPropertyNode>& ChildNode = InParentNode.GetChildNode(ChildIndex);
		FProperty* Property = ChildNode->GetProperty();
		if( Property && Property->GetFName() == *PropertyName )
		{
			FoundNode = ChildNode;
			break;
		}
	}

	// Find the array element.
	if( FoundNode.IsValid() && Index != INDEX_NONE )
	{
		// The found node is the top array so get its child which is the actual node
		FoundNode = FoundNode->GetChildNode( Index );
	}

	return FoundNode;
}

TSharedPtr<FPropertyNode> FDetailLayoutBuilderImpl::GetPropertyNode( TSharedPtr<IPropertyHandle> PropertyHandle ) const
{
	TSharedPtr<FPropertyNode> PropertyNode = nullptr;
	if( PropertyHandle->IsValidHandle() )
	{
		PropertyNode = StaticCastSharedPtr<FPropertyHandleBase>(PropertyHandle)->GetPropertyNode();
	}

	return PropertyNode;
}

/**
 Contains the location of a property, by path

 Supported format: instance_name'outer.outer.value[optional_index]
 Instance name is needed if multiple UProperties of the same type exist (such as two identical structs, the instance name is one of the struct variable names)
 Items in arrays are indexed by []
 Example setup
*/

TSharedPtr<FPropertyNode> FDetailLayoutBuilderImpl::GetPropertyNodeInternal( const FName PropertyPath, const UStruct* ClassOutermost, FName InstanceName ) const
{
	FName PropertyName;
	TArray<FString> PathList;
	PropertyPath.ToString().ParseIntoArray( PathList, TEXT("."), true );

	int32 ArrayFoundIndex = 0;
	if( PathList.Num() == 1 && !PathList[0].FindChar(TEXT('['), ArrayFoundIndex))
	{
		PropertyName = FName( *PathList[0] );
	}
	// The class to find properties in defaults to the class currently being customized
	FName ClassName = CurrentCustomizationClass ? CurrentCustomizationClass->GetFName() : NAME_None;
	if( ClassOutermost != nullptr)
	{
		// The requested a different class
		ClassName = ClassOutermost->GetFName();
	}

	// Find the outer variable name.  This only matters if there are multiple instances of the same property
	FName OuterVariableName = CurrentCustomizationVariableName;
	if( InstanceName != NAME_None )
	{
		OuterVariableName = InstanceName;
	}

	// If this fails there are no properties associated with the class name provided
	FClassInstanceToPropertyMap* ClassInstanceToPropertyMapPtr = PropertyMap.Find( ClassName );

	if( ClassInstanceToPropertyMapPtr )
	{
		FClassInstanceToPropertyMap& ClassInstanceToPropertyMap = *ClassInstanceToPropertyMapPtr;
		if( OuterVariableName == NAME_None && ClassInstanceToPropertyMap.Num() == 1 )
		{
			// If the outer  variable name still wasnt specified and there is only one instance, just use that
			auto FirstKey = ClassInstanceToPropertyMap.CreateIterator();
			OuterVariableName = FirstKey.Key();
		}

		FPropertyNodeMap* PropertyNodeMapPtr = ClassInstanceToPropertyMap.Find( OuterVariableName );

		if( PropertyNodeMapPtr )
		{
			FPropertyNodeMap& PropertyNodeMap = *PropertyNodeMapPtr;
			// Check for property name fast path first
			if( PropertyName != NAME_None )
			{
				// The property name was ambiguous or not found if this fails.  If ambiguous, it means there are multiple data same typed data structures(components or structs) in the class which
				// causes multiple properties by the same name to exist.  These properties must be found via the path method.
				return PropertyNodeMap.PropertyNameToNode.FindRef( PropertyName );
			}
			else
			{
				// We need to search through the tree for a property with the given path
				TSharedPtr<FPropertyNode> PropertyNode;

				// Path should be in the format A[optional_index].B.C
				if( PathList.Num() )
				{
					// Get the base property and index
					FString Property;
					int32 Index;
					GetPropertyAndIndex( PathList[0], Property, Index );

					// Get the parent most property node which is the one in the map.  Its children need to be searched
					PropertyNode = PropertyNodeMap.PropertyNameToNode.FindRef( FName( *Property ) );
					if( PropertyNode.IsValid() )
					{
						if( Index != INDEX_NONE )
						{
							if (Index >= PropertyNode->GetNumChildNodes())
							{
								return nullptr;
							}

							// The parent is the actual array, its children are array elements
							PropertyNode = PropertyNode->GetChildNode( Index );
						}

						// Search any additional paths for the child
						for( int32 PathIndex = 1; PathIndex < PathList.Num(); ++PathIndex )
						{
							GetPropertyAndIndex( PathList[PathIndex], Property, Index );

							PropertyNode = FindChildPropertyNode( *PropertyNode, Property, Index );
						}
					}
				}

				return PropertyNode;
			}
		}
	}

	return nullptr;
}


TSharedRef<IPropertyHandle> FDetailLayoutBuilderImpl::GetPropertyHandle( TSharedPtr<FPropertyNode> PropertyNodePtr ) const
{
	TSharedPtr<IPropertyHandle> PropertyHandle;
	if( PropertyNodePtr.IsValid() )
	{ 
		TSharedRef<FPropertyNode> PropertyNode = PropertyNodePtr.ToSharedRef();
		FNotifyHook* NotifyHook = GetPropertyUtilities()->GetNotifyHook();

		PropertyHandle = PropertyEditorHelpers::GetPropertyHandle( PropertyNode, NotifyHook, PropertyDetailsUtilities.Pin() );
	}
	else
	{
		// Invalid handle
		PropertyHandle = MakeShareable( new FPropertyHandleBase(nullptr, nullptr, nullptr) );
	}	

	return PropertyHandle.ToSharedRef();
}

void FDetailLayoutBuilderImpl::AddExternalRootPropertyNode(TSharedRef<FComplexPropertyNode> InExternalRootNode)
{
	ExternalRootPropertyNodes.Add(InExternalRootNode);
	if (TSharedPtr<IDetailsViewPrivate> DetailsViewPinned = DetailsView.Pin())
	{
		DetailsViewPinned->RestoreExpandedItems(InExternalRootNode);
	}
}

void FDetailLayoutBuilderImpl::RemoveExternalRootPropertyNode(TSharedRef<FComplexPropertyNode> InExternalRootNode)
{
	int32 NumRemoved = ExternalRootPropertyNodes.RemoveAll([InExternalRootNode](TSharedPtr<FComplexPropertyNode> ExternalRootPropertyNode)
	{
		return ExternalRootPropertyNode == InExternalRootNode;
	});

	if (NumRemoved > 0)
	{
		if (TSharedPtr<IDetailsViewPrivate> DetailsViewPinned = DetailsView.Pin())
		{
			DetailsViewPinned->SaveExpandedItems(InExternalRootNode);
		}
	}
}

void FDetailLayoutBuilderImpl::ClearExternalRootPropertyNodes()
{
	if (TSharedPtr<IDetailsViewPrivate> DetailsViewPinned = DetailsView.Pin())
	{
		for (const TSharedPtr<FComplexPropertyNode>& ExternalRootPropertyNode : ExternalRootPropertyNodes)
		{
			DetailsViewPinned->SaveExpandedItems(ExternalRootPropertyNode.ToSharedRef());
		}
	}

	ExternalRootPropertyNodes.Empty();
}

FDelegateHandle FDetailLayoutBuilderImpl::AddNodeVisibilityChangedHandler(FSimpleMulticastDelegate::FDelegate InOnNodeVisibilityChanged)
{
	return OnNodeVisibilityChanged.Add(InOnNodeVisibilityChanged);
}

void FDetailLayoutBuilderImpl::RemoveNodeVisibilityChangedHandler(FDelegateHandle DelegateHandle)
{
	OnNodeVisibilityChanged.Remove(DelegateHandle);
}

void FDetailLayoutBuilderImpl::NotifyNodeVisibilityChanged()
{
	OnNodeVisibilityChanged.Broadcast();
}

IPropertyGenerationUtilities& FDetailLayoutBuilderImpl::GetPropertyGenerationUtilities() const
{
	TSharedPtr<IPropertyGenerationUtilities> PropertyGenerationUtilitiesPinned = PropertyGenerationUtilities.Pin();
	checkf(PropertyGenerationUtilitiesPinned.IsValid(), TEXT("Property generation utilities were destroyed while the layout builder was still in use."));
	return *PropertyGenerationUtilitiesPinned.Get();
}

FCustomPropertyTypeLayoutMap FDetailLayoutBuilderImpl::GetInstancedPropertyTypeLayoutMap() const
{
	FCustomPropertyTypeLayoutMap TypeLayoutMap = GetPropertyGenerationUtilities().GetInstancedPropertyTypeLayoutMap();
	TypeLayoutMap.Append(InstancePropertyTypeExtensions);
	return TypeLayoutMap;
}

void FDetailLayoutBuilderImpl::RefreshNodeVisbility()
{
	for(FDetailTreeNode* Node : TickableNodes)
	{
		if (!TickableNodesToRemove.Contains(Node))
		{
			Node->RefreshVisibility();
		}
	}
}

TSharedPtr<FAssetThumbnailPool> FDetailLayoutBuilderImpl::GetThumbnailPool() const
{
	return PropertyDetailsUtilities.Pin()->GetThumbnailPool();
}

bool FDetailLayoutBuilderImpl::IsPropertyVisible( TSharedRef<IPropertyHandle> PropertyHandle ) const
{
	if (PropertyHandle->IsValidHandle())
	{
		if (TSharedPtr<IDetailsViewPrivate> DetailsViewPinned = DetailsView.Pin())
		{
			TSharedPtr<FPropertyNode> PropertyNode = StaticCastSharedRef<FPropertyHandleBase>(PropertyHandle)->GetPropertyNode();
			const FCategoryPropertyNode* CategoryNode = PropertyNode.IsValid() ? PropertyNode->AsCategoryNode() : nullptr;
			if (CategoryNode != nullptr)
			{
				// this is a subcategory
				FName CategoryName = CategoryNode->GetCategoryName();
				return DetailsViewPinned->IsCustomRowVisible(FName(), CategoryName);
			}
			else if (PropertyHandle->GetProperty() != nullptr)
			{
				FPropertyAndParent PropertyAndParent(PropertyHandle);
				return DetailsViewPinned->IsPropertyVisible(PropertyAndParent);
			}
		}
	}

	return true;
}

bool FDetailLayoutBuilderImpl::IsPropertyVisible( const struct FPropertyAndParent& PropertyAndParent ) const
{
	TSharedPtr<IDetailsViewPrivate> DetailsViewPinned = DetailsView.Pin();
	return DetailsViewPinned ? DetailsViewPinned->IsPropertyVisible( PropertyAndParent ) : true;
}

void FDetailLayoutBuilderImpl::HideCategory( FName CategoryName )
{
	ForceHiddenCategories.Add(CategoryName);
}

TSharedPtr<const IDetailsView> FDetailLayoutBuilderImpl::GetDetailsViewSharedPtr() const
{ 
	return DetailsView.Pin();
}

void FDetailLayoutBuilderImpl::GetObjectsBeingCustomized( TArray< TWeakObjectPtr<UObject> >& OutObjects ) const
{
	OutObjects.Empty();

	FObjectPropertyNode* RootObjectNode = RootNode.IsValid() ? RootNode.Pin()->AsObjectNode() : nullptr;

	// The class to find properties in defaults to the class currently being customized
	FName ClassName = CurrentCustomizationClass ? CurrentCustomizationClass->GetFName() : NAME_None;

	if(ClassName != NAME_None && CurrentCustomizationVariableName != NAME_None)
	{
		// If this fails there are no properties associated with the class name provided
		FClassInstanceToPropertyMap* ClassInstanceToPropertyMapPtr = PropertyMap.Find(ClassName);

		if(ClassInstanceToPropertyMapPtr)
		{
			FClassInstanceToPropertyMap& ClassInstanceToPropertyMap = *ClassInstanceToPropertyMapPtr;

			FPropertyNodeMap* PropertyNodeMapPtr = ClassInstanceToPropertyMap.Find(CurrentCustomizationVariableName);

			if(PropertyNodeMapPtr)
			{
				FPropertyNodeMap& PropertyNodeMap = *PropertyNodeMapPtr;
				FObjectPropertyNode* ParentObjectProperty = PropertyNodeMap.ParentProperty ? PropertyNodeMap.ParentProperty->AsObjectNode() : NULL;
				if(ParentObjectProperty)
				{
					for(int32 ObjectIndex = 0; ObjectIndex < ParentObjectProperty->GetNumObjects(); ++ObjectIndex)
					{
						OutObjects.Add(ParentObjectProperty->GetUObject(ObjectIndex));
					}
				}
			}
		}
	}
	else if(RootObjectNode)
	{
		for (int32 ObjectIndex = 0; ObjectIndex < RootObjectNode->GetNumObjects(); ++ObjectIndex)
		{
			OutObjects.Add(RootObjectNode->GetUObject(ObjectIndex));
		}
	}
}

void FDetailLayoutBuilderImpl::GetStructsBeingCustomized( TArray< TSharedPtr<FStructOnScope> >& OutStructs ) const
{
	OutStructs.Reset();

	FStructurePropertyNode* RootStructNode = RootNode.IsValid() ? RootNode.Pin()->AsStructureNode() : nullptr;

	// The class to find properties in defaults to the class currently being customized
	FName ClassName = CurrentCustomizationClass ? CurrentCustomizationClass->GetFName() : NAME_None;

	if(ClassName != NAME_None && CurrentCustomizationVariableName != NAME_None)
	{
		// If this fails there are no properties associated with the class name provided
		FClassInstanceToPropertyMap* ClassInstanceToPropertyMapPtr = PropertyMap.Find(ClassName);

		if(ClassInstanceToPropertyMapPtr)
		{
			FClassInstanceToPropertyMap& ClassInstanceToPropertyMap = *ClassInstanceToPropertyMapPtr;

			FPropertyNodeMap* PropertyNodeMapPtr = ClassInstanceToPropertyMap.Find(CurrentCustomizationVariableName);

			if(PropertyNodeMapPtr)
			{
				FPropertyNodeMap& PropertyNodeMap = *PropertyNodeMapPtr;
				FComplexPropertyNode* ParentComplexProperty = PropertyNodeMap.ParentProperty ? PropertyNodeMap.ParentProperty->AsComplexNode() : nullptr;
				FStructurePropertyNode* StructureNode = ParentComplexProperty ? ParentComplexProperty->AsStructureNode() : nullptr;
				if(StructureNode)
				{
					StructureNode->GetAllStructureData(OutStructs);
				}
			}
		}
	}

	if(RootStructNode)
	{
		RootStructNode->GetAllStructureData(OutStructs);
	}
}

TSharedRef< IPropertyUtilities > FDetailLayoutBuilderImpl::GetPropertyUtilities() const
{
	return PropertyDetailsUtilities.Pin().ToSharedRef();
}

UClass* FDetailLayoutBuilderImpl::GetBaseClass() const
{
	return RootNode.IsValid() ? Cast<UClass>(RootNode.Pin()->GetBaseStructure()) : nullptr;
}

const TArray<TWeakObjectPtr<UObject>>& FDetailLayoutBuilderImpl::GetSelectedObjects() const
{
	return PropertyDetailsUtilities.Pin()->GetSelectedObjects();
}


bool FDetailLayoutBuilderImpl::HasClassDefaultObject() const
{
	return PropertyDetailsUtilities.Pin()->HasClassDefaultObject();
}

void FDetailLayoutBuilderImpl::SetCustomProperty( const TSharedPtr<FPropertyNode>& PropertyNode ) 
{
	PropertyNode->SetNodeFlags( EPropertyNodeFlags::IsCustomized, true );
}

void FDetailLayoutBuilderImpl::Tick(float DeltaTime)
{
	TGuardValue Guard(bIsIteratingTickableNodes, true);

	auto TickNode = [this, DeltaTime](FDetailTreeNode* Node)
	{
		if (!TickableNodesToRemove.Contains(Node))
		{
			// Skip ticking tree nodes which point to destroyed property nodes.
			// This can happen when because the update order is this:
			//	- update property nodes, calling DestroyTree(), and creating new nodes
			//	- update layout builders (but old ones might still be referenced by the tree view) 
			//  - tick layout builders, which includes the stale builders
			//  - refresh tree view, which finally gets rid of the stale builders
			TSharedPtr<FPropertyNode> PropertyNode = Node->GetPropertyNode();
			if (!(PropertyNode.IsValid() && PropertyNode->IsDestroyed()))
			{
				Node->Tick(DeltaTime);
			}
		}
	};

	for( FDetailTreeNode* Node : TickableNodes)
	{
		TickNode(Node);
	}

	while (!TickableNodesToAdd.IsEmpty())
	{
		// Tick the new nodes, but keep them in a local set so we don't modify TickableNodesToAdd while ticking them
		TSet<FDetailTreeNode*> NodesToAdd = MoveTemp(TickableNodesToAdd);
		TickableNodesToAdd.Reset();

		for( FDetailTreeNode* Node : NodesToAdd)
		{
			TickNode(Node);
		}

		TickableNodes.Append(MoveTemp(NodesToAdd));
	}

	for( FDetailTreeNode* Node : TickableNodesToRemove )
	{
		TickableNodes.Remove(Node);
	}
	TickableNodesToRemove.Reset();
}

void FDetailLayoutBuilderImpl::AddTickableNode( FDetailTreeNode& TickableNode )
{
	if (bIsIteratingTickableNodes)
	{
		TickableNodesToAdd.Add(&TickableNode);
	}
	else
	{
		TickableNodes.Add(&TickableNode);
	}
}

void FDetailLayoutBuilderImpl::RemoveTickableNode( FDetailTreeNode& TickableNode )
{
	if (bIsIteratingTickableNodes)
	{
		TickableNodesToRemove.Add(&TickableNode);
	}
	else
	{
		TickableNodes.Remove(&TickableNode);
	}
}

void FDetailLayoutBuilderImpl::SaveExpansionState( const FString& NodePath, bool bIsExpanded )
{
	if (TSharedPtr<IDetailsViewPrivate> DetailsViewPinned = DetailsView.Pin())
	{
		DetailsViewPinned->SaveCustomExpansionState(NodePath, bIsExpanded);
	}
}

bool FDetailLayoutBuilderImpl::GetSavedExpansionState( const FString& NodePath ) const
{
	TSharedPtr<IDetailsViewPrivate> DetailsViewPinned = DetailsView.Pin();
	return DetailsViewPinned ? DetailsViewPinned->GetCustomSavedExpansionState(NodePath) : false;
}

void FDetailLayoutBuilderImpl::RegisterInstancedCustomPropertyTypeLayout(FName PropertyTypeName, FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate, TSharedPtr<IPropertyTypeIdentifier> Identifier)
{
	FPropertyTypeLayoutCallback Callback;
	Callback.PropertyTypeLayoutDelegate = PropertyTypeLayoutDelegate;
	Callback.PropertyTypeIdentifier = Identifier;

	FPropertyTypeLayoutCallbackList* LayoutCallbacks = InstancePropertyTypeExtensions.Find(PropertyTypeName);
	if (LayoutCallbacks)
	{
		LayoutCallbacks->Add(Callback);
	}
	else
	{
		FPropertyTypeLayoutCallbackList NewLayoutCallbacks;
		NewLayoutCallbacks.Add(Callback);
		InstancePropertyTypeExtensions.Add(PropertyTypeName, NewLayoutCallbacks);
	}
}

void FDetailLayoutBuilderImpl::SortCategories(const FOnCategorySortOrderFunction& InSortFunction)
{
	CategorySortOrderFunctions.Add(InSortFunction);
}

void FDetailLayoutBuilderImpl::SetPropertyGenerationAllowListPaths(const TSet<FString>& InPropertyGenerationAllowListPaths)
{
	PropertyGenerationAllowListPaths = InPropertyGenerationAllowListPaths;
}

bool FDetailLayoutBuilderImpl::IsPropertyPathAllowed(const FString& InPath) const
{
	if (PropertyGenerationAllowListPaths.IsEmpty())
	{
		return true;
	}

	for (const FString& PropertyName : PropertyGenerationAllowListPaths)
	{
		if (InPath.StartsWith(PropertyName) || PropertyName.StartsWith(InPath))
		{
			return true;
		}
	}

	return false;
}

void FDetailLayoutBuilderImpl::DisableInstancedReference(TSharedRef<IPropertyHandle> PropertyHandle) const
{
	TSharedPtr<FPropertyNode> PropertyNode = GetPropertyNode(PropertyHandle);
	if (PropertyNode.IsValid())
	{
		PropertyNode->SetIgnoreInstancedReference();
	}
}

bool FDetailLayoutBuilderImpl::AddEmptyCategoryIfNeeded(TSharedPtr<FComplexPropertyNode> Node)
{
	TSharedPtr<IDetailsViewPrivate> DetailsViewPinned = DetailsView.Pin();

	const bool bHasNoValidCategories = DefaultCategoryMap.IsEmpty();
	const bool bHasValidDisplayManager = DetailsViewPinned && DetailsViewPinned->GetDisplayManager().IsValid();
	const bool bHasValidPropertyNode = Node.IsValid();
	
	if ( bHasNoValidCategories &&
		 bHasValidDisplayManager &&
		 bHasValidPropertyNode )
	{
 		return DetailsViewPinned->GetDisplayManager()->AddEmptyCategoryToDetailLayoutIfNeeded(Node.ToSharedRef(), SharedThis(this));
	}
	return false;
}
