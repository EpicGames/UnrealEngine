// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IPropertyUtilities.h"
#include "PropertyNode.h"
#include "PropertyHandle.h"
#include "IDetailsViewPrivate.h"
#include "SDetailsViewBase.h"
#include "DetailLayoutBuilder.h"

class FDetailCategoryImpl;
class IDetailCategoryBuilder;
class IPropertyUtilities;
class IPropertyGenerationUtilities;

class FDetailLayoutBuilderImpl : public IDetailLayoutBuilder, public TSharedFromThis<FDetailLayoutBuilderImpl>
{
public:
	FDetailLayoutBuilderImpl(
		TSharedPtr<FComplexPropertyNode>& InRootNode,
		FClassToPropertyMap& InPropertyMap,
		const TSharedRef<IPropertyUtilities>& InPropertyUtilities,
		const TSharedRef<IPropertyGenerationUtilities>& InPropertyGenerationUtilities,
		const TSharedPtr<IDetailsViewPrivate>& InDetailsView,
		bool bIsExternal);

	virtual ~FDetailLayoutBuilderImpl() override;

	/** IDetailLayoutBuilder Interface */
	virtual TSharedPtr<const IDetailsView> GetDetailsViewSharedPtr() const override;
	virtual void GetObjectsBeingCustomized(TArray< TWeakObjectPtr<UObject> >& OutObjects) const override;
	virtual void GetStructsBeingCustomized(TArray< TSharedPtr<FStructOnScope> >& OutStructs) const override;
	virtual IDetailCategoryBuilder& EditCategory(FName CategoryName, const FText& NewLocalizedDisplayName = FText::GetEmpty(), ECategoryPriority::Type CategoryType = ECategoryPriority::Default) override;
	virtual IDetailCategoryBuilder& EditCategoryAllowNone(FName CategoryName, const FText& NewLocalizedDisplayName = FText::GetEmpty(), ECategoryPriority::Type CategoryType = ECategoryPriority::Default) override;
	virtual void GetCategoryNames(TArray<FName>& OutCategoryNames) const override;
	virtual IDetailPropertyRow& AddPropertyToCategory(TSharedPtr<IPropertyHandle> InPropertyHandle) override;
	virtual FDetailWidgetRow& AddCustomRowToCategory(TSharedPtr<IPropertyHandle> InPropertyHandle, const FText& InCustomSearchString, bool bForAdvanced = false) override;
	virtual TSharedPtr<IPropertyHandle> AddObjectPropertyData(TConstArrayView<UObject*> Objects, FName PropertyName) override;
	virtual TSharedPtr<IPropertyHandle> AddStructurePropertyData(const TSharedPtr<FStructOnScope>& StructData, FName PropertyName) override;
	virtual IDetailPropertyRow* EditDefaultProperty(TSharedPtr<IPropertyHandle> InPropertyHandle) override;
	virtual IDetailPropertyRow* EditPropertyFromRoot(TSharedPtr<IPropertyHandle> InPropertyHandle) override;
	virtual bool DoesCategoryHaveGeneratedChildren(FName CategoryName) override;
	virtual TSharedRef<IPropertyHandle> GetProperty(const FName PropertyPath, const UStruct* ClassOutermost, FName InInstanceName) const override;
	virtual FName GetTopLevelProperty() override;
	virtual void HideProperty(const TSharedPtr<IPropertyHandle> Property) override;
	virtual void HideProperty(FName PropertyPath, const UStruct* ClassOutermost = NULL, FName InstanceName = NAME_None) override;
	virtual void ForceRefreshDetails() override;
	virtual TSharedPtr<FAssetThumbnailPool> GetThumbnailPool() const override;
	virtual bool IsPropertyVisible(TSharedRef<IPropertyHandle> PropertyHandle) const override;
	virtual bool IsPropertyVisible(const struct FPropertyAndParent& PropertyAndParent) const override;
	virtual void HideCategory(FName CategoryName) override;
	virtual TSharedRef<IPropertyUtilities> GetPropertyUtilities() const override;
	virtual UClass* GetBaseClass() const override;
	virtual const TArray<TWeakObjectPtr<UObject>>& GetSelectedObjects() const override;
	virtual bool HasClassDefaultObject() const override;
	virtual void RegisterInstancedCustomPropertyTypeLayout(FName PropertyTypeName, FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate, TSharedPtr<IPropertyTypeIdentifier> Identifier = nullptr) override;
	virtual void SortCategories(const FOnCategorySortOrderFunction& SortFunction) override;
	virtual void SetPropertyGenerationAllowListPaths(const TSet<FString>& InPropertyGenerationAllowListPaths) override;
	virtual bool IsPropertyPathAllowed(const FString& InPath) const override;
	virtual void DisableInstancedReference(TSharedRef<IPropertyHandle> PropertyHandle) const override;

	/**
	 * Creates an empty category row if there currently are no categories and one is required for the
	 * @code TSharedPtr<FComplexPropertyNode> @endcode Node
	 *
	 * @param Node The @code TSharedPtr<FComplexPropertyNode> @endcode that we will add an empty category for, if needed
	 *
	 * @return true if an empty/stub category was added, else it returns false 
	 */
	bool AddEmptyCategoryIfNeeded(TSharedPtr<FComplexPropertyNode> Node);

	/**
	 * Creates a default category. The SDetails view will generate widgets in default categories
	 *
	 * @param CategoryName	The name of the category to create
	 */
	FDetailCategoryImpl& DefaultCategory(FName CategoryName);

	/** 
	 * Find a subcategory with the given name, if it exists
	 */
	TSharedPtr<FDetailCategoryImpl> GetSubCategoryImpl(FName CategoryName) const;

	/**
	 * @returns true if a category with the given name exists
	 */
	bool HasCategory(FName CategoryName) const;

	/**
	 * Generates the layout for this detail builder
	 */
	void GenerateDetailLayout();

	/**
	 * Filters the layout based on the passed in filter
	 */
	void FilterDetailLayout(const FDetailFilter& InFilter);

	/**
	 * Sets the current class being asked for customization
	 * Used for looking up properties if a class is not provided
	 *
	 * @param CurrentClass	The current class being customized
	 * @param VariableName The variable name of the class being customized (used for inner classes/structs where there can be multiple instances of them)
	 */
	void SetCurrentCustomizationClass(UStruct* CurrentClass, FName VariableName);

	/** @return The current class variable name being customized */
	FName GetCurrentCustomizationVariableName() const { return CurrentCustomizationVariableName; }

	/**
	 * Finds a property node for the current property
	 *
	 * @param PropertyPath	The path to the property
	 * @param ClassOutermost	The outer class of the property
	 * @return The found property node
	 */
	TSharedPtr<FPropertyNode> GetPropertyNode(FName PropertyPath, const UStruct* ClassOutermost, FName InstanceName) const;

	/**
	 * Gets the property node from the provided handle
	 *
	 * @param PropertyHandle	The property handle to get the node from
	 */
	TSharedPtr<FPropertyNode> GetPropertyNode(TSharedPtr<IPropertyHandle> PropertyHandle) const;

	/**
	 * Marks a property as customized
	 *
	 * @param PropertyNode	The property node to mark as customized
	 */
	void SetCustomProperty(const TSharedPtr<FPropertyNode>& PropertyNode);

	/**
	 * @return All tree nodes that should be visible in the tree
	 */
	FDetailNodeList& GetFilteredRootTreeNodes() { return FilteredRootTreeNodes; }

	/**
	 * @return All root tree nodes, regardless of visibility.
	 */
	FDetailNodeList& GetAllRootTreeNodes() { return AllRootTreeNodes; }

	/**
	 * @return true if the layout has any details
	 */
	bool HasDetails() const { return AllRootTreeNodes.Num() > 0; }

	/**
	 * Ticks tickable nodes (if any)
	 */
	void Tick(float DeltaTime);

	/**
	 * Adds a node that should be ticked each frame
	 *
	 * @param TickableNode	The node to tick
	 */
	void AddTickableNode(FDetailTreeNode& TickableNode);

	/**
	 * Removes a node that should no longer be ticked each frame
	 *
	 * @param TickableNode	The node to remove
	 */
	void RemoveTickableNode(FDetailTreeNode& TickableNode);

	/**
	 * @return The current filter that is being used to show or hide rows
	 */
	const FDetailFilter& GetCurrentFilter() const { return CurrentFilter; }

	/**
	 * Saves the expansion state of a tree node
	 *
	 * @param NodePath	The path to the detail node to save
	 * @param bIsExpanded	true if the node is expanded, false otherwise
	 */
	void SaveExpansionState(const FString& NodePath, bool bIsExpanded);

	/**
	 * Gets the saved expansion state of a tree node in this category
	 *
	 * @param NodePath	The path to the detail node to get
	 * @return true if the node should be expanded, false otherwise
	 */
	bool GetSavedExpansionState(const FString& NodePath) const;

	/**
	 * Makes a property handle from a property node
	 *
	 * @param PropertyNodePtr	The property node to make a handle for.
	 */
	TSharedRef<IPropertyHandle> GetPropertyHandle(TSharedPtr<FPropertyNode> PropertyNodePtr) const;

	/**
	 * Adds an external property root node to the list of root nodes that the details new needs to manage
	 *
	 * @param InExternalRootNode		The node to add
	 */
	void AddExternalRootPropertyNode(TSharedRef<FComplexPropertyNode> InExternalRootNode);

	/**
	* Removes an external property root node to the list of root nodes that the details new needs to manage
	*
	* @param InExternalRootNode		The node to remove
	*/
	void RemoveExternalRootPropertyNode(TSharedRef<FComplexPropertyNode> InExternalRootNode);

	void ClearExternalRootPropertyNodes();

	/** @return The details view that owns this layout */
	TSharedPtr<IDetailsView> GetDetailsViewSharedPtr() override { return DetailsView.Pin(); }
	/** @return The root node for this customization */
	TSharedPtr<FComplexPropertyNode> GetRootNode() const { return RootNode.Pin(); }

	FRootPropertyNodeList& GetExternalRootPropertyNodes() { return ExternalRootPropertyNodes;  }

	/** @return True if the layout is for an external root property node and not in the main set of objects the details panel is observing */
	bool IsLayoutForExternalRoot() const { return bLayoutForExternalRoot; }

	/** Adds a handler for when a node this builder owns has had a forced visibility change. */
	FDelegateHandle AddNodeVisibilityChangedHandler(FSimpleMulticastDelegate::FDelegate InOnNodeVisibilityChanged);

	/** Removes a handler for when a node this builder owns has had a forced visibility change. */
	void RemoveNodeVisibilityChangedHandler(FDelegateHandle DelegateHandle);
	
	/** Notifies this detail layout builder that a node it owns had it's visibility forcibly changed. */
	void NotifyNodeVisibilityChanged();

	/** Gets internal utilities for generating property layouts. */
	IPropertyGenerationUtilities& GetPropertyGenerationUtilities() const;

	/** Combines the type layout map from our PropertyGenerationUtilities object, with the local InstancePropertyTypeExtensions map - which provides instance based customizations/overrides. */
	FCustomPropertyTypeLayoutMap GetInstancedPropertyTypeLayoutMap() const;

	void RefreshNodeVisbility();
	
private:
	/**
	 * Finds a property node for the current property by searching in a fast lookup map or a path search if required
	 *
	 * @param PropertyPath	The path to the property
	 * @param ClassOutermost	The outer class of the property
	 * @param bMarkPropertyAsCustomized	Whether or not to mark the property as customized so it does not appear in the default layout
	 * @return The found property node
	 */
	TSharedPtr<FPropertyNode> GetPropertyNodeInternal( FName PropertyPath, const UStruct* ClassOutermost, FName InstanceName ) const;

	/**
	 * Builds a list of simple and advanced categories that should be displayed
	 */
	void BuildCategories( const FCategoryMap& CategoryMap, TArray< TSharedRef<FDetailCategoryImpl> >& OutSimpleCategories, TArray< TSharedRef<FDetailCategoryImpl> >& OutAdvancedCategories );

private:
	/** The root property node for this customization */
	TWeakPtr<FComplexPropertyNode> RootNode;
	/** External property nodes which need to validated each tick */
	FRootPropertyNodeList ExternalRootPropertyNodes;
	/** A mapping of category names to categories which have been customized */
	FCategoryMap CustomCategoryMap;
	/** A mapping of category names to categories which have been not customized */
	FCategoryMap DefaultCategoryMap;
	/** Mapping of subcategory names to their subcategory builder */
	FCategoryMap SubCategoryMap;
	/** A mapping of classes to top level properties in that class */
	FClassToPropertyMap& PropertyMap;
	/** Force hidden categories set by the user */
	TSet<FName> ForceHiddenCategories;
	/** If not empty only nodes for property specified paths will be generated */
	TSet<FString> PropertyGenerationAllowListPaths;
	/** Nodes that require ticking */
	TSet<FDetailTreeNode*> TickableNodes;
	/** Nodes to add to being ticked, only used while iterating */
	TSet<FDetailTreeNode*> TickableNodesToAdd;
	/** Nodes to remove from being ticked, only used while iterating */
	TSet<FDetailTreeNode*> TickableNodesToRemove;
	/** Current filter applied to the view */
	FDetailFilter CurrentFilter;
	/** All RootTreeNodes */
	FDetailNodeList AllRootTreeNodes;
	/** All generated category widgets for rebuilding them if needed */
	FDetailNodeList FilteredRootTreeNodes;
	/** The current variable name of the class being customized (inner class instances)*/
	FName CurrentCustomizationVariableName;
	/** The global property utilties.  This is weak to avoid circular ref but it should always be valid if this class exists*/
	const TWeakPtr<IPropertyUtilities> PropertyDetailsUtilities;
	/** Internal utilities for generating property layouts. */
	const TWeakPtr<IPropertyGenerationUtilities> PropertyGenerationUtilities;
	/** The view where this detail customizer resides */
	TWeakPtr<IDetailsViewPrivate> DetailsView;
	/** The current class being customized */
	UStruct* CurrentCustomizationClass;
	/** True if the layout is for an external root property node and not in the main set of objects the details panel is observing */
	bool bLayoutForExternalRoot;
	/** True if the layout is for an external root property node and not in the main set of objects the details panel is observing */
	bool bIsIteratingTickableNodes = false;
	/** A delegate which is called whenever a node owned by this layout builder has it's visibility forcibly changed. */
	FSimpleMulticastDelegate OnNodeVisibilityChanged;

	/** Optionally provided functions to sort categories, which allow caller to reconfigure sort order. */
	TArray<FOnCategorySortOrderFunction> CategorySortOrderFunctions;

	/** Local customizations for this detail layout only. Provides a way for IDetailCustomizations to override type customizations without polluting customizations for other instances. */
	FCustomPropertyTypeLayoutMap InstancePropertyTypeExtensions;
};
