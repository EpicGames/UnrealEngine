// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailLayoutHelpers.h"
#include "DetailLayoutBuilderImpl.h"
#include "PropertyRowGenerator.h"
#include "DetailCategoryBuilderImpl.h"
#include "CategoryPropertyNode.h"
#include "ObjectEditorUtils.h"
#include "DetailPropertyRow.h"
#include "IDetailCustomization.h"
#include "ObjectPropertyNode.h"
#include "Modules/ModuleManager.h"
#include "UserInterface/PropertyEditor/SPropertyEditorEditInline.h"

namespace DetailLayoutHelpers
{
	void UpdateSinglePropertyMapRecursive(FPropertyNode& InNode, FName CurCategory, FComplexPropertyNode* CurObjectNode, FUpdatePropertyMapArgs& InUpdateArgs)
	{
		FDetailLayoutData& LayoutData = *InUpdateArgs.LayoutData;
		FDetailLayoutBuilderImpl& DetailLayout = *LayoutData.DetailLayout;

		const FStructProperty* ParentStructProp = CastField<FStructProperty>(InNode.GetProperty());
		for (int32 ChildIndex = 0; ChildIndex < InNode.GetNumChildNodes(); ++ChildIndex)
		{
			//Use the original value for each child
			bool LocalUpdateFavoriteSystemOnly = InUpdateArgs.bUpdateFavoriteSystemOnly;

			FUpdatePropertyMapArgs ChildArgs = InUpdateArgs;
			ChildArgs.bUpdateFavoriteSystemOnly = LocalUpdateFavoriteSystemOnly;

			TSharedPtr<FPropertyNode> ChildNodePtr = InNode.GetChildNode(ChildIndex);
			FPropertyNode& ChildNode = *ChildNodePtr;
			FProperty* Property = ChildNode.GetProperty();

			if (FObjectPropertyNode* ObjNode = ChildNode.AsObjectNode())
			{
				// Currently object property nodes do not provide any useful information other than being a container for its children.  We do not draw anything for them.
				// When we encounter object property nodes, add their children instead of adding them to the tree.
				UpdateSinglePropertyMapRecursive(ChildNode, CurCategory, ObjNode, ChildArgs);
			}
			else if (FCategoryPropertyNode* CategoryNode = ChildNode.AsCategoryNode())
			{
				if (!LocalUpdateFavoriteSystemOnly)
				{
					FName InstanceName = NAME_None;
					FName CategoryName = CurCategory;
					FString CategoryDelimiterString;
					CategoryDelimiterString.AppendChar(FPropertyNodeConstants::CategoryDelimiterChar);
					if (CurCategory != NAME_None && CategoryNode->GetCategoryName().ToString().Contains(CategoryDelimiterString))
					{
						// This property is child of another property so add it to the parent detail category
						FDetailCategoryImpl& CategoryImpl = DetailLayout.DefaultCategory(CategoryName);
						CategoryImpl.AddPropertyNode(ChildNodePtr.ToSharedRef(), InstanceName);
					}
				}

				// For category nodes, we just set the current category and recurse through the children
				UpdateSinglePropertyMapRecursive(ChildNode, CategoryNode->GetCategoryName(), CurObjectNode, ChildArgs);
			}
			else
			{
				// Whether or not the property can be visible in the default detail layout
				bool bVisibleByDefault = PropertyEditorHelpers::IsVisibleStandaloneProperty(ChildNode, InNode);

				// Whether or not the property is a struct
				const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
				const bool bIsStruct = StructProperty != NULL;

				static FName ShowOnlyInners("ShowOnlyInnerProperties");

				bool bIsCustomizedStruct = false;
				bool bIsChildOfCustomizedStruct = false;

				const UStruct* Struct = StructProperty ? StructProperty->Struct.Get() : NULL;
				const UStruct* ParentStruct = ParentStructProp ? ParentStructProp->Struct.Get() : NULL;
				if (Struct || ParentStruct)
				{
					FPropertyEditorModule& ParentPlugin = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
					if (Struct)
					{
						bIsCustomizedStruct = ParentPlugin.IsCustomizedStruct(Struct, *InUpdateArgs.InstancedPropertyTypeToDetailLayoutMap);
					}

					if (ParentStruct)
					{
						bIsChildOfCustomizedStruct = ParentPlugin.IsCustomizedStruct(ParentStruct, *InUpdateArgs.InstancedPropertyTypeToDetailLayoutMap);
					}
				}

				// Whether or not to push out struct properties to their own categories or show them inside an expandable struct
				// This recursively applies for any nested structs that have the ShowOnlyInners metadata 
				const bool bPushOutStructProps = bIsStruct && !bIsCustomizedStruct && Property->HasMetaData(ShowOnlyInners);

				// Is the property edit inline new 
				const bool bIsEditInlineNew = ChildNode.HasNodeFlags(EPropertyNodeFlags::ShowInnerObjectProperties) || SPropertyEditorEditInline::Supports(&ChildNode, ChildNode.GetArrayIndex());

				// Is this a property of a container property
				const bool bIsChildOfContainer = PropertyEditorHelpers::IsChildOfArray(ChildNode) || PropertyEditorHelpers::IsChildOfSet(ChildNode) || PropertyEditorHelpers::IsChildOfMap(ChildNode) || PropertyEditorHelpers::IsChildOfOption(ChildNode);

				// Edit inline new properties should be visible by default
				bVisibleByDefault |= bIsEditInlineNew;

				// Children of arrays are not visible directly,
				bVisibleByDefault &= !bIsChildOfContainer;

				// Inners of customized in structs should not be taken into consideration for customizing.  They are not designed to be individually customized when their parent is already customized
				if (!bIsChildOfCustomizedStruct && !LocalUpdateFavoriteSystemOnly)
				{
					// Add any object classes with properties so we can ask them for custom property layouts later
					LayoutData.ClassesWithProperties.Add(Property->GetOwnerStruct());
				}

				// If there is no outer object then the class is the object root and there is only one instance
				FName InstanceName = NAME_None;
				if (CurObjectNode && CurObjectNode->GetParentNode())
				{
					InstanceName = CurObjectNode->GetParentNode()->GetProperty()->GetFName();
				}
				else if (ParentStructProp)
				{
					InstanceName = ParentStructProp->GetFName();
				}

				// Do not add children of customized in struct properties or arrays
				if (!bIsChildOfCustomizedStruct && !bIsChildOfContainer && !LocalUpdateFavoriteSystemOnly)
				{
					// Get the class property map
					FClassInstanceToPropertyMap& ClassInstanceMap = LayoutData.ClassToPropertyMap.FindOrAdd(Property->GetOwnerStruct()->GetFName());

					FPropertyNodeMap& PropertyNodeMap = ClassInstanceMap.FindOrAdd(InstanceName);

					if (!PropertyNodeMap.ParentProperty)
					{
						PropertyNodeMap.ParentProperty = CurObjectNode;
					}
					else
					{
						ensure(PropertyNodeMap.ParentProperty == CurObjectNode);
					}

					checkSlow(!PropertyNodeMap.Contains(Property->GetFName()));

					PropertyNodeMap.Add(Property->GetFName(), ChildNodePtr);
				}

				if (bVisibleByDefault && !bPushOutStructProps)
				{
					FName CategoryName = CurCategory;
					// For properties inside a struct, add them to their own category unless they just take the name of the parent struct.  
					// In that case push them to the parent category
					FName PropertyCategoryName = FObjectEditorUtils::GetCategoryFName(Property);
					if (!ParentStructProp || (PropertyCategoryName != ParentStructProp->Struct->GetFName()))
					{
						CategoryName = PropertyCategoryName;
					}

					if (!LocalUpdateFavoriteSystemOnly)
					{
						// Add a property to the default category
						FDetailCategoryImpl& CategoryImpl = DetailLayout.DefaultCategory(CategoryName);
						CategoryImpl.AddPropertyNode(ChildNodePtr.ToSharedRef(), InstanceName);
					}

					if (InUpdateArgs.bEnableFavoriteSystem)
					{
						if (ChildNodePtr->IsFavorite())
						{
							// Find or create the favorite category, we have to duplicate favorite property row under this category
							static const FName FavoritesCategoryName(TEXT("Favorites"));
							FDetailCategoryImpl& FavoritesCategory = DetailLayout.DefaultCategory(FavoritesCategoryName);

							if (LocalUpdateFavoriteSystemOnly)
							{
								//If the parent has a condition that is not met, mark the child as read-only
								FDetailLayoutCustomization ParentTmpCustomization;
								ParentTmpCustomization.PropertyRow = MakeShared<FDetailPropertyRow>(InNode.AsShared(), FavoritesCategory.AsShared());
								if (ParentTmpCustomization.PropertyRow->GetPropertyEditor()->IsPropertyEditingEnabled() == false)
								{
									ChildNode.SetNodeFlags(EPropertyNodeFlags::IsReadOnly, true);
								}
							}

							// Add the property to the favorite by preserving its category, this will duplicate the row
							FavoritesCategory.AddPropertyNode(ChildNodePtr.ToSharedRef(), CurCategory);
						}

						if (bIsStruct)
						{
							LocalUpdateFavoriteSystemOnly = true;
						}

						ChildArgs.bUpdateFavoriteSystemOnly = LocalUpdateFavoriteSystemOnly;
					}
				}

				bool bRecurseIntoChildren =
					!bIsChildOfCustomizedStruct // Don't recurse into built in struct children, we already know what they are and how to display them
					&& !bIsCustomizedStruct // Don't recurse into customized structs
					&& !bIsChildOfContainer // Don't recurse into containers, the children are drawn by the container property parent
					&& !bIsEditInlineNew // Edit inline new children are not supported for customization yet
					&& (!bIsStruct || bPushOutStructProps); //  Only recurse into struct properties if they are going to be displayed as standalone properties in categories instead of inside an expandable area inside a category

				if (bRecurseIntoChildren || LocalUpdateFavoriteSystemOnly)
				{
					// Built in struct properties or children of arras 
					UpdateSinglePropertyMapRecursive(ChildNode, CurCategory, CurObjectNode, ChildArgs);
				}
			}
		}
	}

	void QueryLayoutForClass(FDetailLayoutData& LayoutData, UStruct* Class, const FCustomDetailLayoutMap& InstancedDetailLayoutMap)
	{
		LayoutData.DetailLayout->SetCurrentCustomizationClass(Class, NAME_None);

		FPropertyEditorModule& ParentPlugin = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		const FCustomDetailLayoutNameMap& GlobalCustomLayoutNameMap = ParentPlugin.GetClassNameToDetailLayoutNameMap();

		// Check the instanced map first
		const FDetailLayoutCallback* Callback = InstancedDetailLayoutMap.Find(Class);

		if (!Callback)
		{
			// callback wasn't found in the per instance map, try the global instances instead
			Callback = GlobalCustomLayoutNameMap.Find(Class->GetFName());
		}

		if (Callback && Callback->DetailLayoutDelegate.IsBound())
		{
			// Create a new instance of the custom detail layout for the current class
			TSharedRef<IDetailCustomization> CustomizationInstance = Callback->DetailLayoutDelegate.Execute();

			// Ask for details immediately
			CustomizationInstance->CustomizeDetails(LayoutData.DetailLayout);

			// Save the instance from destruction until we refresh
			LayoutData.CustomizationClassInstances.Add(CustomizationInstance);
		}
	}


	void QueryCustomDetailLayout(FDetailLayoutData& LayoutData, const FCustomDetailLayoutMap& InstancedDetailLayoutMap, const FOnGetDetailCustomizationInstance& GenericLayoutDelegate)
	{
		FPropertyEditorModule& ParentPlugin = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		// Get the registered classes that customize details
		const FCustomDetailLayoutNameMap& GlobalCustomLayoutNameMap = ParentPlugin.GetClassNameToDetailLayoutNameMap();

		UStruct* BaseStruct = LayoutData.DetailLayout->GetRootNode()->GetBaseStructure();

		LayoutData.CustomizationClassInstances.Empty();

		//Ask for generic details not specific to an object being viewed 
		if (GenericLayoutDelegate.IsBound())
		{
			// Create a new instance of the custom detail layout for the current class
			TSharedRef<IDetailCustomization> CustomizationInstance = GenericLayoutDelegate.Execute();

			// Ask for details immediately
			CustomizationInstance->CustomizeDetails(LayoutData.DetailLayout);

			// Save the instance from destruction until we refresh
			LayoutData.CustomizationClassInstances.Add(CustomizationInstance);
		}


		// Sort them by query order.  @todo not good enough
		struct FCompareFDetailLayoutCallback
		{
			FORCEINLINE bool operator()(const FDetailLayoutCallback& A, const FDetailLayoutCallback& B) const
			{
				return A.Order < B.Order;
			}
		};

		TMap< TWeakObjectPtr<UStruct>, const FDetailLayoutCallback*> FinalCallbackMap;

		for (auto ClassIt = LayoutData.ClassesWithProperties.CreateConstIterator(); ClassIt; ++ClassIt)
		{
			// Must be a class
			UClass* Class = Cast<UClass>(ClassIt->Get());
			if (!Class)
			{
				continue;
			}

			// Check the instanced map first
			const FDetailLayoutCallback* Callback = InstancedDetailLayoutMap.Find(Class);

			if (!Callback)
			{
				// callback wasn't found in the per instance map, try the global instances instead
				Callback = GlobalCustomLayoutNameMap.Find(Class->GetFName());
			}

			if (Callback)
			{
				FinalCallbackMap.Add(Class, Callback);
			}
		}

		FinalCallbackMap.ValueSort(FCompareFDetailLayoutCallback());

		TSet<UStruct*> QueriedClasses;

		if (FinalCallbackMap.Num() > 0)
		{
			// Ask each class that we have properties for to customize its layout
			for (auto LayoutIt(FinalCallbackMap.CreateConstIterator()); LayoutIt; ++LayoutIt)
			{
				const TWeakObjectPtr<UStruct> WeakClass = LayoutIt.Key();

				if (WeakClass.IsValid())
				{
					UStruct* Class = WeakClass.Get();

					TSet<FName> ProcessedClasses;
					bool bHasItemsToProcess = true;
					while (bHasItemsToProcess)
					{
						bHasItemsToProcess = false;

						// Copy ClassToProperty map since it could be modified during customization (through AddObjectPropertyData for example)
						FClassInstanceToPropertyMap InstancedPropertyMapCopy = LayoutData.ClassToPropertyMap.FindChecked(Class->GetFName());

						// Stamp this rounds number of item to detect if current map has changed after going through it
						const int32 InitialCount = InstancedPropertyMapCopy.Num();
						for (FClassInstanceToPropertyMap::TIterator InstanceIt(InstancedPropertyMapCopy); InstanceIt; ++InstanceIt)
						{
							// Stamp classes that have been processed to avoid reprocessing them when doing subsequent rounds
							const FName Key = InstanceIt.Key();
							if (ProcessedClasses.Contains(Key))
							{
								continue;
							}
							ProcessedClasses.Add(Key);

							LayoutData.DetailLayout->SetCurrentCustomizationClass(Class, Key);

							const FOnGetDetailCustomizationInstance& DetailDelegate = LayoutIt.Value()->DetailLayoutDelegate;

							if (DetailDelegate.IsBound())
							{
								QueriedClasses.Add(Class);

								// Create a new instance of the custom detail layout for the current class
								TSharedRef<IDetailCustomization> CustomizationInstance = DetailDelegate.Execute();

								// Ask for details immediately
								CustomizationInstance->CustomizeDetails(LayoutData.DetailLayout);

								// Save the instance from destruction until we refresh
								LayoutData.CustomizationClassInstances.Add(CustomizationInstance);
							}
						}
						
						// Verify if current mapping has changed after customizations which would require another pass
						const int32 NewCount = LayoutData.ClassToPropertyMap.FindChecked(Class->GetFName()).Num();
						if (NewCount > InitialCount)
						{
							bHasItemsToProcess = true;
						}
					}
				}
			}
		}

		// Ensure that the base class and its parents are always queried
		TSet<UStruct*> ParentClassesToQuery;
		if (BaseStruct && !QueriedClasses.Contains(BaseStruct))
		{
			ParentClassesToQuery.Add(BaseStruct);
			LayoutData.ClassesWithProperties.Add(BaseStruct);
		}

		// Find base classes of queried classes that were not queried and add them to the query list
		// this supports cases where a parent class has no properties but still wants to add customization
		for (auto QueriedClassIt = LayoutData.ClassesWithProperties.CreateConstIterator(); QueriedClassIt; ++QueriedClassIt)
		{
			UStruct* ParentStruct = (*QueriedClassIt)->GetSuperStruct();

			while (ParentStruct && ParentStruct->IsA(UClass::StaticClass()) && !QueriedClasses.Contains(ParentStruct) && !LayoutData.ClassesWithProperties.Contains(ParentStruct))
			{
				ParentClassesToQuery.Add(ParentStruct);
				ParentStruct = ParentStruct->GetSuperStruct();
			}
		}

		// Query extra base classes and structs
		for (auto ParentIt = ParentClassesToQuery.CreateConstIterator(); ParentIt; ++ParentIt)
		{
			QueryLayoutForClass(LayoutData, *ParentIt, InstancedDetailLayoutMap);
		}
	}


}
