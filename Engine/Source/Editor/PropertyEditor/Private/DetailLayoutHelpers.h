// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PropertyEditorModule.h"

struct FDetailLayoutData;
class FPropertyNode;
class FComplexPropertyNode;

namespace DetailLayoutHelpers
{
	struct FUpdatePropertyMapArgs
	{
		FDetailLayoutData* LayoutData;
		FCustomPropertyTypeLayoutMap* InstancedPropertyTypeToDetailLayoutMap;
		bool bEnableFavoriteSystem;
		bool bUpdateFavoriteSystemOnly;
	};

	/**
	 * Recursively updates children of property nodes. Generates default layout for properties
	 *
	 * @param InNode	The parent node to get children from
	 * @param The detail layout builder that will be used for customization of this property map
	 * @param CurCategory The current category name
	 */
	void UpdateSinglePropertyMapRecursive(FPropertyNode& InNode, FName CurCategory, FComplexPropertyNode* CurObjectNode, FUpdatePropertyMapArgs& InUpdateArgs);

	/**
	 * Calls a delegate for each registered class that has properties visible to get any custom detail layouts
	 */
	void QueryCustomDetailLayout(FDetailLayoutData& LayoutData, const FCustomDetailLayoutMap& InstancedDetailLayoutMap, const FOnGetDetailCustomizationInstance& GenericLayoutDelegate);
}