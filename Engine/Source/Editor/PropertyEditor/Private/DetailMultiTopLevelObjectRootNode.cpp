// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailMultiTopLevelObjectRootNode.h"
#include "IDetailRootObjectCustomization.h"
#include "DetailWidgetRow.h"
#include "ObjectPropertyNode.h"
#include "Misc/ConfigCacheIni.h"
#include "PropertyCustomizationHelpers.h"

void SDetailMultiTopLevelObjectTableRow::Construct( const FArguments& InArgs, TSharedRef<FDetailTreeNode> InOwnerTreeNode, const TSharedRef<STableViewBase>& InOwnerTableView )
{
	OwnerTreeNode = InOwnerTreeNode;
	ExpansionArrowUsage = InArgs._ExpansionArrowUsage;
	OwnerTableViewWeak = InOwnerTableView;

	STableRow< TSharedPtr< FDetailTreeNode > >::ConstructInternal(
		STableRow::FArguments()
			.Style(FAppStyle::Get(), "DetailsView.TreeView.TableRow")
			.ShowSelection(false),
		InOwnerTableView
	);
}

void SDetailMultiTopLevelObjectTableRow::SetContent(TSharedRef<SWidget> InContent)
{
	if (ExpansionArrowUsage == EExpansionArrowUsage::Default)
	{
		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 2.0f, 2.0f, 2.0f)
			.AutoWidth()
			[
				SNew(SExpanderArrow, SharedThis(this))
				.Visibility(ExpansionArrowUsage == EExpansionArrowUsage::Default ? EVisibility::Visible : EVisibility::Collapsed)
			]
			+ SHorizontalBox::Slot()
			.Expose(ContentSlot)
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 16.0f))
			[
				InContent
			]
		];
	}
	else
	{
		ChildSlot
		[
			InContent
		];
	}
}

FReply SDetailMultiTopLevelObjectTableRow::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (ExpansionArrowUsage != EExpansionArrowUsage::None && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		ToggleExpansion();
		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply SDetailMultiTopLevelObjectTableRow::OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	return OnMouseButtonDown(InMyGeometry, InMouseEvent);
}


FDetailMultiTopLevelObjectRootNode::FDetailMultiTopLevelObjectRootNode(const TSharedPtr<IDetailRootObjectCustomization>& InRootObjectCustomization, const TSharedPtr<IDetailsViewPrivate>& InDetailsView, const FObjectPropertyNode* RootNode)
	: DetailsView(InDetailsView)
	, RootObjectCustomization(InRootObjectCustomization)
	, bShouldBeVisible(false)
	, bHasFilterStrings(false)
{
	RootObjectSet.RootObjects.Reserve(RootNode->GetNumObjects());
	for (int32 ObjectIndex = 0; ObjectIndex < RootNode->GetNumObjects(); ++ObjectIndex)
	{
		RootObjectSet.RootObjects.Add(RootNode->GetUObject(ObjectIndex));
	}

	RootObjectSet.CommonBaseClass = RootNode->GetObjectBaseClass();

	NodeName = RootObjectSet.CommonBaseClass->GetFName();
}

void FDetailMultiTopLevelObjectRootNode::SetChildren(const FDetailNodeList& InChildNodes)
{
	ChildNodes = InChildNodes;

	for (TSharedRef<FDetailTreeNode> Node : ChildNodes)
	{
		Node->SetParentNode(AsShared());
	}
}

void FDetailMultiTopLevelObjectRootNode::OnItemExpansionChanged(bool bIsExpanded, bool bShouldSaveState)
{
	if (bShouldSaveState)
	{
		GConfig->SetBool(TEXT("DetailMultiObjectNodeExpansion"), *NodeName.ToString(), bIsExpanded, GEditorPerProjectIni);
	}
}

bool FDetailMultiTopLevelObjectRootNode::ShouldBeExpanded() const
{
	if (bHasFilterStrings)
	{
		return true;
	}
	else
	{
		bool bShouldBeExpanded = true;
		GConfig->GetBool(TEXT("DetailMultiObjectNodeExpansion"), *NodeName.ToString(), bShouldBeExpanded, GEditorPerProjectIni);
		return bShouldBeExpanded;
	}
}

ENodeVisibility FDetailMultiTopLevelObjectRootNode::GetVisibility() const
{
	ENodeVisibility FinalVisibility = ENodeVisibility::Visible;
	if(RootObjectCustomization.IsValid() && !RootObjectCustomization.Pin()->AreObjectsVisible(RootObjectSet))
	{
		FinalVisibility = ENodeVisibility::ForcedHidden;
	}
	else
	{
		FinalVisibility = bShouldBeVisible ? ENodeVisibility::Visible : ENodeVisibility::HiddenDueToFiltering;
	}

	return FinalVisibility;
}

TSharedRef< ITableRow > FDetailMultiTopLevelObjectRootNode::GenerateWidgetForTableView(const TSharedRef<STableViewBase>& OwnerTable, bool bAllowFavoriteSystem)
{
	EExpansionArrowUsage ExpansionArrowUsage = EExpansionArrowUsage::None;

	TSharedPtr<IDetailRootObjectCustomization> Customization = RootObjectCustomization.Pin();
	if (Customization)
	{
		ExpansionArrowUsage = Customization->GetExpansionArrowUsage();
	}

	TSharedRef<SDetailMultiTopLevelObjectTableRow> TableRowWidget =
		SNew(SDetailMultiTopLevelObjectTableRow, AsShared(), OwnerTable)
		.ExpansionArrowUsage(ExpansionArrowUsage);

	FDetailWidgetRow Row;
	GenerateWidget_Internal(Row, TableRowWidget);

	TableRowWidget->SetContent(Row.NameWidget.Widget);

	return TableRowWidget;
}


bool FDetailMultiTopLevelObjectRootNode::GenerateStandaloneWidget(FDetailWidgetRow& OutRow) const
{
	GenerateWidget_Internal(OutRow, nullptr);
	return true;
}

void FDetailMultiTopLevelObjectRootNode::GetChildren(FDetailNodeList& OutChildren, const bool& bInIgnoreVisibility)
{
	for( int32 ChildIndex = 0; ChildIndex < ChildNodes.Num(); ++ChildIndex )
	{
		TSharedRef<FDetailTreeNode>& Child = ChildNodes[ChildIndex];
		if( Child->GetVisibility() == ENodeVisibility::Visible )
		{
			if( Child->ShouldShowOnlyChildren() )
			{
				Child->GetChildren( OutChildren, bInIgnoreVisibility );
			}
			else
			{
				OutChildren.Add( Child );
			}
		}
	}
}

void FDetailMultiTopLevelObjectRootNode::FilterNode( const FDetailFilter& InFilter )
{
	bShouldBeVisible = false;
	bHasFilterStrings = InFilter.FilterStrings.Num() > 0;

	for( int32 ChildIndex = 0; ChildIndex < ChildNodes.Num(); ++ChildIndex )
	{
		TSharedRef<FDetailTreeNode>& Child = ChildNodes[ChildIndex];

		Child->FilterNode( InFilter );

		if( Child->GetVisibility() == ENodeVisibility::Visible )
		{
			bShouldBeVisible = true;

			if (TSharedPtr<IDetailsViewPrivate> DetailsViewPinned = DetailsView.Pin())
			{
				DetailsViewPinned->RequestItemExpanded(Child, Child->ShouldBeExpanded());
			}
		}
	}
}

bool FDetailMultiTopLevelObjectRootNode::ShouldShowOnlyChildren() const
{
	return RootObjectCustomization.IsValid() && RootObjectSet.RootObjects.Num() ? !RootObjectCustomization.Pin()->ShouldDisplayHeader(RootObjectSet) : false;
}

void FDetailMultiTopLevelObjectRootNode::GenerateWidget_Internal(FDetailWidgetRow& OutRow, TSharedPtr<SDetailMultiTopLevelObjectTableRow> TableRowWidget) const
{
	TSharedPtr<SWidget> HeaderWidget = SNullWidget::NullWidget;
	if (RootObjectCustomization.IsValid() && RootObjectSet.RootObjects.Num())
	{
		HeaderWidget = RootObjectCustomization.Pin()->CustomizeObjectHeader(RootObjectSet, TableRowWidget);
	}

	OutRow.NameContent()
	[
		HeaderWidget.ToSharedRef()
	];

}
