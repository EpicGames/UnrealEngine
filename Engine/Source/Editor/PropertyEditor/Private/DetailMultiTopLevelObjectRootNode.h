// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "IPropertyUtilities.h"
#include "DetailTreeNode.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "SDetailsViewBase.h"
#include "SDetailTableRowBase.h"
#include "IDetailRootObjectCustomization.h"
#include "PropertyCustomizationHelpers.h"

using EExpansionArrowUsage = IDetailRootObjectCustomization::EExpansionArrowUsage;

class SDetailMultiTopLevelObjectTableRow : public SDetailTableRowBase
{
public:
	SLATE_BEGIN_ARGS(SDetailMultiTopLevelObjectTableRow)
		: _DisplayName()
		, _ExpansionArrowUsage(EExpansionArrowUsage::None)
	{}
		SLATE_ARGUMENT( FText, DisplayName )
		SLATE_ARGUMENT(IDetailRootObjectCustomization::EExpansionArrowUsage, ExpansionArrowUsage)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FDetailTreeNode> InOwnerTreeNode, const TSharedRef<STableViewBase>& InOwnerTableView);
	void SetContent(TSharedRef<SWidget> InContent) override;

private:
	const FSlateBrush* GetBackgroundImage() const;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

private:
	EExpansionArrowUsage ExpansionArrowUsage;
	SHorizontalBox::FSlot* ContentSlot = nullptr;
	TWeakPtr<STableViewBase> OwnerTableViewWeak;
};

class FDetailMultiTopLevelObjectRootNode : public FDetailTreeNode, public TSharedFromThis<FDetailMultiTopLevelObjectRootNode>
{
public:
	FDetailMultiTopLevelObjectRootNode(const TSharedPtr<IDetailRootObjectCustomization>& RootObjectCustomization, const TSharedPtr<IDetailsViewPrivate>& InDetailsView, const FObjectPropertyNode* RootNode);
	void SetChildren(const FDetailNodeList& InChildNodes);
private:
	virtual TSharedPtr<IDetailsView> GetNodeDetailsViewSharedPtr() const override { return DetailsView.Pin(); }
	virtual TSharedPtr<IDetailsViewPrivate> GetDetailsViewSharedPtr() const override { return DetailsView.Pin(); }
	virtual void OnItemExpansionChanged(bool bIsExpanded, bool bShouldSaveState) override;
	virtual bool ShouldBeExpanded() const override;
	virtual ENodeVisibility GetVisibility() const override;
	virtual TSharedRef<ITableRow> GenerateWidgetForTableView(const TSharedRef<STableViewBase>& OwnerTable, bool bAllowFavoriteSystem) override;
	virtual bool GenerateStandaloneWidget(FDetailWidgetRow& OutRow) const override;
	virtual void GetChildren(FDetailNodeList& OutChildren, const bool& bInIgnoreVisibility) override;
	virtual void FilterNode(const FDetailFilter& InFilter) override;
	virtual void Tick(float DeltaTime) override {}
	virtual bool ShouldShowOnlyChildren() const override;
	virtual FName GetNodeName() const override { return NodeName; }
	virtual EDetailNodeType GetNodeType() const override { return EDetailNodeType::Object; }
	virtual TSharedPtr<IPropertyHandle> CreatePropertyHandle() const override { return nullptr; }
	void GenerateWidget_Internal(FDetailWidgetRow& Row, TSharedPtr<SDetailMultiTopLevelObjectTableRow> TableRow) const;

private:
	FDetailNodeList ChildNodes;
	TWeakPtr<IDetailsViewPrivate> DetailsView;
	TWeakPtr<IDetailRootObjectCustomization> RootObjectCustomization;
	FDetailsObjectSet RootObjectSet;
	const UClass* CommonBaseClass;
	FName NodeName;
	bool bShouldBeVisible;
	bool bHasFilterStrings;
};
