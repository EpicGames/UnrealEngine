// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailPropertyRow.h"
#include "CategoryPropertyNode.h"
#include "CustomChildBuilder.h"
#include "DetailCategoryGroupNode.h"
#include "DetailItemNode.h"
#include "DetailWidgetRow.h"
#include "ItemPropertyNode.h"
#include "ObjectPropertyNode.h"
#include "ObjectPropertyNode.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorHelpers.h"
#include "PropertyHandleImpl.h"
#include "StructurePropertyNode.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SSpacer.h"
#include "IDetailPropertyChildrenCustomizationHandler.h"

#include "UObject/PropertyOptional.h"

#define LOCTEXT_NAMESPACE	"DetailPropertyRow"

FDetailPropertyRow::FDetailPropertyRow(TSharedPtr<FPropertyNode> InPropertyNode, TSharedRef<FDetailCategoryImpl> InParentCategory, TSharedPtr<FComplexPropertyNode> InExternalRootNode)
	: PropertyNode( InPropertyNode )
	, ParentCategory( InParentCategory )
	, ExternalRootNode( InExternalRootNode )
	, bShowPropertyButtons( true )
	, bShowCustomPropertyChildren( true )
	, bForceAutoExpansion( false )
	, bCachedCustomTypeInterface(false)
{
	if (PropertyNode.IsValid())
	{
		// Is this a set optional property?
		if (PropertyEditorHelpers::IsOptionalProperty(*PropertyNode) && PropertyNode->GetNumChildNodes())
		{
			// If we are selecting multiple options with different states (ie set/unset) dont use the
			// Value node so optional multi-select logic is displayed (see SPropertyEditorOptional.h).
			uint8 MixedValues = 0;
			FProperty* MyProperty = PropertyNode->GetProperty();
			if (FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(MyProperty))
			{
				void* Optional = NULL;
				FReadAddressList Addresses;
				if (PropertyNode->GetReadAddress(Addresses))
				{
					for (int i = 0; i < Addresses.Num(); i++)
					{
						Optional = Addresses.GetAddress(i);
						MixedValues |= OptionalProperty->IsSet(Optional) ? 1 : 2;
						if (MixedValues == 3)
						{
							break;
						}
					}
				}
			}

			if (MixedValues != 3)
			{
				// If we are set, only display our set value which is our child node
				bForceShowOnlyChildren = true;
			}
		}

		TSharedRef<FPropertyNode> PropertyNodeRef = PropertyNode.ToSharedRef();

		const TSharedRef<IPropertyUtilities> Utilities = InParentCategory->GetParentLayoutImpl()->GetPropertyUtilities();

		if (PropertyNode->AsCategoryNode() == nullptr)
		{
			MakePropertyEditor(PropertyNodeRef, Utilities, PropertyEditor);
		}
		
		static FName InlineCustomizationKeyMeta("AllowEditInlineCustomization");
		if (PropertyNode->AsComplexNode() && ExternalRootNode.IsValid()) // AsComplexNode works both for objects and structs
		{
			// We are showing an entirely different object inline.  Generate a layout for it now.
			if (TSharedPtr<IDetailsViewPrivate> DetailsView = InParentCategory->GetDetailsViewSharedPtr())
			{
				ExternalObjectLayout = MakeShared<FDetailLayoutData>();
				DetailsView->UpdateSinglePropertyMap(InExternalRootNode, *ExternalObjectLayout, true);
			}
		}
		else if ((PropertyNode->HasNodeFlags(EPropertyNodeFlags::EditInlineNew) || PropertyNode->HasNodeFlags(EPropertyNodeFlags::DynamicInstance)) && PropertyNode->GetProperty()->HasMetaData(InlineCustomizationKeyMeta))
		{
			// Allow customization of 'edit inline new' objects if the metadata key has been specified.
			// The child of this node, if set, will be an object node that we will want to treat as an 'external object layout'
			TSharedPtr<FPropertyNode> ChildNode = PropertyNode->GetNumChildNodes() > 0 ? PropertyNode->GetChildNode(0) : nullptr;
			TSharedPtr<FComplexPropertyNode> ComplexChildNode = StaticCastSharedPtr<FComplexPropertyNode>(ChildNode);
			if (ComplexChildNode.IsValid())
			{
				// We are showing an entirely different object inline.  Generate a layout for it now.
				if (TSharedPtr<IDetailsViewPrivate> DetailsView = InParentCategory->GetDetailsViewSharedPtr())
				{
					ExternalObjectLayout = MakeShared<FDetailLayoutData>();
					DetailsView->UpdateSinglePropertyMap(ComplexChildNode, *ExternalObjectLayout, true);
				}
			}
		}

		if (PropertyNode->GetPropertyKeyNode().IsValid())
		{							
			TSharedPtr<IPropertyTypeCustomization> FoundPropertyCustomisation = GetPropertyCustomization(PropertyNode->GetPropertyKeyNode().ToSharedRef(), ParentCategory.Pin().ToSharedRef());

			bool bInlineRow = FoundPropertyCustomisation != nullptr ? FoundPropertyCustomisation->ShouldInlineKey() : false;

			static FName InlineKeyMeta("ForceInlineRow");
			bInlineRow |= InPropertyNode->GetParentNode()->GetProperty()->HasMetaData(InlineKeyMeta);

			// Only create the property editor if it's not a struct or if it requires to be inlined (and has customization)
			if (!NeedsKeyNode(PropertyNodeRef, InParentCategory) || (bInlineRow && FoundPropertyCustomisation != nullptr))
			{
				CachedKeyCustomTypeInterface = FoundPropertyCustomisation;
				
				MakePropertyEditor(PropertyNode->GetPropertyKeyNode().ToSharedRef(), Utilities, PropertyKeyEditor);
			}
		}
	}

	PropertyHandle = InParentCategory->GetParentLayoutImpl()->GetPropertyHandle(PropertyNode);
}

bool FDetailPropertyRow::NeedsKeyNode(TSharedRef<FPropertyNode> InPropertyNode, TSharedRef<FDetailCategoryImpl> InParentCategory)
{
	FStructProperty* KeyStructProp = CastField<FStructProperty>(InPropertyNode->GetPropertyKeyNode()->GetProperty());
	return KeyStructProp != nullptr;
}

IDetailPropertyRow& FDetailPropertyRow::DisplayName( const FText& InDisplayName )
{
	if (PropertyNode.IsValid())
	{
		PropertyNode->SetDisplayNameOverride( InDisplayName );
	}
	return *this;
}

IDetailPropertyRow& FDetailPropertyRow::ToolTip( const FText& InToolTip )
{
	if (PropertyNode.IsValid())
	{
		PropertyNode->SetToolTipOverride( InToolTip );
	}
	return *this;
}

IDetailPropertyRow& FDetailPropertyRow::ShowPropertyButtons( bool bInShowPropertyButtons )
{
	bShowPropertyButtons = bInShowPropertyButtons;
	return *this;
}

IDetailPropertyRow& FDetailPropertyRow::EditCondition( TAttribute<bool> EditConditionValue, FOnBooleanValueChanged OnEditConditionValueChanged, ECustomEditConditionMode EditConditionMode )
{
	CustomEditConditionValue = EditConditionValue;
	CustomEditConditionValueChanged = OnEditConditionValueChanged;
	CustomEditConditionMode = EditConditionMode;
	return *this;
}

IDetailPropertyRow& FDetailPropertyRow::EditConditionHides(bool bEditConditionHidesValue)
{
	bCustomEditConditionHides = bEditConditionHidesValue;
	return *this;
}

IDetailPropertyRow& FDetailPropertyRow::IsEnabled( TAttribute<bool> InIsEnabled )
{
	CustomIsEnabledAttrib = InIsEnabled;
	return *this;
}

IDetailPropertyRow& FDetailPropertyRow::ShouldAutoExpand(bool bInForceExpansion)
{
	bForceAutoExpansion = bInForceExpansion;
	return *this;
}

IDetailPropertyRow& FDetailPropertyRow::Visibility( TAttribute<EVisibility> Visibility )
{
	PropertyVisibility = Visibility;
	return *this;
}

IDetailPropertyRow& FDetailPropertyRow::OverrideResetToDefault(const FResetToDefaultOverride& ResetToDefault)
{
	CustomResetToDefault = ResetToDefault;
	return *this;
}

IDetailPropertyRow& FDetailPropertyRow::DragDropHandler(TSharedPtr<IDetailDragDropHandler> InDragDropHandler)
{
	CustomDragDropHandler = InDragDropHandler;
	return *this;
}

bool FDetailPropertyRow::IsExpanded() const
{
	if (GetPropertyNode())
	{
		return GetPropertyNode()->HasNodeFlags(EPropertyNodeFlags::Expanded);
	}
	return false;
}

void FDetailPropertyRow::GetDefaultWidgets( TSharedPtr<SWidget>& OutNameWidget, TSharedPtr<SWidget>& OutValueWidget, bool bAddWidgetDecoration )
{
	FDetailWidgetRow Row;
	GetDefaultWidgets(OutNameWidget, OutValueWidget, Row, bAddWidgetDecoration);
}

void FDetailPropertyRow::GetDefaultWidgets( TSharedPtr<SWidget>& OutNameWidget, TSharedPtr<SWidget>& OutValueWidget, FDetailWidgetRow& Row, bool bAddWidgetDecoration )
{
	TSharedPtr<FDetailWidgetRow> CustomTypeRow;

	TSharedPtr<IPropertyTypeCustomization>& CustomTypeInterface = GetTypeInterface();
	if ( CustomTypeInterface.IsValid() ) 
	{
		CustomTypeRow = MakeShared<FDetailWidgetRow>();

		CustomTypeInterface->CustomizeHeader(PropertyHandle.ToSharedRef(), *CustomTypeRow, *this);
	}

	SetWidgetRowProperties(Row);
	MakeNameOrKeyWidget(Row, CustomTypeRow);
	MakeValueWidget(Row, CustomTypeRow, bAddWidgetDecoration);

	OutNameWidget = Row.NameWidget.Widget;
	OutValueWidget = Row.ValueWidget.Widget;
}

bool FDetailPropertyRow::HasColumns() const
{
	// Regular properties always have columns
	return !CustomPropertyWidget.IsValid() || CustomPropertyWidget->HasColumns();
}

bool FDetailPropertyRow::ShowOnlyChildren() const
{
	return bForceShowOnlyChildren || (PropertyTypeLayoutBuilder.IsValid() && CustomPropertyWidget.IsValid() && !CustomPropertyWidget->HasAnyContent());
}

bool FDetailPropertyRow::RequiresTick() const
{
	return PropertyVisibility.IsBound() || IsOnlyVisibleWhenEditConditionMet();
}

FDetailWidgetRow& FDetailPropertyRow::CustomWidget( bool bShowChildren )
{
	bShowCustomPropertyChildren = bShowChildren;
	CustomPropertyWidget = MakeShared<FDetailWidgetRow>();
	return *CustomPropertyWidget;
}

FDetailWidgetDecl* FDetailPropertyRow::CustomNameWidget()
{
	return CustomPropertyWidget.IsValid() ? &CustomPropertyWidget->NameContent() : nullptr;
}

FDetailWidgetDecl* FDetailPropertyRow::CustomValueWidget()
{
	return CustomPropertyWidget.IsValid() ? &CustomPropertyWidget->ValueContent() : nullptr;
}

FDetailWidgetDecl* FDetailPropertyRow::CustomResetToDefaultWidget()
{
	return CustomPropertyWidget.IsValid() ? &CustomPropertyWidget->ResetToDefaultContent() : nullptr;
}

TSharedPtr<FAssetThumbnailPool> FDetailPropertyRow::GetThumbnailPool() const
{
	TSharedPtr<FDetailCategoryImpl> ParentCategoryPinned = ParentCategory.Pin();
	return ParentCategoryPinned.IsValid() ? ParentCategoryPinned->GetParentLayout().GetThumbnailPool() : NULL;
}

TSharedPtr<IPropertyUtilities> FDetailPropertyRow::GetPropertyUtilities() const
{
	TSharedPtr<FDetailCategoryImpl> ParentCategoryPinned = ParentCategory.Pin();
	if (ParentCategoryPinned.IsValid() && ParentCategoryPinned->IsParentLayoutValid())
	{
		return ParentCategoryPinned->GetParentLayout().GetPropertyUtilities();
	}
	
	return nullptr;
}

FDetailWidgetRow FDetailPropertyRow::GetWidgetRow()
{
	if( HasColumns() )
	{
		FDetailWidgetRow Row;

		SetWidgetRowProperties(Row);
		MakeNameOrKeyWidget( Row, CustomPropertyWidget );
		MakeValueWidget( Row, CustomPropertyWidget );

		return Row;
	}
	else
	{
		return *CustomPropertyWidget;
	}
}

TArrayView<TSharedPtr<IPropertyHandle>> FDetailPropertyRow::GetPropertyHandles() const
{
	if (CustomPropertyWidget)
	{
		return CustomPropertyWidget->PropertyHandles;
	}

	// view single item as a c-array of size one
	return TArrayView<TSharedPtr<IPropertyHandle>>(const_cast<TSharedPtr<IPropertyHandle>*>(&PropertyHandle), 1);
}

FText FDetailPropertyRow::GetFilterTextString() const
{
	if (CustomPropertyWidget)
	{
		return CustomPropertyWidget->FilterTextString;
	}
	else
	{
		return {};
	}
}

static bool IsHeaderRowRequired(const TSharedPtr<IPropertyHandle>& PropertyHandle)
{
	TSharedPtr<IPropertyHandle> ParentHandle = PropertyHandle->GetParentHandle();
	while (ParentHandle.IsValid())
	{
		if (ParentHandle->AsMap().IsValid())
		{
			return true;
		}

		ParentHandle = ParentHandle->GetParentHandle();
	}

	return false;
}

static void FixEmptyHeaderRowInContainers(const TSharedPtr<IPropertyHandle>& PropertyHandle, const TSharedPtr<FDetailWidgetRow>& HeaderRow)
{
	if (IsHeaderRowRequired(PropertyHandle))
	{
		if (!HeaderRow->HasAnyContent())
		{
			if (!HeaderRow->HasNameContent())
			{
				HeaderRow->NameContent()
				[
					PropertyHandle->CreatePropertyNameWidget()
				];
			}

			if (!HeaderRow->HasValueContent())
			{
				HeaderRow->ValueContent()
				[
					PropertyHandle->CreatePropertyValueWidget(false)
				];
			}
		}
	}
}

void FDetailPropertyRow::OnItemNodeInitialized( TSharedRef<FDetailCategoryImpl> InParentCategory, const TAttribute<bool>& InIsParentEnabled, TSharedPtr<IDetailGroup> InParentGroup)
{
	IsParentEnabled = InIsParentEnabled;

	TSharedPtr<IPropertyTypeCustomization>& CustomTypeInterface = GetTypeInterface();
	// Don't customize the user already customized
	if (!CustomPropertyWidget.IsValid() && CustomTypeInterface.IsValid())
	{
		CustomPropertyWidget = MakeShared<FDetailWidgetRow>();

		CustomTypeInterface->CustomizeHeader(PropertyHandle.ToSharedRef(), *CustomPropertyWidget, *this);

		FixEmptyHeaderRowInContainers(PropertyHandle, CustomPropertyWidget);


		// set initial value of auto-expand from struct customization
		if (CustomPropertyWidget->ForceAutoExpansion.IsSet())
		{
			bForceAutoExpansion = CustomPropertyWidget->ForceAutoExpansion.GetValue();
		}
	}

	TSharedPtr<IDetailsView> DetailsView = InParentCategory->GetDetailsViewSharedPtr();
	IDetailPropertyChildrenCustomizationHandler* CustomizationHandler = DetailsView ? DetailsView->GetChildrenCustomizationHandler().Get() : nullptr;
	if (CustomizationHandler && CustomizationHandler->ShouldCustomizeChildren(PropertyHandle.ToSharedRef()))
	{
		PropertyTypeLayoutBuilder = MakeShared<FCustomChildrenBuilder>(InParentCategory, InParentGroup);
		CustomizationHandler->CustomizeChildren(*PropertyTypeLayoutBuilder, PropertyHandle);
	}
	else if( bShowCustomPropertyChildren && CustomTypeInterface.IsValid() )
	{
		PropertyTypeLayoutBuilder = MakeShared<FCustomChildrenBuilder>(InParentCategory, InParentGroup);

		/** Does this row pass its custom reset behavior to its children? */
		if (CustomResetToDefault.IsSet() && CustomResetToDefault->PropagatesToChildren())
		{
			PropertyTypeLayoutBuilder->OverrideResetChildrenToDefault(CustomResetToDefault.GetValue());
		}

		CustomTypeInterface->CustomizeChildren(PropertyHandle.ToSharedRef(), *PropertyTypeLayoutBuilder, *this);
	}
}

void FDetailPropertyRow::OnGenerateChildren( FDetailNodeList& OutChildren )
{
	if (PropertyNode->AsCategoryNode() && PropertyNode->GetParentNode() && !PropertyNode->GetParentNode()->AsObjectNode())
	{
		// This is a sub-category.  Populate from SubCategory builder
		TSharedRef<FDetailCategoryImpl> ParentCategoryRef = ParentCategory.Pin().ToSharedRef();
		TSharedPtr<FDetailLayoutBuilderImpl> LayoutBuilder = ParentCategoryRef->GetParentLayoutImpl();
		TSharedPtr<FDetailCategoryImpl> MyCategory = LayoutBuilder->GetSubCategoryImpl(PropertyNode->AsCategoryNode()->GetCategoryName());
		if(MyCategory.IsValid())
		{
			MyCategory->GenerateLayout();

			// Ignore the header of the category by just getting the categories children directly. We are the header in this case.
			// Also ignore visibility here as we dont have a filter yet and the children will be filtered later anyway
			const bool bIgnoreVisibility = true;
			const bool bIgnoreAdvancedDropdown = true;
			MyCategory->GetGeneratedChildren(OutChildren, bIgnoreVisibility, bIgnoreAdvancedDropdown);
		}
		else
		{
			// Fall back to the default if we can't find the category implementation
			GenerateChildrenForPropertyNode(PropertyNode, OutChildren);
		}
	}
	else if (PropertyNode->AsCategoryNode() || PropertyNode->GetProperty() || ExternalObjectLayout.IsValid())
	{
		GenerateChildrenForPropertyNode( PropertyNode, OutChildren );
	}
}

void FDetailPropertyRow::GenerateChildrenForPropertyNode( TSharedPtr<FPropertyNode>& RootPropertyNode, FDetailNodeList& OutChildren )
{
	// Children should be disabled if we are disabled
	TAttribute<bool> ParentEnabledState = TAttribute<bool>::CreateSP(this, &FDetailPropertyRow::GetEnabledState);

	if( PropertyTypeLayoutBuilder.IsValid() && bShowCustomPropertyChildren )
	{
		const TArray< FDetailLayoutCustomization >& ChildRows = PropertyTypeLayoutBuilder->GetChildCustomizations();

		for( int32 ChildIndex = 0; ChildIndex < ChildRows.Num(); ++ChildIndex )
		{
			TSharedRef<FDetailItemNode> ChildNodeItem = MakeShared<FDetailItemNode>(ChildRows[ChildIndex], ParentCategory.Pin().ToSharedRef(), ParentEnabledState);
			ChildNodeItem->Initialize();
			OutChildren.Add( ChildNodeItem );
		}
	}
	else if (ExternalObjectLayout.IsValid() && ExternalObjectLayout->DetailLayout->HasDetails())
	{
		OutChildren.Append(ExternalObjectLayout->DetailLayout->GetAllRootTreeNodes());
	}
	else if ((bShowCustomPropertyChildren || !CustomPropertyWidget.IsValid()) && RootPropertyNode->GetNumChildNodes() > 0)
	{
		TSharedRef<FDetailCategoryImpl> ParentCategoryRef = ParentCategory.Pin().ToSharedRef();
		IDetailLayoutBuilder& LayoutBuilder = ParentCategoryRef->GetParentLayout();
		FProperty* ParentProperty = RootPropertyNode->GetProperty();

		const bool bStructProperty = ParentProperty && ParentProperty->IsA<FStructProperty>();
		const bool bMapProperty = ParentProperty && ParentProperty->IsA<FMapProperty>();
		const bool bSetProperty = ParentProperty && ParentProperty->IsA<FSetProperty>();

		for( int32 ChildIndex = 0; ChildIndex < RootPropertyNode->GetNumChildNodes(); ++ChildIndex )
		{
			TSharedPtr<FPropertyNode> ChildNode = RootPropertyNode->GetChildNode(ChildIndex);
			
			if (!LayoutBuilder.IsPropertyPathAllowed(ChildNode->GetPropertyPath()))
			{
				ChildNode->SetNodeFlags( EPropertyNodeFlags::RequiresValidation, false); 
				ChildNode->SetNodeFlags( EPropertyNodeFlags::IsBeingFiltered | EPropertyNodeFlags::SkipChildValidation, true);
				continue;
			}
			
			if( ChildNode.IsValid() && ChildNode->HasNodeFlags( EPropertyNodeFlags::IsCustomized ) == 0 )
			{
				if( ChildNode->AsObjectNode() )
				{
					// Skip over object nodes and generate their children.  Object nodes are not visible
					GenerateChildrenForPropertyNode( ChildNode, OutChildren );
				}
				// Only struct children can have custom visibility that is different from their parent.
				else if ( !bStructProperty || LayoutBuilder.IsPropertyVisible(FPropertyAndParent(ChildNode.ToSharedRef())) )
				{	
					TArray<TSharedRef<FDetailTreeNode>> PropNodes;
					bool bHasKeyNode = false;

					FCategoryPropertyNode* CategoryNode = ChildNode->AsCategoryNode();
					if (CategoryNode && CategoryNode->GetCategoryName() == NAME_None)
					{
						// Skip the category node and repoarent everything to the ParentCategory
						GenerateChildrenForPropertyNode(ChildNode, OutChildren);
					}
					else
					{
						// Create and initialize the child first
						FDetailLayoutCustomization Customization;
						Customization.PropertyRow = MakeShared<FDetailPropertyRow>(ChildNode, ParentCategoryRef);

						if (CustomResetToDefault.IsSet() && CustomResetToDefault->PropagatesToChildren())
						{
							Customization.PropertyRow->OverrideResetToDefault(CustomResetToDefault.GetValue());
						}

						TSharedRef<FDetailItemNode> ChildNodeItem = MakeShared<FDetailItemNode>(Customization, ParentCategoryRef, ParentEnabledState);
						ChildNodeItem->Initialize();

						if ( ChildNode->GetPropertyKeyNode().IsValid() )
						{
							// If the child has a key property, only create a second node for the key if the child did not already create a property
							// editor for it
							if ( !Customization.PropertyRow->PropertyKeyEditor.IsValid() )
							{
								FDetailLayoutCustomization KeyCustom;
								KeyCustom.PropertyRow = MakeShared<FDetailPropertyRow>(ChildNode->GetPropertyKeyNode(), ParentCategoryRef);
								TSharedRef<FDetailItemNode> KeyNodeItem = MakeShared<FDetailItemNode>(KeyCustom, ParentCategoryRef, ParentEnabledState);
								KeyNodeItem->Initialize();
							
								PropNodes.Add(KeyNodeItem);
								bHasKeyNode = true;
							}
						}

						// Add the child node 
						PropNodes.Add(ChildNodeItem);
					
						// For set properties, set the name override to match the index
						if (bSetProperty)
						{
							ChildNode->SetDisplayNameOverride(FText::AsNumber(ChildIndex));
						}

						if (bMapProperty && bHasKeyNode)
						{
							// Group the key/value nodes for map properties
							static FText KeyValueGroupNameFormat = LOCTEXT("KeyValueGroupName", "Element {0}");
							FText KeyValueGroupName = FText::Format(KeyValueGroupNameFormat, ChildIndex);

							TSharedRef<FDetailCategoryGroupNode> KeyValueGroupNode = MakeShared<FDetailCategoryGroupNode>(FName(*KeyValueGroupName.ToString()), ParentCategoryRef);
							KeyValueGroupNode->SetChildren(PropNodes);
							KeyValueGroupNode->SetShowBorder(false);
							KeyValueGroupNode->SetHasSplitter(true);

							OutChildren.Add(KeyValueGroupNode);
						}
						else
						{
							OutChildren.Append(PropNodes);
						}
					}
				}
			}
		}
	}
}

FName FDetailPropertyRow::GetRowName() const
{
	if (HasExternalProperty())
	{
		if (GetCustomExpansionId() != NAME_None)
		{
			return GetCustomExpansionId();
		}
		else if (FProperty* ExternalRootProperty = ExternalRootNode->GetProperty())
		{
			return ExternalRootProperty->GetFName();
		}
	}
	if (GetPropertyNode())
	{
		if (FProperty* Property = GetPropertyNode()->GetProperty())
		{
			return Property->GetFName();
		}
	}
	return NAME_None;
}

TSharedRef<FPropertyEditor> FDetailPropertyRow::MakePropertyEditor(const TSharedRef<FPropertyNode>& InPropertyNode, const TSharedRef<IPropertyUtilities>& PropertyUtilities, TSharedPtr<FPropertyEditor>& InEditor )
{
	if( !InEditor.IsValid() )
	{
		InEditor = FPropertyEditor::Create( InPropertyNode, PropertyUtilities );
	}

	return InEditor.ToSharedRef();
}

TSharedPtr<IPropertyTypeCustomization> FDetailPropertyRow::GetPropertyCustomization(const TSharedRef<FPropertyNode>& InPropertyNode, const TSharedRef<FDetailCategoryImpl>& InParentCategory)
{
	TSharedPtr<IPropertyTypeCustomization> CustomInterface;

	if (!PropertyEditorHelpers::IsStaticArray(*InPropertyNode))
	{
		FProperty* Property = InPropertyNode->GetProperty();
		TSharedPtr<IPropertyHandle> PropHandle = InParentCategory->GetParentLayoutImpl()->GetPropertyHandle(InPropertyNode);

		static FName NAME_PropertyEditor("PropertyEditor");
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(NAME_PropertyEditor);

		FPropertyTypeLayoutCallback LayoutCallback;
		if (Property != nullptr)
		{
			LayoutCallback = PropertyEditorModule.GetPropertyTypeCustomization(Property, *PropHandle, InParentCategory->GetCustomPropertyTypeLayoutMap());
		}
		else
		{
			// This add support to objects and structs added to the category with AddExternalObjectProperty / AddExternalStructureProperty
			if (FComplexPropertyNode* ComplexNode = InPropertyNode->AsComplexNode())
			{
				if (FObjectPropertyNode* ObjectNode = ComplexNode->AsObjectNode())
				{
					UClass* PropertyClass = ObjectNode->GetObjectBaseClass();
					while (PropertyClass)
					{
						LayoutCallback = PropertyEditorModule.FindPropertyTypeLayoutCallback(PropertyClass->GetFName(), *PropHandle, InParentCategory->GetCustomPropertyTypeLayoutMap());
						if (LayoutCallback.IsValid())
						{
							break;
						}

						PropertyClass = PropertyClass->GetSuperClass();
					}
				}
				else if (FStructurePropertyNode* StructureNode = ComplexNode->AsStructureNode())
				{
					const FName PropertyTypeName = StructureNode->GetBaseStructure()->GetFName();
					LayoutCallback = PropertyEditorModule.FindPropertyTypeLayoutCallback(PropertyTypeName, *PropHandle, InParentCategory->GetCustomPropertyTypeLayoutMap());
				}
			}
		}

		if (LayoutCallback.IsValid())
		{
			if (PropHandle->IsValidHandle())
			{
				CustomInterface = LayoutCallback.GetCustomizationInstance();
			}
		}
	}

	return CustomInterface;
}

template<typename T>
void MakeExternalStructPropertyRowCustomization(const T& Struct, const UStruct* StructClass,
	FName PropertyName, TSharedRef<FDetailCategoryImpl> ParentCategory, FDetailLayoutCustomization& OutCustomization,
	const FAddPropertyParams& Parameters, bool bAllowChildren)
{
	TSharedRef<FStructurePropertyNode> RootPropertyNode = MakeShared<FStructurePropertyNode>();

	//SET
	RootPropertyNode->SetStructure(Struct);

	FPropertyNodeInitParams InitParams;
	InitParams.ParentNode = nullptr;
	InitParams.Property = nullptr;
	InitParams.ArrayOffset = 0;
	InitParams.ArrayIndex = INDEX_NONE;
	InitParams.bForceHiddenPropertyVisibility = Parameters.ShouldForcePropertyVisible() || FPropertySettings::Get().ShowHiddenProperties();
	InitParams.bCreateCategoryNodes = PropertyName == NAME_None;
	InitParams.bAllowChildren = bAllowChildren;

	Parameters.OverrideAllowChildren(InitParams.bAllowChildren);
	Parameters.OverrideCreateCategoryNodes(InitParams.bCreateCategoryNodes);

	RootPropertyNode->InitNode(InitParams);

	ParentCategory->GetParentLayoutImpl()->AddExternalRootPropertyNode(RootPropertyNode);

	if (PropertyName != NAME_None)
	{
		TSharedPtr<FPropertyNode> PropertyNode = RootPropertyNode->GenerateSingleChild(PropertyName);
		if (PropertyNode.IsValid())
		{
			PropertyNode->RebuildChildren();

			OutCustomization.PropertyRow = MakeShared<FDetailPropertyRow>(PropertyNode, ParentCategory, RootPropertyNode);
			OutCustomization.PropertyRow->SetCustomExpansionId(Parameters.GetUniqueId());
		}
	}
	else
	{
		if (bAllowChildren && InitParams.bCreateCategoryNodes)
		{
			// Using the RootPropertyNode as the property node enables sub categories
			OutCustomization.PropertyRow = MakeShared<FDetailPropertyRow>(RootPropertyNode, ParentCategory, RootPropertyNode);
			OutCustomization.PropertyRow->SetCustomExpansionId(Parameters.GetUniqueId());
		}
		else
		{
			static const FName PropertyEditorModuleName("PropertyEditor");
			FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditorModuleName);

			// Make a "fake" struct property to represent the entire struct
			FStructProperty* StructProperty = PropertyEditorModule.RegisterStructProperty(StructClass);

			// Generate a node for the struct
			TSharedPtr<FItemPropertyNode> ItemNode = MakeShared<FItemPropertyNode>();

			FPropertyNodeInitParams ItemNodeInitParams;
			ItemNodeInitParams.ParentNode = RootPropertyNode;
			ItemNodeInitParams.Property = StructProperty;
			ItemNodeInitParams.ArrayOffset = 0;
			ItemNodeInitParams.ArrayIndex = INDEX_NONE;
			ItemNodeInitParams.bAllowChildren = true;
			ItemNodeInitParams.bForceHiddenPropertyVisibility = Parameters.ShouldForcePropertyVisible() || FPropertySettings::Get().ShowHiddenProperties();
			ItemNodeInitParams.bCreateCategoryNodes = false;

			ItemNode->InitNode(ItemNodeInitParams);

			RootPropertyNode->AddChildNode(ItemNode);

			OutCustomization.PropertyRow = MakeShared<FDetailPropertyRow>(ItemNode, ParentCategory, RootPropertyNode);
			OutCustomization.PropertyRow->SetCustomExpansionId(Parameters.GetUniqueId());
		}
	}
}

void FDetailPropertyRow::MakeExternalPropertyRowCustomization(TSharedPtr<FStructOnScope> StructData, FName PropertyName, TSharedRef<FDetailCategoryImpl> ParentCategory, FDetailLayoutCustomization& OutCustomization, const FAddPropertyParams& Parameters)
{
	MakeExternalStructPropertyRowCustomization<>(
		StructData,
		StructData->GetStruct(),
		PropertyName,
		ParentCategory,
		OutCustomization,
		Parameters,
		/* bAllowChildren= */ false
	);
}

void FDetailPropertyRow::MakeExternalPropertyRowCustomization(TSharedPtr<IStructureDataProvider> StructDataProvider, FName PropertyName, TSharedRef<FDetailCategoryImpl> ParentCategory, struct FDetailLayoutCustomization& OutCustomization, const FAddPropertyParams& Parameters)
{
	MakeExternalStructPropertyRowCustomization<>(
		StructDataProvider,
		StructDataProvider->GetBaseStructure(),
		PropertyName,
		ParentCategory,
		OutCustomization,
		Parameters,
		/* bAllowChildren= */ true
	);
}

void FDetailPropertyRow::MakeExternalPropertyRowCustomization(const TArray<UObject*>& InObjects, FName PropertyName, TSharedRef<FDetailCategoryImpl> ParentCategory, struct FDetailLayoutCustomization& OutCustomization, const FAddPropertyParams& Parameters)
{
	TSharedRef<FObjectPropertyNode> RootPropertyNode = MakeShared<FObjectPropertyNode>();

	for (UObject* Object : InObjects)
	{
		RootPropertyNode->AddObject(Object);
	}

	FPropertyNodeInitParams InitParams;
	InitParams.ParentNode = nullptr;
	InitParams.Property = nullptr;
	InitParams.ArrayOffset = 0;
	InitParams.ArrayIndex = INDEX_NONE;
	InitParams.bAllowChildren = false;
	InitParams.bForceHiddenPropertyVisibility = Parameters.ShouldForcePropertyVisible() || FPropertySettings::Get().ShowHiddenProperties();
	InitParams.bCreateCategoryNodes = PropertyName == NAME_None;

	Parameters.OverrideAllowChildren(InitParams.bAllowChildren);
	Parameters.OverrideCreateCategoryNodes(InitParams.bCreateCategoryNodes);

	RootPropertyNode->InitNode(InitParams);


	if (PropertyName != NAME_None)
	{
		TSharedPtr<FPropertyNode> PropertyNode = RootPropertyNode->GenerateSingleChild(PropertyName);
		if(PropertyNode.IsValid())
		{
			// This is useless as PropertyNode should already be in the child nodes
			RootPropertyNode->AddChildNode(PropertyNode);

			if (InitParams.bCreateCategoryNodes)
			{
				PropertyNode->SetNodeFlags(EPropertyNodeFlags::ShowCategories, true);
			}
			else
			{
				PropertyNode->SetNodeFlags(EPropertyNodeFlags::ShowCategories, false);
			}
			
			PropertyNode->RebuildChildren();

			OutCustomization.PropertyRow = MakeShared<FDetailPropertyRow>(PropertyNode, ParentCategory, RootPropertyNode);
			OutCustomization.PropertyRow->SetCustomExpansionId(Parameters.GetUniqueId());
		}
	}
	else
	{
		OutCustomization.PropertyRow = MakeShared<FDetailPropertyRow>(RootPropertyNode, ParentCategory, RootPropertyNode);
		OutCustomization.PropertyRow->SetCustomExpansionId(Parameters.GetUniqueId());
	}
	
	ParentCategory->GetParentLayoutImpl()->AddExternalRootPropertyNode(RootPropertyNode);
}

void FDetailPropertyRow::MakeChildPropertyRowCustomization(TSharedRef<IPropertyHandle> PropertyHandle,
	TSharedPtr<IStructureDataProvider> StructDataProvider, FName PropertyName, TSharedRef<FDetailCategoryImpl> ParentCategory,
	FDetailLayoutCustomization& OutCustomization, const FAddPropertyParams& Parameters, const FText& DisplayNameOverride)
{
	TSharedRef<FPropertyHandleBase> PropertyHandleImpl = StaticCastSharedRef<FPropertyHandleBase>(PropertyHandle);
	TSharedPtr<FStructurePropertyNode> RootPropertyNode = StaticCastSharedPtr<FStructurePropertyNode>(PropertyHandleImpl->GetPropertyNode());

	if (PropertyName != NAME_None)
	{
		TSharedPtr<FPropertyNode> PropertyNode = RootPropertyNode->GenerateSingleChild(PropertyName);
		if (PropertyNode.IsValid())
		{
			PropertyNode->RebuildChildren();

			OutCustomization.PropertyRow = MakeShared<FDetailPropertyRow>(PropertyNode, ParentCategory);
			OutCustomization.PropertyRow->SetCustomExpansionId(Parameters.GetUniqueId());
		}
	}
	else
	{
		// Generate a node for the struct
		TSharedRef<FStructurePropertyNode> StructPropertyNode = MakeShared<FStructurePropertyNode>();
		StructPropertyNode->SetStructure(StructDataProvider);
		StructPropertyNode->SetDisplayNameOverride(DisplayNameOverride);

		// Make a "fake" struct property to represent the entire struct
		static const FName PropertyEditorModuleName("PropertyEditor");
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditorModuleName);
		FStructProperty* StructProperty = PropertyEditorModule.RegisterStructProperty(StructDataProvider->GetBaseStructure());

		FPropertyNodeInitParams ItemNodeInitParams;
		ItemNodeInitParams.ParentNode = RootPropertyNode;
		ItemNodeInitParams.Property = StructProperty;
		ItemNodeInitParams.ArrayOffset = 0;
		ItemNodeInitParams.ArrayIndex = INDEX_NONE;
		ItemNodeInitParams.bAllowChildren = true;
		ItemNodeInitParams.bForceHiddenPropertyVisibility = Parameters.ShouldForcePropertyVisible() || FPropertySettings::Get().ShowHiddenProperties();
		ItemNodeInitParams.bCreateCategoryNodes = false;

		StructPropertyNode->InitNode(ItemNodeInitParams);
		RootPropertyNode->AddChildNode(StructPropertyNode);

		OutCustomization.PropertyRow = MakeShared<FDetailPropertyRow>(StructPropertyNode, ParentCategory);
		OutCustomization.PropertyRow->SetCustomExpansionId(Parameters.GetUniqueId());
	}
}

EVisibility FDetailPropertyRow::GetPropertyVisibility() const
{
	if (IsOnlyVisibleWhenEditConditionMet() && !IsEditConditionMet())
	{
		return EVisibility::Collapsed;
	}
	else if (CustomPropertyWidget.IsValid() && CustomPropertyWidget->VisibilityAttr.IsBound())
	{
		return CustomPropertyWidget->VisibilityAttr.Get();
	}

	return PropertyVisibility.Get();
}

bool FDetailPropertyRow::HasEditCondition() const
{
	return (PropertyEditor.IsValid() && PropertyEditor->HasEditCondition()) || CustomEditConditionValue.IsSet();
}

bool FDetailPropertyRow::GetEnabledState() const
{
	bool Result = IsParentEnabled.Get(true);

	Result = Result && CustomIsEnabledAttrib.Get(true);
	Result = Result && (CustomPropertyWidget == nullptr || CustomPropertyWidget->IsEnabledAttr.Get(true));
	Result = Result && IsEditConditionMet();

	return Result;
}

bool FDetailPropertyRow::IsEditConditionMet() const
{
	bool bIsEditConditionMet = true;

	if (HasEditCondition())
	{
		if (CustomEditConditionValue.IsSet())
		{
			bIsEditConditionMet = bIsEditConditionMet && CustomEditConditionValue.Get();
			
			// In override mode, we ship checking the native property edit condition
			if (CustomEditConditionMode == ECustomEditConditionMode::Override)
			{
				return bIsEditConditionMet;
			}
		}

		if (PropertyEditor.IsValid())
		{
			bIsEditConditionMet = bIsEditConditionMet && PropertyEditor->IsEditConditionMet();
		}
	}

	return bIsEditConditionMet;
}

bool FDetailPropertyRow::IsOnlyVisibleWhenEditConditionMet() const
{
	return (PropertyEditor.IsValid() && PropertyEditor->IsOnlyVisibleWhenEditConditionMet()) || (bCustomEditConditionHides && HasEditCondition());
}

TSharedPtr<IPropertyTypeCustomization>& FDetailPropertyRow::GetTypeInterface()
{
	if (!bCachedCustomTypeInterface)
	{
		if (PropertyNode.IsValid() && ParentCategory.IsValid())
		{
			CachedCustomTypeInterface = GetPropertyCustomization(PropertyNode.ToSharedRef(), ParentCategory.Pin().ToSharedRef());
		}
		bCachedCustomTypeInterface = true;
	}

	return CachedCustomTypeInterface;
}

bool FDetailPropertyRow::GetForceAutoExpansion() const
{
	return bForceAutoExpansion;
}

static void TogglePropertyEditorEditCondition(bool bValue, TWeakPtr<FPropertyEditor> PropertyEditorWeak)
{
	TSharedPtr<FPropertyEditor> PropertyEditorPtr = PropertyEditorWeak.Pin();
	if (PropertyEditorPtr.IsValid() && PropertyEditorPtr->IsEditConditionMet() != bValue)
	{
		PropertyEditorPtr->ToggleEditConditionState();
	}
}

static void ExecuteCustomEditConditionToggle(bool bValue, FOnBooleanValueChanged CustomEditConditionToggle, TWeakPtr<FPropertyEditor> PropertyEditorWeak)
{
	CustomEditConditionToggle.ExecuteIfBound(bValue);

	TSharedPtr<FPropertyEditor> PropertyEditorPtr = PropertyEditorWeak.Pin();
	if (PropertyEditorPtr.IsValid())
	{
		 PropertyEditorPtr->GetPropertyNode()->InvalidateCachedState();
	}
}

void FDetailPropertyRow::SetWidgetRowProperties(FDetailWidgetRow& Row) const
{
	// set edit condition handlers - use customized if provided
	TAttribute<bool> EditConditionValue;
	if (HasEditCondition())
	{
		EditConditionValue = TAttribute<bool>::CreateSP(this, &FDetailPropertyRow::IsEditConditionMet);
	}

	FOnBooleanValueChanged OnEditConditionValueChanged;
	if (CustomEditConditionValueChanged.IsBound())
	{
		TWeakPtr<FPropertyEditor> PropertyEditorWeak = PropertyEditor;
		OnEditConditionValueChanged = FOnBooleanValueChanged::CreateStatic(&ExecuteCustomEditConditionToggle, CustomEditConditionValueChanged, PropertyEditorWeak);
	}
	else if (PropertyEditor.IsValid() && PropertyEditor->SupportsEditConditionToggle())
	{
		TWeakPtr<FPropertyEditor> PropertyEditorWeak = PropertyEditor;
		OnEditConditionValueChanged = FOnBooleanValueChanged::CreateStatic(&TogglePropertyEditorEditCondition, PropertyEditorWeak);
	}

	Row.EditCondition(EditConditionValue, OnEditConditionValueChanged);
	Row.IsEnabled(CustomIsEnabledAttrib);
	Row.CustomResetToDefault = CustomResetToDefault;
	Row.CustomDragDropHandler = CustomDragDropHandler;
	Row.PropertyHandles.Add(GetPropertyHandle());

	// set custom actions and reset to default
	if (CustomPropertyWidget.IsValid())
	{
		Row.CopyMenuAction = CustomPropertyWidget->CopyMenuAction;
		Row.PasteMenuAction = CustomPropertyWidget->PasteMenuAction;
		Row.CustomMenuItems = CustomPropertyWidget->CustomMenuItems;
		Row.OnCopyToTextDelegate = CustomPropertyWidget->OnCopyToTextDelegate;
        Row.OnPasteFromTextDelegate = CustomPropertyWidget->OnPasteFromTextDelegate;
		Row.FilterTextString = CustomPropertyWidget->FilterTextString;

		if (CustomPropertyWidget->CustomResetToDefault.IsSet())
		{
			ensureMsgf(!CustomResetToDefault.IsSet(), TEXT("Duplicate reset to default handlers set on both FDetailPropertyRow and CustomWidget()!"));
			Row.CustomResetToDefault = CustomPropertyWidget->CustomResetToDefault;
		}

		if(CustomPropertyWidget->HasResetToDefaultContent())
		{
			Row.ResetToDefaultContent()
			[
				CustomPropertyWidget->ResetToDefaultWidget.Widget
			];
		}
	}
}

void FDetailPropertyRow::MakeNameOrKeyWidget( FDetailWidgetRow& Row, const TSharedPtr<FDetailWidgetRow> InCustomRow ) const
{
	EVerticalAlignment VerticalAlignment = VAlign_Center;
	EHorizontalAlignment HorizontalAlignment = HAlign_Fill;

	// We will only use key widgets for non-struct keys
	const bool bHasKeyNode = PropertyKeyEditor.IsValid();

	if (!bHasKeyNode && InCustomRow.IsValid())
	{
		VerticalAlignment = InCustomRow->NameWidget.VerticalAlignment;
		HorizontalAlignment = InCustomRow->NameWidget.HorizontalAlignment;
	}

	TAttribute<bool> IsEnabledAttrib = TAttribute<bool>::CreateSP( this, &FDetailPropertyRow::GetEnabledState );

	TSharedRef<SHorizontalBox> NameHorizontalBox = SNew(SHorizontalBox)
		.Clipping(EWidgetClipping::OnDemand);
	
	TSharedPtr<SWidget> NameWidget = SNullWidget::NullWidget;

	// Key nodes take precedence over custom rows
	if (bHasKeyNode)
	{
		if (PropertyHandle->HasMetaData(TEXT("ReadOnlyKeys")))
		{
			PropertyKeyEditor->GetPropertyNode()->SetNodeFlags(EPropertyNodeFlags::IsReadOnly, true);
		}

		// Does this key have a custom type, use it
		if (CachedKeyCustomTypeInterface)
		{
			// Create a widget that will properly represent the key
			const TSharedPtr<FDetailWidgetRow> CustomTypeWidget = MakeShared<FDetailWidgetRow>();
			CachedKeyCustomTypeInterface->CustomizeHeader(PropertyKeyEditor->GetPropertyHandle(), *CustomTypeWidget, const_cast<FDetailPropertyRow&>(*this));

			NameWidget = CustomTypeWidget->ValueWidget.Widget;
		}
		else
		{
			NameWidget =
				SNew(SPropertyValueWidget, PropertyKeyEditor, ParentCategory.Pin()->GetParentLayoutImpl()->GetPropertyUtilities())
				.IsEnabled(IsEnabledAttrib)
				.ShowPropertyButtons(false);
		}

	}
	else if (PropertyEditorHelpers::IsChildOfOption(*PropertyNode))
	{
		TSharedRef<FPropertyEditor> ParentEditor = FPropertyEditor::Create(
			PropertyNode->GetParentNode()->AsShared(), 
			ParentCategory.Pin()->GetParentLayoutImpl()->GetPropertyUtilities()
		);

		NameWidget = 
			SNew( SPropertyNameWidget, ParentEditor )
			.IsEnabled( IsEnabledAttrib );
	}
	else if (InCustomRow.IsValid())
	{
		NameWidget = 
			SNew( SBox )
			.IsEnabled( IsEnabledAttrib )
			[
				InCustomRow->NameWidget.Widget
			];
	}
	else if (PropertyEditor.IsValid())
	{
		NameWidget = 
			SNew( SPropertyNameWidget, PropertyEditor )
			.IsEnabled( IsEnabledAttrib );
	}

	SHorizontalBox::FSlot* SlotPointer = nullptr;
	NameHorizontalBox->AddSlot()
	.Expose(SlotPointer)
	[
		NameWidget.ToSharedRef()
	];

	if (bHasKeyNode)
	{
		SlotPointer->SetPadding(FMargin(0.0f, 0.0f, 2.0f, 0.0f));
	}
	else if (InCustomRow.IsValid())
	{
		// Allow custom name slots to fill all of the area. Eg., the user adds a SHorizontalBox with left and right align slots.
		SlotPointer->SetFillWidth(1.0f);
	}
	else
	{
		SlotPointer->SetAutoWidth();
	}

	Row.NameContent()
	.HAlign( HorizontalAlignment )
	.VAlign( VerticalAlignment )
	[
		NameHorizontalBox
	];
}

void FDetailPropertyRow::MakeValueWidget( FDetailWidgetRow& Row, const TSharedPtr<FDetailWidgetRow> InCustomRow, bool bAddWidgetDecoration ) const
{
	EVerticalAlignment VerticalAlignment = VAlign_Center;
	EHorizontalAlignment HorizontalAlignment = HAlign_Left;

	TOptional<float> MinWidth;
	TOptional<float> MaxWidth;

	TAttribute<bool> IsEnabledAttrib = TAttribute<bool>::CreateSP( this, &FDetailPropertyRow::GetEnabledState );

	TSharedRef<SHorizontalBox> ValueWidget = 
		SNew( SHorizontalBox )
		.IsEnabled( IsEnabledAttrib );

	if (InCustomRow.IsValid())
	{
		VerticalAlignment = InCustomRow->ValueWidget.VerticalAlignment;
		HorizontalAlignment = InCustomRow->ValueWidget.HorizontalAlignment;
		MinWidth = InCustomRow->ValueWidget.MinWidth;
		MaxWidth = InCustomRow->ValueWidget.MaxWidth;

		ValueWidget->AddSlot()
			.VAlign(VerticalAlignment)
		[
			InCustomRow->ValueWidget.Widget
		];

		Row
		.ExtensionContent()
		[
			InCustomRow->ExtensionWidget.Widget
		];
	}
	else if (PropertyEditor.IsValid())
	{
		TSharedPtr<SPropertyValueWidget> PropertyValue;
		ValueWidget->AddSlot()
		[
			SAssignNew( PropertyValue, SPropertyValueWidget, PropertyEditor, GetPropertyUtilities() )
			.ShowPropertyButtons( false ) // We handle this ourselves
			.InWidgetRow(&Row)
		];
		
		MinWidth = PropertyValue->GetMinDesiredWidth();
		MaxWidth = PropertyValue->GetMaxDesiredWidth();
	}

	if (bAddWidgetDecoration && PropertyEditor.IsValid())
	{
		if (bShowPropertyButtons)
		{
			TArray< TSharedRef<SWidget> > RequiredButtons;
			PropertyEditorHelpers::MakeRequiredPropertyButtons( PropertyEditor.ToSharedRef(), /*OUT*/RequiredButtons );

			for( int32 ButtonIndex = 0; ButtonIndex < RequiredButtons.Num(); ++ButtonIndex )
			{
				ValueWidget->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(4.0f, 1.0f, 0.0f, 1.0f)
				[ 
					RequiredButtons[ButtonIndex]
				];
			}
		}

		// Don't add config hierarchy to container children, can't edit child properties at the hiearchy's per file level
		TSharedPtr<IPropertyHandle> ParentHandle = PropertyHandle->GetParentHandle();
		bool bIsChildProperty = ParentHandle && (ParentHandle->AsArray() || ParentHandle->AsMap() || ParentHandle->AsSet());

		if (!bIsChildProperty && PropertyHandle->HasMetaData(TEXT("ConfigHierarchyEditable")))
		{
			ValueWidget->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(4.0f, 0.0f, 4.0f, 0.0f)
			[
				PropertyCustomizationHelpers::MakeEditConfigHierarchyButton(FSimpleDelegate::CreateSP(PropertyEditor.ToSharedRef(), &FPropertyEditor::EditConfigHierarchy))
			];
		}
	}

	Row.ValueContent()
		.HAlign( HorizontalAlignment )
		.VAlign( VerticalAlignment )	
		.MinDesiredWidth( MinWidth )
		.MaxDesiredWidth( MaxWidth )
		[
			ValueWidget
		];
}

#undef LOCTEXT_NAMESPACE
