// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "AssetThumbnail.h"
#include "IPropertyUtilities.h"
#include "PropertyNode.h"
#include "IPropertyTypeCustomization.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilderImpl.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "IDetailPropertyRow.h"
#include "PropertyEditorHelpers.h"

struct FAddPropertyParams;

class FCustomChildrenBuilder;
class IDetailGroup;

class FDetailPropertyRow : public IDetailPropertyRow, public IPropertyTypeCustomizationUtils, public IDetailLayoutRow, public TSharedFromThis<FDetailPropertyRow>
{
public:
	FDetailPropertyRow(TSharedPtr<FPropertyNode> InPropertyNode, TSharedRef<FDetailCategoryImpl> InParentCategory, TSharedPtr<FComplexPropertyNode> InExternalRootNode = nullptr);

	/** IDetailPropertyRow interface */
	virtual TSharedPtr<IPropertyHandle> GetPropertyHandle() const override { return PropertyHandle; }
	virtual IDetailPropertyRow& DisplayName( const FText& InDisplayName ) override;
	virtual IDetailPropertyRow& ToolTip( const FText& InToolTip ) override;
	virtual IDetailPropertyRow& ShowPropertyButtons( bool bInShowPropertyButtons ) override;
	virtual IDetailPropertyRow& EditCondition( TAttribute<bool> EditConditionValue, FOnBooleanValueChanged OnEditConditionValueChanged, ECustomEditConditionMode EditConditionMode = ECustomEditConditionMode::Override ) override;
	virtual IDetailPropertyRow& EditConditionHides(bool bEditConditionHideValue) override;
	virtual IDetailPropertyRow& IsEnabled(TAttribute<bool> InIsEnabled) override;
	virtual IDetailPropertyRow& ShouldAutoExpand(bool bForceExpansion) override;
	virtual IDetailPropertyRow& Visibility( TAttribute<EVisibility> Visibility ) override;
	virtual IDetailPropertyRow& OverrideResetToDefault(const FResetToDefaultOverride& ResetToDefault) override;
	virtual IDetailPropertyRow& DragDropHandler(TSharedPtr<IDetailDragDropHandler> InDragDropHandler) override;
	virtual bool IsExpanded() const override;
	virtual FDetailWidgetRow& CustomWidget( bool bShowChildren = false ) override;
	virtual FDetailWidgetDecl* CustomNameWidget() override;
	virtual FDetailWidgetDecl* CustomValueWidget() override;
	virtual FDetailWidgetDecl* CustomResetToDefaultWidget() override;
	virtual void GetDefaultWidgets( TSharedPtr<SWidget>& OutNameWidget, TSharedPtr<SWidget>& OutValueWidget, bool bAddWidgetDecoration = false) override;
	virtual void GetDefaultWidgets( TSharedPtr<SWidget>& OutNameWidget, TSharedPtr<SWidget>& OutValueWidget, FDetailWidgetRow& Row, bool bAddWidgetDecoration = false) override;

	/** IPropertyTypeCustomizationUtils interface */
	virtual TSharedPtr<class FAssetThumbnailPool> GetThumbnailPool() const override;
	virtual TSharedPtr<class IPropertyUtilities> GetPropertyUtilities() const override;

	/** IDetailLayoutRow interface */
	virtual FName GetRowName() const override;
	virtual TOptional<FResetToDefaultOverride> GetCustomResetToDefault() const override { return CustomResetToDefault; }


	/** @return true if this row has widgets with columns */
	bool HasColumns() const;

	/** @return true if this row shows only children and is not visible itself */
	bool ShowOnlyChildren() const;

	/** @return true if this row should be ticked */
	bool RequiresTick() const;

	/** @return true if this row has an external property */
	bool HasExternalProperty() const { return ExternalRootNode.IsValid(); }

	/** Sets the custom name that should be used to save and restore this nodes expansion state */
	void SetCustomExpansionId(const FName ExpansionIdName) { CustomExpansionIdName = ExpansionIdName; }

	/** @return Gets the custom name that should be used to save and restore this nodes expansion state */
	FName GetCustomExpansionId() const { return CustomExpansionIdName; }

	/** 
	 * Sets whether or not we force only children to be shown regardless of what the customization wants.
	 * Typically used when this row is only for organizational purposes and doesnt have anything valid to show (like external object nodes)
	 */
	void SetForceShowOnlyChildren(bool bInForceShowOnlyChildren) { bForceShowOnlyChildren = bInForceShowOnlyChildren; }

	/**
	 * Called when the owner node is initialized
	 *
	 * @param InParentCategory	The category that this property is in
	 * @param InIsParentEnabled	Whether or not our parent is enabled
	 */
	void OnItemNodeInitialized( TSharedRef<FDetailCategoryImpl> InParentCategory, const TAttribute<bool>& InIsParentEnabled, TSharedPtr<IDetailGroup> InParentGroup);

	/** @return The widget row that should be displayed for this property row */
	FDetailWidgetRow GetWidgetRow();

	/** @return properties being customized */
	TArrayView<TSharedPtr<IPropertyHandle>> GetPropertyHandles() const;

	/** @return the filter text associated with this row, if any */
	FText GetFilterTextString() const;

	/**
	 * @return The property node for this row
	 */
	TSharedPtr<FPropertyNode> GetPropertyNode() const { return PropertyNode; }

	/**
	 * @return The external root node for this row if it has one.
	 */
	TSharedPtr<FComplexPropertyNode> GetExternalRootNode() const { return ExternalRootNode; }

	/**
	 * @return The property editor for this row
	 */
	TSharedPtr<FPropertyEditor> GetPropertyEditor() { return PropertyEditor; }

	/**
	 * Called when children of this row should be generated
	 *
	 * @param OutChildren	The list of children created
	 */
	void OnGenerateChildren( FDetailNodeList& OutChildren );
	
	/**
	 * @return Whether or not this row wants to force expansion
	 */
	bool GetForceAutoExpansion() const;

	/** 
	 * @return The visibility of this property
	 */
	EVisibility GetPropertyVisibility() const;

	static void MakeExternalPropertyRowCustomization(TSharedPtr<FStructOnScope> StructData, FName PropertyName, TSharedRef<FDetailCategoryImpl> ParentCategory, struct FDetailLayoutCustomization& OutCustomization, const FAddPropertyParams& Parameters);
	static void MakeExternalPropertyRowCustomization(TSharedPtr<IStructureDataProvider> StructDataProvider, FName PropertyName, TSharedRef<FDetailCategoryImpl> ParentCategory, struct FDetailLayoutCustomization& OutCustomization, const FAddPropertyParams& Parameters);
	static void MakeExternalPropertyRowCustomization(const TArray<UObject*>& InObjects, FName PropertyName, TSharedRef<FDetailCategoryImpl> ParentCategory, struct FDetailLayoutCustomization& OutCustomization, const FAddPropertyParams& Parameters);
	static void MakeChildPropertyRowCustomization(TSharedRef<IPropertyHandle> PropertyHandle, TSharedPtr<IStructureDataProvider> StructDataProvider, FName PropertyName, TSharedRef<FDetailCategoryImpl> ParentCategory, struct FDetailLayoutCustomization& OutCustomization, const FAddPropertyParams& Parameters, const FText& DisplayNameOverride = FText());

private:
	/**
	 * Makes a name widget or key widget for the tree
	 */
	void MakeNameOrKeyWidget( FDetailWidgetRow& Row, const TSharedPtr<FDetailWidgetRow> InCustomRow ) const;

	/**
	 * Makes the value widget for the tree
	 */
	void MakeValueWidget( FDetailWidgetRow& Row, const TSharedPtr<FDetailWidgetRow> InCustomRow, bool bAddWidgetDecoration = true ) const;

	/**
	 * Set the given widget row's custom properties. Note: Should be called before MakeNameOrKeyWidget and MakeValueWidget.
	 */
	void SetWidgetRowProperties( FDetailWidgetRow& Row ) const;

	/**
	 * @return true if this row has an edit condition
	 */
	bool HasEditCondition() const;

	/**
	 * @return Whether or not this row is enabled
	 */
	bool GetEnabledState() const;

	/**
	 * @return true if this row's edit condition has been met
	 */
	bool IsEditConditionMet() const;

	/**
	 * @return true if this row should be shown/hidden based on the edit condition state
	 */
	bool IsOnlyVisibleWhenEditConditionMet() const;

	/** 
	 * Looks up and caches (if not already) the associated type interface, and then returns it.
	 * Could be null/invalid if no customization was specified for this row.
	 *
	 * Creating the CustomTypeInterface on demand, instead of at construction, allows for 
	 * IDetailCustomization's to override custom property type layouts via a IDetailLayoutBuilder.
	 * 
	 */
	TSharedPtr<IPropertyTypeCustomization>& GetTypeInterface();

	/**
	 * Generates children for the property node
	 *
	 * @param RootPropertyNode	The property node to generate children for (May be a child of the property node being stored)
	 * @param OutChildren	The list of children created 
	 */
	void GenerateChildrenForPropertyNode( TSharedPtr<FPropertyNode>& RootPropertyNode, FDetailNodeList& OutChildren );

	/**
	 * Makes a property editor from the property node on this row
	 *
	 * @param InPropertyNode	The node to create the editor for
	 * @param PropertyUtilities	Utilities for the property editor
	 * @param InEditor			The editor we wish to create if it does not yet exist.
	 */
	static TSharedRef<FPropertyEditor> MakePropertyEditor( const TSharedRef<FPropertyNode>& InPropertyNode, const TSharedRef<IPropertyUtilities>& PropertyUtilities, TSharedPtr<FPropertyEditor>& InEditor );

	/**
	 * Gets the property customization for the specified node
	 *
	 * @param	InPropertyNode		The node to check for customizations
	 * @param	InParentCategory	The node's parent category
	 * @return	The property customization for the node, or null if one does not exist
	 */
	static TSharedPtr<IPropertyTypeCustomization> GetPropertyCustomization( const TSharedRef<FPropertyNode>& InPropertyNode, const TSharedRef<FDetailCategoryImpl>& InParentCategory );

	/** Does the given property node require a key node to be created, or is a key property editor sufficient? */
	static bool NeedsKeyNode(TSharedRef<FPropertyNode> InPropertyNode, TSharedRef<FDetailCategoryImpl> InParentCategory);

private:
	/** User driven enabled state */
	TAttribute<bool> CustomIsEnabledAttrib;
	/** Whether or not our parent is enabled */
	TAttribute<bool> IsParentEnabled;
	/** Visibility of the property */
	TAttribute<EVisibility> PropertyVisibility;
	/** If the property on this row is a customized property type, this is the interface to that customization */
	TSharedPtr<IPropertyTypeCustomization> CachedCustomTypeInterface;
	/** Builder for children of a customized property type */
	TSharedPtr<class FCustomChildrenBuilder> PropertyTypeLayoutBuilder;
	/** The property handle for this row */
	TSharedPtr<IPropertyHandle> PropertyHandle;
	/** The property node for this row */
	TSharedPtr<FPropertyNode> PropertyNode;
	/** The property editor for this row */
	TSharedPtr<FPropertyEditor> PropertyEditor;
	/** The property editor for this row's key */
	TSharedPtr<FPropertyEditor> PropertyKeyEditor;
	/** The property Type customization for this row's key */
	TSharedPtr<IPropertyTypeCustomization> CachedKeyCustomTypeInterface;
	/** Custom widgets to use for this row instead of the default ones */
	TSharedPtr<FDetailWidgetRow> CustomPropertyWidget;
	/** User customized edit condition */
	TAttribute<bool> CustomEditConditionValue;
	/** User customized edit condition change handler. */
	FOnBooleanValueChanged CustomEditConditionValueChanged;
	/** Should this custom edit condition override or merge with the existing edit condition of the property (if any)? */
	ECustomEditConditionMode CustomEditConditionMode = ECustomEditConditionMode::Override;
	/** User customized edit condition hides specifier */
	bool bCustomEditConditionHides = false;
	/** User customized reset to default */
	TOptional<FResetToDefaultOverride> CustomResetToDefault;
	/** User customized drag/drop handler */
	TSharedPtr<IDetailDragDropHandler> CustomDragDropHandler;
	/** The category this row resides in */
	TWeakPtr<FDetailCategoryImpl> ParentCategory;
	/** Root of the property node if this node comes from an external tree */
	TSharedPtr<FComplexPropertyNode> ExternalRootNode;

	TSharedPtr<struct FDetailLayoutData> ExternalObjectLayout;
	/** The custom expansion ID name used to save and restore expansion state on this node */
	FName CustomExpansionIdName;
	/** Whether or not to show standard property buttons */
	bool bShowPropertyButtons;
	/** True to show custom property children */
	bool bShowCustomPropertyChildren;
	/** True to force auto-expansion */
	bool bForceAutoExpansion;
	/** True if we've already attempted to fill out our CustomTypeInterface (no need to rerun the query) */
	bool bCachedCustomTypeInterface;
	/** True if this row is only for organizational purposes and doesnt have anything valid to show itself*/
	bool bForceShowOnlyChildren = false;
};
