// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "DetailRowMenuContextPrivate.generated.h"

class SDetailTableRowBase;

/** The private context provides the slate widget. */
UCLASS()
class UDetailRowMenuContextPrivate : public UObject
{
	GENERATED_BODY()

public:
	template <typename RowType>
	TSharedPtr<RowType> GetRowWidget() const
	{
		if (const TWeakPtr<RowType> RowContextWeak = StaticCastWeakPtr<RowType>(Row);
			RowContextWeak.IsValid())
		{
			if (TSharedPtr<RowType> RowContext = RowContextWeak.Pin())
			{
				return RowContext;
			}
		}

		return nullptr;
	}

public:
	TWeakPtr<SDetailTableRowBase> Row;
};

