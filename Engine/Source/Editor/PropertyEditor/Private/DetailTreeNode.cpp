// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailTreeNode.h"
#include "DetailWidgetRow.h"
#include "PropertyEditorHelpers.h"
#include "ObjectPropertyNode.h"
#include "DetailCategoryBuilderImpl.h"

FNodeWidgets FDetailTreeNode::CreateNodeWidgets() const
{
	FDetailWidgetRow Row;
	GenerateStandaloneWidget(Row);

	FNodeWidgets Widgets;

	if (Row.HasAnyContent())
	{
		if (Row.HasColumns())
		{
			Widgets.NameWidget = Row.NameWidget.Widget;
			Widgets.NameWidgetLayoutData = FNodeWidgetLayoutData(
				Row.NameWidget.HorizontalAlignment, Row.NameWidget.VerticalAlignment, Row.NameWidget.MinWidth, Row.NameWidget.MaxWidth);
			Widgets.ValueWidget = Row.ValueWidget.Widget;
			Widgets.ValueWidgetLayoutData = FNodeWidgetLayoutData(
				Row.ValueWidget.HorizontalAlignment, Row.ValueWidget.VerticalAlignment, Row.ValueWidget.MinWidth, Row.ValueWidget.MaxWidth);
		}
		else
		{
			Widgets.WholeRowWidget = Row.WholeRowWidget.Widget;
			Widgets.WholeRowWidgetLayoutData = FNodeWidgetLayoutData(
				Row.WholeRowWidget.HorizontalAlignment, Row.WholeRowWidget.VerticalAlignment, Row.WholeRowWidget.MinWidth, Row.WholeRowWidget.MaxWidth);
		}

		Widgets.EditConditionWidget = SNew(SEditConditionWidget)
			.EditConditionValue(Row.EditConditionValue)
			.OnEditConditionValueChanged(Row.OnEditConditionValueChanged);
	}

	Widgets.Actions.CopyMenuAction = Row.CopyMenuAction;
	Widgets.Actions.PasteMenuAction = Row.PasteMenuAction;
	for (const FDetailWidgetRow::FCustomMenuData& CustomMenuItem : Row.CustomMenuItems)
	{
		Widgets.Actions.CustomMenuItems.Add(FNodeWidgetActionsCustomMenuData(CustomMenuItem.Action, CustomMenuItem.Name, CustomMenuItem.Tooltip, CustomMenuItem.SlateIcon));
	}

	return Widgets;
}

void FDetailTreeNode::GetChildren(TArray<TSharedRef<IDetailTreeNode>>& OutChildren, const bool& bInIgnoreVisibility)
{
	FDetailNodeList Children;
	GetChildren(Children, bInIgnoreVisibility);

	OutChildren.Reset(Children.Num());

	OutChildren.Append(Children);
}

const UStruct* FDetailTreeNode::GetPropertyNodeBaseStructure(const FPropertyNode* PropertyNode)
{
	while (PropertyNode)
	{
		if (const FComplexPropertyNode* ComplexNode = PropertyNode->AsComplexNode())
		{
			return ComplexNode->GetBaseStructure();
		}
		else if (const FProperty* CurrentProperty = PropertyNode->GetProperty())
		{
			FFieldClass* PropertyClass = CurrentProperty->GetClass();
			if (PropertyClass == FStructProperty::StaticClass())
			{

				return static_cast<const FStructProperty*>(CurrentProperty)->Struct;
			}
			// Object Properties seem to always show up as ObjectPropertyNodes, which are handled above
			//else if (PropertyClass == FObjectProperty::StaticClass())
			//{
			//}
		}
		PropertyNode = PropertyNode->GetParentNode();
	}
	return nullptr;
}

const UStruct* FDetailTreeNode::GetParentBaseStructure() const
{
	if (TSharedPtr<FPropertyNode> PropertyNode = GetPropertyNode())
	{
		// We don't want this node to return its own UStruct - we want its parent
		if (const UStruct* BaseStructure = GetPropertyNodeBaseStructure(PropertyNode->GetParentNode()))
		{
			return BaseStructure;
		}
	}

	TWeakPtr<FDetailTreeNode> CurrentParentNode = GetParentNode();
	while (TSharedPtr<FDetailTreeNode> PinnedParent = CurrentParentNode.Pin())
	{
		if (TSharedPtr<FPropertyNode> PropertyNode = PinnedParent->GetPropertyNode())
		{
			// Since we're already checking parents, we can use the first property node directly instead of jumping to its parent like before
			if (const UStruct* BaseStructure = GetPropertyNodeBaseStructure(PropertyNode.Get()))
			{
				return BaseStructure;
			}
		}
		CurrentParentNode = PinnedParent->GetParentNode();
	}

	// If property nodes fail, check the detail layout
	if (TSharedPtr<FDetailCategoryImpl> ParentCategory = GetParentCategory())
	{
		return ParentCategory->GetParentLayout().GetBaseClass();
	}
	// If category fails, use the first selected object. This should be rare, if it's even possible
	// (maybe a custom builder appearing in a detail property row or something?)
	for (TWeakObjectPtr<UObject> Selected : GetDetailsViewSharedPtr()->GetSelectedObjects())
	{
		if (UObject* Object = Selected.Get())
		{
			return Object->GetClass();
		}
	}
	return nullptr;
}
