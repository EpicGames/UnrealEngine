// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailWidgetRow.h"
#include "PropertyEditorHelpers.h"

const float FDetailWidgetRow::DefaultValueMinWidth = 125.0f;
const float FDetailWidgetRow::DefaultValueMaxWidth = 125.0f;

const FName FDetailWidgetRow::FCustomMenuData::GetEntryName() const
{
	if (!EntryName.IsNone())
	{
		return EntryName;
	}
	
	return FName(Name.ToString().Replace(TEXT(" "), TEXT("")));
}
