//  Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsDisplayManager.h"

#include "DetailsViewStyle.h"
#include "SDetailsView.h"
#include "DetailLayoutBuilderImpl.h"
#include "DetailCategoryBuilderImpl.h"
#include "ObjectPropertyNode.h"

#define LOCTEXT_NAMESPACE "DetailsDisplayManager"

static TAutoConsoleVariable<bool> CVarForceShowComponentEditor(
	TEXT("DetailsPanel.UI.ForceShowComponentEditor"),
	false,
	TEXT("If true, forces the component editor to show in the main viewport and blueprint details panel for UObjects which normally have it hidden."));


FConstructPropertyUpdatedWidgetBuilderArgs::~FConstructPropertyUpdatedWidgetBuilderArgs() = default;

FDetailsDisplayManager::FDetailsDisplayManager(): bIsOuterCategory(false)
{
	PrimaryStyleKey = SDetailsView::GetPrimaryDetailsViewStyleKey();
}

FDetailsDisplayManager::~FDetailsDisplayManager()
{
	OnDetailsNeedsUpdate.Unbind();
}

TOptional<bool> FDetailsDisplayManager::OverrideCreateCategoryNodes() const
{
	return TOptional<bool>();
}

bool FDetailsDisplayManager::ShouldHideComponentEditor()
{
	return !GetForceShowSubObjectEditor();
}

bool FDetailsDisplayManager::ShouldShowCategoryMenu()
{
	return false;
}

TSharedPtr<SWidget> FDetailsDisplayManager::GetCategoryMenu(FName InCategoryObjectName)
{
	return nullptr;
}

void FDetailsDisplayManager::UpdateView() const
{
	OnDetailsNeedsUpdate.ExecuteIfBound();
}

const FDetailsViewStyleKey& FDetailsDisplayManager::GetDetailsViewStyleKey() const
{
	return PrimaryStyleKey;
}

void FDetailsDisplayManager::SetIsOuterCategory(bool bInIsOuterCategory)
{
	bIsOuterCategory = bInIsOuterCategory;
}

const FDetailsViewStyle* FDetailsDisplayManager::GetDetailsViewStyle() const
{
	const FDetailsViewStyle* ViewStyle = FDetailsViewStyle::GetStyle(GetDetailsViewStyleKey());
	return ViewStyle;
}

FMargin FDetailsDisplayManager::GetTablePadding() const
{
	const FDetailsViewStyle* Style = GetDetailsViewStyle();
	return Style ? Style->GetTablePadding(bIsScrollBarNeeded) : 0;
}

TSharedRef<SWidget> FDetailsDisplayManager::ConstructDetailTreeOverlay()
{
	return SNullWidget::NullWidget;
}

void FDetailsDisplayManager::UpdatePropertyForCategory(FName InCategoryObjectName, FProperty* Property, bool bAddProperty)
{
	TSet<FProperty*> Set;
	const TSet<FProperty*>* PropertySetPtr =  CategoryNameToUpdatePropertySetMap.Find(InCategoryObjectName);

	if ( PropertySetPtr )
	{
		Set = *PropertySetPtr;
	}

	if ( bAddProperty )
	{
		Set.Add(Property );
	}
	else
	{
		Set.Remove( Property );
	}
	
	CategoryNameToUpdatePropertySetMap.Emplace(InCategoryObjectName, Set);
}

bool FDetailsDisplayManager::GetCategoryHasAnyUpdatedProperties(FName InCategoryObjectName) const
{
	if ( const TSet<FProperty*>* PropertySetPtr =  CategoryNameToUpdatePropertySetMap.Find(InCategoryObjectName) )
	{
		TSet<FProperty*> Set = *PropertySetPtr;
		return Set.Num() > 0;
	}
	return false;
}

bool FDetailsDisplayManager::ShowEmptyCategoryIfRootUObjectHasNoPropertyData(UObject* InNode) const
{
	return false;
}


bool FDetailsDisplayManager::AddEmptyCategoryToDetailLayoutIfNeeded(TSharedRef<FComplexPropertyNode> Node,
																	TSharedRef<FDetailLayoutBuilderImpl> DetailLayoutBuilder)
{
	if (FObjectPropertyNode* ObjectPropertyNode = Node->AsObjectNode())
	{
		/* a pointer to the UObject for this Category, if we have one */
		UObject* UObjectForCategory = nullptr;

		const bool bPropertyNodeHasOneUObject =  ObjectPropertyNode && ObjectPropertyNode->GetNumObjects() == 1;
		if ( bPropertyNodeHasOneUObject )
		{
			UObjectForCategory = ObjectPropertyNode->GetUObject(0);
		}

		if (UObjectForCategory &&
			/* we're supposed to show an empty category for this particular object if it has no UProperties */
			ShowEmptyCategoryIfRootUObjectHasNoPropertyData(UObjectForCategory) &&
			UObjectForCategory->GetClass())
		{
			/* Base empty category display text off the display text of the UObject class */
			const FText UObjectClassNameDisplayText = UObjectForCategory->GetClass()->GetDisplayNameText();
			
			/* Base empty category name off the name of the UObject class */
			const FName UObjectClassName = UObjectForCategory->GetClass()->GetFName();
			
			FDetailCategoryImpl& EmptyCategory = DetailLayoutBuilder->DefaultCategory( UObjectClassName );
			EmptyCategory.SetDisplayName(UObjectClassName, UObjectClassNameDisplayText);

			/* property node for now will always show empty ~ eventually we will want to add hooks to optionally show
			 * a "No properties here" or "This empty component does XYZ" string of some kind that can be overridden
			 * based on object type and usage */
			EmptyCategory.AddPropertyNode(Node, UObjectClassName);
			EmptyCategory.SetIsEmpty(true);
			return true;
		}
	}
	return false;
}

TSharedPtr<FPropertyUpdatedWidgetBuilder> FDetailsDisplayManager::GetPropertyUpdatedWidget(FResetToDefault ResetToDefault, bool bIsCategoryUpdateWidget, FName InCategoryObjectName )
{
	return nullptr;
}

TSharedPtr<FPropertyUpdatedWidgetBuilder> FDetailsDisplayManager::GetPropertyUpdatedWidget(FResetToDefault ResetToDefault, TSharedRef<FEditPropertyChain> Chain, FName InCategoryObjectName)
{
	return nullptr;
}

bool FDetailsDisplayManager::CanConstructPropertyUpdatedWidgetBuilder() const
{
	return false;
}

TSharedPtr<FPropertyUpdatedWidgetBuilder> FDetailsDisplayManager::ConstructPropertyUpdatedWidgetBuilder(const FConstructPropertyUpdatedWidgetBuilderArgs& Args)
{
	return nullptr;
}

bool FDetailsDisplayManager::GetIsScrollBarNeeded() const
{
	return bIsScrollBarNeeded;
}

void FDetailsDisplayManager::SetIsScrollBarNeeded(bool bInIsScrollBarNeeded)
{
	bIsScrollBarNeeded = bInIsScrollBarNeeded;
}

bool FDetailsDisplayManager::GetForceShowSubObjectEditor()
{
	return CVarForceShowComponentEditor.GetValueOnAnyThread();
}

#undef LOCTEXT_NAMESPACE