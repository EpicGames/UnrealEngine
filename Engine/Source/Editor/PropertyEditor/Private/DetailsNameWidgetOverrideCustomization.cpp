// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsNameWidgetOverrideCustomization.h"
#include "PropertyPath.h"
#include "Widgets/SWidget.h"

FDetailsNameWidgetOverrideCustomization::FDetailsNameWidgetOverrideCustomization()
{
}

TSharedRef<SWidget> FDetailsNameWidgetOverrideCustomization::CustomizeName(TSharedRef<SWidget> InnerNameContent, FPropertyPath& Path)
{
	return InnerNameContent;
}
