// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsViewConfig.h"

#include "UObject/ObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DetailsViewConfig)

TObjectPtr<UDetailsConfig> UDetailsConfig::Instance = nullptr;

void UDetailsConfig::Initialize()
{
	if (Instance == nullptr)
	{
		Instance = NewObject<UDetailsConfig>();
		Instance->AddToRoot();
	}
}

UDetailsConfig* UDetailsConfig::Get()
{
	return Instance;
}
