// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/Set.h"
#include "EditorConfigBase.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"

#include "DetailsViewConfig.generated.h"

class UObject;
template <typename T> struct TObjectPtr;

USTRUCT()
struct FDetailsSectionSelection
{
	GENERATED_BODY()

public:

	UPROPERTY()
	TSet<FName> SectionNames;
};

USTRUCT()
struct FDetailsViewConfig
{
	GENERATED_BODY()

public:
	
	/** If we should show the favorites category. */
	UPROPERTY()
	bool bShowFavoritesCategory { false };

	/** When enabled, the Advanced Details will always auto expand. */
	UPROPERTY()
	bool bShowAllAdvanced { false };

	/** When Playing or Simulating, shows all properties (even non-visible and non-editable properties), if the object belongs to a simulating world.  This is useful for debugging. */
	UPROPERTY()
	bool bShowHiddenPropertiesWhilePlaying { false };

	/** Show all category children if the category matches the filter. */
	UPROPERTY()
	bool bShowAllChildrenIfCategoryMatches { false };

	/** Show only keyable properties. */
	UPROPERTY()
	bool bShowOnlyKeyable { false };

	/** Show only animated properties. */
	UPROPERTY()
	bool bShowOnlyAnimated { false };

	/** Show only modified properties. */
	UPROPERTY()
	bool bShowOnlyModified { false };

	/** Show sections. */
	UPROPERTY()
	bool bShowSections { true };

	/** Width of the value column in the details view (0.0-1.0). */
	UPROPERTY()
	float ValueColumnWidth { 0 };

	/** A map of class name to a set of selected sections for that class. */
	UPROPERTY()
	TMap<FName, FDetailsSectionSelection> SelectedSections;
	
	/** Show Favorites and Modified sections */
	UPROPERTY()
	bool bShowHelperSections = false;
};

UCLASS(EditorConfig="DetailsView")
class UDetailsConfig : public UEditorConfigBase
{
	GENERATED_BODY()
	
public:

	UPROPERTY(meta=(EditorConfig))
	TMap<FName, FDetailsViewConfig> Views;

	static void Initialize();
	static UDetailsConfig* Get();

private:

	static TObjectPtr<UDetailsConfig> Instance;
};