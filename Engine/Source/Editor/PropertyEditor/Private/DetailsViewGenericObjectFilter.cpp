// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsViewGenericObjectFilter.h"

#include "HAL/PlatformCrt.h"

class UObject;


FDetailsViewDefaultObjectFilter::FDetailsViewDefaultObjectFilter(bool bInAllowMultipleRoots)
	: bAllowMultipleRoots(bInAllowMultipleRoots)
{
}

TArray<FDetailsViewObjectRoot> FDetailsViewDefaultObjectFilter::FilterObjects(const TArray<UObject*>& SourceObjects)
{
	TArray<FDetailsViewObjectRoot> Roots;
	if (bAllowMultipleRoots)
	{
		Roots.Reserve(SourceObjects.Num());
		for (UObject* Object : SourceObjects)
		{
			Roots.Emplace(Object);
		}
	}
	else
	{
		Roots.Emplace(SourceObjects);
	}

	return Roots;
}