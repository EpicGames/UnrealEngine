// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "DetailsViewObjectFilter.h"

class UObject;

/**
 * The default object set is just a pass through to the details panel.  
 */
class FDetailsViewDefaultObjectFilter : public FDetailsViewObjectFilter
{
public:
	FDetailsViewDefaultObjectFilter(bool bInAllowMultipleRoots);

	/**
	 * FDetailsViewObjectFilter interface
	 */
	virtual TArray<FDetailsViewObjectRoot> FilterObjects(const TArray<UObject*>& SourceObjects);

private:
	bool bAllowMultipleRoots;
};