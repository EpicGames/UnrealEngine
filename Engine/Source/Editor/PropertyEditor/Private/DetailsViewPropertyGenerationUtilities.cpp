// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsViewPropertyGenerationUtilities.h"

FDetailsViewPropertyGenerationUtilities::FDetailsViewPropertyGenerationUtilities(SDetailsViewBase& InDetailsView)
	: DetailsViewPtr(StaticCastWeakPtr<SDetailsViewBase>(InDetailsView.AsWeak()))
{
}

const FCustomPropertyTypeLayoutMap& FDetailsViewPropertyGenerationUtilities::GetInstancedPropertyTypeLayoutMap() const
{
	if (TSharedPtr<SDetailsViewBase> DetailsView = DetailsViewPtr.Pin())
	{
		return DetailsView->GetCustomPropertyTypeLayoutMap();
	}

	static FCustomPropertyTypeLayoutMap Empty;
	return Empty;
}

void FDetailsViewPropertyGenerationUtilities::RebuildTreeNodes()
{
	if (TSharedPtr<SDetailsViewBase> DetailsView = DetailsViewPtr.Pin())
	{
		DetailsView->RerunCurrentFilter();
	}
}