// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyGenerationUtilities.h"
#include "SDetailsViewBase.h"

/** Property generation utilities for widgets derived from SDetailsViewBase. */
class FDetailsViewPropertyGenerationUtilities : public IPropertyGenerationUtilities
{
public:
	FDetailsViewPropertyGenerationUtilities(SDetailsViewBase& InDetailsView);
	/** IPropertyGenerationUtilities interface */
	virtual const FCustomPropertyTypeLayoutMap& GetInstancedPropertyTypeLayoutMap() const override;
	virtual void RebuildTreeNodes() override;

private:
	TWeakPtr<SDetailsViewBase> DetailsViewPtr;
};