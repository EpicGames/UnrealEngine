//  Copyright Epic Games, Inc. All Rights Reserved.


#include "DetailsViewStyle.h"
#include "Containers/Map.h"
#include "Brushes/SlateImageBrush.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "UObject/OverridableManager.h"
#include "Layout/Visibility.h"
#include "Styling/AppStyle.h"
#include "Subsystems/PropertyVisibilityOverrideSubsystem.h"

#define LOCTEXT_NAMESPACE "DetailsViewStyle"

TArray< TSharedRef< const FOverridesWidgetStyleKey >> FOverridesWidgetStyleKeys::OverridesWidgetStyleKeys;

const FSlateBrush& FOverridesWidgetStyleKey::GetConstStyleBrush() const
{
	const FSlateBrush* ImageBrush = FAppStyle::GetBrush( *(TEXT("DetailsView.") + Name.ToString()) );
	checkf(ImageBrush, TEXT("Expecting a valid brush"));
	return *ImageBrush;
}

const FSlateBrush& FOverridesWidgetStyleKey::GetConstStyleBrushHovered() const
{
	const FSlateBrush* ImageBrush = FAppStyle::GetBrush( *(TEXT("DetailsView.") + Name.ToString() + TEXT(".Hovered")) );
	checkf(ImageBrush, TEXT("Expecting a valid brush"));
	return *ImageBrush;
}

FText FOverridesWidgetStyleKey::GetToolTipText(bool bIsCategory) const
{
	return bIsCategory ? CategoryTooltip : PropertyTooltip;
}

void FOverridesWidgetStyleKeys::Initialize()
{
	OverridesWidgetStyleKeys.Add( MakeShared<FOverridesWidgetStyleKey>(Here()));
	OverridesWidgetStyleKeys.Add( MakeShared<FOverridesWidgetStyleKey>(Inside()));
	OverridesWidgetStyleKeys.Add( MakeShared<FOverridesWidgetStyleKey>(HereInside()));
	OverridesWidgetStyleKeys.Add( MakeShared<FOverridesWidgetStyleKey>(Added()));
	OverridesWidgetStyleKeys.Add( MakeShared<FOverridesWidgetStyleKey>(Removed()));
	OverridesWidgetStyleKeys.Add( MakeShared<FOverridesWidgetStyleKey>(None()));
	OverridesWidgetStyleKeys.Add( MakeShared<FOverridesWidgetStyleKey>(Inherited()));
}

const FOverridesWidgetStyleKey& FOverridesWidgetStyleKeys::Here()
{
	static const FOverridesWidgetStyleKey Here{"OverrideHere",
		EOverriddenState::AllOverridden,
		false,
		LOCTEXT("HereCategoryTooltip", "This component has been overridden."),
		LOCTEXT("HerePropertyTooltip", "This property has been overridden.")
	};
	return Here;
}

const FOverridesWidgetStyleKey& FOverridesWidgetStyleKeys::Added()
{
	static const FOverridesWidgetStyleKey Added{"OverrideAdded",
		EOverriddenState::Added,
		TOptional<bool>(),
		LOCTEXT("AddedCategoryTooltip", "This component has been added."),
		LOCTEXT("AddedPropertyTooltip", "This property has been added.")
	};
	return Added;
}

const FOverridesWidgetStyleKey& FOverridesWidgetStyleKeys::None()
{
	static const FOverridesWidgetStyleKey None{"OverrideNone",
		EOverriddenState::NoOverrides,
		TOptional<bool>(),
		LOCTEXT("NoneCategoryTooltip", "No override."),
		LOCTEXT("NonePropertyTooltip", "No override.")
	};
	return None;
}

const FOverridesWidgetStyleKey& FOverridesWidgetStyleKeys::Removed()
{
	static const FOverridesWidgetStyleKey Removed{"OverrideRemoved"};
	return Removed;
}

const FOverridesWidgetStyleKey& FOverridesWidgetStyleKeys::Inside()
{
	static const FOverridesWidgetStyleKey Inside{"OverrideInside",
		EOverriddenState::HasOverrides,
		TOptional<bool>(),
		LOCTEXT("InsideCategoryTooltip", "At least one of this component's properties has been overridden."),
		LOCTEXT("InsidePropertyTooltip", "At least one of this property's values has been overridden.")
	};
	return Inside;
}

const FOverridesWidgetStyleKey& FOverridesWidgetStyleKeys::HereInside()
{
	static const FOverridesWidgetStyleKey HereInside{"OverrideHereInside" };
	return HereInside;
}

const FOverridesWidgetStyleKey& FOverridesWidgetStyleKeys::Inherited()
{
	static const FOverridesWidgetStyleKey Inherited{"OverrideInherited",
		EOverriddenState::AllOverridden,
		true,
		LOCTEXT("InheritedCategoryTooltip", "This component's parent entity has been overridden."),
		LOCTEXT("InheritedPropertyTooltip", "This property's parent component has been overridden.")
	};
	return Inherited;
}

TArray< TSharedRef< const FOverridesWidgetStyleKey >> FOverridesWidgetStyleKeys::GetKeys()
{
	return OverridesWidgetStyleKeys;
}

TAttribute<EVisibility> FOverridesWidgetStyleKey::GetVisibilityAttribute(const TSharedPtr<FEditPropertyChain>& PropertyChain, TWeakObjectPtr<UObject>& OverriddenObjectWeakPtr) const
{
	static FOverridableManager& Manager = FOverridableManager::Get();
	
	return TAttribute<EVisibility>::CreateLambda( [ this, PropertyChain,  OverriddenObjectWeakPtr ] ()
	{
		const bool bIsProperty = PropertyChain.IsValid();
		
	    if ( OverriddenObjectWeakPtr.IsValid() )
	    {
		    UObject* OverriddenObject = OverriddenObjectWeakPtr.Get();

			FPropertyVisitorPath Path;
			if (bIsProperty)
			{
				Path = FPropertyVisitorPath(FPropertyChangedEvent(nullptr), *PropertyChain.Get());
			}

			bool bInherited = false;
			const bool bIsVisible = VisibleOverriddenState.GetValue() == Manager.GetOverriddenState(OverriddenObject, &UE::PropertyVisibility::ConsiderPropertyForOverriddenState, Path, &bInherited) &&
									(!bStateInherited.IsSet() || bStateInherited.GetValue() == bInherited);

			if (bIsVisible)
			{
				return EVisibility::Visible;
			}
	    }

		return EVisibility::Collapsed;
	});
}

FOverridesWidgetStyleKey::FOverridesWidgetStyleKey(FName InName) : Name{InName}
{
}

FOverridesWidgetStyleKey::FOverridesWidgetStyleKey(FName InName,
													const TOptional<EOverriddenState>& InOverriddenState,
													const TOptional<bool>& bInStateInherited /*= TOptional<bool>()*/,
													const FText& InCategoryTooltip,
													const FText& InPropertyTooltip) :
   Name(InName),
   VisibleOverriddenState(InOverriddenState),
   bStateInherited(bInStateInherited),
   bCanBeVisible(true),
   CategoryTooltip(InCategoryTooltip),
   PropertyTooltip(InPropertyTooltip)
{
}

FDetailsViewStyle::FDetailsViewStyle()
{
}

FDetailsViewStyle::FDetailsViewStyle(
	const FDetailsViewStyleKey& InKey, 
	float InTopCategoryPadding) :
		FSlateWidgetStyle(),
		Key(InKey),
		TopCategoryPadding(InTopCategoryPadding)
{
	Initialize(Key);
}

FMargin FDetailsViewStyle::GetTablePadding(bool bIsScrollBarNeeded) const
{
	if (bIsScrollBarNeeded)
	{
		return TablePaddingWithScrollbar;
	}
	return TablePaddingWithNoScrollbar;
}

void FDetailsViewStyle::Initialize(const FDetailsViewStyle* Style)
{
	if ( Style )
	{
		Key = Style->Key;
		TopCategoryPadding = Style->TopCategoryPadding;
		TablePaddingWithScrollbar = Style->TablePaddingWithScrollbar;
		TablePaddingWithNoScrollbar = Style->TablePaddingWithNoScrollbar;
	}
}

FDetailsViewStyle::FDetailsViewStyle(FDetailsViewStyleKey& InKey) :
		FSlateWidgetStyle(),
		Key(InKey)
{
	Initialize(Key);
}

void FDetailsViewStyle::Initialize(FDetailsViewStyleKey& InKey)
{
	if (const FDetailsViewStyle** Style = StyleKeyToStyleTemplateMap.Find(InKey.GetName()))
	{
		Initialize( *Style );
	}
}

FDetailsViewStyle::FDetailsViewStyle(FDetailsViewStyle& InStyle) :
	FDetailsViewStyle(InStyle.Key, InStyle.TopCategoryPadding ) 
{
}

FDetailsViewStyle::FDetailsViewStyle(const FDetailsViewStyle& InStyle) :
	FDetailsViewStyle(InStyle.Key, InStyle.TopCategoryPadding ) 
{
}

FMargin FDetailsViewStyle::GetOuterCategoryRowPadding() const
{
	return FMargin(0, TopCategoryPadding, 0, 1);
}

FMargin FDetailsViewStyle::GetRowPadding(bool bIsOuterCategory) const
{
	return bIsOuterCategory ?
		FMargin(0, TopCategoryPadding, 0, 1) :
		FMargin(0, 0, 0, 1);
}

bool FDetailsViewStyle::operator==(FDetailsViewStyle& OtherLayoutType) const
{
	return Key == OtherLayoutType.Key;
}

FDetailsViewStyle& FDetailsViewStyle::operator=(FDetailsViewStyleKey& OtherLayoutTypeKey)
{
	if (const FDetailsViewStyle* Style = GetStyle(OtherLayoutTypeKey))
	{
		Initialize(OtherLayoutTypeKey, Style->TopCategoryPadding);
	}
	return *this;
}

const FName FDetailsViewStyle::GetTypeName() const
{
	return Key.GetName();
}

const FSlateBrush* FDetailsViewStyle::GetBackgroundImageForCategoryRow(
	const bool bShowBorder, 
	const bool bIsInnerCategory,
	const bool bIsCategoryExpanded) const
{
	static const FSlateBrush* InnerCategoryRowBrush = FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle");
	static const FSlateBrush* ClassicStyleTopLevelCategoryRowBrush = FAppStyle::Get().GetBrush("DetailsView.CategoryTop");
	static const FSlateBrush* CardStyleTopLevelCategoryCollapsedScrollBarNeededRowBrush = FAppStyle::Get().GetBrush("DetailsView.CardHeaderRounded");
	static const FSlateBrush* CardStyleTopLevelCategoryExpandedScrollBarNeededRowBrush = FAppStyle::Get().GetBrush("DetailsView.CardHeaderTopRounded");

	if (bShowBorder)
	{
		if (bIsInnerCategory)
		{
			return InnerCategoryRowBrush;
		}
			
		const bool bIsCardStyle = Key == FDetailsViewStyleKeys::Card();
			
		if (!bIsCardStyle)
		{
			return ClassicStyleTopLevelCategoryRowBrush;
		}
		if (!bIsCategoryExpanded)
		{
			return CardStyleTopLevelCategoryCollapsedScrollBarNeededRowBrush;
		}
		return CardStyleTopLevelCategoryExpandedScrollBarNeededRowBrush;
	}

	return nullptr;
}

void FDetailsViewStyle::InitializeDetailsViewStyles()
{
	static FDetailsViewStyle CardStyle{FDetailsViewStyleKeys::Card(), 8.0f};
	CardStyle.TablePaddingWithScrollbar = FMargin(8, 0, 20, 8);
	CardStyle.TablePaddingWithNoScrollbar = FMargin(8, 0, 8, 8);
	StyleKeyToStyleTemplateMap.Add(FDetailsViewStyleKeys::Card().GetName(), &CardStyle);

	static FDetailsViewStyle DefaultStyle{FDetailsViewStyleKeys::Default(), 0.0f };
	StyleKeyToStyleTemplateMap.Add(FDetailsViewStyleKeys::Default().GetName(), &DefaultStyle);
	
	static FDetailsViewStyle ClassicStyle{FDetailsViewStyleKeys::Classic(), 0.0f };
	StyleKeyToStyleTemplateMap.Add(FDetailsViewStyleKeys::Classic().GetName(), &ClassicStyle);

	FOverridesWidgetStyleKeys::Initialize();
}

const FSlateBrush* FDetailsViewStyle::GetBackgroundImageForScrollBarWell(
	const bool bShowBorder,
	const bool bIsInnerCategory,
	const bool bIsCategoryExpanded,
	const bool bIsScrollBarNeeded) const
{
	static const FSlateBrush* InnerCategoryWellBrush = FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle");
	static const FSlateBrush* ClassicStyleTopLevelCategoryRowBrush = FAppStyle::Get().GetBrush("DetailsView.CategoryTop");
	static const FSlateBrush* CardStyleCollapsedScrollBarNeededWellBrush = FAppStyle::Get().GetBrush("DetailsView.CardHeaderRightSideRounded");
	static const FSlateBrush* CardStyleExpandedScrollBarNeededWellBrush = FAppStyle::Get().GetBrush("DetailsView.CardHeaderTopRightSideRounded");

	if (bShowBorder)
	{
		if (bIsInnerCategory)
		{
			return InnerCategoryWellBrush;
		}
		const bool bIsCardStyle = this->Key == FDetailsViewStyleKeys::Card();
			
		if (!bIsCardStyle)
		{
			return ClassicStyleTopLevelCategoryRowBrush;
		}
		if (!bIsCategoryExpanded){
			return CardStyleCollapsedScrollBarNeededWellBrush;
		}
		return CardStyleExpandedScrollBarNeededWellBrush;
	}

	return nullptr;
}

void FDetailsViewStyle::Initialize(
	FDetailsViewStyleKey& InKey,
	const float InTopCategoryPadding)
{
	Key = InKey;
	TopCategoryPadding = InTopCategoryPadding;
}

const FDetailsViewStyle* FDetailsViewStyle::GetStyle(const FDetailsViewStyleKey& Key)
{
	const FDetailsViewStyle** StylePtr = StyleKeyToStyleTemplateMap.Find(Key.GetName());
	StylePtr = StylePtr ?
					StylePtr :
					StyleKeyToStyleTemplateMap.Find(FDetailsViewStyleKeys::Default().GetName());
		
	return StylePtr ? *StylePtr : nullptr;
}

#undef LOCTEXT_NAMESPACE