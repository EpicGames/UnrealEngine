//  Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Brushes/SlateImageBrush.h"
#include "DetailsViewStyleKey.h"
#include "Layout/Margin.h"
#include "Styling/SlateTypes.h"
#include "Misc/Paths.h"
#include "Styling/SlateWidgetStyle.h"

enum class EOverriddenState : uint8;
struct EVisibility;

/**
 * A class which provides a key with the information to create the Overrides widget style (including the Icom)
 */
class FOverridesWidgetStyleKey : public TSharedFromThis< FOverridesWidgetStyleKey >
{
public:

	/**
	 * The default constructor for the style key.
	 *
	 * @param InName the FName which is the name of the key
	 */
	PROPERTYEDITOR_API FOverridesWidgetStyleKey(FName InName);

	/**
	 * The constructor for the style key that initializes the Overridden state members
	 *
	 * @param InName the FName which is the name of the key
	 * @param InOverriddenState the overridden state for components for which this style is visible
	 * @param bInStateInherited take into account if the state was inherited or not, not set means anything
	 * @param InCategoryTooltip tooltip for this widget if it is a category
	 * @param InPropertyTooltip tooltip for this widget if it is a property
	 */
	PROPERTYEDITOR_API FOverridesWidgetStyleKey(FName InName, 
												const TOptional<EOverriddenState>& InOverriddenState,
												const TOptional<bool>& bInStateInherited = TOptional<bool>(),
												const FText& InCategoryTooltip = FText::GetEmpty(),
												const FText& InPropertyTooltip = FText::GetEmpty());

	/**
	 * returns the const FSlateBrush& that creates the icon for this widget style
	 */
	const FSlateBrush& GetConstStyleBrush() const;

	/**
	* returns the const FSlateBrush& that creates the hovered icon for this widget style
	*/
	const FSlateBrush& GetConstStyleBrushHovered() const;
	
	/**
	* returns the tooltip for this widget style
	*/
	FText GetToolTipText(bool bIsCategory = false) const;

	/**
	 * Creates a TAttribute<EVisibility> which indicates the EVisibility for this widget style
	 * 
	 * @param PropertyChain points to the FEditPropertyChain for the Property, if this style is for a Property, else it is nullptr
	 * @param OverriddenObjectWeakPtr the UObject associated with this widget style
	 * @return 
	 */
	TAttribute<EVisibility> GetVisibilityAttribute(const TSharedPtr<FEditPropertyChain>& PropertyChain, TWeakObjectPtr<UObject>& OverriddenObjectWeakPtr) const;

	/**
	 * the name of the key
	 */
	const FName Name;

	/**
	 * The EOverriddenState for which this style is visible for a UObject
	 */
	const TOptional<EOverriddenState> VisibleOverriddenState;

	/**
	 * Whether the state was inherited from a parent node or not
	 */
	TOptional<bool> bStateInherited;

	const bool bCanBeVisible = false;

	/**
	 * Tooltip for this widget if it is a category
	 */
	FText CategoryTooltip;
	
	/**
	 * tTooltip for this widget if it is a property
	 */
	FText PropertyTooltip;
};

/**s
 * The FOverridesWidgetStyleKeys class provides style keys which can
 * create the needed styles for overrides widgets
 */
class FOverridesWidgetStyleKeys
{
public:

	/**
	 * The style for an override widget when an item is completely overridden and has no
	 * nested properties which are not.
	 */
	PROPERTYEDITOR_API static const FOverridesWidgetStyleKey& Here();

	/**
	 * The style for the override widget when an item has been newly added 
	 */
	static const FOverridesWidgetStyleKey& Added();

	/**
	 * The style for the override widget when the user hovers over it which shows that they have associated
	 * action no overrides to choose from
	 */
	PROPERTYEDITOR_API static const FOverridesWidgetStyleKey& None();

	/**
	 * The style for the override widget when an item has been removed 
	 */
	static const FOverridesWidgetStyleKey& Removed();

	/**
	 * The style for the override widget when it has some nested items inside that have been overridden, but
	 * not all of them
	 */
	static const FOverridesWidgetStyleKey& Inside();

	static const FOverridesWidgetStyleKey& HereInside();

	/**
	 * The override state was inherited from a parent node
	 */
	static const FOverridesWidgetStyleKey& Inherited();

	static TArray< TSharedRef< const FOverridesWidgetStyleKey >> GetKeys();
	
	static void Initialize(); 

private:
	static TArray< TSharedRef< const FOverridesWidgetStyleKey >> OverridesWidgetStyleKeys;
};

/**
 * A Class which holds information regarding the style of a Details View
 */
class FDetailsViewStyle : public FSlateWidgetStyle
{
public:

	/**
	 * The default constructor of this @code FDetailsViewStyle @endcode.
	 */
	FDetailsViewStyle();
	
	/**
	 * Constructs this @code FDetailsViewStyle @endcode with the specified parameters
	 *
	 * @param InKey the Key for  the @code FDetailsViewStyle @endcode
	 */
	FDetailsViewStyle(FDetailsViewStyleKey& InKey);

	/**
	 * Copy constructor for @code FDetailsViewStyle @endcode.
	 *
	 * @param InStyle the @code FDetailsViewStyle @endcode instance from which to initialize
	 * this @code FDetailsViewStyle @endcode.
	 */
	FDetailsViewStyle(FDetailsViewStyle& InStyle);

	/**
	 * Const copy constructor for @code FDetailsViewStyle @endcode.
	 *
	 * @param InStyle the @code FDetailsViewStyle @endcode instance from which to initialize
	 * this @code FDetailsViewStyle @endcode.
	 */
	FDetailsViewStyle(const FDetailsViewStyle& InStyle);

	/**
	 * Returns the padding for the outer Category row
	 */
	FMargin GetOuterCategoryRowPadding() const;
	
	/**
	 * Returns the padding for details panel rows which are not outer Category rows
	 */
	FMargin GetRowPadding(bool bIsOuterCategory) const;

	/**
	 * The equality operator for @code FDetailsViewStyle @endcode
	 */
	bool operator==(FDetailsViewStyle& OtherLayoutType) const;

	/**
	 * The assignment operator for @code FDetailsViewStyle @endcode
	 */
	FDetailsViewStyle& operator=(FDetailsViewStyleKey& OtherLayoutTypeKey);

	/**
   	 * Gets the @code FName @endcode Name of this style.
   	 */
	virtual const FName GetTypeName() const override;

	/**
	 * Returns the background image for the Category row, minus the scrollbar well
	 *
 	 * @param bShowBorder a bool to indicate whether the border should be shown, at all
	 * @param bIsCategoryExpanded a bool that indicates whether this Category is expanded 
	 */	
	const FSlateBrush* GetBackgroundImageForCategoryRow(
		const bool bShowBorder,
		const bool bIsInnerCategory,
		const bool bIsCategoryExpanded) const;

	/**
	 * Returns the background image for the scroll bar well
	 *
	 * @param bShowBorder a bool to indicate whether the border should be shown, at all
	 * @param bIsInnerCategory a bool that indicates whether this background is for an inner Category
	 * @param bIsCategoryExpanded a bool that indicates whether this Category is expanded 
	 * @param bIsScrollBarVisible a bool that indicates whether the scrollbar for the details view is visible
	 */	
	const FSlateBrush* GetBackgroundImageForScrollBarWell(
			const bool bShowBorder,
			const bool bIsInnerCategory,
			const bool bIsCategoryExpanded, 
			const bool bIsScrollBarVisible) const;

	/**
	 * Initializes all Details View Styles
	 */
	static void InitializeDetailsViewStyles();


	/**
	 * Constructs this FDetailsViewStyle with the specified parameters
	 *
	 * @param InKey the name of the FDetailsViewStyle
	 * @param InTopCategoryPadding the top padding for the Category row
	 */
	FDetailsViewStyle(
		const FDetailsViewStyleKey& InKey,
		float InTopCategoryPadding = 0.f);

	/**
	 * Returns the FMargin which provides the padding around the whole details view table
	 *
	 * @param bIsScrollBarVisible whether the scrollbar is visible
	 */
	FMargin GetTablePadding(bool bIsScrollBarVisible) const;

	/**
	 * Returns the FMargin which provides the padding around the Category buttons
	 */
	FMargin GetCategoryButtonsMargin() const
	{
		return CategoryButtonsMargin;
	}
	
	/**
	* Returns the @code FDetailsViewStyle @endcode for which this is the @code FDetailsViewStyleKey @endcode .
	*
	* @param InKey the name of the FDetailsViewStyle
	*/
	static const FDetailsViewStyle* GetStyle(const FDetailsViewStyleKey& InKey);
	
private:

	void Initialize( const FDetailsViewStyle* Style );

	/**
   	 * Initializes this FDetailsViewStyle with the specified parameters
   	 *
   	 * @param InKey the name of the FDetailsViewStyle
   	 * @param InTopCategoryPadding the top padding for the Category row
   	 */
	void Initialize( FDetailsViewStyleKey& InKey, 
		const float InTopCategoryPadding);

	/**
	* Initializes this FDetailsViewStyle with the style specified by @code FDetailsViewStyleKey @endcode InKey
	*
	* @param InKey the name of the FDetailsViewStyle
	*/
	void Initialize(FDetailsViewStyleKey& InKey);

	/** the Name of the Style */
	FDetailsViewStyleKey Key;

	/**
	 * A map of @code FDetailsViewStyleKey @endcode instances to the
	 * @code const FDetailsViewStyle* @endcode instances for which they are the keys. 
	 */
	static inline TMap<FName, const FDetailsViewStyle*> StyleKeyToStyleTemplateMap;
	
	/** the Slate Units of the top padding for an outer Category row */
	float TopCategoryPadding = 0.f;
	
	/**
   * The FMargin which provides the padding around the whole details view table with scrollbar
   */
	FMargin TablePaddingWithScrollbar = FMargin(0, 0, 16, 1);

	/**
	* The FMargin which provides the padding around the whole details view table with no scrollbar
	*/
	FMargin TablePaddingWithNoScrollbar = FMargin(0, 0, 0, 1);
	
	/**
	* The FMargin which provides the padding around the Category buttons
	*/
	FMargin CategoryButtonsMargin = FMargin(0, 0,  0, 0);
	
};
	

