//  Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsViewStyleKey.h"

FDetailsViewStyleKey::FDetailsViewStyleKey(): Name("")
{
}

FDetailsViewStyleKey::FDetailsViewStyleKey(FDetailsViewStyleKey& Key): FDetailsViewStyleKey(Key.GetName())
{
}

FDetailsViewStyleKey::FDetailsViewStyleKey(const FDetailsViewStyleKey& Key): FDetailsViewStyleKey(Key.GetName())
{
}

FDetailsViewStyleKey& FDetailsViewStyleKey::operator=(FDetailsViewStyleKey& OtherStyleKey)
{
	this->Name = OtherStyleKey.Name;
	return *this;
}

FDetailsViewStyleKey& FDetailsViewStyleKey::operator=(const FDetailsViewStyleKey& OtherStyleKey)
{
	this->Name = OtherStyleKey.Name;
	return *this;
}

bool FDetailsViewStyleKey::operator==(const FDetailsViewStyleKey& OtherStyleKey) const
{
	return this->Name == OtherStyleKey.Name;
}

FName FDetailsViewStyleKey::GetName() const
{
	return Name;
}

const FDetailsViewStyleKey& FDetailsViewStyleKeys::Classic()
{
	static const FDetailsViewStyleKey Classic{"Classic"};
	return Classic;
}

const FDetailsViewStyleKey& FDetailsViewStyleKeys::Card()
{
	static const FDetailsViewStyleKey Card{"Card"};
	return Card;
}

const FDetailsViewStyleKey& FDetailsViewStyleKeys::Default()
{
	static const FDetailsViewStyleKey Default{"Default"};
	return Default;
}

FDetailsViewStyleKey::FDetailsViewStyleKey(FName InName): Name(InName)
{
}

bool FDetailsViewStyleKeys::IsDefault(const FDetailsViewStyleKey& StyleKey)
{
	return StyleKey.GetName() == Default().GetName();
}
