// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditConditionContext.h"
#include "EditConditionParser.h"

#include "ObjectPropertyNode.h"
#include "PropertyNode.h"
#include "PropertyEditorHelpers.h"
#include "PropertyPathHelpers.h"

DEFINE_LOG_CATEGORY(LogEditCondition);

FEditConditionContext::FEditConditionContext(FPropertyNode& InPropertyNode)
{
	PropertyNode = InPropertyNode.AsShared();
}

FName FEditConditionContext::GetContextName() const
{
	TSharedPtr<FPropertyNode> PinnedNode = PropertyNode.Pin();
	if (!PinnedNode.IsValid())
	{
		return FName();
	}

	return PinnedNode->GetProperty()->GetOwnerStruct()->GetFName();
}

const FBoolProperty* FEditConditionContext::GetSingleBoolProperty(const TSharedPtr<FEditConditionExpression>& Expression) const
{
	TSharedPtr<FPropertyNode> PinnedNode = PropertyNode.Pin();
	if (!PinnedNode.IsValid())
	{
		return nullptr;
	}

	const FProperty* Property = PinnedNode->GetProperty();
	if (Property == nullptr)
	{
		return nullptr;
	}

	const FBoolProperty* BoolProperty = nullptr;

	for (const FCompiledToken& Token : Expression->Tokens)
	{
		if (const EditConditionParserTokens::FPropertyToken* PropertyToken = Token.Node.Cast<EditConditionParserTokens::FPropertyToken>())
		{
			if (BoolProperty != nullptr)
			{
				// second property token in the same expression, this can't be a simple expression like "bValue == false"
				return nullptr;
			}

			BoolProperty = FindFProperty<FBoolProperty>(Property->GetOwnerStruct(), *PropertyToken->PropertyName);
			if (BoolProperty == nullptr)
			{
				return nullptr;
			}
		}
	}

	return BoolProperty;
}

const TWeakObjectPtr<UFunction> FEditConditionContext::GetFunction(const FString& FieldName) const
{
	TSharedPtr<FPropertyNode> PinnedNode = PropertyNode.Pin();
	if (!PinnedNode.IsValid())
	{
		return nullptr;
	}

	const FProperty* Property = PinnedNode->GetProperty();
	if (Property == nullptr)
	{
		return nullptr;
	}

	TWeakObjectPtr<UFunction> Function = FindUField<UFunction>(Property->GetOwnerStruct(), *FieldName);

	if (Function == nullptr && FieldName.Contains(TEXT(".")))
	{
		// Function not found in struct, try to see if this is a static function
		Function = FindObject<UFunction>(nullptr, *FieldName, EFindObjectFlags::ExactClass);

		if (Function.IsValid() && !Function->HasAnyFunctionFlags(EFunctionFlags::FUNC_Static))
		{
			Function = nullptr;
		}
	}

	return Function;
}

static TSet<TPair<FName, FString>> AlreadyLogged;

/**
 * Attempt to property within the node & log if not found.
 * 
 * @param PropertyNode		Property owning the field to search for, ex: a UObject
 * @param FieldNam			Name of field to search for
 * @return					Field for given fieldname if found, nullptr otherwise 
 */
template<typename T>
const T* FindTypedField(const TSharedPtr<FPropertyNode>& PropertyNode, const FString& FieldName)
{
	if (!PropertyNode.IsValid())
	{
		return nullptr;
	}

	const FProperty* Property = PropertyNode->GetProperty();
	if (Property == nullptr)
	{
		return nullptr;
	}

	auto ShouldLog = [&]() -> bool
		{
			TPair<FName, FString> FieldKey(Property->GetOwnerStruct()->GetFName(), FieldName);
			if (!AlreadyLogged.Find(FieldKey))
			{
				AlreadyLogged.Add(FieldKey);
				return true;
			}
			return false;
		};

	const FProperty* Field = FindFProperty<FProperty>(Property->GetOwnerStruct(), *FieldName);
	if (Field == nullptr)
	{
		if (ShouldLog())
		{
			UE_LOGF(LogEditCondition, Error, "EditCondition parsing failed: Field name \"%ls\" was not found in class \"%ls\".", *FieldName, *Property->GetOwnerStruct()->GetName());
		}
		return nullptr;
	}
	else if (Field->HasAnyPropertyFlags(CPF_Deprecated))
	{
		if (ShouldLog())
		{
			UE_LOGF(LogEditCondition, Warning, "EditCondition parsing failed: Field name \"%ls\" was found in class \"%ls\" but marked deprecated.", *FieldName, *Property->GetOwnerStruct()->GetName());
		}
	}

	return CastField<T>(Field);
}

/** 
 * Get the parent to use as the context when evaluating the edit condition.
 * For normal properties inside a UObject, this is the UObject. 
 * For children of containers, this is the UObject the container is in. 
 * Note: We do not support nested containers.
 * The result can be nullptr in exceptional cases, eg. if the UI is getting rebuilt.
 */
static const FPropertyNode* GetEditConditionParentNode(const TSharedPtr<FPropertyNode>& PropertyNode)
{
	const FPropertyNode* ParentNode = PropertyNode->GetParentNode();
	if (ParentNode == nullptr)
	{
		return nullptr;
	}

	FFieldVariant PropertyOuter = PropertyNode->GetProperty()->GetOwnerVariant();

	if (PropertyOuter.Get<FArrayProperty>() != nullptr ||
		PropertyOuter.Get<FSetProperty>() != nullptr ||
		PropertyOuter.Get<FMapProperty>() != nullptr)
	{
		// in a dynamic container, parent is actually one level up
		return ParentNode->GetParentNode();
	}

	if (PropertyNode->GetProperty()->ArrayDim > 1 && PropertyNode->GetArrayIndex() != INDEX_NONE)
	{
		// in a fixed size container, parent node is just the header field
		return ParentNode->GetParentNode();
	}

	return ParentNode;
}

static uint8* ComputeEditConditionValuePointer(FPropertyNode& ConditionedProperty, FReadAddressList& ConditionPropertyAddresses, const FProperty& EditConditionProperty, int32 AddressIndex)
{
	// The strategy for getting the pointer to the vaue of the edit condition is to use the value pointer of the property
	// that is being conditioned and then walk back that property's offset to find the owning struct's base address. 
	// This owning struct's base address is then offset forward by the edit condition's property to find the edit condition's value pointer.
	// 
	// The assumption is that the edit condition property is on the same struct/class as the property that is being "conditioned".
	// 
	// Since the edit condition is inline, there will not be a PropertyNode available, therefore it is necessary to use
	// the FProperty API to find the value pointer for the edit condition.
	//
	// It is also not possible to use the conditioned property's parent node which is typically used in the non-sparse class data case
	// since the parent may not point to the struct that either property is in.  
	// In the case of property directly on the SparseClassData, the parent node is often a category or object node.
	// In the case of a property within a sub-struct in an array property of the SparseClassData, the offset between the SparseClassData instance
	// and the edit condition property depends on the property chain.
	// 
	// Therefore, it is easiest to find the direct owning struct address that is shared by both conditioned and edit condition property and compute
	// the edit condition value pointer directly.
		
	uint8* ConditionedPropertyValueAddress = ConditionPropertyAddresses.GetAddress(AddressIndex);
	const int32 ConditionedPropertyOffset = ConditionedProperty.GetProperty()->GetOffset_ForInternal();
	uint8* OwningStructStartAddress = ConditionedPropertyValueAddress - ConditionedPropertyOffset;

	const int32 ArrayIndex = 0; // EditConditions does not allow indexing into arrays
	uint8* EditConditionValuePtr = EditConditionProperty.ContainerPtrToValuePtr<uint8>(OwningStructStartAddress, ArrayIndex);
	return EditConditionValuePtr;
}

TOptional<bool> FEditConditionContext::GetBoolValue(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction) const
{
	TSharedPtr<FPropertyNode> PinnedNode = PropertyNode.Pin();
	if (!PinnedNode.IsValid())
	{
		return TOptional<bool>();
	}

	if (const UFunction* Function = CachedFunction.Get())
	{
		if (CastField<FBoolProperty>(Function->GetReturnProperty()) == nullptr)
		{
			// Function return type not bool, return undefined
			return TOptional<bool>();
		}

		// Check for external static function references 
		if (Function->HasAnyFunctionFlags(EFunctionFlags::FUNC_Static))
		{
			FString StaticFunctionName = {};
			UObject* EditConditionExpressionCDO = Function->GetOuterUClass()->GetDefaultObject();
			Function->GetName(StaticFunctionName);

			{
				FEditorScriptExecutionGuard ScriptExecutionGuard;
				FCachedPropertyPath Path(StaticFunctionName);
				bool bResult = true;
				if (PropertyPathHelpers::GetPropertyValue(EditConditionExpressionCDO, Path, bResult))
				{
					return bResult;
				}
				else
				{
					// Execution failed, return undefined
					return TOptional<bool>();
				}
			}
		}

		// We might be selecting multiple objects, account for that
		TArray<UObject*> OutObjects;
		FObjectPropertyNode* ObjectNode = PinnedNode->FindObjectItemParent();
		if (ObjectNode)
		{
			int32 NumObjects = ObjectNode->GetNumObjects();
			for (int32 ObjectIndex = 0; ObjectIndex < NumObjects; ++ObjectIndex)
			{
				FEditorScriptExecutionGuard ScriptExecutionGuard;

				UObject* FunctionTarget = ObjectNode->GetUObject(ObjectIndex);
				FCachedPropertyPath FunctionPath(PropertyName);
				bool bResult = true;
				if (FunctionTarget && PropertyPathHelpers::GetPropertyValue(FunctionTarget, FunctionPath, bResult))
				{
					if (!bResult)
					{
						return false;
					}
				}
				else
				{
					// Execution failed, return undefined
					return TOptional<bool>();
				}
			}

			// Executed our target function over relevant objects with no condtion failures, return true
			if (NumObjects > 0)
			{
				return true;
			}
		}
	}

	const FBoolProperty* OperandProperty = FindTypedField<FBoolProperty>(PinnedNode, PropertyName);
	if (OperandProperty == nullptr)
	{
		return TOptional<bool>();
	}

	FComplexPropertyNode* ComplexParentNode = PinnedNode->FindComplexParent();
	if (ComplexParentNode == nullptr)
	{
		return TOptional<bool>();
	}

	TOptional<bool> Result;
	if (!PinnedNode->HasNodeFlags(EPropertyNodeFlags::IsSparseClassData))
	{
		const FPropertyNode* ParentNode = GetEditConditionParentNode(PinnedNode);
		if (ParentNode == nullptr)
		{
			return TOptional<bool>();
		}
		
		for (int32 Index = 0; Index < ComplexParentNode->GetInstancesNum(); ++Index)
		{
			const uint8* ValuePtr = ComplexParentNode->GetValuePtrOfInstance(Index, OperandProperty, ParentNode);
			if (ValuePtr == nullptr)
			{
				return TOptional<bool>();
			}

			bool bValue = OperandProperty->GetPropertyValue(ValuePtr);
			if (!Result.IsSet())
			{
				Result = bValue;
			}
			else if (Result.GetValue() != bValue)
			{
				// all values aren't the same...
				return TOptional<bool>();
			}
		}
	}
	else
	{
		FReadAddressList ThisPropertyReadAddresses;
		PinnedNode->GetReadAddress(ThisPropertyReadAddresses);

		for (int32 Index = 0; Index < ThisPropertyReadAddresses.Num(); ++Index)
		{
			uint8* EditConditionValuePtr = ComputeEditConditionValuePointer(*PinnedNode, ThisPropertyReadAddresses, *OperandProperty, Index);
			bool Value = OperandProperty->GetPropertyValue(EditConditionValuePtr);

			if (!Result.IsSet())
			{
				Result = Value;
			}
			else if (Result.GetValue() != Value)
			{
				return TOptional<bool>();
			}
		}
	}
	return Result;
}

TOptional<int64> FEditConditionContext::GetIntegerValue(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction) const
{
	TSharedPtr<FPropertyNode> PinnedNode = PropertyNode.Pin();
	if (!PinnedNode.IsValid())
	{
		return TOptional<int64>();
	}

	if (const UFunction* Function = CachedFunction.Get())
	{ 
		// EditConditions Currently only support bool, see: UE-175891
		return TOptional<int64>();
	}

	const FProperty* Property = FindTypedField<FProperty>(PinnedNode, PropertyName);
	const FNumericProperty* OperandProperty = CastField<FNumericProperty>(Property);

	if (OperandProperty == nullptr)
	{
		// Retry with an enum and its underlying property
		if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			OperandProperty = EnumProperty->GetUnderlyingProperty();
		}
	}

	if (OperandProperty == nullptr || !OperandProperty->IsInteger())
	{
		return TOptional<int64>();
	}

	FComplexPropertyNode* ComplexParentNode = PinnedNode->FindComplexParent();
	if (ComplexParentNode == nullptr)
	{
		return TOptional<int64>();
	}
	
	TOptional<int64> Result;
	if (!PinnedNode->HasNodeFlags(EPropertyNodeFlags::IsSparseClassData))
	{
		const FPropertyNode* ParentNode = GetEditConditionParentNode(PinnedNode);
		if (ParentNode == nullptr)
		{
			return TOptional<int64>();
		}

		for (int32 Index = 0; Index < ComplexParentNode->GetInstancesNum(); ++Index)
		{
			const uint8* ValuePtr = ComplexParentNode->GetValuePtrOfInstance(Index, Property, ParentNode);
			if (ValuePtr == nullptr)
			{
				return TOptional<int64>();
			}

			int64 Value = OperandProperty->GetSignedIntPropertyValue(ValuePtr);
			if (!Result.IsSet())
			{
				Result = Value;
			}
			else if (Result.GetValue() != Value)
			{
				// all values aren't the same...
				return TOptional<int64>();
			}
		}
	}
	else
	{
		FReadAddressList ThisPropertyReadAddresses;
		PinnedNode->GetReadAddress(ThisPropertyReadAddresses);

		for (int32 Index = 0; Index < ThisPropertyReadAddresses.Num(); ++Index)
		{
			uint8* EditConditionValuePtr = ComputeEditConditionValuePointer(*PinnedNode, ThisPropertyReadAddresses, *OperandProperty, Index);
			int64 Value = OperandProperty->GetSignedIntPropertyValue(EditConditionValuePtr);
			
			if (!Result.IsSet())
			{
				Result = Value;
			}
			else if (Result.GetValue() != Value)
			{
				return TOptional<int64>();
			}
		}
	}
	return Result;
}

TOptional<double> FEditConditionContext::GetNumericValue(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction) const
{
	TSharedPtr<FPropertyNode> PinnedNode = PropertyNode.Pin();
	if (!PinnedNode.IsValid())
	{
		return TOptional<double>();
	}

	if (const UFunction* Function = CachedFunction.Get())
	{
		// EditConditions Currently only support bool, see: UE-175891
		return TOptional<double>();
	}

	const FNumericProperty* OperandProperty = FindTypedField<FNumericProperty>(PinnedNode, PropertyName);
	if (OperandProperty == nullptr)
	{
		return TOptional<double>();
	}

	FComplexPropertyNode* ComplexParentNode = PinnedNode->FindComplexParent();
	if (ComplexParentNode == nullptr)
	{
		return TOptional<double>();
	}

	TOptional<double> Result;
	if (!PinnedNode->HasNodeFlags(EPropertyNodeFlags::IsSparseClassData))
	{
		const FPropertyNode* ParentNode = GetEditConditionParentNode(PinnedNode);
		if (ParentNode == nullptr)
		{
			return TOptional<double>();
		}
	
		for (int32 Index = 0; Index < ComplexParentNode->GetInstancesNum(); ++Index)
		{
			const uint8* ValuePtr = ComplexParentNode->GetValuePtrOfInstance(Index, OperandProperty, ParentNode);
			if (ValuePtr == nullptr)
			{
				return TOptional<double>();
			}

			double Value = 0;

			if (OperandProperty->IsInteger())
			{
				Value = (double) OperandProperty->GetSignedIntPropertyValue(ValuePtr);
			}
			else if (OperandProperty->IsFloatingPoint())
			{
				Value = OperandProperty->GetFloatingPointPropertyValue(ValuePtr);
			}

			if (!Result.IsSet())
			{
				Result = Value;
			}
			else if (!FMath::IsNearlyEqual(Result.GetValue(), Value))
			{
				// all values aren't the same...
				return TOptional<double>();
			}
		}
	}
	else
	{
		FReadAddressList ThisPropertyReadAddresses;
		PinnedNode->GetReadAddress(ThisPropertyReadAddresses);

		for (int32 Index = 0; Index < ThisPropertyReadAddresses.Num(); ++Index)
		{
			uint8* EditConditionValuePtr = ComputeEditConditionValuePointer(*PinnedNode, ThisPropertyReadAddresses, *OperandProperty, Index);
			
			double Value = 0;
			if (OperandProperty->IsInteger())
			{
				Value = (double) OperandProperty->GetSignedIntPropertyValue(EditConditionValuePtr);
			}
			else if (OperandProperty->IsFloatingPoint())
			{
				Value = OperandProperty->GetFloatingPointPropertyValue(EditConditionValuePtr);
			}
			if (!Result.IsSet())
			{
				Result = Value;
			}
			else if (Result.GetValue() != Value)
			{
				return TOptional<double>();
			}
		}
	}

	return Result;
}

TOptional<FString> FEditConditionContext::GetEnumValue(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction) const
{
	TSharedPtr<FPropertyNode> PinnedNode = PropertyNode.Pin();
	if (!PinnedNode.IsValid())
	{
		return TOptional<FString>();
	}

	if (const UFunction* Function = CachedFunction.Get())
	{
		// EditConditions Currently only support bool, see: UE-175891
		return TOptional<FString>();
	}

	const FProperty* Property = FindTypedField<FProperty>(PinnedNode, PropertyName);
	if (Property == nullptr)
	{
		return TOptional<FString>();
	}

	const UEnum* EnumType = nullptr;
	const FNumericProperty* OperandProperty = nullptr;
	if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		OperandProperty = EnumProperty->GetUnderlyingProperty();
		EnumType = EnumProperty->GetEnum();
	}
	else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
	{
		OperandProperty = ByteProperty;
		EnumType = ByteProperty->GetIntPropertyEnum();
	}

	if (EnumType == nullptr || OperandProperty == nullptr || !OperandProperty->IsInteger())
	{
		return TOptional<FString>();
	}

	FComplexPropertyNode* ComplexParentNode = PinnedNode->FindComplexParent();
	if (ComplexParentNode == nullptr)
	{
		return TOptional<FString>();
	}

	TOptional<int64> Result;
	if (!PinnedNode->HasNodeFlags(EPropertyNodeFlags::IsSparseClassData))
	{
		const FPropertyNode* ParentNode = GetEditConditionParentNode(PinnedNode);
		if (ParentNode == nullptr)
		{
			return TOptional<FString>();
		}
	
		for (int32 Index = 0; Index < ComplexParentNode->GetInstancesNum(); ++Index)
		{
			// NOTE: this very intentionally fetches the value from Property, not NumericProperty, 
			// because the underlying property of an enum does not return a valid value
			const uint8* ValuePtr = ComplexParentNode->GetValuePtrOfInstance(Index, Property, ParentNode);
		
			if (ValuePtr == nullptr)
			{
				return TOptional<FString>();
			}

			const int64 Value = OperandProperty->GetSignedIntPropertyValue(ValuePtr);
			if (!Result.IsSet())
			{
				Result = Value;
			}
			else if (Result.GetValue() != Value)
			{
				// all values aren't the same...
				return TOptional<FString>();
			}
		}
	}
	else
	{
		FReadAddressList ThisPropertyReadAddresses;
		PinnedNode->GetReadAddress(ThisPropertyReadAddresses);

		for (int32 Index = 0; Index < ThisPropertyReadAddresses.Num(); ++Index)
		{
			uint8* EditConditionValuePtr = ComputeEditConditionValuePointer(*PinnedNode, ThisPropertyReadAddresses, *OperandProperty, Index);
			const int64 Value = OperandProperty->GetSignedIntPropertyValue(EditConditionValuePtr);
			if (!Result.IsSet())
			{
				Result = Value;
			}
			else if (Result.GetValue() != Value)
			{
				return TOptional<FString>();
			}
		}
	}

	if (!Result.IsSet())
	{
		return TOptional<FString>();
	}

	return EnumType->GetNameStringByValue(Result.GetValue());
}

TOptional<UObject*> FEditConditionContext::GetPointerValue(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction) const
{
	TSharedPtr<FPropertyNode> PinnedNode = PropertyNode.Pin();
	if (!PinnedNode.IsValid())
	{
		return TOptional<UObject*>();
	}

	if (const UFunction* Function = CachedFunction.Get())
	{
		// EditConditions Currently only support bool, see: UE-175891
		return TOptional<UObject*>();
	}
	
	FComplexPropertyNode* ComplexParentNode = PinnedNode->FindComplexParent();
	if (ComplexParentNode == nullptr)
	{
		return TOptional<UObject*>();
	}

	// Get the property of the EditCondition operand
	// EditCondition pointers can only be UObjects
	const FObjectPropertyBase* OperandProperty = FindTypedField<FObjectPropertyBase>(PinnedNode, PropertyName);
	if (OperandProperty == nullptr)
	{
		return TOptional<UObject*>();
	}

	if (!PinnedNode->HasNodeFlags(EPropertyNodeFlags::IsSparseClassData))
	{
		const FPropertyNode* ParentNode = GetEditConditionParentNode(PinnedNode);
		if (ParentNode == nullptr)
		{
			return TOptional<UObject*>();
		}

		TOptional<UObject*> Result;
		for (int32 Index = 0; Index < ComplexParentNode->GetInstancesNum(); ++Index)
		{
			const uint8* ValuePtr = ComplexParentNode->GetValuePtrOfInstance(Index, OperandProperty, ParentNode);
			if (ValuePtr == nullptr)
			{
				return TOptional<UObject*>();
			}

			UObject* Value = OperandProperty->GetObjectPropertyValue(ValuePtr);
			if (!Result.IsSet())
			{
				Result = Value;
			}
			else if (Result.GetValue() != Value)
			{
				// all values aren't the same
				return TOptional<UObject*>();
			}
		}
		return Result;
	}
	else
	{
		TOptional<UObject*> Result;
		FReadAddressList ThisPropertyReadAddresses;
		PinnedNode->GetReadAddress(ThisPropertyReadAddresses);

		for (int32 Index = 0; Index < ThisPropertyReadAddresses.Num(); ++Index)
		{
			uint8* EditConditionValuePtr = ComputeEditConditionValuePointer(*PinnedNode, ThisPropertyReadAddresses, *OperandProperty, Index);
			UObject* Value = OperandProperty->GetObjectPropertyValue(EditConditionValuePtr);
			if (!Result.IsSet())
			{
				Result = Value;
			}
			else if (Result.GetValue() != Value)
			{
				return TOptional<UObject*>();
			}
		}
		return Result;
	}
}

TOptional<FString> FEditConditionContext::GetTypeName(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction) const
{
	TSharedPtr<FPropertyNode> PinnedNode = PropertyNode.Pin();
	if (!PinnedNode.IsValid())
	{
		return TOptional<FString>();
	}

	auto GetPropertyName = [](const FProperty* Property) -> TOptional<FString>
	{
		if (Property == nullptr)
		{
			return TOptional<FString>();
		}

		if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			return EnumProperty->GetEnum()->GetName();
		}
		else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
		{
			const UEnum* EnumType = ByteProperty->GetIntPropertyEnum();
			if (EnumType != nullptr)
			{
				return EnumType->GetName();
			}
		}

		return Property->GetCPPType();
	};

	if (const UFunction* Function = CachedFunction.Get())
	{
		return GetPropertyName(Function->GetReturnProperty());
	}

	const FProperty* Property = FindTypedField<FProperty>(PinnedNode, PropertyName);

	return GetPropertyName(Property);
}

TOptional<int64> FEditConditionContext::GetIntegerValueOfEnum(const FString& EnumTypeName, const FString& MemberName) const
{
	const UEnum* EnumType = UClass::TryFindTypeSlow<UEnum>(EnumTypeName, EFindFirstObjectOptions::ExactClass);
	if (EnumType == nullptr)
	{
		return TOptional<int64>();
	}

	const int64 EnumValue = EnumType->GetValueByName(FName(*MemberName));
	if (EnumValue == INDEX_NONE)
	{
		return TOptional<int64>();
	}

	return EnumValue;
}
