// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

DECLARE_LOG_CATEGORY_EXTERN(LogEditCondition, Log, All);

class UFunction;

class IEditConditionContext
{
public:
	virtual FName GetContextName() const = 0;

	virtual TOptional<bool> GetBoolValue(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction = nullptr) const = 0;
	virtual TOptional<int64> GetIntegerValue(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction = nullptr) const = 0;
	virtual TOptional<double> GetNumericValue(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction = nullptr) const = 0;
	virtual TOptional<FString> GetEnumValue(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction = nullptr) const = 0;
	virtual TOptional<UObject*> GetPointerValue(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction = nullptr) const = 0;
	virtual TOptional<FString> GetTypeName(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction = nullptr) const = 0;
	virtual TOptional<int64> GetIntegerValueOfEnum(const FString& EnumType, const FString& EnumValue) const = 0;

	/**
	 * Get field as a function, avoiding multiple function searches
	 *
	 * @TODO: Get property in general, not just for functions. See: UE-175891.
	 *
	 * @param FieldName Name of the field to try to interpret as a function
	 * @return Function if found or already cached, else
	 */
	virtual const TWeakObjectPtr<UFunction> GetFunction(const FString& FieldName) const = 0;
};

class FProperty;
class FPropertyNode;
class FComplexPropertyNode;
class FEditConditionExpression;

class FEditConditionContext : public IEditConditionContext
{
public:
	FEditConditionContext(FPropertyNode& InPropertyNode);
	virtual ~FEditConditionContext() = default;

	virtual FName GetContextName() const override;

	virtual TOptional<bool> GetBoolValue(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction = nullptr) const override;
	virtual TOptional<int64> GetIntegerValue(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction = nullptr) const override;
	virtual TOptional<double> GetNumericValue(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction = nullptr) const override;
	virtual TOptional<FString> GetEnumValue(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction = nullptr) const override;
	virtual TOptional<UObject*> GetPointerValue(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction = nullptr) const override;
	virtual TOptional<FString> GetTypeName(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction = nullptr) const override;
	virtual TOptional<int64> GetIntegerValueOfEnum(const FString& EnumType, const FString& EnumValue) const override;

	virtual const TWeakObjectPtr<UFunction> GetFunction(const FString& FieldName) const override;

	/**
	 * Fetch the single boolean property referenced.
	 * Returns nullptr if more than one property is referenced.
	 */
	const FBoolProperty* GetSingleBoolProperty(const TSharedPtr<FEditConditionExpression>& Expression) const;

private:
	TWeakPtr<FPropertyNode> PropertyNode;
};
