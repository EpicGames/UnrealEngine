// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditConditionParser.h"

#include "Containers/Set.h"
#include "EditConditionContext.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/BasicMathExpressionEvaluator.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ExpressionParser.h"
#include "Misc/Optional.h"
#include "Trace/Detail/Channel.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectThreadContext.h"

class UObject;

#define LOCTEXT_NAMESPACE "EditConditionParser"

namespace EditConditionParserTokens
{
	const TCHAR* const FEqual::Moniker = TEXT("==");
	const TCHAR* const FNotEqual::Moniker = TEXT("!=");
	const TCHAR* const FGreater::Moniker = TEXT(">");
	const TCHAR* const FGreaterEqual::Moniker = TEXT(">=");
	const TCHAR* const FLess::Moniker = TEXT("<");
	const TCHAR* const FLessEqual::Moniker = TEXT("<=");
	const TCHAR* const FNot::Moniker = TEXT("!");
	const TCHAR* const FAnd::Moniker = TEXT("&&");
	const TCHAR* const FOr::Moniker = TEXT("||");
	const TCHAR* const FAdd::Moniker = TEXT("+");
	const TCHAR* const FSubtract::Moniker = TEXT("-");
	const TCHAR* const FMultiply::Moniker = TEXT("*");
	const TCHAR* const FDivide::Moniker = TEXT("/");
	const TCHAR* const FBitwiseAnd::Moniker = TEXT("&");
	const TCHAR* const FSubExpressionStart::Moniker = TEXT("(");
	const TCHAR* const FSubExpressionEnd::Moniker = TEXT(")");
}

static const TCHAR PropertyBreakingChars[] = { '|', '=', '&', '>', '<', '!', '+', '-', '*', '/', ' ', '\t', '(', ')' };

static TOptional<FExpressionError> ConsumeBool(FExpressionTokenConsumer& Consumer)
{
	TOptional<FStringToken> TrueToken = Consumer.GetStream().ParseTokenIgnoreCase(TEXT("true"));
	if (TrueToken.IsSet())
	{
		Consumer.Add(TrueToken.GetValue(), true);
	}

	TOptional<FStringToken> FalseToken = Consumer.GetStream().ParseTokenIgnoreCase(TEXT("false"));
	if (FalseToken.IsSet())
	{
		Consumer.Add(FalseToken.GetValue(), false);
	}

	return TOptional<FExpressionError>();
}

static TOptional<FExpressionError> ConsumeNullPtr(FExpressionTokenConsumer& Consumer)
{
	TOptional<FStringToken> NullToken = Consumer.GetStream().ParseToken(TEXT("nullptr"));
	if (NullToken.IsSet())
	{
		Consumer.Add(NullToken.GetValue(), EditConditionParserTokens::FNullPtrToken());
	}

	return TOptional<FExpressionError>();
}

static TOptional<FExpressionError> ConsumeIndexNone(FExpressionTokenConsumer& Consumer)
{
	TOptional<FStringToken> IndexNoneToken = Consumer.GetStream().ParseToken(TEXT("INDEX_NONE"));
	if (IndexNoneToken.IsSet())
	{
		Consumer.Add(IndexNoneToken.GetValue(), EditConditionParserTokens::FIndexNoneToken());
	}

	return TOptional<FExpressionError>();
}

static TOptional<FExpressionError> ConsumePropertyName(FExpressionTokenConsumer& Consumer)
{
	enum class EParsedStringType : uint8
	{
		Unknown,
		Unquoted,
		Quoted,
	};

	FString PropertyName;
	bool bShouldBeEnum = false;
	EParsedStringType ParsedStringType = EParsedStringType::Unknown;

	TCHAR OpeningQuoteChar = TEXT('\0');
	int32 NumConsecutiveSlashes = 0;

	TOptional<FStringToken> StringToken = Consumer.GetStream().ParseToken([&PropertyName, &bShouldBeEnum, &ParsedStringType, &OpeningQuoteChar, &NumConsecutiveSlashes](TCHAR InC)
	{
		if (ParsedStringType == EParsedStringType::Unknown)
		{
			if (InC == '"' || InC == '\'')
			{
				ParsedStringType = EParsedStringType::Quoted;

				OpeningQuoteChar = InC;
				NumConsecutiveSlashes = 0;
				return EParseState::Continue;
			}
			
			ParsedStringType = EParsedStringType::Unquoted;
		}

		check(ParsedStringType != EParsedStringType::Unknown);

		if (InC == ':')
		{
			bShouldBeEnum = true;
		}

		if (ParsedStringType == EParsedStringType::Unquoted)
		{
			for (const TCHAR BreakingChar : PropertyBreakingChars)
			{
				if (InC == BreakingChar)
				{
					return EParseState::StopBefore;
				}
			}

			PropertyName.AppendChar(InC);
		}
		else
		{
			check(ParsedStringType == EParsedStringType::Quoted);

			if (InC == OpeningQuoteChar && NumConsecutiveSlashes % 2 == 0)
			{
				return EParseState::StopAfter;
			}

			PropertyName.AppendChar(InC);

			if (InC == '\\')
			{
				NumConsecutiveSlashes++;
			}
			else
			{
				NumConsecutiveSlashes = 0;
			}
		}

		return EParseState::Continue;
	});

	if (ParsedStringType == EParsedStringType::Quoted)
	{
		PropertyName.ReplaceEscapedCharWithCharInline();
	}

	if (StringToken.IsSet())
	{
		if (bShouldBeEnum)
		{
			int32 DoubleColonIndex = PropertyName.Find("::");
			if (DoubleColonIndex == INDEX_NONE)
			{
				return FExpressionError(FText::Format(LOCTEXT("PropertyContainsSingleColon", "EditCondition contains single colon in property name \"{0}\", expected double colons."), FText::FromString(PropertyName)));
			}

			if (DoubleColonIndex == 0)
			{
				return FExpressionError(FText::Format(LOCTEXT("PropertyDoubleColonAtStart", "EditCondition contained double colon at start of property name \"{0}\", expected enum type."), FText::FromString(PropertyName)));
			}

			FString EnumType = PropertyName.Left(DoubleColonIndex);
			FString EnumValue = PropertyName.RightChop(DoubleColonIndex + 2);
			
			if (EnumValue.Len() == 0)
			{
				return FExpressionError(FText::Format(LOCTEXT("PropertyDoubleColonAtEnd", "EditCondition contained double colon at end of property name \"{0}\", expected enum value."), FText::FromString(PropertyName)));
			}

			Consumer.Add(StringToken.GetValue(), EditConditionParserTokens::FEnumToken(MoveTemp(EnumType), MoveTemp(EnumValue)));
		}
		else
		{
			Consumer.Add(StringToken.GetValue(), EditConditionParserTokens::FPropertyToken(MoveTemp(PropertyName)));
		}
	}

	return TOptional<FExpressionError>();
}

template <typename ValueType>
static void LogEditConditionError(const TValueOrError<ValueType, FExpressionError>& Error, const IEditConditionContext* Context = nullptr)
{
	if (!Error.HasError())
	{
		return;
	}

	const FString Message = Error.GetError().Text.ToString();
	FString Formatted = Message;
	if (Context != nullptr)
	{
		Formatted = FString::Printf(TEXT("%s - %s"), *Context->GetContextName().ToString(), *Message);
	}

	static TSet<FString> ErrorsAlreadyLogged;
	if (!ErrorsAlreadyLogged.Find(Formatted))
	{
		ErrorsAlreadyLogged.Add(Formatted);
		UE_LOGF(LogEditCondition, Error, "%ls", *Formatted);
	}
}

template <typename T>
TOptional<T> GetValueInternal(const IEditConditionContext& Context, const FString& PropertyName)
{
	return TOptional<T>();
}

template<>
TOptional<bool> GetValueInternal<bool>(const IEditConditionContext& Context, const FString& PropertyName)
{
	return Context.GetBoolValue(PropertyName, Context.GetFunction(PropertyName));
}

template<>
TOptional<double> GetValueInternal<double>(const IEditConditionContext& Context, const FString& PropertyName)
{
	return Context.GetNumericValue(PropertyName, Context.GetFunction(PropertyName));
}

template <typename T>
struct TOperand
{
	TOperand(T InValue) : Value(InValue), Property(nullptr), Context(nullptr) {}
	TOperand(const EditConditionParserTokens::FPropertyToken& InProperty, const IEditConditionContext& InContext) : 
		Property(&InProperty), Context(&InContext) {}

	bool IsProperty() const { return Property != nullptr; }
	TOptional<T> GetValue() const
	{
		if (IsProperty())
		{
			return GetValueInternal<T>(*Context, Property->PropertyName);	
		}
		
		return TOptional<T>(Value);
	}

	const FString& GetName() const { check(IsProperty()); return Property->PropertyName; }

private:
	T Value;
	const EditConditionParserTokens::FPropertyToken* Property;
	const IEditConditionContext* Context;
};

static FExpressionResult ApplyNot(TOperand<bool> A)
{
	TOptional<bool> Value = A.GetValue();
	if (Value.IsSet())
	{
		return MakeValue(!Value.GetValue());
	}
	return MakeError(FText::Format(LOCTEXT("InvalidOperand", "EditCondition attempted to use an invalid operand \"{0}\"."), FText::FromString(A.GetName())));
}

template <typename T, typename Function>
FExpressionResult ApplyBinary(TOperand<T> A, TOperand<T> B, Function Apply)
{
	TOptional<T> ValueA = A.GetValue();
	if (!ValueA.IsSet())
	{
		return MakeError(FText::Format(LOCTEXT("InvalidOperand", "EditCondition attempted to use an invalid operand \"{0}\"."), FText::FromString(A.GetName())));
	}

	TOptional<T> ValueB = B.GetValue();
	if (!ValueB.IsSet())
	{
		return MakeError(FText::Format(LOCTEXT("InvalidOperand", "EditCondition attempted to use an invalid operand \"{0}\"."), FText::FromString(B.GetName())));
	}

	return MakeValue(Apply(ValueA.GetValue(), ValueB.GetValue()));
}

static FExpressionResult ApplyBitwiseAnd(const EditConditionParserTokens::FPropertyToken& Property, const EditConditionParserTokens::FEnumToken& Enum, const IEditConditionContext& Context)
{
	TOptional<int64> EnumValue = Context.GetIntegerValueOfEnum(Enum.Type, Enum.Value);
	if (!EnumValue.IsSet())
	{
		return MakeError(FText::Format(LOCTEXT("InvalidEnumValue", "EditCondition attempted to use an invalid enum value \"{0}::{1}\"."), FText::FromString(Enum.Type), FText::FromString(Enum.Value)));
	}

	TOptional<int64> PropertyValue = Context.GetIntegerValue(Property.PropertyName, Context.GetFunction(Property.PropertyName));
	if (!PropertyValue.IsSet())
	{
		return MakeError(FText::Format(LOCTEXT("InvalidOperand", "EditCondition attempted to use an invalid operand \"{0}\"."), FText::FromString(Property.PropertyName)));
	}

	return MakeValue((PropertyValue.Get(0) & EnumValue.Get(0)) != 0);
}

static FExpressionResult ApplyPropertyIsNull(const EditConditionParserTokens::FPropertyToken& Property, const IEditConditionContext& Context, bool bNegate)
{
	TWeakObjectPtr<UFunction> CachedFunction = Context.GetFunction(Property.PropertyName);

	TOptional<FString> TypeName = Context.GetTypeName(Property.PropertyName, CachedFunction);
	if (!TypeName.IsSet())
	{
		return MakeError(FText::Format(LOCTEXT("InvalidOperand", "EditCondition attempted to use an invalid operand \"{0}\"."), FText::FromString(Property.PropertyName)));
	}

	TOptional<UObject*> Ptr = Context.GetPointerValue(Property.PropertyName, CachedFunction);
	if (!Ptr.IsSet())
	{
		return MakeError(FText::Format(LOCTEXT("InvalidOperand", "EditCondition attempted to use an invalid operand \"{0}\"."), FText::FromString(Property.PropertyName)));
	}

	bool bIsNull = Ptr.GetValue() == nullptr;
	if (bNegate)
	{
		bIsNull = !bIsNull;
	}

	return MakeValue(bIsNull);
}

static FExpressionResult ApplyPropertyIsIndexNone(const EditConditionParserTokens::FPropertyToken& Property, const IEditConditionContext& Context, bool bNegate)
{
	TWeakObjectPtr<UFunction> CachedFunction = Context.GetFunction(Property.PropertyName);

	TOptional<FString> TypeName = Context.GetTypeName(Property.PropertyName, CachedFunction);
	if (!TypeName.IsSet())
	{
		return MakeError(FText::Format(LOCTEXT("InvalidOperand", "EditCondition attempted to use an invalid operand \"{0}\"."), FText::FromString(Property.PropertyName)));
	}

	TOptional<int64> Value = Context.GetIntegerValue(Property.PropertyName, CachedFunction);
	if (!Value.IsSet())
	{
		return MakeError(FText::Format(LOCTEXT("InvalidOperand", "EditCondition attempted to use an invalid operand \"{0}\"."), FText::FromString(Property.PropertyName)));
	}

	bool bIsIndexNone = Value.GetValue() == (int64)INDEX_NONE;
	if (bNegate)
	{
		bIsIndexNone = !bIsIndexNone;
	}

	return MakeValue(bIsIndexNone);
}

static FExpressionResult ApplyPropertiesEqual(const EditConditionParserTokens::FPropertyToken& A, const EditConditionParserTokens::FPropertyToken& B, const IEditConditionContext& Context, bool bNegate)
{
	TWeakObjectPtr<UFunction> CachedFunctionA = Context.GetFunction(A.PropertyName);
	TWeakObjectPtr<UFunction> CachedFunctionB = Context.GetFunction(A.PropertyName);

	TOptional<UObject*> PtrA = Context.GetPointerValue(A.PropertyName, CachedFunctionA);
	TOptional<UObject*> PtrB = Context.GetPointerValue(B.PropertyName, CachedFunctionB);
	if (PtrA.IsSet() && PtrB.IsSet())
	{
		const bool bAreEqual = PtrA.GetValue() == PtrB.GetValue();
		return MakeValue(bNegate ? !bAreEqual : bAreEqual);
	}

	TOptional<FString> TypeNameA = Context.GetTypeName(A.PropertyName, CachedFunctionA);
	TOptional<FString> TypeNameB = Context.GetTypeName(B.PropertyName, CachedFunctionB);
	if (!TypeNameA.IsSet())
	{
		return MakeError(FText::Format(LOCTEXT("InvalidOperand", "EditCondition attempted to use an invalid operand \"{0}\"."), FText::FromString(A.PropertyName)));
	}

	if (!TypeNameB.IsSet())
	{
		return MakeError(FText::Format(LOCTEXT("InvalidOperand", "EditCondition attempted to use an invalid operand \"{0}\"."), FText::FromString(B.PropertyName)));
	}

	if (TypeNameA.GetValue() != TypeNameB.GetValue())
	{
		return MakeError(FText::Format(LOCTEXT("OperandTypeMismatch", "EditCondition attempted to compare operands of different types: \"{0}\" and \"{1}\"."), FText::FromString(A.PropertyName), FText::FromString(B.PropertyName)));
	}

	TOptional<bool> BoolA = Context.GetBoolValue(A.PropertyName, CachedFunctionA);
	TOptional<bool> BoolB = Context.GetBoolValue(B.PropertyName, CachedFunctionB);
	if (BoolA.IsSet() && BoolB.IsSet())
	{
		const bool bAreEqual = BoolA.GetValue() == BoolB.GetValue();
		return MakeValue(bNegate ? !bAreEqual : bAreEqual);
	}

	TOptional<double> DoubleA = Context.GetNumericValue(A.PropertyName, CachedFunctionA);
	TOptional<double> DoubleB = Context.GetNumericValue(B.PropertyName, CachedFunctionB);
	if (DoubleA.IsSet() && DoubleB.IsSet())
	{
		const bool bAreEqual = DoubleA.GetValue() == DoubleB.GetValue();
		return MakeValue(bNegate ? !bAreEqual : bAreEqual);
	}

	TOptional<FString> EnumA = Context.GetEnumValue(A.PropertyName, CachedFunctionA);
	TOptional<FString> EnumB = Context.GetEnumValue(B.PropertyName, CachedFunctionB);
	if (EnumA.IsSet() && EnumB.IsSet())
	{
		const bool bAreEqual = EnumA.GetValue() == EnumB.GetValue();
		return MakeValue(bNegate ? !bAreEqual : bAreEqual);
	}

	return MakeError(FText::Format(LOCTEXT("OperandTypeMismatch", "EditCondition attempted to compare operands of different types: \"{0}\" and \"{1}\"."), FText::FromString(A.PropertyName), FText::FromString(B.PropertyName)));
}

static void CreateBooleanOperators(TOperatorJumpTable<IEditConditionContext>& OperatorJumpTable)
{
	using namespace EditConditionParserTokens;

	OperatorJumpTable.MapPreUnary<FNot>([](bool A) { return !A; });
	OperatorJumpTable.MapPreUnary<FNot>([](const FPropertyToken& A, const IEditConditionContext* Context)
	{
		return ApplyNot(TOperand<bool>(A, *Context));
	});

	// AND
	{
		auto ApplyAnd = [](bool First, bool Second) { return First && Second; };
	
		OperatorJumpTable.MapBinary<FAnd>(ApplyAnd);
		OperatorJumpTable.MapBinary<FAnd>([ApplyAnd](const FPropertyToken& A, bool B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<bool>(A, *Context), TOperand<bool>(B), ApplyAnd);
		});
		OperatorJumpTable.MapBinary<FAnd>([ApplyAnd](bool A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<bool>(A), TOperand<bool>(B, *Context), ApplyAnd);
		});
		OperatorJumpTable.MapBinary<FAnd>([ApplyAnd](const FPropertyToken& A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<bool>(A, *Context), TOperand<bool>(B, *Context), ApplyAnd);
		});
	}

	// OR
	{
		auto ApplyOr = [](bool First, bool Second) { return First || Second; };

		OperatorJumpTable.MapBinary<FOr>(ApplyOr);
		OperatorJumpTable.MapBinary<FOr>([ApplyOr](const FPropertyToken& A, bool B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<bool>(A, *Context), TOperand<bool>(B), ApplyOr);
		});
		OperatorJumpTable.MapBinary<FOr>([ApplyOr](bool A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<bool>(A), TOperand<bool>(B, *Context), ApplyOr);
		});
		OperatorJumpTable.MapBinary<FOr>([ApplyOr](const FPropertyToken& A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<bool>(A, *Context), TOperand<bool>(B, *Context), ApplyOr);
		});
	}

	// EQUALS
	{
		auto ApplyEqual = [](bool First, bool Second) { return First == Second; };

		OperatorJumpTable.MapBinary<FEqual>(ApplyEqual);
		OperatorJumpTable.MapBinary<FEqual>([ApplyEqual](const FPropertyToken& A, bool B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<bool>(A, *Context), TOperand<bool>(B), ApplyEqual);
		});
		OperatorJumpTable.MapBinary<FEqual>([ApplyEqual](bool A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<bool>(A), TOperand<bool>(B, *Context), ApplyEqual);
		});
	}

	// NOT-EQUALS
	{
		auto ApplyNotEqual = [](bool First, bool Second) { return First != Second; };

		OperatorJumpTable.MapBinary<FNotEqual>(ApplyNotEqual);
		OperatorJumpTable.MapBinary<FNotEqual>([ApplyNotEqual](const FPropertyToken& A, bool B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<bool>(A, *Context), TOperand<bool>(B), ApplyNotEqual);
		});
		OperatorJumpTable.MapBinary<FNotEqual>([ApplyNotEqual](bool A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<bool>(A), TOperand<bool>(B, *Context), ApplyNotEqual);
		});
	}
}

template <typename T>
void CreateNumberOperators(TOperatorJumpTable<IEditConditionContext>& OperatorJumpTable)
{
	using namespace EditConditionParserTokens;

	// EQUAL
	{
		auto ApplyEqual = [](T First, T Second) { return First == Second; };

		OperatorJumpTable.MapBinary<FEqual>(ApplyEqual);
		OperatorJumpTable.MapBinary<FEqual>([ApplyEqual](const FPropertyToken& A, T B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A, *Context), TOperand<T>(B), ApplyEqual);
		});
		OperatorJumpTable.MapBinary<FEqual>([ApplyEqual](T A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A), TOperand<T>(B, *Context), ApplyEqual);
		});
	}

	// NOT-EQUAL
	{
		auto ApplyNotEqual = [](T First, T Second) { return First != Second; };

		OperatorJumpTable.MapBinary<FNotEqual>(ApplyNotEqual);
		OperatorJumpTable.MapBinary<FNotEqual>([ApplyNotEqual](const FPropertyToken& A, T B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A, *Context), TOperand<T>(B), ApplyNotEqual);
		});
		OperatorJumpTable.MapBinary<FNotEqual>([ApplyNotEqual](T A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A), TOperand<T>(B, *Context), ApplyNotEqual);
		});
	}

	// GREATER
	{
		auto ApplyGreater = [](T First, T Second) { return First > Second; };

		OperatorJumpTable.MapBinary<FGreater>(ApplyGreater);
		OperatorJumpTable.MapBinary<FGreater>([ApplyGreater](const FPropertyToken& A, T B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A, *Context), TOperand<T>(B), ApplyGreater);
		});
		OperatorJumpTable.MapBinary<FGreater>([ApplyGreater](T A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A), TOperand<T>(B, *Context), ApplyGreater);
		});
		OperatorJumpTable.MapBinary<FGreater>([ApplyGreater](const FPropertyToken& A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A, *Context), TOperand<T>(B, *Context), ApplyGreater);
		});
	}

	// GREATER-EQUAL
	{
		auto ApplyGreaterEqual = [](T First, T Second) { return First >= Second; };

		OperatorJumpTable.MapBinary<FGreaterEqual>(ApplyGreaterEqual);
		OperatorJumpTable.MapBinary<FGreaterEqual>([ApplyGreaterEqual](const FPropertyToken& A, T B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A, *Context), TOperand<T>(B), ApplyGreaterEqual);
		});
		OperatorJumpTable.MapBinary<FGreaterEqual>([ApplyGreaterEqual](T A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A), TOperand<T>(B, *Context), ApplyGreaterEqual);
		});
		OperatorJumpTable.MapBinary<FGreaterEqual>([ApplyGreaterEqual](const FPropertyToken& A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A, *Context), TOperand<T>(B, *Context), ApplyGreaterEqual);
		});
	}

	// LESS
	{
		auto ApplyLess = [](T First, T Second) { return First < Second; };

		OperatorJumpTable.MapBinary<FLess>(ApplyLess);
		OperatorJumpTable.MapBinary<FLess>([ApplyLess](const FPropertyToken& A, T B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A, *Context), TOperand<T>(B), ApplyLess);
		});
		OperatorJumpTable.MapBinary<FLess>([ApplyLess](T A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A), TOperand<T>(B, *Context), ApplyLess);
		});
		OperatorJumpTable.MapBinary<FLess>([ApplyLess](const FPropertyToken& A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A, *Context), TOperand<T>(B, *Context), ApplyLess);
		});
	}

	// LESS-EQUAL
	{
		auto ApplyLessEqual = [](T First, T Second) { return First <= Second; };

		OperatorJumpTable.MapBinary<FLessEqual>(ApplyLessEqual);
		OperatorJumpTable.MapBinary<FLessEqual>([ApplyLessEqual](const FPropertyToken& A, T B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A, *Context), TOperand<T>(B), ApplyLessEqual);
		});
		OperatorJumpTable.MapBinary<FLessEqual>([ApplyLessEqual](T A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A), TOperand<T>(B, *Context), ApplyLessEqual);
		});
		OperatorJumpTable.MapBinary<FLessEqual>([ApplyLessEqual](const FPropertyToken& A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A, *Context), TOperand<T>(B, *Context), ApplyLessEqual);
		});
	}

	// ADD
	{
		auto ApplyAdd = [](T First, T Second) { return First + Second; };

		OperatorJumpTable.MapBinary<FAdd>(ApplyAdd);
		OperatorJumpTable.MapBinary<FAdd>([ApplyAdd](const FPropertyToken& A, T B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A, *Context), TOperand<T>(B), ApplyAdd);
		});
		OperatorJumpTable.MapBinary<FAdd>([ApplyAdd](T A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A), TOperand<T>(B, *Context), ApplyAdd);
		});
		OperatorJumpTable.MapBinary<FAdd>([ApplyAdd](const FPropertyToken& A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A, *Context), TOperand<T>(B, *Context), ApplyAdd);
		});
	}

	// SUBTRACT
	{
		auto ApplySubtract = [](T First, T Second) { return First - Second; };

		OperatorJumpTable.MapBinary<FSubtract>(ApplySubtract);
		OperatorJumpTable.MapBinary<FSubtract>([ApplySubtract](const FPropertyToken& A, T B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A, *Context), TOperand<T>(B), ApplySubtract);
		});
		OperatorJumpTable.MapBinary<FSubtract>([ApplySubtract](T A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A), TOperand<T>(B, *Context), ApplySubtract);
		});
		OperatorJumpTable.MapBinary<FSubtract>([ApplySubtract](const FPropertyToken& A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A, *Context), TOperand<T>(B, *Context), ApplySubtract);
		});
	}

	// MULTIPLY
	{
		auto ApplyMultiply = [](T First, T Second) { return First * Second; };

		OperatorJumpTable.MapBinary<FMultiply>(ApplyMultiply);
		OperatorJumpTable.MapBinary<FMultiply>([ApplyMultiply](const FPropertyToken& A, T B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A, *Context), TOperand<T>(B), ApplyMultiply);
		});
		OperatorJumpTable.MapBinary<FMultiply>([ApplyMultiply](T A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A), TOperand<T>(B, *Context), ApplyMultiply);
		});
		OperatorJumpTable.MapBinary<FMultiply>([ApplyMultiply](const FPropertyToken& A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A, *Context), TOperand<T>(B, *Context), ApplyMultiply);
		});
	}

	// DIVIDE
	{
		auto ApplyDivide = [](T First, T Second) { return First / Second; };

		OperatorJumpTable.MapBinary<FDivide>(ApplyDivide);
		OperatorJumpTable.MapBinary<FDivide>([ApplyDivide](const FPropertyToken& A, T B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A, *Context), TOperand<T>(B), ApplyDivide);
		});
		OperatorJumpTable.MapBinary<FDivide>([ApplyDivide](T A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A), TOperand<T>(B, *Context), ApplyDivide);
		});
		OperatorJumpTable.MapBinary<FDivide>([ApplyDivide](const FPropertyToken& A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return ApplyBinary(TOperand<T>(A, *Context), TOperand<T>(B, *Context), ApplyDivide);
		});
	}
}

static FExpressionResult EnumPropertyEquals(const EditConditionParserTokens::FEnumToken& Enum, const EditConditionParserTokens::FPropertyToken& Property, const IEditConditionContext& Context, bool bNegate)
{
	TWeakObjectPtr<UFunction> CachedFunction = Context.GetFunction(Property.PropertyName);

	TOptional<FString> TypeName = Context.GetTypeName(Property.PropertyName, CachedFunction);
	if (!TypeName.IsSet())
	{
		return MakeError(FText::Format(LOCTEXT("InvalidOperand_Type", "EditCondition attempted to use an invalid operand \"{0}\" (type error)."), FText::FromString(Property.PropertyName)));
	}

	if (TypeName.GetValue() != Enum.Type)
	{
		return MakeError(FText::Format(LOCTEXT("OperandTypeMismatch", "EditCondition attempted to compare operands of different types: \"{0}\" and \"{1}\"."), FText::FromString(Property.PropertyName), FText::FromString(Enum.Type + TEXT("::") + Enum.Value)));
	}

	TOptional<FString> ValueProp = Context.GetEnumValue(Property.PropertyName, CachedFunction);
	if (!ValueProp.IsSet())
	{
		return MakeError(FText::Format(LOCTEXT("InvalidOperand_Value", "EditCondition attempted to use an invalid operand \"{0}\" (value error)."), FText::FromString(Property.PropertyName)));
	}

	bool bEqual = ValueProp.GetValue() == Enum.Value;
	return MakeValue(bNegate ? !bEqual : bEqual);
}

static void CreateEnumOperators(TOperatorJumpTable<IEditConditionContext>& OperatorJumpTable)
{
	using namespace EditConditionParserTokens;

	// EQUALS
	{
		OperatorJumpTable.MapBinary<FEqual>([](const FEnumToken& A, const FEnumToken& B, const IEditConditionContext* Context)
		{
			return A.Type == B.Type && A.Value == B.Value;
		});
		OperatorJumpTable.MapBinary<FEqual>([](const FPropertyToken& A, const FEnumToken& B, const IEditConditionContext* Context)
		{
			return EnumPropertyEquals(B, A, *Context, false);
		});
		OperatorJumpTable.MapBinary<FEqual>([](const FEnumToken& A, const FPropertyToken& B, const IEditConditionContext* Context)
		{
			return EnumPropertyEquals(A, B, *Context, false);
		});
	}

	// NOT-EQUALS
	{
		OperatorJumpTable.MapBinary<FNotEqual>([](const FEnumToken& A, const FEnumToken& B, const IEditConditionContext* Context)
		{
			return A.Type != B.Type || A.Value != B.Value;
		});
		OperatorJumpTable.MapBinary<FNotEqual>([](const FPropertyToken& A, const FEnumToken& B, const IEditConditionContext* Context) -> FExpressionResult
		{
			return EnumPropertyEquals(B, A, *Context, true);
		});
		OperatorJumpTable.MapBinary<FNotEqual>([](const FEnumToken& A, const FPropertyToken& B, const IEditConditionContext* Context) -> FExpressionResult
		{
			return EnumPropertyEquals(A, B, *Context, true);
		});
	}
}

FEditConditionParser::FEditConditionParser()
{
	using namespace EditConditionParserTokens;

	TokenDefinitions.IgnoreWhitespace();
	TokenDefinitions.DefineToken(&ExpressionParser::ConsumeSymbol<FEqual>);
	TokenDefinitions.DefineToken(&ExpressionParser::ConsumeSymbol<FNotEqual>);
	TokenDefinitions.DefineToken(&ExpressionParser::ConsumeSymbol<FLessEqual>);
	TokenDefinitions.DefineToken(&ExpressionParser::ConsumeSymbol<FLess>);
	TokenDefinitions.DefineToken(&ExpressionParser::ConsumeSymbol<FGreaterEqual>);
	TokenDefinitions.DefineToken(&ExpressionParser::ConsumeSymbol<FGreater>);
	TokenDefinitions.DefineToken(&ExpressionParser::ConsumeSymbol<FNot>);
	TokenDefinitions.DefineToken(&ExpressionParser::ConsumeSymbol<FAnd>);
	TokenDefinitions.DefineToken(&ExpressionParser::ConsumeSymbol<FOr>);
	TokenDefinitions.DefineToken(&ExpressionParser::ConsumeSymbol<FAdd>);
	TokenDefinitions.DefineToken(&ExpressionParser::ConsumeSymbol<FSubtract>);
	TokenDefinitions.DefineToken(&ExpressionParser::ConsumeSymbol<FMultiply>);
	TokenDefinitions.DefineToken(&ExpressionParser::ConsumeSymbol<FDivide>);
	TokenDefinitions.DefineToken(&ExpressionParser::ConsumeSymbol<FBitwiseAnd>);
	TokenDefinitions.DefineToken(&ExpressionParser::ConsumeSymbol<FSubExpressionStart>);
	TokenDefinitions.DefineToken(&ExpressionParser::ConsumeSymbol<FSubExpressionEnd>);
	TokenDefinitions.DefineToken(&ExpressionParser::ConsumeNumber);
	TokenDefinitions.DefineToken(&ConsumeNullPtr);
	TokenDefinitions.DefineToken(&ConsumeIndexNone);
	TokenDefinitions.DefineToken(&ConsumeBool);
	TokenDefinitions.DefineToken(&ConsumePropertyName);
				
	ExpressionGrammar.DefineBinaryOperator<FAnd>(4);
	ExpressionGrammar.DefineBinaryOperator<FOr>(4);
	ExpressionGrammar.DefineBinaryOperator<FEqual>(3);
	ExpressionGrammar.DefineBinaryOperator<FNotEqual>(3);
	ExpressionGrammar.DefineBinaryOperator<FLess>(3);
	ExpressionGrammar.DefineBinaryOperator<FLessEqual>(3);
	ExpressionGrammar.DefineBinaryOperator<FGreater>(3);
	ExpressionGrammar.DefineBinaryOperator<FGreaterEqual>(3);
	ExpressionGrammar.DefineBinaryOperator<FBitwiseAnd>(2);
	ExpressionGrammar.DefineBinaryOperator<FAdd>(2);
	ExpressionGrammar.DefineBinaryOperator<FSubtract>(2);
	ExpressionGrammar.DefineBinaryOperator<FMultiply>(1);
	ExpressionGrammar.DefineBinaryOperator<FDivide>(1);
	ExpressionGrammar.DefinePreUnaryOperator<FNot>();
	ExpressionGrammar.DefineGrouping<FSubExpressionStart, FSubExpressionEnd>();

	// POINTER EQUALITY
	OperatorJumpTable.MapBinary<FEqual>([](const FPropertyToken& A, const FPropertyToken& B, const IEditConditionContext* Context) -> FExpressionResult
	{
		return ApplyPropertiesEqual(A, B, *Context, false);
	});

	OperatorJumpTable.MapBinary<FNotEqual>([](const FPropertyToken& A, const FPropertyToken& B, const IEditConditionContext* Context) -> FExpressionResult
	{
		return ApplyPropertiesEqual(A, B, *Context, true);
	});

	// POINTER NULL
	OperatorJumpTable.MapBinary<FEqual>([](const FPropertyToken& A, const FNullPtrToken& B, const IEditConditionContext* Context) -> FExpressionResult
	{
		return ApplyPropertyIsNull(A, *Context, false);
	});

	OperatorJumpTable.MapBinary<FNotEqual>([](const FPropertyToken& A, const FNullPtrToken& B, const IEditConditionContext* Context) -> FExpressionResult
	{
		return ApplyPropertyIsNull(A, *Context, true);
	});

	// INDEX_NONE
	OperatorJumpTable.MapBinary<FEqual>([](const FPropertyToken& A, const FIndexNoneToken& B, const IEditConditionContext* Context) -> FExpressionResult
	{
		return ApplyPropertyIsIndexNone(A, *Context, false);
	});

	OperatorJumpTable.MapBinary<FNotEqual>([](const FPropertyToken& A, const FIndexNoneToken& B, const IEditConditionContext* Context) -> FExpressionResult
	{
		return ApplyPropertyIsIndexNone(A, *Context, true);
	});

	// BITWISE AND
	OperatorJumpTable.MapBinary<FBitwiseAnd>([](const FPropertyToken& A, const FEnumToken& B, const IEditConditionContext* Context) -> FExpressionResult
	{
		return ApplyBitwiseAnd(A, B, *Context);
	});

	CreateBooleanOperators(OperatorJumpTable);
	CreateNumberOperators<double>(OperatorJumpTable);
	CreateEnumOperators(OperatorJumpTable);
}

TValueOrError<bool, FText> FEditConditionParser::Evaluate(const FEditConditionExpression& Expression, const IEditConditionContext& Context) const
{
	using namespace EditConditionParserTokens;
	
	// Edit conditions may invoke UFUNCTION()s by going through UObject::ProcessEvent, which checkf()s this condition.
	const bool bIsSafeToCall = !FUObjectThreadContext::Get().IsRoutingPostLoad;
	// If you hit this ensure, then adjust your logic to not call this during PostLoad, e.g. try deferring it to end of frame / next tick.
	if (!ensure(bIsSafeToCall))
	{
		return MakeError(LOCTEXT("Evaluate.InternalError", "Internal Error: Invalid call timing."));
	}
	
	FExpressionResult Result = ExpressionParser::Evaluate(Expression.Tokens, OperatorJumpTable, &Context);
	if (Result.HasValue())
	{
		const bool* BoolResult = Result.GetValue().Cast<bool>();
		if (BoolResult != nullptr)
		{
			return MakeValue(*BoolResult);
		}

		const FPropertyToken* PropertyResult = Result.GetValue().Cast<FPropertyToken>();
		if (PropertyResult != nullptr)
		{
			TOptional<bool> PropertyValue = Context.GetBoolValue(PropertyResult->PropertyName, Context.GetFunction(PropertyResult->PropertyName));
			if (PropertyValue.IsSet())
			{
				return MakeValue(PropertyValue.GetValue());
			}
		}
	}
	else
	{
		LogEditConditionError(Result, &Context);
	}

	const FText ErrorText = Result.HasError() ? Result.StealError().Text : FText::GetEmpty();
	return MakeError(ErrorText);
}

TSharedPtr<FEditConditionExpression> FEditConditionParser::Parse(const FString& ExpressionString) const
{
	using namespace ExpressionParser;

	LexResultType LexResult = ExpressionParser::Lex(*ExpressionString, TokenDefinitions);
	if (LexResult.IsValid())
	{
		CompileResultType CompileResult = ExpressionParser::Compile(LexResult.StealValue(), ExpressionGrammar);
		if (CompileResult.IsValid())
		{
			return MakeShared<FEditConditionExpression>(CompileResult.StealValue());
		}
		else
		{
			LogEditConditionError(CompileResult);
		}
	}
	else
	{
		LogEditConditionError(LexResult);
	}

	return TSharedPtr<FEditConditionExpression>();
}

#undef LOCTEXT_NAMESPACE
