// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Misc/ExpressionParserTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/ValueOrError.h"

class FText;

namespace EditConditionParserTokens
{
	struct FPropertyToken 
	{
		FPropertyToken(FString&& InProperty) :
			PropertyName(InProperty) {}

		FPropertyToken(const FPropertyToken& Other) :
			PropertyName(Other.PropertyName) {}

		FPropertyToken& operator=(const FPropertyToken& Other)
		{
			PropertyName = Other.PropertyName;
			return *this;
		}

		FString PropertyName;
	};

	struct FEnumToken 
	{
		FEnumToken(FString&& InType, FString&& InValue) :
			Type(InType), Value(InValue) {}

		FEnumToken(const FEnumToken& Other) :
			Type(Other.Type), Value(Other.Value) {}

		FEnumToken& operator=(const FEnumToken& Other)
		{
			Type = Other.Type;
			Value = Other.Value;
			return *this;
		}

		FString Type;
		FString Value;
	};

	struct FNullPtrToken
	{
		FNullPtrToken() = default;
		FNullPtrToken(const FNullPtrToken&) = default;
		FNullPtrToken& operator=(const FNullPtrToken&) = default;
	};

	struct FIndexNoneToken
	{
		FIndexNoneToken() = default;
		FIndexNoneToken(const FIndexNoneToken&) = default;
		FIndexNoneToken& operator=(const FIndexNoneToken&) = default;
	};
}

#define DEFINE_EDIT_CONDITION_NODE(TYPE, ...) \
	namespace EditConditionParserTokens { struct TYPE { static const TCHAR* const Moniker; }; } \
	DEFINE_EXPRESSION_NODE_TYPE(EditConditionParserTokens::TYPE, __VA_ARGS__)

	DEFINE_EDIT_CONDITION_NODE(FEqual, 0x3AF0EE1B, 0xC3F847C7, 0xA2B95102, 0x4EFC202D)
	DEFINE_EDIT_CONDITION_NODE(FNotEqual, 0x5CDF3FA4, 0x35614D88, 0x8F94017C, 0xF9229625)
	DEFINE_EDIT_CONDITION_NODE(FLessEqual, 0x0D097F12, 0x2268447B, 0xA9C6769E, 0x5FE086EA)
	DEFINE_EDIT_CONDITION_NODE(FLess, 0xD3B1D8E9, 0x552249C7, 0x92646CF8, 0x6D065F45)
	DEFINE_EDIT_CONDITION_NODE(FGreaterEqual, 0x5BC680B9, 0x148448FB, 0xA37A0EBB, 0x34C4BCA8)
	DEFINE_EDIT_CONDITION_NODE(FGreater, 0xBD99934C, 0x5E7F4BE9, 0x99A1BD6E, 0x582F8FC9)
	DEFINE_EDIT_CONDITION_NODE(FNot, 0xE0FB4BB7, 0x66A646C2, 0xB2360362, 0x0E1FAE33)
	DEFINE_EDIT_CONDITION_NODE(FAnd, 0x71F839BC, 0xD75F6B47, 0x9C68017B, 0x33FAB080)
	DEFINE_EDIT_CONDITION_NODE(FOr, 0x9C317A4C, 0xCE50304D, 0x8BF6D471, 0x9D791746)
	DEFINE_EDIT_CONDITION_NODE(FAdd, 0xF4086317, 0x80428041, 0x903149EB, 0xC5A60ACF)
	DEFINE_EDIT_CONDITION_NODE(FSubtract, 0xE80D53E9, 0x32F1E145, 0x8535B767, 0x0634DE9E)
	DEFINE_EDIT_CONDITION_NODE(FMultiply, 0x0D4D37DB, 0x53C7EA41, 0x8297C200, 0xF5B28D27)
	DEFINE_EDIT_CONDITION_NODE(FDivide, 0x48009789, 0x2EE84F40, 0x83467A99, 0x92617168)
	DEFINE_EDIT_CONDITION_NODE(FBitwiseAnd, 0xF5335505, 0xACB75044, 0xAA468084, 0x1042ADE8)
	DEFINE_EDIT_CONDITION_NODE(FSubExpressionStart, 0x78CE8411, 0x34076241, 0xA2BD0548, 0x54458A3A)
	DEFINE_EDIT_CONDITION_NODE(FSubExpressionEnd, 0x5CB25466, 0x780CAE4D, 0x8A88E000, 0xD3695885)

	DEFINE_EXPRESSION_NODE_TYPE(EditConditionParserTokens::FPropertyToken, 0x9A3FAF6F, 0xB2E45E4D, 0xA80A70C6, 0x47A89BD7)
	DEFINE_EXPRESSION_NODE_TYPE(EditConditionParserTokens::FEnumToken, 0xC9A35C24, 0x21FC904B, 0x9F1B2B6A, 0xDF6F4BC4)
	DEFINE_EXPRESSION_NODE_TYPE(EditConditionParserTokens::FNullPtrToken, 0xC3E91735, 0xB3B90C4D, 0x99BA3DCA, 0xC6CC6231)
	DEFINE_EXPRESSION_NODE_TYPE(EditConditionParserTokens::FIndexNoneToken, 0x6A9CA107, 0x8ED649E4, 0xAF4525CA, 0xEBF84F97)

#undef DEFINE_EDIT_CONDITION_NODE

class IEditConditionContext;

class FEditConditionExpression
{
public:
	FEditConditionExpression(TArray<FCompiledToken>&& InTokens) :
		Tokens(MoveTemp(InTokens))
	{}

	TArray<FCompiledToken> Tokens;
};

class FEditConditionParser
{
public:
	FEditConditionParser();

	FEditConditionParser(const FEditConditionParser&) = delete;
	FEditConditionParser(FEditConditionParser&&) = default;
	FEditConditionParser& operator=(const FEditConditionParser&) = delete;
	FEditConditionParser& operator=(FEditConditionParser&&) = default;

	/**
	 * Parse the given string into an expression.
	 * @returns The parsed expression if valid, invalid TSharedPtr if the string is not a valid expression string.
	 */
	TSharedPtr<FEditConditionExpression> Parse(const FString& ExpressionString) const;

	/** 
	 * Evaluate the given expression within the given context.
	 * @returns The result of the evaluated expression if valid, invalid TOptional if the evaluation failed or produced a non-bool result.
	 */
	TValueOrError<bool, FText> Evaluate(const FEditConditionExpression& Expression, const IEditConditionContext& Context) const;

private:
	FTokenDefinitions TokenDefinitions;
	FExpressionGrammar ExpressionGrammar;
	TOperatorJumpTable<IEditConditionContext> OperatorJumpTable;
};
