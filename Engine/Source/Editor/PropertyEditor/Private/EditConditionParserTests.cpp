// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditConditionParserTests.h"
#include "EditConditionParser.h"
#include "EditConditionContext.h"
#include "ObjectPropertyNode.h"
#include "Misc/AutomationTest.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditConditionParserTests)

UEditConditionTestObject::UEditConditionTestObject(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UEditConditionTestObject::VoidFunction() const
{
	ensureAlwaysMsgf(false, TEXT("Error. Should not be executing a void function in edit conditions."));
}

bool UEditConditionTestObject::GetBoolFunction() const
{
	return BoolProperty;
}

EditConditionTestEnum UEditConditionTestObject::GetEnumFunction() const
{
	return EnumProperty;
}

TEnumAsByte<EditConditionByteEnum> UEditConditionTestObject::GetByteEnumFunction() const
{
	return ByteEnumProperty;
}

double UEditConditionTestObject::GetDoubleFunction() const
{
	return DoubleProperty;
}

int32 UEditConditionTestObject::GetIntegerFunction() const
{
	return IntegerProperty;
}

uint8 UEditConditionTestObject::GetUintBitfieldFunction() const
{
	return UintBitfieldProperty;
}

UObject* UEditConditionTestObject::GetUObjectPtrFunction() const
{
	return UObjectPtr;
}

TSoftClassPtr<UObject> UEditConditionTestObject::GetSoftClassPtrFunction() const
{
	return SoftClassPtr;
}

TWeakObjectPtr<UObject> UEditConditionTestObject::GetWeakObjectPtrFunction() const
{
	return WeakObjectPtr;
}

void UEditConditionTestObject::StaticVoidFunction() 
{
	ensureAlwaysMsgf(false, TEXT("Error. Should not be executing a void function in edit conditions."));
}

bool UEditConditionTestObject::StaticGetBoolFunction()
{
	return UEditConditionTestObject::StaticClass()->GetDefaultObject<UEditConditionTestObject>()->BoolProperty;
}

EditConditionTestEnum UEditConditionTestObject::StaticGetEnumFunction()
{
	return UEditConditionTestObject::StaticClass()->GetDefaultObject<UEditConditionTestObject>()->EnumProperty;
}

TEnumAsByte<EditConditionByteEnum> UEditConditionTestObject::StaticGetByteEnumFunction()
{
	return UEditConditionTestObject::StaticClass()->GetDefaultObject<UEditConditionTestObject>()->ByteEnumProperty;
}

double UEditConditionTestObject::StaticGetDoubleFunction()
{
	return UEditConditionTestObject::StaticClass()->GetDefaultObject<UEditConditionTestObject>()->DoubleProperty;
}

int32 UEditConditionTestObject::StaticGetIntegerFunction()
{
	return UEditConditionTestObject::StaticClass()->GetDefaultObject<UEditConditionTestObject>()->IntegerProperty;
}

uint8 UEditConditionTestObject::StaticGetUintBitfieldFunction()
{
	return UEditConditionTestObject::StaticClass()->GetDefaultObject<UEditConditionTestObject>()->UintBitfieldProperty;
}

UObject* UEditConditionTestObject::StaticGetUObjectPtrFunction()
{
	return UEditConditionTestObject::StaticClass()->GetDefaultObject<UEditConditionTestObject>()->UObjectPtr;
}

TSoftClassPtr<UObject> UEditConditionTestObject::StaticGetSoftClassPtrFunction()
{
	return UEditConditionTestObject::StaticClass()->GetDefaultObject<UEditConditionTestObject>()->SoftClassPtr;
}

TWeakObjectPtr<UObject> UEditConditionTestObject::StaticGetWeakObjectPtrFunction()
{
	return UEditConditionTestObject::StaticClass()->GetDefaultObject<UEditConditionTestObject>()->WeakObjectPtr;
}

#if WITH_DEV_AUTOMATION_TESTS
struct FTestEditConditionContext : IEditConditionContext
{
	TMap<FString, bool> BoolValues;
	TMap<FString, int64> IntegerValues;
	TMap<FString, double> DoubleValues;
	TMap<FString, FString> EnumValues;

	FString EnumTypeName;
	TMap<FString, int64> EnumTypeValues;

	FTestEditConditionContext(){}
	virtual ~FTestEditConditionContext() {}

	virtual FName GetContextName() const
	{
		return "TestEditConditionContext";
	}

	virtual TOptional<bool> GetBoolValue(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction = nullptr) const override
	{
		if (const bool* Value = BoolValues.Find(PropertyName))
		{
			return *Value;
		}
		return TOptional<bool>();
	}

	virtual TOptional<int64> GetIntegerValue(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction = nullptr) const override
	{
		if (const int64* Value = IntegerValues.Find(PropertyName))
		{
			return *Value;
		}
		return TOptional<int64>();
	}

	virtual TOptional<double> GetNumericValue(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction = nullptr) const override
	{
		if (const double* Value = DoubleValues.Find(PropertyName))
		{
			return *Value;
		}
		return TOptional<double>();
	}

	virtual TOptional<FString> GetEnumValue(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction = nullptr) const override
	{
		if (const FString* Value = EnumValues.Find(PropertyName))
		{
			return *Value;
		}
		return TOptional<FString>();
	}

	virtual TOptional<int64> GetIntegerValueOfEnum(const FString& TypeName, const FString& ValueName) const override
	{
		check(TypeName == EnumTypeName);

		if (const int64* Value = EnumTypeValues.Find(ValueName))
		{
			return *Value;
		}
		return TOptional<int64>();
	}

	virtual const TWeakObjectPtr<UFunction> GetFunction(const FString& FieldName) const override
	{
		return nullptr;
	}

	virtual TOptional<UObject*> GetPointerValue(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction = nullptr) const override
	{
		return TOptional<UObject*>();
	}

	virtual TOptional<FString> GetTypeName(const FString& PropertyName, TWeakObjectPtr<UFunction> CachedFunction = nullptr) const override
	{
		TOptional<FString> Result;

		if (BoolValues.Find(PropertyName) != nullptr)
		{
			Result = TEXT("bool");
		}
		else if (DoubleValues.Find(PropertyName) != nullptr)
		{
			Result = TEXT("double");
		}
		else if (EnumValues.Find(PropertyName) != nullptr)
		{
			Result = EnumTypeName;
		}

		return Result;
	}

	void SetupBool(const FString& PropertyName, bool Value)
	{
		BoolValues.Add(PropertyName, Value);
	}

	void SetupInteger(const FString& PropertyName, int64 Value)
	{
		IntegerValues.Add(PropertyName, Value);
	}

	void SetupDouble(const FString& PropertyName, double Value)
	{
		DoubleValues.Add(PropertyName, Value);
	}

	void SetupEnum(const FString& PropertyName, const FString& Value)
	{
		EnumValues.Add(PropertyName, Value);
	}

	void SetupEnumType(const FString& EnumType)
	{
		EnumTypeName = EnumType;
	}

	void SetupEnumTypeValue(const FString& Name, int64 Value)
	{
		EnumTypeValues.Add(Name, Value);
	}
};

static bool CanParse(const FEditConditionParser& Parser, const FString& Expression, int32 ExpectedTokens, int32 ExpectedProperties)
{
	 TSharedPtr<FEditConditionExpression> Parsed = Parser.Parse(Expression);

	 if (!Parsed.IsValid())
	 {
		ensureMsgf(false, TEXT("Failed to parse expression: %s"), *Expression);
		return false;
	 }
	 
	 int PropertyCount = 0;

	 for (const auto& Token : Parsed->Tokens)
	 {
		const EditConditionParserTokens::FPropertyToken* PropertyToken = Token.Node.Cast<EditConditionParserTokens::FPropertyToken>();
		if (PropertyToken != nullptr)
		{
			++PropertyCount;
		}
	 }

	 if (Parsed->Tokens.Num() != ExpectedTokens || PropertyCount != ExpectedProperties)
	 {
		 ensureMsgf(false, TEXT("Parsing produced an invalid number of tokens or properties. Expression: %s"), *Expression);
		 return false;
	 }

	 return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditConditionParser_Parse, "EditConditionParser.Parse", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FEditConditionParser_Parse::RunTest(const FString& Parameters)
{
	FEditConditionParser Parser;
	bool bResult = true;

	bResult &= CanParse(Parser, TEXT("false"), 1, 0);
	bResult &= CanParse(Parser, TEXT("TRUE"), 1, 0);
	bResult &= CanParse(Parser, TEXT("fAlsE"), 1, 0);
	bResult &= CanParse(Parser, TEXT("true == false"), 3, 0);
	bResult &= CanParse(Parser, TEXT("BoolProperty"), 1, 1);
	bResult &= CanParse(Parser, TEXT("!BoolProperty"), 2, 1);
	bResult &= CanParse(Parser, TEXT("BoolProperty == true"), 3, 1);
	bResult &= CanParse(Parser, TEXT("BoolProperty == false"), 3, 1);
	bResult &= CanParse(Parser, TEXT("GetBoolFunction"), 1, 1);
	bResult &= CanParse(Parser, TEXT("!GetBoolFunction"), 2, 1);
	bResult &= CanParse(Parser, TEXT("GetBoolFunction == true"), 3, 1);
	bResult &= CanParse(Parser, TEXT("GetBoolFunction == false"), 3, 1);
	bResult &= CanParse(Parser, TEXT("PropertyEditor.EditConditionTestObject.StaticGetBoolFunction"), 1, 1);
	bResult &= CanParse(Parser, TEXT("!PropertyEditor.EditConditionTestObject.StaticGetBoolFunction"), 2, 1);
	bResult &= CanParse(Parser, TEXT("PropertyEditor.EditConditionTestObject.StaticGetBoolFunction == true"), 3, 1);
	bResult &= CanParse(Parser, TEXT("PropertyEditor.EditConditionTestObject.StaticGetBoolFunction == false"), 3, 1);
	bResult &= CanParse(Parser, TEXT("IntProperty == 0"), 3, 1);
	bResult &= CanParse(Parser, TEXT("IntProperty != 0"), 3, 1);
	bResult &= CanParse(Parser, TEXT("IntProperty > 0"), 3, 1);
	bResult &= CanParse(Parser, TEXT("IntProperty < 0"), 3, 1);
	bResult &= CanParse(Parser, TEXT("IntProperty <= 0"), 3, 1);
	bResult &= CanParse(Parser, TEXT("IntProperty >= 0"), 3, 1);
	bResult &= CanParse(Parser, TEXT("Foo > Bar"), 3, 2);
	bResult &= CanParse(Parser, TEXT("Foo && Bar"), 3, 2);
	bResult &= CanParse(Parser, TEXT("Foo || Bar"), 3, 2);
	bResult &= CanParse(Parser, TEXT("Foo == Bar + 5"), 5, 2);
	bResult &= CanParse(Parser, TEXT("Foo == Bar - 5"), 5, 2);
	bResult &= CanParse(Parser, TEXT("Foo == Bar * 5"), 5, 2);
	bResult &= CanParse(Parser, TEXT("Foo == Bar / 5"), 5, 2);
	bResult &= CanParse(Parser, TEXT("Enum == EType::Value"), 3, 1);
	bResult &= CanParse(Parser, TEXT("Enum != EType::Value"), 3, 1);
	bResult &= CanParse(Parser, TEXT("Enum != EType::Value && BoolProperty"), 5, 2);
	bResult &= CanParse(Parser, TEXT("Enum == EType::Value || BoolProperty == false"), 7, 2);
	bResult &= CanParse(Parser, TEXT("Enum != EType::Value || BoolProperty == bFoo"), 7, 3);
	bResult &= CanParse(Parser, TEXT("Enum != EType::Value && GetBoolFunction"), 5, 2);
	bResult &= CanParse(Parser, TEXT("Enum == EType::Value || GetBoolFunction == false"), 7, 2);
	bResult &= CanParse(Parser, TEXT("Enum != EType::Value || GetBoolFunction == bFoo"), 7, 3);
	bResult &= CanParse(Parser, TEXT("Enum != EType::Value && PropertyEditor.EditConditionTestObject.StaticGetBoolFunction"), 5, 2);
	bResult &= CanParse(Parser, TEXT("Enum == EType::Value || PropertyEditor.EditConditionTestObject.StaticGetBoolFunction == false"), 7, 2);
	bResult &= CanParse(Parser, TEXT("Enum != EType::Value || PropertyEditor.EditConditionTestObject.StaticGetBoolFunction == bFoo"), 7, 3);
	bResult &= CanParse(Parser, TEXT("Enum == EType::Value && Foo != 5"), 7, 2);
	bResult &= CanParse(Parser, TEXT("Enum != EType::Value && Foo == Bar"), 7, 3);
	bResult &= CanParse(Parser, TEXT("PointerProperty == nullptr"), 3, 1);
	bResult &= CanParse(Parser, TEXT("PointerProperty != nullptr"), 3, 1);
	bResult &= CanParse(Parser, TEXT("Flags & EType::Value"), 3, 1);
	bResult &= CanParse(Parser, TEXT("Flags & EType::Value == false"), 5, 1);

	return bResult;
}

static bool CanEvaluate(const FEditConditionParser& Parser, const IEditConditionContext& Context, const FString& Expression, bool Expected)
{
	TSharedPtr<FEditConditionExpression> Parsed = Parser.Parse(Expression);
	if (!Parsed.IsValid())
	{
		ensureMsgf(false, TEXT("Failed to parse expression: %s"), *Expression);
		return false;
	}

	TValueOrError<bool, FText> Result = Parser.Evaluate(*Parsed.Get(), Context);
	if (!Result.IsValid())
	{
		ensureMsgf(false, TEXT("Expression failed to evaluate: %s, Error: %s"), *Expression, *Result.GetError().ToString());
		return false;
	}

	if (Result.GetValue() != Expected)
	{
		auto BoolToString = [](bool Value) { return Value ? TEXT("true") : TEXT("false"); };
		ensureMsgf(false, TEXT("Expression evaluated to unexpected value: %s, Expected: %s, Actual: %s"), *Expression, BoolToString(Expected), BoolToString(Result.GetValue()));
		return false;
	}

	return true;
}

/** For expressions we expect / wish to be considered mal-formed */
static bool CanNotEvaluate(const FEditConditionParser& Parser, const IEditConditionContext& Context, const FString& Expression)
{
	TSharedPtr<FEditConditionExpression> Parsed = Parser.Parse(Expression);
	if (!Parsed.IsValid())
	{
		ensureMsgf(false, TEXT("Failed to parse expression: %s"), *Expression);
		return false;
	}

	TValueOrError<bool, FText> Result = Parser.Evaluate(*Parsed.Get(), Context);
	if (Result.IsValid())
	{
		ensureMsgf(false, TEXT("Error. Expected expression to fail evaluation: %s"), *Expression);
		return false;
	}

	return true;
}

static bool RunVoidTests(const IEditConditionContext& Context)
{
	FEditConditionParser Parser;
	bool bResult = true;

	bResult &= CanNotEvaluate(Parser, Context, TEXT("VoidFunction"));
	bResult &= CanNotEvaluate(Parser, Context, TEXT("!VoidFunction"));

	bResult &= CanNotEvaluate(Parser, Context, TEXT("VoidFunction == true"));
	bResult &= CanNotEvaluate(Parser, Context, TEXT("VoidFunction == false"));

	bResult &= CanNotEvaluate(Parser, Context, TEXT("VoidFunction == VoidFunction"));
	bResult &= CanNotEvaluate(Parser, Context, TEXT("VoidFunction != VoidFunction"));

	bResult &= CanNotEvaluate(Parser, Context, TEXT("VoidFunction != true"));
	bResult &= CanNotEvaluate(Parser, Context, TEXT("VoidFunction != false"));

	bResult &= CanNotEvaluate(Parser, Context, TEXT("PropertyEditor.EditConditionTestObject.StaticVoidFunction"));
	bResult &= CanNotEvaluate(Parser, Context, TEXT("!PropertyEditor.EditConditionTestObject.StaticVoidFunction"));

	bResult &= CanNotEvaluate(Parser, Context, TEXT("PropertyEditor.EditConditionTestObject.StaticVoidFunction == true"));
	bResult &= CanNotEvaluate(Parser, Context, TEXT("PropertyEditor.EditConditionTestObject.StaticVoidFunction == false"));

	bResult &= CanNotEvaluate(Parser, Context, TEXT("PropertyEditor.EditConditionTestObject.StaticVoidFunction == PropertyEditor.EditConditionTestObject.StaticVoidFunction"));
	bResult &= CanNotEvaluate(Parser, Context, TEXT("PropertyEditor.EditConditionTestObject.StaticVoidFunction != PropertyEditor.EditConditionTestObject.StaticVoidFunction"));

	bResult &= CanNotEvaluate(Parser, Context, TEXT("PropertyEditor.EditConditionTestObject.StaticVoidFunction != true"));
	bResult &= CanNotEvaluate(Parser, Context, TEXT("PropertyEditor.EditConditionTestObject.StaticVoidFunction != false"));

	return bResult;
}

static bool RunBoolTests(const IEditConditionContext& Context)
{
	FEditConditionParser Parser;
	bool bResult = true;

	bResult &= CanEvaluate(Parser, Context, TEXT("true"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("TRUE"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("false"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("False"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("fAlSe"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("!true"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("!false"), true);

	bResult &= CanEvaluate(Parser, Context, TEXT("BoolProperty"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("!BoolProperty"), false);
	
	bResult &= CanEvaluate(Parser, Context, TEXT("BoolProperty == true"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("BoolProperty == false"), false);

	bResult &= CanEvaluate(Parser, Context, TEXT("BoolProperty == BoolProperty"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("BoolProperty != BoolProperty"), false);

	bResult &= CanEvaluate(Parser, Context, TEXT("BoolProperty != true"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("BoolProperty != false"), true);

	bResult &= CanEvaluate(Parser, Context, TEXT("GetBoolFunction"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("!GetBoolFunction"), false);

	bResult &= CanEvaluate(Parser, Context, TEXT("GetBoolFunction == true"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("GetBoolFunction == false"), false);

	bResult &= CanEvaluate(Parser, Context, TEXT("GetBoolFunction == GetBoolFunction"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("GetBoolFunction != GetBoolFunction"), false);

	bResult &= CanEvaluate(Parser, Context, TEXT("GetBoolFunction != true"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("GetBoolFunction != false"), true);

	bResult &= CanEvaluate(Parser, Context, TEXT("PropertyEditor.EditConditionTestObject.StaticGetBoolFunction"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("!PropertyEditor.EditConditionTestObject.StaticGetBoolFunction"), true);

	bResult &= CanEvaluate(Parser, Context, TEXT("PropertyEditor.EditConditionTestObject.StaticGetBoolFunction == true"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("PropertyEditor.EditConditionTestObject.StaticGetBoolFunction == false"), true);

	bResult &= CanEvaluate(Parser, Context, TEXT("PropertyEditor.EditConditionTestObject.StaticGetBoolFunction == PropertyEditor.EditConditionTestObject.StaticGetBoolFunction"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("PropertyEditor.EditConditionTestObject.StaticGetBoolFunction != PropertyEditor.EditConditionTestObject.StaticGetBoolFunction"), false);

	bResult &= CanEvaluate(Parser, Context, TEXT("PropertyEditor.EditConditionTestObject.StaticGetBoolFunction != true"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("PropertyEditor.EditConditionTestObject.StaticGetBoolFunction != false"), false);
	
	bResult &= CanEvaluate(Parser, Context, TEXT("true && true"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("true && false"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("false && true"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("false && false"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("true && true && true"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("true && true && false"), false);

	bResult &= CanEvaluate(Parser, Context, TEXT("BoolProperty && BoolProperty"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("BoolProperty && false"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("false && BoolProperty"), false);

	bResult &= CanEvaluate(Parser, Context, TEXT("GetBoolFunction && GetBoolFunction"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("GetBoolFunction && false"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("false && GetBoolFunction"), false);

	bResult &= CanEvaluate(Parser, Context, TEXT("PropertyEditor.EditConditionTestObject.StaticGetBoolFunction && PropertyEditor.EditConditionTestObject.StaticGetBoolFunction"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("PropertyEditor.EditConditionTestObject.StaticGetBoolFunction && false"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("false && PropertyEditor.EditConditionTestObject.StaticGetBoolFunction"), false);
	
	bResult &= CanEvaluate(Parser, Context, TEXT("true || true"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("true || false"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("false || true"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("false || false"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("true || true || true"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("true || true || false"), true);

	bResult &= CanEvaluate(Parser, Context, TEXT("BoolProperty || BoolProperty"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("BoolProperty || false"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("false || BoolProperty"), true);

	bResult &= CanEvaluate(Parser, Context, TEXT("GetBoolFunction || GetBoolFunction"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("GetBoolFunction || false"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("false || GetBoolFunction"), true);

	bResult &= CanEvaluate(Parser, Context, TEXT("PropertyEditor.EditConditionTestObject.StaticGetBoolFunction || PropertyEditor.EditConditionTestObject.StaticGetBoolFunction"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("PropertyEditor.EditConditionTestObject.StaticGetBoolFunction || false"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("false || PropertyEditor.EditConditionTestObject.StaticGetBoolFunction"), false);

	return bResult;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditConditionParser_EvaluateVoidFails, "EditConditionParser.EvaluateVoidFails", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FEditConditionParser_EvaluateVoidFails::RunTest(const FString& Parameters)
{
	// We only expect errors first time, since the API only logs first time
	static bool bExpectErrors = true;

	if (bExpectErrors)
	{
		AddExpectedError(TEXT("EditCondition attempted to use an invalid operand"), EAutomationExpectedErrorFlags::Contains, 2);
		bExpectErrors = false;
	}

	FTestEditConditionContext TestContext;

	return RunVoidTests(TestContext);
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditConditionParser_EvaluateBool, "EditConditionParser.EvaluateBool", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FEditConditionParser_EvaluateBool::RunTest(const FString& Parameters)
{
	FTestEditConditionContext TestContext;
	TestContext.SetupBool(TEXT("BoolProperty"), true);
	TestContext.SetupBool(TEXT("GetBoolFunction"), true);
	TestContext.SetupBool(TEXT("PropertyEditor.EditConditionTestObject.StaticGetBoolFunction"), false);

	return RunBoolTests(TestContext);
}

static bool RunNumericTests(const IEditConditionContext& Context)
{
	FEditConditionParser Parser;
	bool bResult = true;

	bResult &= CanEvaluate(Parser, Context, TEXT("5 == 5"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("5.0 == 5.0"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty == 5.0"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty == 5"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty == DoubleProperty"), true);
	
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty != 5.0"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty != 6.0"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty != 6"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty != DoubleProperty"), false);

	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty > 4.5"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty > 5"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty > 6"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty > DoubleProperty"), false);
	
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty < 4.5"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty < 5"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty < 6"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty < DoubleProperty"), false);
	
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty >= 4.5"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty >= 5"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty >= 6"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty >= DoubleProperty"), true);
	
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty <= 4.5"), false);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty <= 5"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty <= 6"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty <= DoubleProperty"), true);
	
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty == 2 + 3"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty == 6 - 1"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty == 2.5 * 2"), true);
	bResult &= CanEvaluate(Parser, Context, TEXT("DoubleProperty == 10 / 2"), true);

	return bResult;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditConditionParser_EvaluateDouble, "EditConditionParser.EvaluateDouble", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FEditConditionParser_EvaluateDouble::RunTest(const FString& Parameters)
{
	FTestEditConditionContext TestContext;
	TestContext.SetupDouble(TEXT("DoubleProperty"), 5.0);

	return RunNumericTests(TestContext);
}

static bool RunEnumTests(const IEditConditionContext& Context, const FString& EnumName, const FString& PropertyName)
{
	FEditConditionParser Parser;
	bool bResult = true;

	bResult &= CanEvaluate(Parser, Context, EnumName + TEXT("::First == ") + EnumName + TEXT("::First"), true);
	bResult &= CanEvaluate(Parser, Context, EnumName + TEXT("::First == ") + EnumName + TEXT("::Second"), false);

	bResult &= CanEvaluate(Parser, Context, EnumName + TEXT("::First != ") + EnumName + TEXT("::First"), false);
	bResult &= CanEvaluate(Parser, Context, EnumName + TEXT("::First != ") + EnumName + TEXT("::Second"), true);

	bResult &= CanEvaluate(Parser, Context, PropertyName + TEXT(" == ") + PropertyName, true);
	bResult &= CanEvaluate(Parser, Context, PropertyName + TEXT(" != ") + PropertyName, false);

	bResult &= CanEvaluate(Parser, Context, PropertyName + TEXT(" == ") + EnumName + TEXT("::First"), true);
	bResult &= CanEvaluate(Parser, Context, EnumName + TEXT("::First == ") + PropertyName, true);
	bResult &= CanEvaluate(Parser, Context, PropertyName + TEXT(" == ") + EnumName + TEXT("::Second"), false);

	bResult &= CanEvaluate(Parser, Context, PropertyName + TEXT(" != ") + EnumName + TEXT("::Second"), true);
	bResult &= CanEvaluate(Parser, Context, PropertyName + TEXT(" != ") + EnumName + TEXT("::First"), false);
	bResult &= CanEvaluate(Parser, Context, EnumName + TEXT("::Second != ") + PropertyName, true);

	return bResult;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditConditionParser_EvaluateEnum, "EditConditionParser.EvaluateEnum", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FEditConditionParser_EvaluateEnum::RunTest(const FString& Parameters)
{
	FTestEditConditionContext TestContext;

	const FString EnumType = TEXT("EditConditionTestEnum");
	TestContext.SetupEnumType(EnumType);

	const FString PropertyName = TEXT("EnumProperty");
	TestContext.SetupEnum(PropertyName, TEXT("First"));

	return RunEnumTests(TestContext, EnumType, PropertyName);
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditConditionParser_EvaluateBitFlags, "EditConditionParser.EvaluateBitFlags", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FEditConditionParser_EvaluateBitFlags::RunTest(const FString& Parameters)
{
	FTestEditConditionContext TestContext;

	const FString EnumType = TEXT("TestEnum");
	TestContext.SetupEnumType(EnumType);
	TestContext.SetupEnumTypeValue(TEXT("Nil"), 0);
	TestContext.SetupEnumTypeValue(TEXT("One"), 1 << 0);
	TestContext.SetupEnumTypeValue(TEXT("Two"), 1 << 1);
	TestContext.SetupEnumTypeValue(TEXT("Four"), 1 << 2);

	const FString PropertyName = TEXT("FlagsProperty");

	FEditConditionParser Parser;
	bool bResult = true;

	TestContext.SetupInteger(PropertyName, 0);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Nil"), false);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::One"), false);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Two"), false);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Four"), false);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::One == false"), true);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Two == false"), true);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Four == false"), true);

	TestContext.SetupInteger(PropertyName, 1);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Nil"), false);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::One"), true);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Two"), false);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Four"), false);

	TestContext.SetupInteger(PropertyName, 2);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Nil"), false);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::One"), false);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Two"), true);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Four"), false);

	TestContext.SetupInteger(PropertyName, 3);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Nil"), false);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::One"), true);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Two"), true);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Four"), false);

	TestContext.SetupInteger(PropertyName, 5);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Nil"), false);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::One"), true);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Two"), false);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Four"), true);

	TestContext.SetupInteger(PropertyName, 7);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Nil"), false);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::One"), true);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Two"), true);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Four"), true);

	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::One == false"), false);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Two == false"), false);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Four == false"), false);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::One == true"), true);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Two == true"), true);
	bResult &= CanEvaluate(Parser, TestContext, TEXT("FlagsProperty & TestEnum::Four == true"), true);

	return bResult;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditConditionParser_EvaluateUObject, "EditConditionParser.EvaluateUObject", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FEditConditionParser_EvaluateUObject::RunTest(const FString& Parameters)
{
	UEditConditionTestObject* TestObject = NewObject<UEditConditionTestObject>();
	TestObject->AddToRoot();

	TSharedPtr<FObjectPropertyNode> ObjectNode(new FObjectPropertyNode);
	ObjectNode->AddObject(TestObject);

	FPropertyNodeInitParams InitParams;
	ObjectNode->InitNode(InitParams);

	TOptional<bool> bResult;
	bool bAllResults = true;

	// enum comparison
	{
		static const FName EnumPropertyName = TEXT("EnumProperty");
		TSharedPtr<FPropertyNode> PropertyNode = ObjectNode->FindChildPropertyNode(EnumPropertyName, true);
		FEditConditionContext Context(*PropertyNode.Get());

		TestObject->EnumProperty = EditConditionTestEnum::First;
		TestObject->ByteEnumProperty = EditConditionByteEnum::First;

		static const FString EnumType = TEXT("EditConditionTestEnum");
		bAllResults &= RunEnumTests(Context, EnumType, EnumPropertyName.ToString());

		static const FString ByteEnumType = TEXT("EditConditionByteEnum");
		static const FString ByteEnumPropertyName = TEXT("ByteEnumProperty");
		bAllResults &= RunEnumTests(Context, ByteEnumType, ByteEnumPropertyName);
	}

	// void expression failure
	{
		// We only expect errors first time, since the API only logs first time
		static bool bExpectErrors = true;

		if (bExpectErrors)
		{
			AddExpectedError(TEXT("EditCondition parsing failed"), EAutomationExpectedErrorFlags::Contains, 1);
			AddExpectedError(TEXT("EditCondition attempted to use an invalid operand"), EAutomationExpectedErrorFlags::Contains, 2);
			bExpectErrors = false;
		}
		
		// Using bool property for context since doesn't matter as long as parent is same (due to evaluation failing)
		static const FName BoolPropertyName(TEXT("BoolProperty"));
		TSharedPtr<FPropertyNode> PropertyNode = ObjectNode->FindChildPropertyNode(BoolPropertyName, true);
		FEditConditionContext Context(*PropertyNode.Get());

		bAllResults &= RunVoidTests(Context);
	}

	// bool comparison
	{
		static const FName BoolPropertyName(TEXT("BoolProperty"));
		TSharedPtr<FPropertyNode> PropertyNode = ObjectNode->FindChildPropertyNode(BoolPropertyName, true);
		FEditConditionContext Context(*PropertyNode.Get());

		TestObject->BoolProperty = true;

		bAllResults &= RunBoolTests(Context);
	}

	// double comparison
	{
		static const FName DoublePropertyName(TEXT("DoubleProperty"));
		TSharedPtr<FPropertyNode> PropertyNode = ObjectNode->FindChildPropertyNode(DoublePropertyName, true);
		FEditConditionContext Context(*PropertyNode.Get());

		TestObject->DoubleProperty = 5.0;

		bAllResults &= RunNumericTests(Context);
	}

	// integer comparison
	{
		static const FName IntegerPropertyName(TEXT("IntegerProperty"));
		TSharedPtr<FPropertyNode> PropertyNode = ObjectNode->FindChildPropertyNode(IntegerPropertyName, true);
		FEditConditionContext Context(*PropertyNode.Get());

		TestObject->IntegerProperty = 5;

		bAllResults &= RunNumericTests(Context);
	}

	{
		static const FName DoublePropertyName(TEXT("DoubleProperty"));
		TSharedPtr<FPropertyNode> PropertyNode = ObjectNode->FindChildPropertyNode(DoublePropertyName, true);
		FEditConditionContext Context(*PropertyNode.Get());

		TestEqual(TEXT("Boolean Type Name"), Context.GetTypeName(TEXT("BoolProperty")).GetValue(), TEXT("bool"));
		TestEqual(TEXT("Enum Type Name"), Context.GetTypeName(TEXT("EnumProperty")).GetValue(), TEXT("EditConditionTestEnum"));
		TestEqual(TEXT("Byte Enum Type Name"), Context.GetTypeName(TEXT("ByteEnumProperty")).GetValue(), TEXT("EditConditionByteEnum"));
		TestEqual(TEXT("Double Type Name"), Context.GetTypeName(TEXT("DoubleProperty")).GetValue(), TEXT("double"));
	}

	TestObject->RemoveFromRoot();

	return bAllResults;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditConditionParser_EvaluatePointers, "EditConditionParser.EvaluatePointers", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FEditConditionParser_EvaluatePointers::RunTest(const FString& Parameters)
{
	UEditConditionTestObject* TestObject = NewObject<UEditConditionTestObject>();
	TestObject->AddToRoot();

	TSharedPtr<FObjectPropertyNode> ObjectNode(new FObjectPropertyNode);
	ObjectNode->AddObject(TestObject);

	FPropertyNodeInitParams InitParams;
	ObjectNode->InitNode(InitParams);

	static const FName BoolPropertyName(TEXT("BoolProperty"));
	TSharedPtr<FPropertyNode> PropertyNode = ObjectNode->FindChildPropertyNode(BoolPropertyName, true);
	FEditConditionContext Context(*PropertyNode.Get());

	bool bAllResults = true;

	FEditConditionParser Parser;

	{
		TestObject->UObjectPtr = nullptr;

		bAllResults &= CanEvaluate(Parser, Context, TEXT("UObjectPtr != nullptr"), false);
		bAllResults &= CanEvaluate(Parser, Context, TEXT("UObjectPtr == nullptr"), true);

		TestObject->UObjectPtr = TestObject;

		bAllResults &= CanEvaluate(Parser, Context, TEXT("UObjectPtr != nullptr"), true);
		bAllResults &= CanEvaluate(Parser, Context, TEXT("UObjectPtr == nullptr"), false);
	}

	{
		TestObject->SoftClassPtr = nullptr;

		bAllResults &= CanEvaluate(Parser, Context, TEXT("SoftClassPtr != nullptr"), false);
		bAllResults &= CanEvaluate(Parser, Context, TEXT("SoftClassPtr == nullptr"), true);

		TestObject->SoftClassPtr = TestObject;

		bAllResults &= CanEvaluate(Parser, Context, TEXT("SoftClassPtr != nullptr"), true);
		bAllResults &= CanEvaluate(Parser, Context, TEXT("SoftClassPtr == nullptr"), false);
	}

	{
		TestObject->WeakObjectPtr = nullptr;

		bAllResults &= CanEvaluate(Parser, Context, TEXT("WeakObjectPtr != nullptr"), false);
		bAllResults &= CanEvaluate(Parser, Context, TEXT("WeakObjectPtr == nullptr"), true);

		TestObject->WeakObjectPtr = TestObject;

		bAllResults &= CanEvaluate(Parser, Context, TEXT("WeakObjectPtr != nullptr"), true);
		bAllResults &= CanEvaluate(Parser, Context, TEXT("WeakObjectPtr == nullptr"), false);
	}

	{
		// equality & inequality
		bAllResults &= CanEvaluate(Parser, Context, TEXT("UObjectPtr == SoftClassPtr"), true);
		bAllResults &= CanEvaluate(Parser, Context, TEXT("SoftClassPtr == UObjectPtr"), true);

		bAllResults &= CanEvaluate(Parser, Context, TEXT("WeakObjectPtr != UObjectPtr"), false);
		bAllResults &= CanEvaluate(Parser, Context, TEXT("UObjectPtr != WeakObjectPtr"), false);

		bAllResults &= CanEvaluate(Parser, Context, TEXT("SoftClassPtr == WeakObjectPtr"), true);
		bAllResults &= CanEvaluate(Parser, Context, TEXT("WeakObjectPtr == UObjectPtr"), true);

		TestObject->UObjectPtr = nullptr;

		bAllResults &= CanEvaluate(Parser, Context, TEXT("UObjectPtr != SoftClassPtr"), true);
		bAllResults &= CanEvaluate(Parser, Context, TEXT("WeakObjectPtr != UObjectPtr"), true);
		bAllResults &= CanEvaluate(Parser, Context, TEXT("WeakObjectPtr == UObjectPtr"), false);
	}

	TestObject->RemoveFromRoot();

	return bAllResults;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditConditionParser_Grouping, "EditConditionParser.Grouping", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FEditConditionParser_Grouping::RunTest(const FString& Parameters)
{
	UEditConditionTestObject* TestObject = NewObject<UEditConditionTestObject>();
	TestObject->AddToRoot();

	TSharedPtr<FObjectPropertyNode> ObjectNode(new FObjectPropertyNode);
	ObjectNode->AddObject(TestObject);

	FPropertyNodeInitParams InitParams;
	ObjectNode->InitNode(InitParams);

	static const FName BoolPropertyName(TEXT("BoolProperty"));
	TSharedPtr<FPropertyNode> PropertyNode = ObjectNode->FindChildPropertyNode(BoolPropertyName, true);
	FEditConditionContext Context(*PropertyNode.Get());

	bool bAllResults = true;

	FEditConditionParser Parser;

	bAllResults &= CanEvaluate(Parser, Context, TEXT("(true)"), true);
	bAllResults &= CanEvaluate(Parser, Context, TEXT("(true) == true"), true);
	bAllResults &= CanEvaluate(Parser, Context, TEXT("(true) == false"), false);
	bAllResults &= CanEvaluate(Parser, Context, TEXT("(true) == (true)"), true);
	bAllResults &= CanEvaluate(Parser, Context, TEXT("((true == false) == false) == true"), true);

	TestObject->DoubleProperty = 5.0;
	bAllResults &= CanEvaluate(Parser, Context, TEXT("(DoubleProperty == 5.0)"), true);
	bAllResults &= CanEvaluate(Parser, Context, TEXT("(DoubleProperty == 5.0) == false"), false);
	bAllResults &= CanEvaluate(Parser, Context, TEXT("(DoubleProperty == 5.0) == (true == true)"), true);

	TestObject->EnumProperty = EditConditionTestEnum::First;
	bAllResults &= CanEvaluate(Parser, Context, TEXT("(EnumProperty == EditConditionTestEnum::First) == true"), true);
	bAllResults &= CanEvaluate(Parser, Context, TEXT("(EnumProperty == EditConditionTestEnum::First) && (DoubleProperty == 5.0)"), true);
	bAllResults &= CanEvaluate(Parser, Context, TEXT("(EnumProperty == EditConditionTestEnum::Second) && (DoubleProperty == 5.0)"), false);
	bAllResults &= CanEvaluate(Parser, Context, TEXT("(EnumProperty == EditConditionTestEnum::Second) || (DoubleProperty == 5.0)"), true);

	{
		FTestEditConditionContext TestContext;

		const FString EnumType = TEXT("TestEnum");
		TestContext.SetupEnumType(EnumType);
		TestContext.SetupEnumTypeValue(TEXT("Nil"), 0);
		TestContext.SetupEnumTypeValue(TEXT("One"), 1 << 0);

		TestContext.SetupInteger("FlagsProperty", 1);
		TestContext.SetupDouble("DoubleProperty", 5.0);

		bAllResults &= CanEvaluate(Parser, TestContext, TEXT("(FlagsProperty & TestEnum::Nil)"), false); 
		bAllResults &= CanEvaluate(Parser, TestContext, TEXT("(FlagsProperty & TestEnum::One)"), true);
		bAllResults &= CanEvaluate(Parser, TestContext, TEXT("(FlagsProperty & TestEnum::One) && (DoubleProperty == 5.0)"), true);
		bAllResults &= CanEvaluate(Parser, TestContext, TEXT("(FlagsProperty & TestEnum::One) == false"), false);
		bAllResults &= CanEvaluate(Parser, TestContext, TEXT("!(FlagsProperty & TestEnum::One)"), false);
		bAllResults &= CanEvaluate(Parser, TestContext, TEXT("!(FlagsProperty & TestEnum::Nil)"), true);
	}

	TestObject->RemoveFromRoot();

	return bAllResults;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditConditionParser_SingleBool, "EditConditionParser.SingleBool", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FEditConditionParser_SingleBool::RunTest(const FString& Parameters)
{
	UEditConditionTestObject* TestObject = NewObject<UEditConditionTestObject>();
	TestObject->AddToRoot();

	TSharedPtr<FObjectPropertyNode> ObjectNode(new FObjectPropertyNode);
	ObjectNode->AddObject(TestObject);

	FPropertyNodeInitParams InitParams;
	ObjectNode->InitNode(InitParams);

	static const FName BoolPropertyName(TEXT("BoolProperty"));
	TSharedPtr<FPropertyNode> PropertyNode = ObjectNode->FindChildPropertyNode(BoolPropertyName, true);
	FEditConditionContext Context(*PropertyNode.Get());

	FEditConditionParser Parser;

	{
		TSharedPtr<FEditConditionExpression> Expression = Parser.Parse(FString(TEXT("BoolProperty")));
		const FBoolProperty* Property = Context.GetSingleBoolProperty(Expression);
		TestNotNull(TEXT("Bool"), Property);
	}

	{
		TSharedPtr<FEditConditionExpression> Expression = Parser.Parse(FString(TEXT("UintBitfieldProperty")));
		const FBoolProperty* Property = Context.GetSingleBoolProperty(Expression);
		TestNotNull(TEXT("Uint Bitfield"), Property);
	}

	TestObject->RemoveFromRoot();

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
