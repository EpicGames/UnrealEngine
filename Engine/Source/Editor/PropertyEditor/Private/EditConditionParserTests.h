// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/EnumAsByte.h"
#include "HAL/Platform.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "EditConditionParserTests.generated.h"

UENUM()
enum class EditConditionTestEnum
{
	First = 15,
	Second = 31
};

UENUM()
enum EditConditionByteEnum : int
{
	First = 15,
	Second = 31
};


/**
 * Test object for edit condition property checks
 * 
 * Note: Currently only bool functions are supported (Including static). 
 * remaining are not used because currently not supported by edit condition parser
 */
UCLASS(transient)
class UEditConditionTestObject : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Test)
	bool BoolProperty;

	UPROPERTY(EditAnywhere, Category=Test)
	EditConditionTestEnum EnumProperty;

	UPROPERTY(EditAnywhere, Category=Test)
	TEnumAsByte<EditConditionByteEnum> ByteEnumProperty;

	UPROPERTY(EditAnywhere, Category=Test)
	double DoubleProperty;

	UPROPERTY(EditAnywhere, Category=Test)
	int32 IntegerProperty;

	UPROPERTY(EditAnywhere, Category=Test)
	uint8 UintBitfieldProperty : 1;

	UPROPERTY(EditAnywhere, Category=Test)
	TObjectPtr<UObject> UObjectPtr;

	UPROPERTY(EditAnywhere, Category=Test)
	TSoftClassPtr<UObject> SoftClassPtr;

	UPROPERTY(EditAnywhere, Category=Test)
	TWeakObjectPtr<UObject> WeakObjectPtr;

public:
	/** Used in test cases that should fail, should not be able to execute a void function in edit condition */
	UFUNCTION()
	void VoidFunction() const;

	UFUNCTION()
	bool GetBoolFunction() const;

	UFUNCTION()
	EditConditionTestEnum GetEnumFunction() const;

	UFUNCTION()
	TEnumAsByte<EditConditionByteEnum> GetByteEnumFunction() const;

	UFUNCTION()
	double GetDoubleFunction() const;

	UFUNCTION()
	int32 GetIntegerFunction() const;

	UFUNCTION()
	uint8 GetUintBitfieldFunction() const;

	UFUNCTION()
	UObject* GetUObjectPtrFunction() const;

	UFUNCTION()
	TSoftClassPtr<UObject> GetSoftClassPtrFunction() const;

	UFUNCTION()
	TWeakObjectPtr<UObject> GetWeakObjectPtrFunction() const;
	
public:
	/** Used in test cases that should fail, should not be able to execute a void function in edit condition */
	UFUNCTION()
	void StaticVoidFunction();

	UFUNCTION()
	static bool StaticGetBoolFunction();

	UFUNCTION()
	static EditConditionTestEnum StaticGetEnumFunction();

	UFUNCTION()
	static TEnumAsByte<EditConditionByteEnum> StaticGetByteEnumFunction();

	UFUNCTION()
	static double StaticGetDoubleFunction();

	UFUNCTION()
	static int32 StaticGetIntegerFunction();

	UFUNCTION()
	static uint8 StaticGetUintBitfieldFunction();

	UFUNCTION()
	static UObject* StaticGetUObjectPtrFunction();

	UFUNCTION()
	static TSoftClassPtr<UObject> StaticGetSoftClassPtrFunction();

	UFUNCTION()
	static TWeakObjectPtr<UObject> StaticGetWeakObjectPtrFunction();
};
