// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetThumbnail.h"
#include "DetailsDisplayManager.h"
#include "DetailTreeNode.h"
#include "DetailsViewArgs.h"
#include "IDetailsView.h"
#include "PropertyNode.h"

class FDetailsNameWidgetOverrideCustomization;
class FEditConditionParser;
class FNotifyHook;
class IDetailPropertyExtensionHandler;
class IDetailRootObjectCustomization;
class FStringPrefixTree;

class IDetailsViewPrivate : public IDetailsView
{
public:
	/**
	 * Sets the expansion state for a node and optionally all of its children
	 *
	 * @param InTreeNode		The node to change expansion state on
	 * @param bIsItemExpanded	The new expansion state
	 * @param bRecursive		Whether or not to apply the expansion change to any children
	 */
	virtual void SetNodeExpansionState(TSharedRef<FDetailTreeNode> InTreeNode, bool bIsItemExpanded, bool bRecursive) = 0;

	/**
	 * Requests that an item in the tree be expanded or collapsed
	 *
	 * @param TreeNode	The tree node to expand
	 * @param bExpand	true if the item should be expanded, false otherwise
	 */
	virtual void RequestItemExpanded(TSharedRef<FDetailTreeNode> TreeNode, bool bExpand) = 0;

	/**
	 * Refreshes the detail's treeview
	 */
	virtual void RefreshTree() = 0;

	/** Returns the notify hook to use when properties change */
	virtual FNotifyHook* GetNotifyHook() const = 0;

	/**
	 * Returns the property utilities for this view
	 */
	virtual TSharedPtr<IPropertyUtilities> GetPropertyUtilities() = 0;

	/** Causes the details view to move the scroll offset (by item)
	 * @param DeltaOffset	We add this value to the current scroll offset if the result is in the scrolling range
	 */
	virtual void MoveScrollOffset(int32 DeltaOffset) = 0;

	/**
	 * Saves the expansion state of a tree node
	 *
	 * @param NodePath	The path to the detail node to save
	 * @param bIsExpanded	true if the node is expanded, false otherwise
	 */
	virtual void SaveCustomExpansionState(const FString& NodePath, bool bIsExpanded) = 0;

	/**
	 * Gets the saved expansion state of a tree node in this category
	 *
	 * @param NodePath	The path to the detail node to get
	 * @return true if the node should be expanded, false otherwise
	 */
	virtual bool GetCustomSavedExpansionState(const FString& NodePath) const = 0;

	/**
	 * Gets the View Identifier for this details view. If any.
	 *
	 * @return Text set for view identifier for this details view on creation
	 */
	virtual FName GetViewIdentifier() const = 0;

	/**
	 * Gets expansion persistence method for this details view. See FDetailsViewArgs.
	 *
	 * @return Text set for view identifier for this details view on creation
	 */
	virtual FDetailsViewArgs::EExpansionPersistenceMethod GetExpansionPersistenceMethod() const = 0;

	/**
	 * @return True if the property is visible
	 */
	virtual bool IsPropertyVisible( const struct FPropertyAndParent& PropertyAndParent ) const = 0;

	/**
	 * @return True if the property is visible
	 */
	virtual bool IsPropertyReadOnly( const struct FPropertyAndParent& PropertyAndParent ) const = 0;

	/**
	 * @return Whether a custom row with the specified name and parent name is visible.
	 */
	virtual bool IsCustomRowVisible(FName InRowName, FName InParentName) const = 0;

	/**
	 * @return Whether a custom row with the specified name and parent name is read-only.
	 */
	virtual bool IsCustomRowReadOnly(FName InRowName, FName InParentName) const = 0;

	/**
	 * @return The thumbnail pool that should be used for thumbnails being rendered in this view
	 */
	virtual TSharedPtr<class FAssetThumbnailPool> GetThumbnailPool() const = 0;

	/**
	 * @return The set of custom class viewer filters to use for class properties in this view
	 */
	virtual const TArray<TSharedRef<class IClassViewerFilter>>& GetClassViewerFilters() const = 0;

	/**
	 * Creates the color picker window for this property view.
	 *
	 * @param PropertyEditor				The slate property node to edit.
	 * @param bUseAlpha			Whether or not alpha is supported
	 */
	virtual void CreateColorPickerWindow(const TSharedRef< class FPropertyEditor >& PropertyEditor, bool bUseAlpha) = 0;

	/**
	 * Adds an action to execute next tick
	 */
	virtual void EnqueueDeferredAction(FSimpleDelegate& DeferredAction) = 0;

	/**
	 * Called when properties have finished changing (after PostEditChange is called)
	 */
	virtual void NotifyFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) = 0;

	/**
	 * Reruns the current search filter applied to the details panel to account for any new changes                                                              
	 */
	virtual void RerunCurrentFilter() = 0;

	/** If a customization standalone widget is used, the value should be update only once, when its window is closed */
	virtual bool DontUpdateValueWhileEditing() const = 0;

	/** @return Whether or not the details panel was created with multiple unrelated objects visible at once in the details panel */
	virtual bool ContainsMultipleTopLevelObjects() const = 0;

	/** @return the customization instance that defines how the display for a root object looks */
	virtual TSharedPtr<class IDetailRootObjectCustomization> GetRootObjectCustomization() const = 0;

	/** Runs the details customization update on a root property node */
	virtual void UpdateSinglePropertyMap(TSharedPtr<FComplexPropertyNode> InRootPropertyNode, struct FDetailLayoutData& LayoutData, bool bIsExternal) = 0;

	virtual const FCustomPropertyTypeLayoutMap& GetCustomPropertyTypeLayoutMap() const = 0;

	/** Saves the expansion state of property nodes for the selected object set */
	virtual void SaveExpandedItems(TSharedRef<FPropertyNode> StartNode) = 0;

	/**
	* Restores the expansion state of property nodes for the selected object set
	*/
	virtual void RestoreExpandedItems(TSharedRef<FPropertyNode> StartNode) = 0;

	/**
	* Restores the expansion state of property nodes for the selected object set, with optionally provided external expansion states.
	*/
	virtual void RestoreExpandedItems(TSharedRef<FPropertyNode> StartNode, TMap<UStruct*, FStringPrefixTree>* OptionalExpansionStates) = 0;

	/**
	* Mark node as animating, useful if animating during behaviors that trigger widget reconstruction.
	* @param InNode				The slate property node to animate.
	* @param InAnimationDuration	The animation duration in seconds.
	* @param InAnimationBatchId	(Optional) A batch id to enable simultaneous (and performant) animation of multiple rows.
	*/
	virtual void MarkNodeAnimating(TSharedPtr<FPropertyNode> InNode, float InAnimationDuration, TOptional<FGuid> InAnimationBatchId = {}) = 0;

	/** @return true if node is animating */
	virtual bool IsNodeAnimating(TSharedPtr<FPropertyNode> InNode) = 0;

	/** @return the time/progress of the given node (time since creation/play), if it's animating. The time is in seconds. */
	virtual float GetNodeAnimationTime(TSharedPtr<FPropertyNode> InNode) = 0;

	/** Column width accessibility */
	virtual class FDetailColumnSizeData& GetColumnSizeData() = 0;

	/** Does this details view allow favoriting? */
	virtual bool IsFavoritingEnabled() const = 0;

	/** Is the given group a favorite? */
	virtual bool IsGroupFavorite(FStringView GroupPath) const = 0;

	/** Set the given group's favorite status. */
	virtual void SetGroupFavorite(FStringView GroupPath, bool IsFavorite) = 0;

	/** Is the given custom builder a favorite? */
	virtual bool IsCustomBuilderFavorite(FStringView Path) const = 0;

	/** Set the given group's favorite status. */
	virtual void SetCustomBuilderFavorite(FStringView GroupPath, bool IsFavorite) = 0;
	
	/** Retrieve a list of top-most detail tree nodes. */
	virtual void GetHeadNodes(TArray<TWeakPtr<FDetailTreeNode>>& OutNodes) {}


	/**
	* Gets the @code FDetailsViewStyleKey& @endcode which provides a Key to the current style for a Details View
	*/
	virtual const FDetailsViewStyleKey& GetStyleKey() override
	{
		return FDetailsViewStyleKeys::Default(); 
	}
	
	/**
	* Updates @code FDetailsViewStyleKey& StyleKey @endcode for the current @code IDetailsViewPrivate @endcode state
	*/
	virtual void UpdateStyleKey() override
	{
	}
	
	/**
	* Returns a bool indicating whether the given @code FDetailsViewStyleKey @endcode is the default Details View Style 
	*/
	virtual bool IsDefaultStyle() const override
	{
		return true;
	}

	/**
	 * Returns a @code TSharedPtr @endcode to the @code FDetailsDisplayManager @endcode for this
	 * details view
	 */
	virtual TSharedPtr<FDetailsDisplayManager> GetDisplayManager() = 0;

	virtual TSharedPtr<FDetailsNameWidgetOverrideCustomization> GetDetailsNameWidgetOverrideCustomization()
	{
		return nullptr;
	}
};
