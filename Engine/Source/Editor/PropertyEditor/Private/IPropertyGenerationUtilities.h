// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyEditorDelegates.h"

/** Defines internal utilities for generating property layouts. */
class IPropertyGenerationUtilities
{
public:
	virtual ~IPropertyGenerationUtilities() = default;

	/** Gets the instance type customization map for the details implementation providing this utilities object. */
	virtual const FCustomPropertyTypeLayoutMap& GetInstancedPropertyTypeLayoutMap() const = 0;

	/** Rebuilds the details tree nodes for the details implementation providing this utilities object. */
	virtual void RebuildTreeNodes() = 0;
};
