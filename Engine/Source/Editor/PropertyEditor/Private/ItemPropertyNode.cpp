// Copyright Epic Games, Inc. All Rights Reserved.

#include "ItemPropertyNode.h"
#include "Editor.h"
#include "EditorMetadataOverrides.h"
#include "ObjectPropertyNode.h"
#include "PropertyEditorHelpers.h"
#include "PropertyPathHelpers.h"
#include "Settings/EditorStyleSettings.h"
#include "UserInterface/PropertyEditor/SPropertyEditorArrayItem.h"

#include "UObject/PropertyOptional.h"
#include "VerseVM/VVMCVars.h"

#define LOCTEXT_NAMESPACE "ItemPropertyNode"

FItemPropertyNode::FItemPropertyNode()
{
	bCanDisplayFavorite = false;
}

FItemPropertyNode::~FItemPropertyNode()
{

}

uint8* FItemPropertyNode::GetValueBaseAddress(uint8* StartAddress, bool bIsSparseData, bool bIsStruct) const
{
	const FProperty* MyProperty = GetProperty();
	const TSharedPtr<FPropertyNode> ParentNode = ParentNodeWeakPtr.Pin();
	if (MyProperty && ParentNode)
	{
		const FArrayProperty* OuterArrayProp = MyProperty->GetOwner<FArrayProperty>();
		const FSetProperty* OuterSetProp = MyProperty->GetOwner<FSetProperty>();
		const FMapProperty* OuterMapProp = MyProperty->GetOwner<FMapProperty>();

		uint8* ValueBaseAddress = ParentNode->GetValueBaseAddress(StartAddress, bIsSparseData, bIsStruct);

		if (OuterArrayProp != nullptr)
		{
			FScriptArrayHelper ArrayHelper(OuterArrayProp, ValueBaseAddress);
			if (ValueBaseAddress != nullptr && ArrayHelper.IsValidIndex(ArrayIndex))
			{
				return ArrayHelper.GetRawPtr(ArrayIndex);
			}
		}
		else if (OuterSetProp != nullptr)
		{
			FScriptSetHelper SetHelper(OuterSetProp, ValueBaseAddress);
			if (ValueBaseAddress != nullptr)
			{
				int32 ActualIndex = SetHelper.FindInternalIndex(ArrayIndex);
				if (ActualIndex != INDEX_NONE)
				{
					return SetHelper.GetElementPtr(ActualIndex);
				}
			}
		}
		else if (OuterMapProp != nullptr)
		{
			FScriptMapHelper MapHelper(OuterMapProp, ValueBaseAddress);
			if (ValueBaseAddress != nullptr)
			{
				int32 ActualIndex = MapHelper.FindInternalIndex(ArrayIndex);
				if (ActualIndex != INDEX_NONE)
				{
					uint8* PairPtr = MapHelper.GetPairPtr(ActualIndex);
					return MyProperty->ContainerPtrToValuePtr<uint8>(PairPtr);
				}
			}
		}
		else
		{
			uint8* ValueAddress = ValueBaseAddress;
			if (ValueAddress != nullptr && ParentNode->GetProperty() != MyProperty)
			{
				// if this is not a fixed size array (in which the parent property and this property are the same), we need to offset from the property (otherwise, the parent already did that for us)
				ValueAddress = MyProperty->ContainerPtrToValuePtr<uint8>(ValueAddress);
			}

			if (ValueAddress != nullptr)
			{
				ValueAddress += ArrayOffset;
			}
			return ValueAddress;
		}
	}

	return nullptr;
}

uint8* FItemPropertyNode::GetValueAddress(uint8* StartAddress, bool bIsSparseData, bool bIsStruct) const
{
	uint8* Result = GetValueBaseAddress(StartAddress, bIsSparseData, bIsStruct);
	return Result;
}

/**
 * Overridden function for special setup
 */
void FItemPropertyNode::InitBeforeNodeFlags()
{
	FPropertyNode::InitBeforeNodeFlags();
	
	const FProperty* MyProperty = Property.Get();
	const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(MyProperty);

	if (ObjectProperty)
	{
		if (Verse::CVarUseDynamicSubobjectInstancing->GetBool())
		{
			const UClass* OwnerClass = ObjectProperty->GetOwnerClass();
			if (OwnerClass && OwnerClass->ShouldUseDynamicSubobjectInstancing())
			{
				SetNodeFlags(EPropertyNodeFlags::SupportsDynamicInstancing, true);
			}
		}
		else
		{
			static const FName Name_SupportsDynamicInstance("SupportsDynamicInstance");

			if (ObjectProperty->HasMetaData(Name_SupportsDynamicInstance))
			{
				SetNodeFlags(EPropertyNodeFlags::SupportsDynamicInstancing, true);
			}
		}
	}
}

void FItemPropertyNode::InitExpansionFlags(void)
{
	FProperty* MyProperty = GetProperty();

	bool bExpandableType = CastField<FStructProperty>(MyProperty)
						|| CastField<FArrayProperty>(MyProperty)
						|| CastField<FSetProperty>(MyProperty)
						|| CastField<FMapProperty>(MyProperty)
						|| CastField<FOptionalProperty>(MyProperty);
						
	auto IsDynamicInstanced = [this, MyProperty]()
	{
		if (!HasNodeFlags(EPropertyNodeFlags::SupportsDynamicInstancing))
		{
			return false;
		}
		// Handle dynamic instancing
		// The property will instance the object under it only if that object is within this object's outer
		// otherwise the object will be a reference
		// The Property tree needs to dead end the generation of PropertyNodes for references to objects otherwise initializing
		// the child nodes amounts to effectively enumerating all paths to a given property on the referenced object which can have considerable
		// performance and memory issues
		FObjectProperty* ObjectProperty = CastField<FObjectProperty>(MyProperty);
		if (!ObjectProperty)
		{
			return false;
		}
		
		// Walk up the PropertyNode tree to find the nearest ObjectNode, this will be the outer we check against
		const FObjectPropertyNode* ParentObjectNode = FindObjectItemParent();
		if (!ParentObjectNode)
		{
			return false;
		}
		
		FReadAddressList ObjectAddresses;
		if( !GetReadAddress(!!HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ObjectAddresses, false ) )
		{
			return false;
		}
		
		if (ParentObjectNode->GetNumObjects() != ObjectAddresses.Num())
		{
			// Cannot establish a one-to-one mapping of objects, so it is not possible to determine if the object is dynamic instanced
			return false;
		}
		
		// We've got some addresses, and we know they're all NULL or non-NULL.
		// Have a peek at them and check if they are instanced.  They must all be instanced otherwise we dead-end the node expansion
		for (int32 i = 0; i < ObjectAddresses.Num(); i++)
		{
			const UObject* Outer = ParentObjectNode->GetUObject(i);
			UObject* Obj = ObjectProperty->GetObjectPropertyValue(ObjectAddresses.GetAddress(i));
			if(Outer != nullptr && Obj != nullptr)
			{
				if (!Obj->IsInOuter(Outer))
				{
					return false;
				}
			}
			else
			{
				return false;
			}
		}
		
		return true;
	};

	const bool bIsDynamicInstanced = IsDynamicInstanced();
	if (bIsDynamicInstanced)
	{
		SetNodeFlags(EPropertyNodeFlags::DynamicInstance, true);
		bExpandableType = true;
	}

	if (bExpandableType
		|| HasNodeFlags(EPropertyNodeFlags::EditInlineNew)
		|| HasNodeFlags(EPropertyNodeFlags::ShowInnerObjectProperties)
		|| (MyProperty->ArrayDim > 1 && ArrayIndex == -1))
	{
		SetNodeFlags(EPropertyNodeFlags::CanBeExpanded, true);
	}
}
/**
 * Overridden function for Creating Child Nodes
 */
void FItemPropertyNode::InitChildNodes()
{
	//NOTE - this is only turned off as to not invalidate child object nodes.
	FProperty* MyProperty = GetProperty();
	FStructProperty* StructProperty = CastField<FStructProperty>(MyProperty);
	FArrayProperty* ArrayProperty = CastField<FArrayProperty>(MyProperty);
	FSetProperty* SetProperty = CastField<FSetProperty>(MyProperty);
	FMapProperty* MapProperty = CastField<FMapProperty>(MyProperty);
	FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(MyProperty);
	FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(MyProperty);

	const bool bShouldShowHiddenProperties = !!HasNodeFlags(EPropertyNodeFlags::ShouldShowHiddenProperties);
	const bool bShouldShowDisableEditOnInstance = !!HasNodeFlags(EPropertyNodeFlags::ShouldShowDisableEditOnInstance);

	if( MyProperty->ArrayDim > 1 && ArrayIndex == -1 )
	{
		// Do not add array children which are defined by an enum but the enum at the array index is hidden
		// This only applies to static arrays
		static const FName NAME_ArraySizeEnum("ArraySizeEnum");
		UEnum* ArraySizeEnum = NULL; 
		if (MyProperty->HasMetaData(NAME_ArraySizeEnum))
		{
			ArraySizeEnum	= FindObject<UEnum>(NULL, *MyProperty->GetMetaData(NAME_ArraySizeEnum));
		}

		// Expand array.
		for( int32 Index = 0 ; Index < MyProperty->ArrayDim ; Index++ )
		{
			bool bShouldBeHidden = false;
			if( ArraySizeEnum )
			{
				// The enum at this array index is hidden
				bShouldBeHidden = ArraySizeEnum->HasMetaData(TEXT("Hidden"), Index );
			}

			if( !bShouldBeHidden )
			{
				TSharedPtr<FItemPropertyNode> NewItemNode( new FItemPropertyNode);
				FPropertyNodeInitParams InitParams;
				InitParams.ParentNode = SharedThis(this);
				InitParams.Property = MyProperty;
				InitParams.ArrayOffset = Index*MyProperty->GetElementSize();
				InitParams.ArrayIndex = Index;
				InitParams.bAllowChildren = true;
				InitParams.bForceHiddenPropertyVisibility = bShouldShowHiddenProperties;
				InitParams.bCreateDisableEditOnInstanceNodes = bShouldShowDisableEditOnInstance;

				NewItemNode->InitNode( InitParams );
				AddChildNode(NewItemNode);
			}
		}
	}
	else if( ArrayProperty )
	{
		void* Array = NULL;
		FReadAddressList Addresses;
		if ( GetReadAddress(HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), Addresses ) )
		{
			Array = Addresses.GetAddress(0);
		}

		if( Array )
		{
			FScriptArrayHelper ArrayHelper(ArrayProperty, Array);
			for( int32 Index = 0 ; Index < ArrayHelper.Num(); Index++ )
			{
				TSharedPtr<FItemPropertyNode> NewItemNode( new FItemPropertyNode );

				FPropertyNodeInitParams InitParams;
				InitParams.ParentNode = SharedThis(this);
				InitParams.Property = ArrayProperty->Inner;
				InitParams.ArrayOffset = Index * ArrayProperty->Inner->GetElementSize();
				InitParams.ArrayIndex = Index;
				InitParams.bAllowChildren = true;
				InitParams.bForceHiddenPropertyVisibility = bShouldShowHiddenProperties;
				InitParams.bCreateDisableEditOnInstanceNodes = bShouldShowDisableEditOnInstance;

				NewItemNode->InitNode( InitParams );
				AddChildNode(NewItemNode);
			}
		}
	}
	else if ( SetProperty )
	{
		void* Set = NULL;
		FReadAddressList Addresses;
		if (GetReadAddress(!!HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), Addresses))
		{
			Set = Addresses.GetAddress(0);
		}

		if (Set)
		{
			FScriptSetHelper SetHelper(SetProperty, Set);

			for (int32 Index = 0; Index < SetHelper.Num(); ++Index)
			{
				TSharedPtr<FItemPropertyNode> NewItemNode(new FItemPropertyNode);

				FPropertyNodeInitParams InitParams;
				InitParams.ParentNode = SharedThis(this);
				InitParams.Property = SetProperty->ElementProp;
				InitParams.ArrayIndex = Index;
				InitParams.bAllowChildren = true;
				InitParams.bForceHiddenPropertyVisibility = bShouldShowHiddenProperties;
				InitParams.bCreateDisableEditOnInstanceNodes = bShouldShowDisableEditOnInstance;

				NewItemNode->InitNode(InitParams);
				AddChildNode(NewItemNode);
			}
		}
	}
	else if ( MapProperty )
	{
		void* Map = NULL;
		FReadAddressList Addresses;
		if (GetReadAddress(!!HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), Addresses))
		{
			Map = Addresses.GetAddress(0);
		}

		if ( Map )
		{
			FScriptMapHelper MapHelper(MapProperty, Map);

			for (int32 Index = 0; Index < MapHelper.Num(); ++Index)
			{
				// Construct the key node first
				TSharedPtr<FPropertyNode> KeyNode(new FItemPropertyNode());
					
				FPropertyNodeInitParams InitParams;
				InitParams.ParentNode = SharedThis(this);	// Key Node needs to point to this node to ensure its data is set correctly
				InitParams.Property = MapHelper.KeyProp;
				InitParams.ArrayIndex = Index;
				InitParams.ArrayOffset = 0;
				InitParams.bAllowChildren = true;
				InitParams.bForceHiddenPropertyVisibility = bShouldShowHiddenProperties;
				InitParams.bCreateDisableEditOnInstanceNodes = bShouldShowDisableEditOnInstance;

				KeyNode->InitNode(InitParams);
				// Not adding the key node as a child, otherwise it'll show up in the wrong spot.

				// Then the value node
				TSharedPtr<FPropertyNode> ValueNode(new FItemPropertyNode());
					
				// Reuse the init params
				InitParams.ParentNode = SharedThis(this);
				InitParams.Property = MapHelper.ValueProp;

				ValueNode->InitNode(InitParams);
				AddChildNode(ValueNode);

				FPropertyNode::SetupKeyValueNodePair(KeyNode, ValueNode);
			}
		}
	}
	else if( StructProperty )
	{
		// Expand struct.
		TArray<FProperty*> StructMembers;
		for (TFieldIterator<FProperty> It(StructProperty->Struct); It; ++It)
		{
			FProperty* StructMember = *It;
			
			if (PropertyEditorHelpers::ShouldBeVisible(*this, StructMember))
			{
				StructMembers.Add(StructMember);
			}
		}

		PropertyEditorHelpers::OrderPropertiesFromMetadata(StructMembers);

		for (FProperty* StructMember : StructMembers)
		{
			TSharedPtr<FItemPropertyNode> NewItemNode( new FItemPropertyNode );
		
			FPropertyNodeInitParams InitParams;
			InitParams.ParentNode = SharedThis(this);
			InitParams.Property = StructMember;
			InitParams.ArrayOffset = 0;
			InitParams.ArrayIndex = INDEX_NONE;
			InitParams.bAllowChildren = true;
			InitParams.bForceHiddenPropertyVisibility = bShouldShowHiddenProperties;
			InitParams.bCreateDisableEditOnInstanceNodes = bShouldShowDisableEditOnInstance;

			NewItemNode->InitNode( InitParams );
			AddChildNode(NewItemNode);

			if ( FPropertySettings::Get().ExpandDistributions() == false)
			{
				// auto-expand distribution structs
				if ( CastField<FObjectProperty>(StructMember) || CastField<FWeakObjectProperty>(StructMember) || CastField<FLazyObjectProperty>(StructMember) || CastField<FSoftObjectProperty>(StructMember) )
				{
					const FName StructName = StructProperty->Struct->GetFName();
					if (StructName == NAME_RawDistributionFloat || StructName == NAME_RawDistributionVector)
					{
						NewItemNode->SetNodeFlags(EPropertyNodeFlags::Expanded, true);
					}
				}
			}
		}
	}
	else if( ObjectProperty )
	{
		uint8* ReadValue = NULL;

		FReadAddressList ReadAddresses;
		if( GetReadAddress(!!HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ReadAddresses, false ) )
		{
			// We've got some addresses, and we know they're all NULL or non-NULL.
			// Have a peek at the first one, and only build an objects node if we've got addresses.
			if( UObject* Obj = (ReadAddresses.Num() > 0) ? ObjectProperty->GetObjectPropertyValue(ReadAddresses.GetAddress(0)) : nullptr )
			{
				//verify it's not above in the hierarchy somewhere
				FObjectPropertyNode* ParentObjectNode = FindObjectItemParent();
				while (ParentObjectNode)
				{
					for ( TPropObjectIterator Itor( ParentObjectNode->ObjectIterator() ) ; Itor ; ++Itor )
					{
						if (*Itor == Obj)
						{
							SetNodeFlags(EPropertyNodeFlags::NoChildrenDueToCircularReference, true);
							//stop the circular loop!!!
							return;
						}
					}
					FPropertyNode* UpwardTravesalNode = ParentObjectNode->GetParentNode();
					ParentObjectNode = (UpwardTravesalNode==NULL) ? NULL : UpwardTravesalNode->FindObjectItemParent();
				}

			
				TSharedPtr<FObjectPropertyNode> NewObjectNode( new FObjectPropertyNode );
				for ( int32 AddressIndex = 0 ; AddressIndex < ReadAddresses.Num() ; ++AddressIndex )
				{
					NewObjectNode->AddObject( ObjectProperty->GetObjectPropertyValue(ReadAddresses.GetAddress(AddressIndex) ) );
				}

				FPropertyNodeInitParams InitParams;
				InitParams.ParentNode = SharedThis(this);
				InitParams.Property = MyProperty;
				InitParams.ArrayOffset = 0;
				InitParams.ArrayIndex = INDEX_NONE;
				InitParams.bAllowChildren = true;
				InitParams.bForceHiddenPropertyVisibility = bShouldShowHiddenProperties;
				InitParams.bCreateDisableEditOnInstanceNodes = bShouldShowDisableEditOnInstance;

				NewObjectNode->InitNode( InitParams );
				AddChildNode(NewObjectNode);
			}
		}
	}
	else if (OptionalProperty)
	{
		FReadAddressList Addresses;
		if (GetReadAddress(!!HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), Addresses, false))
		{
			bool bAllOptionalsSet = true;
			for (int i = 0; i < Addresses.Num(); i++)
			{
				bAllOptionalsSet &= OptionalProperty->IsSet(Addresses.GetAddress(i));
				if (!bAllOptionalsSet)
				{
					break;
				}
			}

			if (bAllOptionalsSet)
			{
				TSharedPtr<FItemPropertyNode> ValueNode = MakeShared<FItemPropertyNode>();

				FPropertyNodeInitParams InitParams;
				InitParams.ParentNode = SharedThis(this);
				InitParams.Property = OptionalProperty->GetValueProperty();
				InitParams.bAllowChildren = true;
				InitParams.bForceHiddenPropertyVisibility = bShouldShowHiddenProperties;
				InitParams.bCreateDisableEditOnInstanceNodes = bShouldShowDisableEditOnInstance;

				ValueNode->InitNode(InitParams);
				AddChildNode(ValueNode);
			}
		}
	}
}

void FItemPropertyNode::SetFavorite(bool IsFavorite)
{
	if (GEditor == nullptr)
	{
		return;
	}

	UEditorMetadataOverrides* MetadataOverrides = GEditor->GetEditorSubsystem<UEditorMetadataOverrides>();
	if (MetadataOverrides == nullptr)
	{
		return;
	}

	const FObjectPropertyNode* ObjectParent = FindObjectItemParent();
	if (ObjectParent == nullptr)
	{
		return;
	}
				
	FString Path;
	GetQualifiedName(Path, /*bWithArrayIndex=*/true, ObjectParent, /*bIgnoreCategories=*/true);

	static const FName FavoritePropertiesName("FavoriteProperties");

	TArray<FString> FavoritePropertiesList;
	if (MetadataOverrides->GetArrayMetadata(ObjectParent->GetObjectBaseClass(), FavoritePropertiesName, FavoritePropertiesList))
	{
		if (IsFavorite)
		{
			FavoritePropertiesList.AddUnique(Path);
		}
		else
		{
			FavoritePropertiesList.Remove(Path);
		}

		MetadataOverrides->SetArrayMetadata(ObjectParent->GetObjectBaseClass(), FavoritePropertiesName, FavoritePropertiesList);
	}
	else
	{
		if (IsFavorite)
		{
			FavoritePropertiesList.Add(Path);
			MetadataOverrides->SetArrayMetadata(ObjectParent->GetObjectBaseClass(), FavoritePropertiesName, FavoritePropertiesList);
		}
	}
}

bool FItemPropertyNode::IsFavorite() const
{
	if (GEditor == nullptr)
	{
		return false;
	}

	UEditorMetadataOverrides* MetadataOverrides = GEditor->GetEditorSubsystem<UEditorMetadataOverrides>();
	if (MetadataOverrides == nullptr)
	{
		return false;
	}

	const FObjectPropertyNode* ObjectParent = FindObjectItemParent();
	if (ObjectParent == nullptr)
	{
		return false;
	}

	FString Path;
	GetQualifiedName(Path, /*bWithArrayIndex=*/true, ObjectParent, /*bIgnoreCategories=*/true);

	static const FName FavoritePropertiesName("FavoriteProperties");

	TArray<FString> FavoritePropertiesList;
	if (MetadataOverrides->GetArrayMetadata(ObjectParent->GetObjectBaseClass(), FavoritePropertiesName, FavoritePropertiesList))
	{
		return FavoritePropertiesList.Contains(Path);
	}

	return false;
}

void FItemPropertyNode::SetDisplayNameOverride( const FText& InDisplayNameOverride )
{
	DisplayNameOverride = InDisplayNameOverride;
}

FText FItemPropertyNode::GetDisplayName() const
{
	FText FinalDisplayName;

	if( !DisplayNameOverride.IsEmpty() )
	{
		FinalDisplayName = DisplayNameOverride;
	}
	else 
	{
		const FProperty* PropertyPtr = GetProperty();
		if( GetArrayIndex()==-1 && PropertyPtr != NULL  )
		{
			// This item is not a member of an array, get a traditional display name
			if ( FPropertySettings::Get().ShowFriendlyPropertyNames() )
			{
				//We are in "readable display name mode"../ Make a nice name
				FinalDisplayName = PropertyPtr->GetDisplayNameText();
				if ( FinalDisplayName.IsEmpty() )
				{
					FString PropertyDisplayName;
					bool bIsBoolProperty = CastField<const FBoolProperty>(PropertyPtr) != NULL;

					const TSharedPtr<FPropertyNode> ParentNode = ParentNodeWeakPtr.Pin();
					const FStructProperty* ParentStructProperty = CastField<const FStructProperty>(ParentNode->GetProperty());
					if( ParentStructProperty && ParentStructProperty->Struct->GetFName() == NAME_Rotator )
					{
						if( Property->GetFName() == "Roll" )
						{
							PropertyDisplayName = TEXT("X");
						}
						else if( Property->GetFName() == "Pitch" )
						{
							PropertyDisplayName = TEXT("Y");
						}
						else if( Property->GetFName() == "Yaw" )
						{
							PropertyDisplayName = TEXT("Z");
						}
						else
						{
							check(0);
						}
					}
					else
					{
						PropertyDisplayName = Property->GetName();
					}
					if( GetDefault<UEditorStyleSettings>()->bShowFriendlyNames )
					{
						PropertyDisplayName = FName::NameToDisplayString( PropertyDisplayName, bIsBoolProperty );
					}

					FinalDisplayName = FText::FromString( PropertyDisplayName );
				}
			}
			else
			{
				FinalDisplayName =  FText::FromString( PropertyPtr->GetName() );
			}
		}
		else if (const TSharedPtr<FPropertyNode> ParentNode = ParentNodeWeakPtr.Pin())
		{	
			// Sets and maps do not have a display index.
			FProperty* ParentProperty = ParentNode->GetProperty();

			// Get the ArraySizeEnum class from meta data.
			static const FName NAME_ArraySizeEnum("ArraySizeEnum");
			UEnum* ArraySizeEnum = NULL; 
			if (PropertyPtr && PropertyPtr->HasMetaData(NAME_ArraySizeEnum))
			{
				ArraySizeEnum	= FindObject<UEnum>(NULL, *Property->GetMetaData(NAME_ArraySizeEnum));
			}
		
			// Also handle UArray's having the ArraySizeEnum entry...
			if (ArraySizeEnum == nullptr && CastField<FArrayProperty>(ParentProperty) != nullptr && ParentProperty->HasMetaData(NAME_ArraySizeEnum))
			{
				ArraySizeEnum = FindObject<UEnum>(NULL, *ParentProperty->GetMetaData(NAME_ArraySizeEnum));
			}

			if (CastField<FSetProperty>(ParentProperty) == nullptr &&  CastField<FMapProperty>(ParentProperty) == nullptr)
			{
				if (PropertyPtr != nullptr)
				{
					// Check if this property has Title Property Meta
					static const FName NAME_TitleProperty = FName(TEXT("TitleProperty"));
					FString TitleProperty = PropertyPtr->GetMetaData(NAME_TitleProperty);
					if (!TitleProperty.IsEmpty())
					{
						FItemPropertyNode* NonConstThis = const_cast<FItemPropertyNode*>(this);

						FReadAddressListData ReadAddress;
						GetReadAddressUncached(*NonConstThis, ReadAddress);

						const UStruct* PropertyStruct = nullptr;

						if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(PropertyPtr))
						{
							// We do this so we get the classes *exact* value and not just the base from PropertyClass
							uint8* FoundAddress = ReadAddress.GetAddress(0);
							UObject* ObjectValue = FoundAddress != nullptr ? ObjectProperty->GetObjectPropertyValue(FoundAddress) : nullptr;

							if (ObjectValue != nullptr)
							{
								PropertyStruct = ObjectValue->GetClass();
							}
						}

						// Find the property and get the right property handle
						if (PropertyStruct != nullptr)
						{
							const TSharedPtr<IPropertyHandle> ThisAsHandle = PropertyEditorHelpers::GetPropertyHandle(NonConstThis->AsShared(), nullptr, nullptr);
							TSharedPtr<FTitleMetadataFormatter> TitleFormatter = FTitleMetadataFormatter::TryParse(ThisAsHandle, TitleProperty);
							if (TitleFormatter)
							{
								TitleFormatter->GetDisplayText(FinalDisplayName);
							}
						}
					}
				}

				if(FinalDisplayName.IsEmpty())
				{
					// This item is a member of an array, its display name is its index 
					if (PropertyPtr == NULL || ArraySizeEnum == NULL)
					{
						if (ArraySizeEnum == nullptr)
						{
							FinalDisplayName = FText::AsNumber(GetArrayIndex());
						}
						else
						{
							FinalDisplayName = ArraySizeEnum->GetDisplayNameTextByIndex(GetArrayIndex());
						}
					}
					else
					{
						FinalDisplayName = ArraySizeEnum->GetDisplayNameTextByIndex(GetArrayIndex());
					}
				}
			}
			// Maps should have display names that reflect the key and value types
			else if (PropertyPtr != nullptr && CastField<FMapProperty>(ParentProperty) != nullptr)
			{
				FText FormatText = GetPropertyKeyNode().IsValid()
					? LOCTEXT("MapValueDisplayFormat", "Value ({0})")
					: LOCTEXT("MapKeyDisplayFormat", "Key ({0})");

				FString TypeName;

				if (const FStructProperty* StructProp = CastField<FStructProperty>(PropertyPtr))
				{
					// For struct props, use the name of the struct itself
					TypeName = StructProp->Struct->GetName();
				}
				else if (const FEnumProperty* EnumProp = CastField<FEnumProperty>(PropertyPtr))
				{
					// For enum props, use the name of the enum
					if (EnumProp->GetEnum() != nullptr)
					{
						TypeName = EnumProp->GetEnum()->GetName();
					}
					else
					{
						TypeName = TEXT("Enum");
					}
				}
				else if(PropertyPtr->IsA<FStrProperty>())
				{
					// For strings, actually return "String" and not "Str"
					TypeName = TEXT("String");
				}
				else
				{
					// For any other property, get the type from the property class
					TypeName = PropertyPtr->GetClass()->GetName();

					int32 EndIndex = TypeName.Find(TEXT("Property"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);

					if (EndIndex != -1)
					{
						TypeName.MidInline(0, EndIndex, EAllowShrinking::No);
					}
				}

				if (FPropertySettings::Get().ShowFriendlyPropertyNames())
				{
					TypeName = FName::NameToDisplayString(TypeName, false);
				}

				FinalDisplayName = FText::Format(FormatText, FText::FromString(TypeName));
			}
		}
	}
	
	return FinalDisplayName;
}

void FItemPropertyNode::SetToolTipOverride( const FText& InToolTipOverride )
{
	ToolTipOverride = InToolTipOverride;
}

FText FItemPropertyNode::GetToolTipText() const
{
	if(!ToolTipOverride.IsEmpty())
	{
		return ToolTipOverride;
	}

	return PropertyEditorHelpers::GetToolTipText(GetProperty());
}

#undef LOCTEXT_NAMESPACE
