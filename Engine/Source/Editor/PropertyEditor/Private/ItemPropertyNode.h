// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PropertyNode.h"

class FItemPropertyNode : public FPropertyNode
{
public:

	FItemPropertyNode();
	virtual ~FItemPropertyNode();

	virtual uint8* GetValueBaseAddress(uint8* StartAddress, bool bIsSparseData, bool bIsStruct) const override;
	virtual uint8* GetValueAddress(uint8* StartAddress, bool bIsSparseData, bool bIsStruct) const override;

	/**
	 * Overridden function to get the derived object node
	 */
	virtual FItemPropertyNode* AsItemPropertyNode() override { return this; }
	virtual const FItemPropertyNode* AsItemPropertyNode() const override { return this; }

	/** Display name override to use instead of the property name */
	void SetDisplayNameOverride( const FText& InDisplayNameOverride ) override;

	/**
	* @return true if the property is mark as a favorite
	*/
	virtual void SetFavorite(bool FavoriteValue) override;

	/**
	* @return true if the property is mark as a favorite
	*/
	virtual bool IsFavorite() const override;

	/**
	 * @return The formatted display name for the property in this node                                                              
	 */
	virtual FText GetDisplayName() const override;

	/**
	 * Sets the tooltip override to use instead of the property tooltip
	 */
	virtual void SetToolTipOverride( const FText& InToolTipOverride ) override;

	/**
	 * @return The tooltip for the property in this node                                                              
	 */
	virtual FText GetToolTipText() const override;

protected:
	virtual void InitBeforeNodeFlags() override;
	
	/**
	 * Overridden function for special setup
	 */
	virtual void InitExpansionFlags() override;

	/**
	 * Overridden function for Creating Child Nodes
	 */
	virtual void InitChildNodes() override;

private:
	/** Display name override to use instead of the property name */
	FText DisplayNameOverride;
	/** Tooltip override to use instead of the property tooltip */
	FText ToolTipOverride;

	/**
	* Option to know if we can display the favorite icon in the property editor
	*/
	bool bCanDisplayFavorite;
};
