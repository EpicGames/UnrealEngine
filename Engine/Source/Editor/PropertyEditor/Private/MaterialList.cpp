// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialList.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "Engine/Texture2D.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "PropertyEditorEditConstPolicy.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Modules/ModuleManager.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "PropertyCustomizationHelpers"

/** Definition of extra widgets delegate for bottom material value field. */
FMaterialList::FOnAddMaterialItemViewExtraBottomWidget FMaterialList::OnAddMaterialItemViewExtraBottomWidget;

/**
 * Builds up a list of unique materials while creating some information about the materials
 */
class FMaterialListBuilder : public IMaterialListBuilder
{
	friend class FMaterialList;
public:

	/** 
	 * Adds a new material to the list
	 * 
	 * @param SlotIndex		The slot (usually mesh element index) where the material is located on the component
	 * @param Material		The material being used
	 * @param bCanBeReplced	Whether or not the material can be replaced by a user
	 */
	virtual void AddMaterial( uint32 SlotIndex, UMaterialInterface* Material, bool bCanBeReplaced, UActorComponent* InCurrentComponent, FName SlotName ) override
	{
		int32 NumMaterials = MaterialSlots.Num();

		FMaterialListItem MaterialItem( Material, SlotIndex, bCanBeReplaced, InCurrentComponent, SlotName ); 
		if( !UniqueMaterials.Contains( MaterialItem ) ) 
		{
			MaterialSlots.Add( MaterialItem );
			UniqueMaterials.Add( MaterialItem );
		}

		// Did we actually add material?  If we did then we need to increment the number of materials in the element
		if( MaterialSlots.Num() > NumMaterials )
		{
			// Resize the array to support the slot if needed
			if( !MaterialCount.IsValidIndex(SlotIndex) )
			{
				int32 NumToAdd = (SlotIndex - MaterialCount.Num()) + 1;
				if( NumToAdd > 0 )
				{
					MaterialCount.AddZeroed( NumToAdd );
				}
			}

			++MaterialCount[SlotIndex];
		}
	}

	/** Empties the list */
	void Empty()
	{
		UniqueMaterials.Empty();
		MaterialSlots.Reset();
		MaterialCount.Reset();
	}

	/** Sorts the list by slot index */
	void Sort()
	{
		struct FSortByIndex
		{
			bool operator()( const FMaterialListItem& A, const FMaterialListItem& B ) const
			{
				return A.SlotIndex < B.SlotIndex;
			}
		};

		MaterialSlots.Sort( FSortByIndex() );
	}

	/** @return The number of materials in the list */
	uint32 GetNumMaterials() const { return MaterialSlots.Num(); }

	/** @return The number of materials in the list at a given slot */
	uint32 GetNumMaterialsInSlot( uint32 Index ) const { return MaterialCount[Index]; }
private:
	/** All unique materials */
	TSet<FMaterialListItem> UniqueMaterials;
	/** All material items in the list */
	TArray<FMaterialListItem> MaterialSlots;
	/** Material counts for each slot.  The slot is the index and the value at that index is the count */
	TArray<uint32> MaterialCount;
};


TSharedRef<FMaterialItemView> FMaterialItemView::Create(
	const FMaterialListItem& Material, 
	FOnMaterialChanged InOnMaterialChanged,
	FOnGenerateWidgetsForMaterial InOnGenerateNameWidgetsForMaterial, 
	FOnGenerateWidgetsForMaterial InOnGenerateWidgetsForMaterial, 
	FOnResetMaterialToDefaultClicked InOnResetToDefaultClicked,
	FOnGenerateWidgetsForMaterial InOnGenerateExtraBottomWidget,
	int32 InMultipleMaterialCount,
	bool bShowUsedTextures)
{
	// FMaterialItemView has private constructor that is why we need to use MakeShareable 
	return MakeShareable( new FMaterialItemView( Material, InOnMaterialChanged, InOnGenerateNameWidgetsForMaterial, InOnGenerateWidgetsForMaterial, InOnResetToDefaultClicked, InOnGenerateExtraBottomWidget, InMultipleMaterialCount, bShowUsedTextures) );
}

TSharedRef<SWidget> FMaterialItemView::CreateNameContent()
{
	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("ElementIndex"), MaterialItem.SlotIndex);

	return 
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew( STextBlock )
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text( FText::Format(LOCTEXT("ElementIndex", "Element {ElementIndex}"), Arguments ) )
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
		]
		+SVerticalBox::Slot()
		.Padding(0.0f,4.0f)
		.AutoHeight()
		[
			OnGenerateCustomNameWidgets.IsBound() ? OnGenerateCustomNameWidgets.Execute( MaterialItem.Material.Get(), MaterialItem.SlotIndex ) : StaticCastSharedRef<SWidget>( SNullWidget::NullWidget )
		];
}

FString FMaterialItemView::OnGetObjectPath() const
{
	return MaterialItem.Material->GetPathName();
}

namespace ValueExtender
{
	template <typename TContainer, typename TDelegate>
	TSharedRef<SWidget> MakeWidget(TDelegate& InDelegate, const TSharedRef<FMaterialItemView>& InMaterialItemView, IDetailLayoutBuilder& InDetailBuilder, UActorComponent* InCurrentComponent)
	{
		TSharedPtr<TContainer> WidgetContainer = SNew(TContainer);
		TArray<TSharedPtr<SWidget>> Widgets;

		// Execute all delegates and loop through all results widgets
		InDelegate.Broadcast(InMaterialItemView, InCurrentComponent, InDetailBuilder, Widgets);

		for (TSharedPtr<SWidget> Widget : Widgets)
		{
			WidgetContainer->AddSlot()
				.AutoHeight()
			[
				Widget.ToSharedRef()
			];
		}

		return WidgetContainer.ToSharedRef();
	}
};

TSharedRef<SWidget> FMaterialItemView::CreateValueContent(IDetailLayoutBuilder& InDetailBuilder, const TArray<FAssetData>& OwnerAssetDataArray, UActorComponent* InActorComponent)
{	
	// Always consider the InActorComponent's asset location (BP, Level, etc.) as part of the OwnerAssetArray
	TArray<FAssetData> AssetDataArray = OwnerAssetDataArray;
	if (InActorComponent)
	{
		AssetDataArray.Add((FAssetData)InActorComponent->GetOuter());
	}

	return
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding( 0.0f )
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SObjectPropertyEntryBox)
				.ObjectPath(this, &FMaterialItemView::OnGetObjectPath)
				.AllowClear(false)
				.AllowedClass(UMaterialInterface::StaticClass())
				.OnObjectChanged(this, &FMaterialItemView::OnSetObject)
				.ThumbnailPool(InDetailBuilder.GetThumbnailPool())
				.DisplayCompactSize(true)
				.OwnerAssetDataArray(AssetDataArray)
				.CustomContentSlot()
				[
					SNew( SBox )
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(0.0f, 0.0f, 3.0f, 0.0f)
						.AutoWidth()
						[
							// Add a button to browse to any nanite override material
							MakeBrowseNaniteOverrideMaterialButton()
						]
						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(0.0f, 0.0f, 3.0f, 0.0f)
						.AutoWidth()
						[
							// Add a menu for displaying all textures 
							SNew(SComboButton)
							.OnGetMenuContent(this, &FMaterialItemView::OnGetTexturesMenuForMaterial)
							.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButtonWithIcon")
							.VAlign(VAlign_Center)
							.IsEnabled( this, &FMaterialItemView::IsTexturesMenuEnabled )
							.Visibility(bShowUsedTextures ? EVisibility::Visible : EVisibility::Hidden)
							.ToolTipText(this, &FMaterialItemView::GetTexturesMenuToolTipText)
							.ButtonContent()
							[
								SNew(SImage)
								.Image(FSlateIconFinder::FindIconBrushForClass(UTexture2D::StaticClass()))
								.ColorAndOpacity(FSlateColor::UseForeground())
							]
						]
						+SHorizontalBox::Slot()
						.Padding(3.0f, 0.0f)
						.FillWidth(1.f)
						[
							OnGenerateCustomMaterialWidgets.IsBound() ? OnGenerateCustomMaterialWidgets.Execute(MaterialItem.Material.Get(), MaterialItem.SlotIndex) : SNullWidget::NullWidget
						]
					]
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				GetGlobalRowExtensionWidget(InDetailBuilder, InActorComponent).ToSharedRef()
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			OnGenerateExtraBottomWidget.IsBound() ? OnGenerateExtraBottomWidget.Execute(MaterialItem.Material.Get(), MaterialItem.SlotIndex) : SNullWidget::NullWidget
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			ValueExtender::MakeWidget<SVerticalBox>(FMaterialList::OnAddMaterialItemViewExtraBottomWidget, SharedThis(this), InDetailBuilder, InActorComponent)
		];
}

bool FMaterialItemView::GetResetToBaseVisibility() const
{
	// Only show the reset to base button if the current material can be replaced
	return OnMaterialChanged.IsBound() && MaterialItem.bCanBeReplaced;
}


void FMaterialItemView::OnResetToBaseClicked()
{
	// Only allow reset to base if the current material can be replaced
	if( MaterialItem.Material.IsValid() && MaterialItem.bCanBeReplaced )
	{
		bool bReplaceAll = false;
		ReplaceMaterial( nullptr, bReplaceAll );
		OnResetToDefaultClicked.ExecuteIfBound( MaterialItem.Material.Get(), MaterialItem.SlotIndex );
	}
}

FMaterialItemView::FMaterialItemView(	const FMaterialListItem& InMaterial, 
					FOnMaterialChanged& InOnMaterialChanged, 
					FOnGenerateWidgetsForMaterial& InOnGenerateNameWidgets, 
					FOnGenerateWidgetsForMaterial& InOnGenerateMaterialWidgets, 
					FOnResetMaterialToDefaultClicked& InOnResetToDefaultClicked,
					FOnGenerateWidgetsForMaterial& InOnGenerateExtraBottomWidget,
					int32 InMultipleMaterialCount,
					bool bInShowUsedTextures)
					
	: MaterialItem( InMaterial )
	, OnMaterialChanged( InOnMaterialChanged )
	, OnGenerateCustomNameWidgets( InOnGenerateNameWidgets )
	, OnGenerateCustomMaterialWidgets( InOnGenerateMaterialWidgets )
	, OnResetToDefaultClicked( InOnResetToDefaultClicked )
	, OnGenerateExtraBottomWidget (InOnGenerateExtraBottomWidget)
	, MultipleMaterialCount( InMultipleMaterialCount )
	, bShowUsedTextures( bInShowUsedTextures )
{

}

void FMaterialItemView::ReplaceMaterial( UMaterialInterface* NewMaterial, bool bReplaceAll )
{
	UMaterialInterface* PrevMaterial = NULL;
	if( MaterialItem.Material.IsValid() )
	{
		PrevMaterial = MaterialItem.Material.Get();
	}

	if( NewMaterial != PrevMaterial )
	{
		// Replace the material
		OnMaterialChanged.ExecuteIfBound( NewMaterial, PrevMaterial, MaterialItem.SlotIndex, bReplaceAll );
	}
}

void FMaterialItemView::OnSetObject( const FAssetData& AssetData )
{
	const bool bReplaceAll = false;

	UMaterialInterface* NewMaterial = Cast<UMaterialInterface>(AssetData.GetAsset());
	ReplaceMaterial( NewMaterial, bReplaceAll );
}

TSharedRef<SWidget> FMaterialItemView::MakeBrowseNaniteOverrideMaterialButton() const
{
	TSharedRef<SWidget> Widget =
		SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.WidthOverride(22.0f)
		.HeightOverride(22.0f)
		.ToolTipText(LOCTEXT("BrowseToNaniteOverride_Tip", "Browse to the Nanite Override Material in Content Browser"))
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ContentPadding(0.0f)
			.IsFocusable(false)
			.OnClicked(FOnClicked::CreateLambda([WeakMaterial = MaterialItem.Material]()
				{
					UMaterialInterface* Material = WeakMaterial.Get();
					UMaterialInterface* NaniteOverrideMaterial = Material != nullptr ? Material->GetNaniteOverride() : nullptr;
					if (GEditor && NaniteOverrideMaterial != nullptr)
					{
						TArray<UObject*> Objects;
						Objects.Add(NaniteOverrideMaterial);
						GEditor->SyncBrowserToObjects(Objects);
					}
					return FReply::Handled();
				}))
			[ 
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.NaniteBrowseContent"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];

	Widget->SetVisibility(TAttribute<EVisibility>::CreateLambda([WeakMaterial = MaterialItem.Material]()
		{
			UMaterialInterface* Material = WeakMaterial.Get();
			return Material != nullptr && Material->GetNaniteOverride() != nullptr ? EVisibility::Visible : EVisibility::Collapsed;
		}));

	return Widget;
}


bool FMaterialItemView::IsTexturesMenuEnabled() const
{
	if (UMaterialInterface* Material = MaterialItem.Material.Get())
	{
		// Don't enable the menu unless there are textures, otherwise, the user might want to think the button is broken while really, it just has nothing to display
		TArray<UTexture*> Textures;
		Material->GetUsedTextures(Textures, GetCurrentMaterialQualityLevelChecked());
		return !Textures.IsEmpty();
	}
	return false;
}

FText FMaterialItemView::GetTexturesMenuToolTipText() const
{
	if (UMaterialInterface* Material = MaterialItem.Material.Get())
	{
		// Don't enable the menu unless there are textures, otherwise, the user might want to think the button is broken while really, it just has nothing to display
		TArray<UTexture*> Textures;
		Material->GetUsedTextures(Textures, GetCurrentMaterialQualityLevelChecked());

		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("MaterialName"), FText::AsCultureInvariant(Material->GetName()));
		if (Textures.IsEmpty())
		{
			return LOCTEXT("GetTexturesMenuToolTipText_NoTexture", "Find the material's textures in the content browser (no texture)");
		}
		else
		{
			return FText::Format(LOCTEXT("GetTexturesMenuToolTipText_MaterialName", "Find {MaterialName}'s textures in the content browser"), Arguments);
		}
	}

	return LOCTEXT("GetTexturesMenuToolTipText_Default", "Find the material's textures in the content browser");
}

TSharedRef<SWidget> FMaterialItemView::OnGetTexturesMenuForMaterial()
{
	FMenuBuilder MenuBuilder( true, NULL );

	if( MaterialItem.Material.IsValid() )
	{
		UMaterialInterface* Material = MaterialItem.Material.Get();

		TArray< UTexture* > Textures;
		Material->GetUsedTextures(Textures, GetCurrentMaterialQualityLevelChecked());

		// Add a menu item for each texture.  Clicking on the texture will display it in the content browser
		// UObject for delegate compatibility
		for( UObject* Texture : Textures )
		{
			FUIAction Action( FExecuteAction::CreateSP( this, &FMaterialItemView::GoToAssetInContentBrowser, MakeWeakObjectPtr(Texture) ) );

			MenuBuilder.AddMenuEntry( FText::FromString( Texture->GetName() ), LOCTEXT( "BrowseTexture_ToolTip", "Find this texture in the content browser" ), FSlateIcon(), Action );
		}
	}

	return MenuBuilder.MakeWidget();
}


void FMaterialItemView::GoToAssetInContentBrowser( TWeakObjectPtr<UObject> Object )
{
	if( Object.IsValid() )
	{
		TArray< UObject* > Objects;
		Objects.Add( Object.Get() );
		GEditor->SyncBrowserToObjects( Objects );
	}
}


FOnGenerateGlobalRowExtensionArgs FMaterialItemView::GetGlobalRowExtensionArgs(IDetailLayoutBuilder& InDetailBuilder, UActorComponent* InCurrentComponent) const
{
	FOnGenerateGlobalRowExtensionArgs OnGenerateGlobalRowExtensionArgs;

	USceneComponent* SceneComponent = Cast<USceneComponent>(InCurrentComponent);
	if (!SceneComponent)
	{
		return OnGenerateGlobalRowExtensionArgs;
	}

	TSharedPtr<IPropertyHandle> PropertyHandle;
	UObject* OwnerObject = nullptr;
	FString PropertyPath;
	FProperty* MaterialProperty = nullptr;
	if (SceneComponent->GetMaterialPropertyPath(MaterialItem.SlotIndex, OwnerObject, PropertyPath, MaterialProperty))
	{
		OnGenerateGlobalRowExtensionArgs.OwnerObject = OwnerObject;
		OnGenerateGlobalRowExtensionArgs.PropertyPath = PropertyPath;
		OnGenerateGlobalRowExtensionArgs.Property = MaterialProperty;
	}

	return OnGenerateGlobalRowExtensionArgs;
}


TSharedPtr<SWidget> FMaterialItemView::GetGlobalRowExtensionWidget(IDetailLayoutBuilder& InDetailBuilder, UActorComponent* InCurrentComponent) const
{
	FSlimHorizontalToolBarBuilder ToolbarBuilder(TSharedPtr<FUICommandList>(), FMultiBoxCustomization::None);
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	// Add Extension buttons to Row Generator
	TArray<FPropertyRowExtensionButton> ExtensionButtons;

	// Add custom property extensions
	FOnGenerateGlobalRowExtensionArgs RowExtensionArgs = GetGlobalRowExtensionArgs(InDetailBuilder, InCurrentComponent);
	PropertyEditorModule.GetGlobalRowExtensionDelegate().Broadcast(RowExtensionArgs, ExtensionButtons);

	// Build extension toolbar 
	ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);
	ToolbarBuilder.SetStyle(&FAppStyle::Get(), "DetailsView.ExtensionToolBar");
	for (const FPropertyRowExtensionButton& Extension : ExtensionButtons)
	{
		ToolbarBuilder.AddToolBarButton(Extension.UIAction, NAME_None, Extension.Label, Extension.ToolTip, Extension.Icon);
	}
		
	return ToolbarBuilder.MakeWidget();
}

FMaterialList::FMaterialList(IDetailLayoutBuilder& InDetailLayoutBuilder, FMaterialListDelegates& InMaterialListDelegates, const TArray<FAssetData>& InOwnerAssetDataArray, bool bInAllowCollapse, bool bInShowUsedTextures)
	: MaterialListDelegates( InMaterialListDelegates )
	, DetailLayoutBuilder( InDetailLayoutBuilder )
	, MaterialListBuilder( new FMaterialListBuilder )
	, bAllowCollpase(bInAllowCollapse)
	, bShowUsedTextures(bInShowUsedTextures)
	, OwnerAssetDataArray(InOwnerAssetDataArray)
{
}

void FMaterialList::OnDisplayMaterialsForElement( int32 SlotIndex )
{
	// We now want to display all the materials in the element
	ExpandedSlots.Add( SlotIndex );

	MaterialListBuilder->Empty();
	MaterialListDelegates.OnGetMaterials.ExecuteIfBound( *MaterialListBuilder );

	OnRebuildChildren.ExecuteIfBound();
}

void FMaterialList::OnHideMaterialsForElement( int32 SlotIndex )
{
	// No longer want to expand the element
	ExpandedSlots.Remove( SlotIndex );

	// regenerate the materials
	MaterialListBuilder->Empty();
	MaterialListDelegates.OnGetMaterials.ExecuteIfBound( *MaterialListBuilder );
	
	OnRebuildChildren.ExecuteIfBound();
}


void FMaterialList::Tick( float DeltaTime )
{
	// Check each material to see if its still valid.  This allows the material list to stay up to date when materials are changed out from under us
	if( MaterialListDelegates.OnGetMaterials.IsBound() )
	{
		// Whether or not to refresh the material list
		bool bRefreshMaterialList = false;

		// Get the current list of materials from the user
		MaterialListBuilder->Empty();
		MaterialListDelegates.OnGetMaterials.ExecuteIfBound( *MaterialListBuilder );

		if( MaterialListBuilder->GetNumMaterials() != DisplayedMaterials.Num() )
		{
			// The array sizes differ so we need to refresh the list
			bRefreshMaterialList = true;
		}
		else
		{
			// Compare the new list against the currently displayed list
			for( int32 MaterialIndex = 0; MaterialIndex < MaterialListBuilder->MaterialSlots.Num(); ++MaterialIndex )
			{
				const FMaterialListItem& Item = MaterialListBuilder->MaterialSlots[MaterialIndex];

				// The displayed materials is out of date if there isn't a 1:1 mapping between the material sets
				if( !DisplayedMaterials.IsValidIndex( MaterialIndex ) || DisplayedMaterials[ MaterialIndex ] != Item )
				{
					bRefreshMaterialList = true;
					break;
				}
			}
		}

		if (!bRefreshMaterialList && MaterialListDelegates.OnMaterialListDirty.IsBound())
		{
			bRefreshMaterialList = MaterialListDelegates.OnMaterialListDirty.Execute();
		}

		if( bRefreshMaterialList )
		{
			OnRebuildChildren.ExecuteIfBound();
		}
	}
}

void FMaterialList::GenerateHeaderRowContent( FDetailWidgetRow& NodeRow )
{
	NodeRow.CopyAction(FUIAction(FExecuteAction::CreateSP(this, &FMaterialList::OnCopyMaterialList), FCanExecuteAction::CreateSP(this, &FMaterialList::OnCanCopyMaterialList)));
	NodeRow.PasteAction(FUIAction(FExecuteAction::CreateSP(this, &FMaterialList::OnPasteMaterialList)));

	if (const TSharedPtr<FOnPasteFromText> OnPasteFromTextDelegate = NodeRow.OnPasteFromTextDelegate.Pin();
		OnPasteFromTextDelegate.IsValid())
	{
		OnPasteFromTextDelegate->AddSP(this, &FMaterialList::OnPasteMaterialListFromText);
	}

	if (bAllowCollpase)
	{
		NodeRow.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("MaterialHeaderTitle", "Materials"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}
}

void FMaterialList::GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder )
{
	ViewedMaterials.Empty();
	DisplayedMaterials.Empty();
	if( MaterialListBuilder->GetNumMaterials() > 0 )
	{
		const FName RowTagName = "Material";
		DisplayedMaterials = MaterialListBuilder->MaterialSlots;

		MaterialListBuilder->Sort();
		TArray<FMaterialListItem>& MaterialSlots = MaterialListBuilder->MaterialSlots;

		int32 CurrentSlot = INDEX_NONE;
		bool bDisplayAllMaterialsInSlot = true;
		for( auto It = MaterialSlots.CreateConstIterator(); It; ++It )
		{
			const FMaterialListItem& Material = *It;

			if( CurrentSlot != Material.SlotIndex )
			{
				// We've encountered a new slot.  Make a widget to display that
				CurrentSlot = Material.SlotIndex;

				uint32 NumMaterials = MaterialListBuilder->GetNumMaterialsInSlot(CurrentSlot);

				// If an element is expanded we want to display all its materials
				bool bWantToDisplayAllMaterials = NumMaterials > 1 && ExpandedSlots.Contains(CurrentSlot);

				// If we are currently displaying an expanded set of materials for an element add a link to collapse all of them
				if( bWantToDisplayAllMaterials )
				{
					FDetailWidgetRow& ChildRow = ChildrenBuilder.AddCustomRow( LOCTEXT( "HideAllMaterialSearchString", "Hide All Materials") );

					FFormatNamedArguments Arguments;
					Arguments.Add(TEXT("ElementSlot"), CurrentSlot);
					ChildRow
					.RowTag(RowTagName)
					.ValueContent()
					.MaxDesiredWidth(0.0f)// No Max Width
					[
						SNew( SBox )
							.HAlign( HAlign_Center )
							[
								SNew( SHyperlink )
									.TextStyle( FAppStyle::Get(), "MaterialList.HyperlinkStyle" )
									.Text( FText::Format(LOCTEXT("HideAllMaterialLinkText", "Hide All Materials on Element {ElementSlot}"), Arguments ) )
									.OnNavigate( this, &FMaterialList::OnHideMaterialsForElement, CurrentSlot )
							]
					];
				}	

				if( NumMaterials > 1 && !bWantToDisplayAllMaterials )
				{
					// The current slot has multiple elements to view
					bDisplayAllMaterialsInSlot = false;

					FDetailWidgetRow& ChildRow = ChildrenBuilder.AddCustomRow( FText::GetEmpty() );
					ChildRow.RowTag(RowTagName);

					AddMaterialItem( ChildRow, CurrentSlot, FMaterialListItem( NULL, CurrentSlot, true ), !bDisplayAllMaterialsInSlot, Material.CurrentComponent.Get() );
				}
				else
				{
					bDisplayAllMaterialsInSlot = true;
				}

			}

			// Display each thumbnail element unless we shouldn't display multiple materials for one slot
			if( bDisplayAllMaterialsInSlot )
			{
				FDetailWidgetRow& ChildRow = ChildrenBuilder.AddCustomRow( Material.Material.IsValid()? FText::FromString(Material.Material->GetName()) : FText::GetEmpty() );
				ChildRow.RowTag(RowTagName);

				AddMaterialItem( ChildRow, CurrentSlot, Material, !bDisplayAllMaterialsInSlot, Material.CurrentComponent.Get() );
			}
		}
	}
	else
	{
		FDetailWidgetRow& ChildRow = ChildrenBuilder.AddCustomRow( LOCTEXT("NoMaterials", "No Materials") );

		ChildRow
		[
			SNew( SBox )
			.HAlign( HAlign_Center )
			[
				SNew( STextBlock )
				.Text( LOCTEXT("NoMaterials", "No Materials") ) 
				.Font( IDetailLayoutBuilder::GetDetailFont() )
			]
		];
	}		
}

bool FMaterialList::OnCanCopyMaterialList() const
{
	if (MaterialListDelegates.OnCanCopyMaterialList.IsBound())
	{
		return MaterialListDelegates.OnCanCopyMaterialList.Execute();
	}

	return false;
}

void FMaterialList::OnCopyMaterialList()
{
	if (MaterialListDelegates.OnCopyMaterialList.IsBound())
	{
		MaterialListDelegates.OnCopyMaterialList.Execute();
	}
}

void FMaterialList::OnPasteMaterialList()
{
	if (MaterialListDelegates.OnPasteMaterialList.IsBound())
	{
		MaterialListDelegates.OnPasteMaterialList.Execute();
	}
}

void FMaterialList::OnPasteMaterialListFromText(
	const FString& InTag,
	const FString& InText,
	const TOptional<FGuid>& InOperationId)
{
	if (const TSharedPtr<FOnPasteFromText> OnPasteFromTextDelegate = MaterialListDelegates.OnPasteFromText.Pin();
		OnPasteFromTextDelegate.IsValid())
	{
		if (OnPasteFromTextDelegate->IsBound())
		{
			OnPasteFromTextDelegate->Broadcast(InTag, InText, InOperationId);
		}
	}
}

bool FMaterialList::OnCanCopyMaterialItem(int32 CurrentSlot) const
{
	if (MaterialListDelegates.OnCanCopyMaterialItem.IsBound())
	{
		return MaterialListDelegates.OnCanCopyMaterialItem.Execute(CurrentSlot);
	}

	return false;
}

void FMaterialList::OnCopyMaterialItem(int32 CurrentSlot)
{
	if (MaterialListDelegates.OnCopyMaterialItem.IsBound())
	{
		MaterialListDelegates.OnCopyMaterialItem.Execute(CurrentSlot);
	}
}

void FMaterialList::OnPasteMaterialItem(int32 CurrentSlot)
{
	if (MaterialListDelegates.OnPasteMaterialItem.IsBound())
	{
		MaterialListDelegates.OnPasteMaterialItem.Execute(CurrentSlot);
	}
}
			
void FMaterialList::AddMaterialItem( FDetailWidgetRow& Row, int32 CurrentSlot, const FMaterialListItem& Item, bool bDisplayLink, UActorComponent* InActorComponent)
{
	uint32 NumMaterials = MaterialListBuilder->GetNumMaterialsInSlot(CurrentSlot);

	TSharedRef<FMaterialItemView> NewView = FMaterialItemView::Create( Item
		, MaterialListDelegates.OnMaterialChanged
		, MaterialListDelegates.OnGenerateCustomNameWidgets
		, MaterialListDelegates.OnGenerateCustomMaterialWidgets
		, MaterialListDelegates.OnResetMaterialToDefaultClicked
		, MaterialListDelegates.OnGenerateExtraBottomWidget
		, NumMaterials
		, bShowUsedTextures);

	TSharedPtr<SWidget> RightSideContent;
	if( bDisplayLink )
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("NumMaterials"), NumMaterials);

		RightSideContent = 
			SNew( SBox )
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Top)
				[
					SNew( SHyperlink )
					.TextStyle( FAppStyle::Get(), "MaterialList.HyperlinkStyle" )
					.Text( FText::Format(LOCTEXT("DisplayAllMaterialLinkText", "Display {NumMaterials} materials"), Arguments) )
					.ToolTipText( LOCTEXT("DisplayAllMaterialLink_ToolTip","Display all materials. Drag and drop a material here to replace all materials.") )
					.OnNavigate( this, &FMaterialList::OnDisplayMaterialsForElement, CurrentSlot )
				];
	}
	else
	{
		RightSideContent = NewView->CreateValueContent( DetailLayoutBuilder, OwnerAssetDataArray, InActorComponent );
		ViewedMaterials.Add( NewView );
	}

	FResetToDefaultOverride ResetToDefaultOverride = FResetToDefaultOverride::Create(
		TAttribute<bool>(NewView, &FMaterialItemView::GetResetToBaseVisibility),
		FSimpleDelegate::CreateSP(NewView, &FMaterialItemView::OnResetToBaseClicked)
	);

	Row.OverrideResetToDefault(ResetToDefaultOverride);

	Row.CopyAction(FUIAction(FExecuteAction::CreateSP(this, &FMaterialList::OnCopyMaterialItem, Item.SlotIndex), FCanExecuteAction::CreateSP(this, &FMaterialList::OnCanCopyMaterialItem, Item.SlotIndex)));
	Row.PasteAction(FUIAction(FExecuteAction::CreateSP(this, &FMaterialList::OnPasteMaterialItem, Item.SlotIndex)));

	Row.NameContent()
	[
		NewView->CreateNameContent()
	]
	.ValueContent()
	.MinDesiredWidth(250.f)
	.MaxDesiredWidth(0.0f) // no maximum
	[
		RightSideContent.ToSharedRef()
	];

	if(USceneComponent* SceneComponent = Cast<USceneComponent>(InActorComponent))
	{
		UObject* OwnerObject = nullptr;
		FString PropertyPath;
		FProperty* MaterialProperty = nullptr;
		if (SceneComponent->GetMaterialPropertyPath(Item.SlotIndex, OwnerObject, PropertyPath, MaterialProperty))
		{
			Row.IsEnabled(!PropertyEditorPolicy::IsPropertyEditConst(MaterialProperty, SceneComponent) && SceneComponent->CanEditChange(MaterialProperty));
		}		
	}
}

#undef LOCTEXT_NAMESPACE