// Copyright Epic Games, Inc. All Rights Reserved.


#include "ObjectPropertyNode.h"
#include "Misc/ConfigCacheIni.h"
#include "CategoryPropertyNode.h"
#include "ItemPropertyNode.h"
#include "ObjectEditorUtils.h"
#include "EditorCategoryUtils.h"
#include "PropertyEditorHelpers.h"
#include "UObject/PropertyBagRepository.h"

#if WITH_EDITORONLY_DATA
#include "Engine/Blueprint.h"
#endif

FObjectPropertyNode::FObjectPropertyNode(void)
	: FComplexPropertyNode()
	, BaseClass(NULL)
{
}

FObjectPropertyNode::~FObjectPropertyNode(void)
{
}

UObject* FObjectPropertyNode::GetUObject(int32 InIndex)
{
	check(Objects.IsValidIndex(InIndex));
	return Objects[InIndex].Get();
}

const UObject* FObjectPropertyNode::GetUObject(int32 InIndex) const
{
	check(Objects.IsValidIndex(InIndex));
	return Objects[InIndex].Get();
}

UPackage* FObjectPropertyNode::GetUPackage(int32 InIndex)
{
	UObject* Obj = GetUObject(InIndex);
	if (Obj)
	{
		TWeakObjectPtr<UPackage>* Package = ObjectToPackageMapping.Find(Obj);
		return Package ? Package->Get() : Obj->GetOutermost();
	}
	return nullptr;
}

const UPackage* FObjectPropertyNode::GetUPackage(int32 InIndex) const
{
	return const_cast<FObjectPropertyNode*>(this)->GetUPackage(InIndex);
}

// Adds a new object to the list.
void FObjectPropertyNode::AddObject( UObject* InObject )
{
	if (HasNodeFlags(EPropertyNodeFlags::ResolveInstanceDataObjects))
	{
		UE::FPropertyBagRepository& Repository = UE::FPropertyBagRepository::Get();
		if (UObject* Found = Repository.FindInstanceDataObject(InObject))
		{
			InObject = Found;
		}
	}
	Objects.Add( InObject );
}

// Adds new objects to the list.
void FObjectPropertyNode::AddObjects(const TArray<UObject*>& InObjects)
{
	for (UObject* Object : InObjects)
	{
		AddObject(Object);
	}
}

// Removes an object from the list.
void FObjectPropertyNode::RemoveObject( UObject* InObject )
{
	const int32 idx = Objects.Find( InObject );

	if( idx != INDEX_NONE )
	{
		Objects.RemoveAt( idx, 1 );
	}
	else
	{
		if (HasNodeFlags(EPropertyNodeFlags::ResolveInstanceDataObjects))
		{
			UE::FPropertyBagRepository& Repository = UE::FPropertyBagRepository::Get();
			if (UObject* Found = Repository.FindInstanceDataObject(InObject))
			{
				RemoveObject(Found);
			}
		}
	}
}

// Removes all objects from the list.
void FObjectPropertyNode::RemoveAllObjects()
{
	Objects.Empty();
}

void FObjectPropertyNode::SetObjectPackageOverrides(const TMap<TWeakObjectPtr<UObject>, TWeakObjectPtr<UPackage>>& InMapping)
{
	ObjectToPackageMapping = InMapping;
}

void FObjectPropertyNode::ClearObjectPackageOverrides()
{
	ObjectToPackageMapping.Empty();
}

// Purges any objects marked pending kill from the object list
bool FObjectPropertyNode::PurgeKilledObjects()
{
	bool bDidPurgeObjects = false;

	// Purge any objects that are marked pending kill from the object list
	for (auto It = Objects.CreateIterator(); It; ++It)
	{
		const TWeakObjectPtr<UObject>& Object = *It;
		if (Object.IsStale())
		{
			bDidPurgeObjects = true;
			It.RemoveCurrent();
		}
	}

	return bDidPurgeObjects;
}

// Called when the object list is finalized, Finalize() finishes the property window setup.
void FObjectPropertyNode::Finalize()
{
	// May be NULL...
	UClass* OldBase = GetObjectBaseClass();

	// Find an appropriate base class.
	SetBestBaseClass();
	if (BaseClass.IsValid() && BaseClass->HasAnyClassFlags(CLASS_CollapseCategories) )
	{
		SetNodeFlags(EPropertyNodeFlags::ShowCategories, false );
	}
}

TArray<UStruct*> FObjectPropertyNode::GetAllStructures()
{
	TArray<UStruct*> RetVal;
	RetVal.Reserve(SparseClassDataInstances.Num() + 1);
	if (UStruct* BaseStruct = GetBaseStructure())
	{
		RetVal.Add(BaseStruct);
	}

	for (const auto& ClassSparseDataPair : SparseClassDataInstances)
	{
		const TTuple<UScriptStruct*, void*>& SparseInstance = ClassSparseDataPair.Value;
		RetVal.AddUnique(SparseInstance.Key);
	}

	return RetVal;
}

TArray<const UStruct*> FObjectPropertyNode::GetAllStructures() const
{
	TArray<const UStruct*> RetVal;
	RetVal.Reserve(SparseClassDataInstances.Num() + 1);
	if (const UStruct* BaseStruct = GetBaseStructure())
	{
		RetVal.Add(BaseStruct);
	}

	for (const auto& ClassSparseDataPair : SparseClassDataInstances)
	{
		const TTuple<UScriptStruct*, void*>& SparseInstance = ClassSparseDataPair.Value;
		RetVal.AddUnique(SparseInstance.Key);
	}
	return RetVal;
}

// Intended for a function that gets the number of elements in a container
DECLARE_DELEGATE_RetVal_OneParam(int32, FGetNumDelegate, const void*);

bool FObjectPropertyNode::GetReadAddressUncached(const FPropertyNode& InNode,
											   bool InRequiresSingleSelection,
											   FReadAddressListData* OutAddresses,
											   bool bComparePropertyContents,
											   bool bObjectForceCompare,
											   bool bArrayPropertiesCanDifferInSize) const
{
	// Are any objects selected for property editing?
	if( !GetNumObjects())
	{
		return false;
	}

	const FProperty* InItemProperty = InNode.GetProperty();
	// Is there a InItemProperty bound to the InItemProperty window?
	if( !InItemProperty )
	{
		return false;
	}

	UStruct* OwnerStruct = InItemProperty->GetOwnerStruct();
	if (!OwnerStruct || OwnerStruct->IsStructTrashed())
	{
		// Verify that the property is not part of an invalid trash class
		return false;
	}

	// Requesting a single selection?
	if( InRequiresSingleSelection && GetNumObjects() > 1)
	{
		// Fail: we're editing properties for multiple objects.
		return false;
	}

	//assume all properties are the same unless proven otherwise
	bool bAllTheSame = true;

	//////////////////////////////////////////

	// If this item is the child of a container, return NULL if there is a different number
	// of items in the container in different objects, when multi-selecting.

	FArrayProperty* ArrayOuter = InItemProperty->GetOwner<FArrayProperty>();
	FSetProperty* SetOuter = InItemProperty->GetOwner<FSetProperty>();
	FMapProperty* MapOuter = InItemProperty->GetOwner<FMapProperty>();

	if( ArrayOuter || SetOuter || MapOuter)
	{
		const FPropertyNode* ParentPropertyNode = InNode.GetParentNode();
		check(ParentPropertyNode);
		const UObject* TempObject = GetUObject(0);
		if( TempObject )
		{
			uint8* BaseAddr = ParentPropertyNode->GetValueBaseAddressFromObject(TempObject);
			if( BaseAddr )
			{
				if ( ArrayOuter )
				{
					FScriptArrayHelper ArrayHelper(ArrayOuter, BaseAddr);
					const int32 Num = ArrayHelper.Num();
					for (int32 ObjIndex = 1; ObjIndex < GetNumObjects(); ++ObjIndex)
					{
						TempObject = GetUObject(ObjIndex);
						BaseAddr = ParentPropertyNode->GetValueBaseAddressFromObject(TempObject);

						ArrayHelper = FScriptArrayHelper(ArrayOuter, BaseAddr);
						if (BaseAddr && Num != ArrayHelper.Num())
						{
							bAllTheSame = false;
						}
					}
				}
				else if ( SetOuter )
				{
					const int32 Num = FScriptSetHelper::Num(BaseAddr);
					for (int32 ObjIndex = 1; ObjIndex < GetNumObjects(); ++ObjIndex)
					{
						TempObject = GetUObject(ObjIndex);
						BaseAddr = ParentPropertyNode->GetValueBaseAddressFromObject(TempObject);

						if (BaseAddr && Num != FScriptSetHelper::Num(BaseAddr))
						{
							bAllTheSame = false;
						}
					}
				}
				else if ( MapOuter )
				{
					FScriptMapHelper MapHelper(MapOuter, BaseAddr);
					const int32 Num = MapHelper.Num();
					for (int32 ObjIndex = 1; ObjIndex < GetNumObjects(); ++ObjIndex)
					{
						TempObject = GetUObject(ObjIndex);
						BaseAddr = ParentPropertyNode->GetValueBaseAddressFromObject(TempObject);

						if (BaseAddr)
						{
							MapHelper = FScriptMapHelper(MapOuter, BaseAddr);
							if (Num != MapHelper.Num())
							{
								bAllTheSame = false;
							}
						}
					}
				}
			}
		}
	}

	uint8* BaseAddr = InNode.GetValueBaseAddressFromObject(GetUObject(0));
	if (BaseAddr)
	{
		// If the item is an array or set itself, return NULL if there are a different number of
		// items in the container in different objects, when multi-selecting.

		const FArrayProperty* ArrayProp = CastField<FArrayProperty>(InItemProperty);
		const FSetProperty* SetProp = CastField<FSetProperty>(InItemProperty);
		const FMapProperty* MapProp = CastField<FMapProperty>(InItemProperty);

		if (ArrayProp || SetProp || MapProp)
		{
			// This flag is an override for array properties which want to display e.g. the "Clear" and "Empty"
			// buttons, even though the array properties may differ in the number of elements.
			if (!bArrayPropertiesCanDifferInSize)
			{
				const UObject* TempObject = GetUObject(0);

				if (ArrayProp)
				{
					FScriptArrayHelper ArrayHelper(ArrayProp, BaseAddr);
					int32 const Num = ArrayHelper.Num();
					for (int32 ObjIndex = 1; ObjIndex < GetNumObjects(); ObjIndex++)
					{
						TempObject = GetUObject(ObjIndex);
						if (TempObject)
						{
							ArrayHelper = FScriptArrayHelper(ArrayProp, InNode.GetValueBaseAddressFromObject(TempObject));
							if (Num != ArrayHelper.Num())
							{
								bAllTheSame = false;
							}
						}
					}
				}
				else if (SetProp)
				{
					int32 const Num = FScriptSetHelper::Num(BaseAddr);
					for (int32 ObjIndex = 1; ObjIndex < GetNumObjects(); ++ObjIndex)
					{
						TempObject = GetUObject(ObjIndex);
						if (TempObject && Num != FScriptSetHelper::Num(InNode.GetValueBaseAddressFromObject(TempObject)))
						{
							bAllTheSame = false;
						}
					}
				}
				else if (MapProp)
				{
					FScriptMapHelper MapHelper(MapProp, BaseAddr);
					int32 const Num = MapHelper.Num();
					for (int32 ObjIndex = 1; ObjIndex < GetNumObjects(); ++ObjIndex)
					{
						TempObject = GetUObject(ObjIndex);
						if (TempObject)
						{
							MapHelper = FScriptMapHelper(MapProp, InNode.GetValueBaseAddressFromObject(TempObject));
							if (Num != MapHelper.Num())
							{
								bAllTheSame = false;
							}
						}
					}
				}
			}
		}
		else
		{
			if ( bComparePropertyContents || !CastField<FObjectPropertyBase>(InItemProperty) || bObjectForceCompare )
			{
				// Make sure the value of this InItemProperty is the same in all selected objects.
				for( int32 ObjIndex = 1 ; ObjIndex < GetNumObjects() ; ObjIndex++ )
				{
					if (!InItemProperty->Identical(BaseAddr, InNode.GetValueBaseAddressFromObject(GetUObject(ObjIndex))))
					{
						bAllTheSame = false;
						break;
					}
				}
			}
			else
			{
				if (CastField<FObjectPropertyBase>(InItemProperty))
				{
					// We don't want to exactly compare component properties.  However, we
					// need to be sure that all references are either valid or invalid.

					// If BaseObj is NON-NULL, all other objects' properties should also be non-NULL.
					// If BaseObj is NULL, all other objects' properties should also be NULL.
					UObject* BaseObj = CastField<FObjectPropertyBase>(InItemProperty)->GetObjectPropertyValue(BaseAddr);

					for (int32 ObjIndex = 1; ObjIndex < GetNumObjects(); ObjIndex++)
					{
						UObject* CurObj = CastField<FObjectPropertyBase>(InItemProperty)->GetObjectPropertyValue(InNode.GetValueBaseAddressFromObject(GetUObject(ObjIndex)));
						if (   ( !BaseObj && CurObj )			// BaseObj is NULL, but this InItemProperty is non-NULL!
							|| ( BaseObj && !CurObj ) )			// BaseObj is non-NULL, but this InItemProperty is NULL!
						{
							bAllTheSame = false;
							break;
						}
					}
				}
			}
		}
	}

		// Write addresses to the output.
	if (OutAddresses != nullptr)
		{
		for (int32 ObjIndex = 0; ObjIndex < GetNumObjects(); ++ObjIndex)
		{	
			const UObject* TempObject = GetUObject(ObjIndex);
			if (TempObject)
			{
				OutAddresses->Add(TempObject, InNode.GetValueBaseAddressFromObject(TempObject));
			}
		}
	}

	// Everything checked out and we have usable addresses.
	return bAllTheSame;
}

/**
 * fills in the OutAddresses array with the addresses of all of the available objects.
 * @param InItem		The property to get objects from.
 * @param OutAddresses	Storage array for all of the objects' addresses.
 */
bool FObjectPropertyNode::GetReadAddressUncached(const FPropertyNode& InNode, FReadAddressListData& OutAddresses ) const
{
	// Are any objects selected for property editing?
	if( !GetNumObjects())
	{
		return false;
	}

	const FProperty* InItemProperty = InNode.GetProperty();
	// Is there a InItemProperty bound to the InItemProperty window?
	if( !InItemProperty )
	{
		return false;
	}


	// Write addresses to the output.
	for (int32 ObjIndex = 0 ; ObjIndex < GetNumObjects() ; ++ObjIndex)
	{
		const UObject* TempObject = GetUObject(ObjIndex);
		if (TempObject != nullptr)
		{
			OutAddresses.Add(TempObject, InNode.GetValueBaseAddressFromObject(TempObject));
		}
	}

	// Everything checked out and we have usable addresses.
	return true;
}

uint8* FObjectPropertyNode::GetValueBaseAddress(uint8* StartAddress, bool bIsSparseData, bool bIsStruct) const
{
	uint8* Result = StartAddress;

	UClass* ClassObject;
	if (!bIsSparseData)
	{
		if ( (ClassObject=Cast<UClass>((UObject*)Result)) != NULL )
		{
			Result = (uint8*)ClassObject->GetDefaultObject();
		}
	}

	return Result;
}


void FObjectPropertyNode::InitBeforeNodeFlags()
{
	// Note: InitBeforeNodeFlags is called _after_ initializing many node flags in FPropertyNode::InitNode
	StoredProperty = Property;
	Property = NULL;
	
	if (HasNodeFlags(EPropertyNodeFlags::ResolveInstanceDataObjects))
	{
		// Replace any objects that have been set with their IDO counterparts if they exist
		for (int32 Index = 0, End = Objects.Num(); Index < End; ++Index)
		{
			TWeakObjectPtr<UObject> Object = Objects[Index];
			TStrongObjectPtr<UObject> Pinned = Object.Pin();
			if (Pinned)
			{
				if (HasNodeFlags(EPropertyNodeFlags::ResolveInstanceDataObjects))
				{
					UE::FPropertyBagRepository& Repository = UE::FPropertyBagRepository::Get();
				
					if (UObject* Found = Repository.FindInstanceDataObject(Pinned.Get()))
					{
						Objects[Index] = Found;
					}					
				}
			}
		}
	}

	Finalize();
}

void FObjectPropertyNode::InitChildNodes()
{
	InternalInitChildNodes();
}

static TSharedPtr<FPropertyNode> FindChildCategory( TSharedPtr<FPropertyNode> ParentNode, FName CategoryName )
{
	for( int32 CurNodeIndex = 0; CurNodeIndex < ParentNode->GetNumChildNodes(); ++CurNodeIndex )
	{
		TSharedPtr<FPropertyNode>& ChildNode = ParentNode->GetChildNode(CurNodeIndex);
		check( ChildNode.IsValid() );

		// Is this a category node?
		FCategoryPropertyNode* ChildCategoryNode = ChildNode->AsCategoryNode();
		if( ChildCategoryNode != nullptr && ChildCategoryNode->GetCategoryName() == CategoryName )
		{
			return ChildNode;
		}
	}
	return nullptr;
}

void FObjectPropertyNode::GetCategoryProperties(const TSet<UClass*>& ClassesToConsider, const FProperty* CurrentProperty, bool bShouldShowDisableEditOnInstance, bool bShouldShowHiddenProperties,
	TSet<FName>& CategoriesFromProperties, TArray<FName>& SortedCategories, int32& NonBlueprintCategoryIndex)
{
	static const FName Name_bShowOnlyWhenTrue("bShowOnlyWhenTrue");
	static const FName Name_ShowOnlyInnerProperties("ShowOnlyInnerProperties");

	FName CategoryName = FObjectEditorUtils::GetCategoryFName(CurrentProperty);

	for (UClass* Class : ClassesToConsider)
	{
		if (FEditorCategoryUtils::IsCategoryHiddenFromClass(Class, CategoryName.ToString()))
		{
			HiddenCategories.Add(CategoryName);
			break;
		}
	}

	bool bMetaDataAllowVisible = true;
	const FString& ShowOnlyWhenTrueString = CurrentProperty->GetMetaData(Name_bShowOnlyWhenTrue);
	if (ShowOnlyWhenTrueString.Len())
	{
		//ensure that the metadata visibility string is actually set to true in order to show this property
		GConfig->GetBool(TEXT("UnrealEd.PropertyFilters"), *ShowOnlyWhenTrueString, bMetaDataAllowVisible, GEditorPerProjectIni);
	}

	if (bMetaDataAllowVisible)
	{
		if (PropertyEditorHelpers::ShouldBeVisible(*this, CurrentProperty) && !HiddenCategories.Contains(CategoryName))
		{
			bool bIsAlreadyInSet;
			CategoriesFromProperties.Add(CategoryName, &bIsAlreadyInSet);
			if (!bIsAlreadyInSet)
			{
				// We add categories that are not visible in Blueprints to the very end of the array, while keeping visible ones in the front.
				// This is because we sort categories by the user-defined order in Blueprints afterwards. However, this user-defined order only
				// includes categories that are visible to Blueprints. All other categories will be sorted to the bottom.
				// By ordering these non-BP-visible categories to the bottom even for non-BP-generated classes, we can keep the category order consistent for both.
				// See also SortCategoriesByUserDefinedOrder and SMyBlueprint::CollectAllActions.
				const bool bShowAsBlueprintVar = !CurrentProperty->HasAnyPropertyFlags(CPF_Parm) && CurrentProperty->HasAllPropertyFlags(CPF_BlueprintVisible);
				if (bShowAsBlueprintVar)
				{
					SortedCategories.Insert(CategoryName, NonBlueprintCategoryIndex++);
				}
				else
				{
					SortedCategories.Add(CategoryName);
				}
			}
		}
	}

	if (CurrentProperty->HasMetaData(Name_ShowOnlyInnerProperties))
	{
		const FStructProperty* StructProperty = CastField<const FStructProperty>(CurrentProperty);
		if (StructProperty)
		{
			for (TFieldIterator<FProperty> It(StructProperty->Struct); It; ++It)
			{
				GetCategoryProperties(ClassesToConsider, *It, bShouldShowDisableEditOnInstance, bShouldShowHiddenProperties, CategoriesFromProperties, SortedCategories, NonBlueprintCategoryIndex);
			}
		}
	}
}

#if WITH_EDITORONLY_DATA

static void SortCategoriesByUserDefinedOrder(TArray<FName>& SortedCategories, const TSet<UClass*>& ClassesToConsider)
{
	// Acquire the sort indices from classes in display text format.
	TMap<FString, int32> DisplayTextSortOrders;

	for (UClass* Class : ClassesToConsider)
	{
		if (UBlueprint* BP = Cast<UBlueprint>(Class->ClassGeneratedBy))
		{
			for (const FName& DisplayTextCategory : BP->CategorySorting)
			{
				// These are already in display text format.
				const int32 SortOrder = DisplayTextSortOrders.Num();
				DisplayTextSortOrders.FindOrAdd(DisplayTextCategory.ToString(), SortOrder);
			}
		}

		TArray<FString> ClassPrioritizeCategories;
		Class->GetPrioritizeCategories(ClassPrioritizeCategories);
		for (const FString& MetaCategory : ClassPrioritizeCategories)
		{
			// These are user declared meta specifiers, so we sanitize them.
			const FString DisplayTextCategory = FEditorCategoryUtils::GetCategoryDisplayString(MetaCategory);
			const int32 SortOrder = DisplayTextSortOrders.Num();
			DisplayTextSortOrders.FindOrAdd(DisplayTextCategory, SortOrder);
		}
	}

	// Stable sort to keep the order of e.g. undefined items.
	Algo::StableSort(SortedCategories, [&DisplayTextSortOrders](const FName& CategoryA, const FName& CategoryB)
		{
			const FString DisplayTextCategoryA = FEditorCategoryUtils::GetCategoryDisplayString(CategoryA.ToString());
			const FString DisplayTextCategoryB = FEditorCategoryUtils::GetCategoryDisplayString(CategoryB.ToString());
			const int32* SortOrderA = DisplayTextSortOrders.Find(DisplayTextCategoryA);
			const int32* SortOrderB = DisplayTextSortOrders.Find(DisplayTextCategoryB);
			if (SortOrderA)
			{
				if (SortOrderB)
				{
					return *SortOrderA < *SortOrderB; // Sort by user-defined order.
				}
				else
				{
					return true; // Move undefined items to the bottom.
				}
			}
			return false; // Move undefined items to the bottom and keep their order.
		});
}

#endif

void FObjectPropertyNode::InternalInitChildNodes( FName SinglePropertyName )
{
	HiddenCategories.Reset();
	SparseClassDataInstances.Reset();
	// Assemble a list of category names by iterating over all fields of BaseClass.

	// build a list of classes that we need to look at
	TSet<UClass*> ClassesToConsider;
	for( int32 i = 0; i < GetNumObjects(); ++i )
	{
		UObject* TempObject = GetUObject( i );
		if( TempObject )
		{
			ClassesToConsider.Add( TempObject->GetClass() );
		}
	}

	// Create a merged list of user-enforced sorted info, hidden category info, etc...
	TArray<FName> SortedCategories;

	const bool bShouldShowHiddenProperties = !!HasNodeFlags(EPropertyNodeFlags::ShouldShowHiddenProperties);
	const bool bShouldShowDisableEditOnInstance = !!HasNodeFlags(EPropertyNodeFlags::ShouldShowDisableEditOnInstance);
	const bool bResolveInstanceDataObjects = !!HasNodeFlags(EPropertyNodeFlags::ResolveInstanceDataObjects);

	if (UClass* ResolvedBaseClass = BaseClass.Get())
	{
		TSet<FName> CategoriesFromProperties;
		int32 NonBlueprintCategoryIndex = 0;
		for (TFieldIterator<FProperty> It(ResolvedBaseClass); It; ++It)
		{
			GetCategoryProperties(ClassesToConsider, *It, bShouldShowDisableEditOnInstance, bShouldShowHiddenProperties, CategoriesFromProperties, SortedCategories, NonBlueprintCategoryIndex);
		}

		if (UScriptStruct* SparseClassDataStruct = ResolvedBaseClass->GetSparseClassDataStruct())
		{
			SparseClassDataInstances.Add(ResolvedBaseClass, TTuple<UScriptStruct*, void*>(SparseClassDataStruct, ResolvedBaseClass->GetOrCreateSparseClassData()));
		
			for (TFieldIterator<FProperty> It(SparseClassDataStruct); It; ++It)
			{
				GetCategoryProperties(ClassesToConsider, *It, bShouldShowDisableEditOnInstance, bShouldShowHiddenProperties, CategoriesFromProperties, SortedCategories, NonBlueprintCategoryIndex);
			}
		}

#if WITH_EDITORONLY_DATA
		SortCategoriesByUserDefinedOrder(SortedCategories, ClassesToConsider);
#endif
	}


	//////////////////////////////////////////
	// Add the category headers and the child items that belong inside of them.

	// Only show category headers if this is the top level object window and the parent window allows headers.
	if( HasNodeFlags(EPropertyNodeFlags::ShowCategories) )
	{
		FString CategoryDelimiterString;
		CategoryDelimiterString.AppendChar( FPropertyNodeConstants::CategoryDelimiterChar );

		TArray< FPropertyNode* > ParentNodesToSort;

		for( const FName& FullCategoryPath : SortedCategories )
		{
			// Figure out the nesting level for this category
			TArray< FString > FullCategoryPathStrings;
			FullCategoryPath.ToString().ParseIntoArray( FullCategoryPathStrings, *CategoryDelimiterString, true );

			TSharedPtr<FPropertyNode> ParentLevelNode = SharedThis(this);
			FString CurCategoryPathString;
			for( int32 PathLevelIndex = 0; PathLevelIndex < FullCategoryPathStrings.Num(); ++PathLevelIndex )
			{
				// Build up the category path name for the current path level index
				if( CurCategoryPathString.Len() != 0 )
				{
					CurCategoryPathString += FPropertyNodeConstants::CategoryDelimiterChar;
				}
				CurCategoryPathString += FullCategoryPathStrings[ PathLevelIndex ];
				const FName CategoryName( *CurCategoryPathString );

				// Check to see if we've already created a category at the specified path level
				TSharedPtr<FPropertyNode> FoundCategory = FindChildCategory( ParentLevelNode, CategoryName );

				// If we didn't find the category, then we'll need to create it now!
				if( !FoundCategory.IsValid() )
				{
					// Create the category node and assign it to its parent node
					TSharedPtr<FCategoryPropertyNode> NewCategoryNode( new FCategoryPropertyNode );
					{
						NewCategoryNode->SetCategoryName( CategoryName );

						FPropertyNodeInitParams InitParams;
						InitParams.ParentNode = ParentLevelNode;
						InitParams.Property = NULL;
						InitParams.ArrayOffset = 0;
						InitParams.ArrayIndex = INDEX_NONE;
						InitParams.bAllowChildren = true;
						InitParams.bForceHiddenPropertyVisibility = bShouldShowHiddenProperties;
						InitParams.bCreateDisableEditOnInstanceNodes = bShouldShowDisableEditOnInstance;

						NewCategoryNode->InitNode( InitParams );

						// Recursively expand category properties if the category has been flagged for auto-expansion.
						if (BaseClass->IsAutoExpandCategory(*CategoryName.ToString())
							&&	!BaseClass->IsAutoCollapseCategory(*CategoryName.ToString()))
						{
							NewCategoryNode->SetNodeFlags(EPropertyNodeFlags::Expanded, true);
						}

						// Add this node to it's parent.  Note that no sorting happens here, so the parent's
						// list of child nodes will not be in the correct order.  We'll keep track of which
						// nodes we added children to so we can sort them after we're finished adding new nodes.
						ParentLevelNode->AddChildNode(NewCategoryNode);
						ParentNodesToSort.AddUnique( ParentLevelNode.Get() );
					}

					// Descend into the newly created category by using this node as the new parent
					ParentLevelNode = NewCategoryNode;
				}
				else
				{
					ParentLevelNode = FoundCategory;
				}
			}
		}
	}
	else
	{
		TArray<FProperty*> SortedProperties;

		// Iterate over all fields, creating items.
		for( TFieldIterator<FProperty> It(BaseClass.Get()); It; ++It )
		{
			FProperty* CurProp = *It;

			// if a SinglePropertyName was provided, bypass the 'property visibility check' and always include it
			if ((SinglePropertyName == NAME_None && PropertyEditorHelpers::ShouldBeVisible(*this, CurProp)) || CurProp->GetFName() == SinglePropertyName)
			{
				SortedProperties.Add(CurProp);

				if( SinglePropertyName != NAME_None )
				{
					// Generate no other children
					break;
				}
			}
		}

		// Sort the properties if needed.
		if (SortedProperties.Num() > 1)
		{
			PropertyEditorHelpers::OrderPropertiesFromMetadata(SortedProperties);
		}

		// Add nodes for the properties.
		for (FProperty* CurProp : SortedProperties)
		{
			TSharedPtr<FItemPropertyNode> NewItemNode( new FItemPropertyNode );

			FPropertyNodeInitParams InitParams;
			InitParams.ParentNode = SharedThis(this);
			InitParams.Property = CurProp;
			InitParams.ArrayOffset =  0;
			InitParams.ArrayIndex = INDEX_NONE;
			InitParams.bAllowChildren = SinglePropertyName == NAME_None;
			InitParams.bForceHiddenPropertyVisibility = bShouldShowHiddenProperties;
			InitParams.bCreateDisableEditOnInstanceNodes = bShouldShowDisableEditOnInstance;

			NewItemNode->InitNode( InitParams );

			AddChildNode(NewItemNode);
		}
	}

}


TSharedPtr<FPropertyNode> FObjectPropertyNode::GenerateSingleChild( FName ChildPropertyName )
{
	bool bDestroySelf = false;
	DestroyTree(bDestroySelf);

	// No category nodes should be created in single property mode
	SetNodeFlags( EPropertyNodeFlags::ShowCategories, false );

	InternalInitChildNodes( ChildPropertyName );

	if( ChildNodes.Num() > 0 )
	{
		// only one node should be been created
		check( ChildNodes.Num() == 1);

		return ChildNodes[0];
	}

	return nullptr;
}

bool FObjectPropertyNode::IsSparseDataStruct(const UScriptStruct* Struct) const
{
	for (const auto& Instance : SparseClassDataInstances)
	{
		const TTuple<UScriptStruct*, void*>& SparseData = Instance.Value;
		if (SparseData.Key == Struct)
		{
			return true;
		}
	}

	return false;
}

/**
 * Appends my path, including an array index (where appropriate)
 */
bool FObjectPropertyNode::GetQualifiedName(FString& PathPlusIndex, bool bWithArrayIndex, const FPropertyNode* StopParent, bool bIgnoreCategories ) const
{
	bool bAddedAnything = false;
	const TSharedPtr<FPropertyNode> ParentNode = ParentNodeWeakPtr.Pin();
	if (ParentNode && StopParent != ParentNode.Get())
	{
		bAddedAnything = ParentNode->GetQualifiedName(PathPlusIndex, bWithArrayIndex, StopParent, bIgnoreCategories);
	}

	if (bAddedAnything)
	{
		PathPlusIndex += TEXT(".");
	}

	PathPlusIndex += TEXT("Object");
	bAddedAnything = true;

	return bAddedAnything;
}

// Looks at the Objects array and returns the best base class.  Called by
// Finalize(); that is, when the list of selected objects is being finalized.
void FObjectPropertyNode::SetBestBaseClass()
{
	BaseClass = NULL;

	for( int32 x = 0 ; x < Objects.Num() ; ++x )
	{
		UObject* Obj = Objects[x].Get();
		
		if( Obj )
		{
			UClass* ObjClass = Cast<UClass>(Obj);
			if (ObjClass == NULL)
			{
				ObjClass = Obj->GetClass();
			}
			check( ObjClass );

			// Initialize with the class of the first object we encounter.
			if( BaseClass == NULL )
			{
				BaseClass = ObjClass;
			}

			// If we've encountered an object that's not a subclass of the current best baseclass,
			// climb up a step in the class hierarchy.
			while( !ObjClass->IsChildOf( BaseClass.Get() ) )
			{
				BaseClass = BaseClass->GetSuperClass();
			}
		}
	}
}
