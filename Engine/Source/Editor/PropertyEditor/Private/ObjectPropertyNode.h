// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PropertyNode.h"
#include "UObject/WeakFieldPtr.h"

//-----------------------------------------------------------------------------
//	FObjectPropertyNode - Used for the root and various sub-nodes
//-----------------------------------------------------------------------------

//////////////////////////////////////////////////////////////////////////
// Object iteration
typedef TArray< TWeakObjectPtr<UObject> >::TIterator TPropObjectIterator;
typedef TArray< TWeakObjectPtr<UObject> >::TConstIterator TPropObjectConstIterator;

class FObjectPropertyNode : public FComplexPropertyNode
{
public:
	FObjectPropertyNode();
	virtual ~FObjectPropertyNode();

	/** FPropertyNode Interface */
	virtual FObjectPropertyNode* AsObjectNode() override { return this;}
	virtual const FObjectPropertyNode* AsObjectNode() const override { return this; }
	virtual bool GetReadAddressUncached(const FPropertyNode& InNode, bool InRequiresSingleSelection, FReadAddressListData* OutAddresses, bool bComparePropertyContents = true, bool bObjectForceCompare = false, bool bArrayPropertiesCanDifferInSize = false) const override;
	virtual bool GetReadAddressUncached(const FPropertyNode& InNode, FReadAddressListData& OutAddresses) const override;

	/**
	 * Returns the UObject at index "n" of the Objects Array
	 * @param InIndex - index to read out of the array
	 */
	UObject* GetUObject(int32 InIndex);
	const UObject* GetUObject(int32 InIndex) const;

	/**
	 * Returns the UPackage at index "n" of the Objects Array
	 * @param InIndex - index to read out of the array
	 */
	UPackage* GetUPackage(int32 InIndex);
	const UPackage* GetUPackage(int32 InIndex) const;

	/** Returns the number of objects for which properties are currently being edited. */
	int32 GetNumObjects() const	{ return Objects.Num(); }

	/**
	 * Adds a new object to the list.
	 */
	void AddObject( UObject* InObject );

	/** Adds new objects to the list. */
	void AddObjects(const TArray<UObject*>& InObjects);

	/**
	 * Removes an object from the list.
	 */
	void RemoveObject(UObject* InObject);
	/**
	 * Removes all objects from the list.
	 */
	void RemoveAllObjects();

	/** Set overrides that should be used when looking for packages that contain the given object */
	void SetObjectPackageOverrides(const TMap<TWeakObjectPtr<UObject>, TWeakObjectPtr<UPackage>>& InMapping);
	/** Clear overrides that should be used when looking for packages that contain the given object */
	void ClearObjectPackageOverrides();

	/**
	 * Purges any objects marked pending kill from the object list
	 * @return True if any objects were purged, or false otherwise
	 */
	bool PurgeKilledObjects();

	// Called when the object list is finalized, Finalize() finishes the property window setup.
	void Finalize();

	/** @return		The base-est baseclass for objects in this list. */
	UClass*			GetObjectBaseClass()       { return BaseClass.IsValid() ? BaseClass.Get() : nullptr; }
	/** @return		The base-est baseclass for objects in this list. */
	const UClass*	GetObjectBaseClass() const { return BaseClass.IsValid() ? BaseClass.Get() : nullptr; }


	// FComplexPropertyNode implementation
	virtual UStruct* GetBaseStructure() override { return GetObjectBaseClass(); }
	virtual const UStruct* GetBaseStructure() const override{ return GetObjectBaseClass(); }

	virtual TArray<UStruct*> GetAllStructures() override;
	virtual TArray<const UStruct*> GetAllStructures() const override;

	virtual int32 GetInstancesNum() const override{ return GetNumObjects(); }
	virtual uint8* GetMemoryOfInstance(int32 Index) const override
	{
		return (uint8*)GetUObject(Index);
	}
	virtual uint8* GetValuePtrOfInstance(int32 Index, const FProperty* InProperty, const FPropertyNode* InParentNode) const override
	{
		if (InParentNode == nullptr || InProperty == nullptr)
		{
			return nullptr;
		}

		const UObject* Obj = GetUObject(Index);
		if (Obj == nullptr)
		{
			return nullptr;
		}

		uint8* ParentPtr = InParentNode->GetValueAddressFromObject(Obj);
		if (ParentPtr == nullptr)
		{
			return nullptr;
		}

		return InProperty->ContainerPtrToValuePtr<uint8>(ParentPtr);
	}
	virtual TWeakObjectPtr<UObject> GetInstanceAsUObject(int32 Index) const override
	{
		check(Objects.IsValidIndex(Index));
		return Objects[Index];
	}
	virtual EPropertyType GetPropertyType() const override { return EPT_Object; }
	virtual void Disconnect() override
	{
		RemoveAllObjects();
	}

	//////////////////////////////////////////////////////////////////////////
	/** @return		The property stored at this node, to be passed to Pre/PostEditChange. */
	virtual FProperty*		GetStoredProperty()		{ return StoredProperty.IsValid() ? StoredProperty.Get() : nullptr; }

	TPropObjectIterator			ObjectIterator()			{ return TPropObjectIterator( Objects ); }
	TPropObjectConstIterator	ObjectConstIterator() const	{ return TPropObjectConstIterator( Objects ); }


	/** Generates a single child from the provided property name.  Any existing children are destroyed */
	virtual TSharedPtr<FPropertyNode> GenerateSingleChild( FName ChildPropertyName ) override;

	/**
	 * @return The hidden categories 
	 */
	const TSet<FName>& GetHiddenCategories() const { return HiddenCategories; }

	bool IsRootNode() const { return ParentNodeWeakPtr.Pin() == nullptr; }

	/**
	 * @return True if Struct is one of the sparse data structures used by this object
	 */
	bool IsSparseDataStruct(const UScriptStruct* Struct) const;
protected:
	/** FPropertyNode interface */
	virtual void InitBeforeNodeFlags() override;
	virtual void InitChildNodes() override;
	virtual bool GetQualifiedName( FString& PathPlusIndex, const bool bWithArrayIndex, const FPropertyNode* StopParent = nullptr, bool bIgnoreCategories = false ) const override;
	virtual uint8* GetValueBaseAddress(uint8* Base, bool bIsSparseData, bool bIsStruct) const override;
	/**
	 * Looks at the Objects array and creates the best base class.  Called by
	 * Finalize(); that is, when the list of selected objects is being finalized.
	 */
	void SetBestBaseClass();

private:
	/**
	 * Creates child nodes
	 * 
	 * @param SingleChildName	The property name of a single child to create instead of all childen
	 */
	void InternalInitChildNodes( FName SingleChildName = NAME_None );

	/** If CurrentProperty should show up in the ClassesToConsider make sure its category is in SortedCategories and CategoriesFromProperties. */
	void GetCategoryProperties(const TSet<UClass*>& ClassesToConsider, const FProperty* CurrentProperty, bool bShouldShowDisableEditOnInstance, bool bShouldShowHiddenProperties,
	TSet<FName>& CategoriesFromProperties, TArray<FName>& SortedCategories, int32& NonBlueprintCategoryIndex);
private:
	/** The list of objects we are editing properties for. */
	TArray< TWeakObjectPtr<UObject> >		Objects;

	/** The lowest level base class for objects in this list. */
	TWeakObjectPtr<UClass>					BaseClass;

	/**
	 * The property passed to Pre/PostEditChange calls.  
	 */
	TWeakFieldPtr<FProperty>				StoredProperty;

	/**
	 * Set of all category names hidden by the objects in this node
	 */
	TSet<FName> HiddenCategories;



	/* 
	 * Contains the structure and memory location for storing data of that type. Used to read and write to sidecar structs for the class
	 */
	TMap<UClass*, TTuple<UScriptStruct*, void*>> SparseClassDataInstances;

	/** Object -> Package re-mapping */
	TMap<TWeakObjectPtr<UObject>, TWeakObjectPtr<UPackage>> ObjectToPackageMapping;
};

