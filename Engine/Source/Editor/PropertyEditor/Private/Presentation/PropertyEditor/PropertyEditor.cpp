// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "Modules/ModuleManager.h"
#include "UnrealEdGlobals.h"
#include "CategoryPropertyNode.h"
#include "ItemPropertyNode.h"
#include "ObjectPropertyNode.h"

#include "SceneOutlinerFilters.h"
#include "IDetailPropertyRow.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorHelpers.h"
#include "Editor.h"
#include "EditorClassUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "IConfigEditorModule.h"
#include "PropertyNode.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "EditConditionParser.h"
#include "EditConditionContext.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "ActorTreeItem.h"
#include "PropertyHandleImpl.h"
#include "IPropertyUtilities.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "PropertyEditor"

DEFINE_LOG_CATEGORY_STATIC(LogPropertyEditor, Log, All);

namespace UE::Private
{
	static FAutoConsoleVariable CVarExpandAddedItem(
		TEXT("DetailsPanel.ExpandAddedItems"),
		true,
		TEXT("Automatically expands items added to container in the details panel"),
		ECVF_Default);
	
	static UClass* GetMetaClass(const FProperty* ForProperty)
	{
		if(!ForProperty)
		{
			return nullptr;
		}
		else if(const FClassProperty* ClassProp = CastField<FClassProperty>(ForProperty))
		{
			return ToRawPtr(ClassProp->MetaClass);
		}
		else if(const FSoftClassProperty* SoftClassProp = CastField<FSoftClassProperty>(ForProperty))
		{
			return ToRawPtr(SoftClassProp->MetaClass);
		}
		else
		{
			return ToRawPtr(FEditorClassUtils::GetClassFromString(ForProperty->GetMetaData("MetaClass")));
		}
	}
}

const FString FPropertyEditor::MultipleValuesDisplayName = NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values").ToString();

TSharedRef< FPropertyEditor > FPropertyEditor::Create( const TSharedRef< class FPropertyNode >& InPropertyNode, const TSharedRef<class IPropertyUtilities >& InPropertyUtilities )
{
	return MakeShareable( new FPropertyEditor( InPropertyNode, InPropertyUtilities ) );
}

FPropertyEditor::FPropertyEditor( const TSharedRef<FPropertyNode>& InPropertyNode, const TSharedRef<IPropertyUtilities>& InPropertyUtilities )
	: PropertyHandle( NULL )
	, PropertyNode( InPropertyNode )
	, PropertyUtilities( InPropertyUtilities )
{
	// FPropertyEditor isn't built to handle CategoryNodes
	check( InPropertyNode->AsCategoryNode() == NULL );

	PropertyHandle = PropertyEditorHelpers::GetPropertyHandle( InPropertyNode, PropertyUtilities->GetNotifyHook(), PropertyUtilities );
	check( PropertyHandle.IsValid() && PropertyHandle->IsValidHandle() );
}


FText FPropertyEditor::GetDisplayName() const
{
	FItemPropertyNode* ItemPropertyNode = PropertyNode->AsItemPropertyNode();

	if ( ItemPropertyNode != NULL )
	{
		return ItemPropertyNode->GetDisplayName();
	}

	if (const FComplexPropertyNode* ComplexPropertyNode = PropertyNode->AsComplexNode())
	{
		const FText DisplayName = ComplexPropertyNode->GetDisplayName();

		// Does this property define its own name?
		if (!DisplayName.IsEmpty())
		{
			return DisplayName;
		}
	}

	FString DisplayName;
	PropertyNode->GetQualifiedName( DisplayName, true );
	return FText::FromString(DisplayName);
}

FText FPropertyEditor::GetToolTipText() const
{
	return PropertyNode->GetToolTipText();
}

FString FPropertyEditor::GetDocumentationLink() const
{
	FString DocumentationLink;

	if( PropertyNode->AsItemPropertyNode() )
	{
		FProperty* Property = PropertyNode->GetProperty();
		DocumentationLink = PropertyEditorHelpers::GetDocumentationLink( Property );
	}

	return DocumentationLink;
}

FString FPropertyEditor::GetDocumentationExcerptName() const
{
	FString ExcerptName;

	if( PropertyNode->AsItemPropertyNode() )
	{
		FProperty* Property = PropertyNode->GetProperty();
		ExcerptName = PropertyEditorHelpers::GetDocumentationExcerptName( Property );
	}

	return ExcerptName;
}

FString FPropertyEditor::GetValueAsString() const 
{
	FString Str;

	if( PropertyHandle->GetValueAsFormattedString( Str ) == FPropertyAccess::MultipleValues )
	{
		Str = MultipleValuesDisplayName;
	}

	return Str;
}

FString FPropertyEditor::GetValueAsDisplayString() const
{
	FString Str;

	if( PropertyHandle->GetValueAsDisplayString( Str ) == FPropertyAccess::MultipleValues )
	{
		Str = MultipleValuesDisplayName;
	}

	return Str;
}

FText FPropertyEditor::GetValueAsText() const
{
	FText Text;

	if( PropertyHandle->GetValueAsFormattedText( Text ) == FPropertyAccess::MultipleValues )
	{
		Text = NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values");
	}

	return Text;
}

FText FPropertyEditor::GetValueAsDisplayText() const
{
	FText Text;

	if( PropertyHandle->GetValueAsDisplayText( Text ) == FPropertyAccess::MultipleValues )
	{
		Text = NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values");
	}

	return Text;
}

bool FPropertyEditor::PropertyIsA(const FFieldClass* Class) const
{
	return PropertyNode->GetProperty() != NULL ? PropertyNode->GetProperty()->IsA( Class ) : false;
}

bool FPropertyEditor::IsFavorite() const 
{ 
	return PropertyNode->HasNodeFlags( EPropertyNodeFlags::IsFavorite ) != 0; 
}

bool FPropertyEditor::IsChildOfFavorite() const 
{ 
	return PropertyNode->IsChildOfFavorite(); 
}

void FPropertyEditor::ToggleFavorite() 
{ 
	PropertyUtilities->ToggleFavorite( SharedThis( this ) ); 
}

void FPropertyEditor::UseSelected()
{
	OnUseSelected();
}

void FPropertyEditor::OnUseSelected()
{
	PropertyHandle->SetObjectValueFromSelection();
}

void FPropertyEditor::AddItem()
{
	// This action must be deferred until next tick so that we avoid accessing invalid data before we have a chance to tick
	PropertyUtilities->EnqueueDeferredAction( FSimpleDelegate::CreateSP( this, &FPropertyEditor::OnAddItem ) );
}

void FPropertyEditor::AddGivenItem(const FString& InGivenItem)
{
	// This action must be deferred until next tick so that we avoid accessing invalid data before we have a chance to tick
	PropertyUtilities->EnqueueDeferredAction(FSimpleDelegate::CreateSP(this, &FPropertyEditor::OnAddGivenItem, InGivenItem));
}

void FPropertyEditor::OnAddItem()
{
	// Check to make sure that the property is a valid container
	TSharedPtr<IPropertyHandleArray> ArrayHandle = PropertyHandle->AsArray();
	TSharedPtr<IPropertyHandleSet> SetHandle = PropertyHandle->AsSet();
	TSharedPtr<IPropertyHandleMap> MapHandle = PropertyHandle->AsMap();

	check(ArrayHandle.IsValid() || SetHandle.IsValid() || MapHandle.IsValid());

	FPropertyHandleItemAddResult Result;

	if (ArrayHandle.IsValid())
	{
		Result = ArrayHandle->AddItem();
	}
	else if (SetHandle.IsValid())
	{
		Result = SetHandle->AddItem();
	}
	else if (MapHandle.IsValid())
	{
		Result = MapHandle->AddItem();
	}

	// Expand containers when an item is added to them
	FPropertyNode* ParentNode = &PropertyNode.Get();
	while (ParentNode)
	{
		// We expand all parents. Normally, they'd all be expanded (otherwise the property would not be visible and the
		// user would not be able to add a new item), but with customizations it's possible to have something like:
		//  > [Some Label Widget]    | 3 Array elements (+)
		// where the row belongs to a struct property, while the label widget and the array are (for example) two child
		// properties of said struct. In this case, without expanding parents, adding an array item would try to expand
		// the property node which does not have its own row, while the row itself would remain collapsed.
		ParentNode->SetNodeFlags(EPropertyNodeFlags::Expanded, true);
		ParentNode = ParentNode->GetParentNode();
	}

	if (Result.GetAccessResult() == FPropertyAccess::Success && UE::Private::CVarExpandAddedItem->GetBool())
	{
		TSharedPtr<FPropertyNode> ChildNode;
		bool bHasChildNode = PropertyNode->GetChildNode(Result.GetIndex(), ChildNode);
		if (bHasChildNode)
		{
			ChildNode->SetNodeFlags(EPropertyNodeFlags::Expanded, true);
		}
	}
	
	//In case the property is show in the favorite category refresh the whole tree
	if (PropertyNode->IsFavorite())
	{
		ForceRefresh();
	}
}

void FPropertyEditor::OnAddGivenItem(const FString InGivenItem)
{

	const FScopedTransaction Transaction(FText::Format(LOCTEXT("AddElementToArray", "Add element to {0} array "), PropertyNode->GetDisplayName()));

	FProperty* NodeProperty = PropertyNode->GetProperty();

	FReadAddressList ReadAddresses;
	PropertyNode->GetReadAddress(!!PropertyNode->HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ReadAddresses, true, false, true);
	if (ReadAddresses.Num())
	{
		TArray< TMap<FString, int32> > ArrayIndicesPerObject;

		// List of top level objects sent to the PropertyChangedEvent
		TArray<const UObject*> TopLevelObjects;
		TopLevelObjects.Reserve(ReadAddresses.Num());

		FObjectPropertyNode* ObjectNode = PropertyNode->FindObjectItemParent();
		FArrayProperty* Array = CastField<FArrayProperty>(NodeProperty);

		check(Array);

		for (int32 i = 0; i < ReadAddresses.Num(); ++i)
		{
			void* Addr = ReadAddresses.GetAddress(i);
			if (Addr)
			{
				//add on array index so we can tell which entry just changed
				ArrayIndicesPerObject.Add(TMap<FString, int32>());
				FPropertyValueImpl::GenerateArrayIndexMapToObjectNode(ArrayIndicesPerObject[i], &PropertyNode.Get());

				UObject* Obj = ObjectNode ? ObjectNode->GetUObject(i) : nullptr;
				if (Obj)
				{
					if ((Obj->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) ||
						(Obj->HasAnyFlags(RF_DefaultSubObject) && Obj->GetOuter()->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))))
					{
						PropertyNode->PropagateContainerPropertyChange(Obj, Addr, EPropertyArrayChangeType::Add, -1);
					}

					TopLevelObjects.Add(Obj);
				}

				int32 Index = INDEX_NONE;

				FScriptArrayHelper	ArrayHelper(Array, Addr);
				Index = ArrayHelper.AddValue();

				ArrayIndicesPerObject[i].Add(NodeProperty->GetName(), Index);
			}
		}

		FPropertyChangedEvent ChangeEvent(NodeProperty, EPropertyChangeType::ArrayAdd, MakeArrayView(TopLevelObjects));
		ChangeEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);
		PropertyNode->FixPropertiesInEvent(ChangeEvent);

		// Both Insert and Add are deferred so you need to rebuild the parent node's children
		GetPropertyNode()->RebuildChildren();

		// Check to make sure that the property is a valid container
		TSharedPtr<IPropertyHandleArray> ArrayHandle = PropertyHandle->AsArray();

		check(ArrayHandle.IsValid());

		TSharedPtr<IPropertyHandle> ElementHandle;

		if (ArrayHandle.IsValid())
		{
			uint32 Last;
			ArrayHandle->GetNumElements(Last);
			ElementHandle = ArrayHandle->GetElement(Last - 1);
		}

		if (ElementHandle.IsValid())
		{
			ElementHandle->SetValueFromFormattedString(InGivenItem);
		}
	}
}

void FPropertyEditor::SetOptionalItem(FProperty* NewProperty, const UClass* NewObjectClass)
{
	// This action must be deferred until next tick so that we avoid accessing invalid data before we have a chance to tick
	PropertyUtilities->EnqueueDeferredAction(FSimpleDelegate::CreateSP(this, &FPropertyEditor::OnSetOptionalValue, NewProperty, NewObjectClass));
}

void FPropertyEditor::ClearOptionalItem()
{
	// This action must be deferred until next tick so that we avoid accessing invalid data before we have a chance to tick
	PropertyUtilities->EnqueueDeferredAction(FSimpleDelegate::CreateSP(this, &FPropertyEditor::OnClearOptionalValue));
}

void FPropertyEditor::OnSetOptionalValue(FProperty* NewProperty, const UClass* NewObjectClass)
{
	TSharedPtr<IPropertyHandleOptional> OptionalHandle = PropertyHandle->AsOptional();
	if (!OptionalHandle.IsValid())
	{
		// This could be the value calling as we share UI between options and their value
		OptionalHandle = PropertyHandle->GetParentHandle()->AsOptional();
	}

	if (OptionalHandle.IsValid())
	{
		OptionalHandle->SetOptionalValue(NewProperty, NewObjectClass);
	}
}

void FPropertyEditor::OnClearOptionalValue()
{
	TSharedPtr<IPropertyHandleOptional> OptionalHandle = PropertyHandle->AsOptional();
	if (!OptionalHandle.IsValid())
	{
		// This could be the value calling as we share UI between options and their value
		OptionalHandle = PropertyHandle->GetParentHandle()->AsOptional();
	}

	if (OptionalHandle.IsValid())
	{
		OptionalHandle->ClearOptionalValue();
	}
}

void FPropertyEditor::ClearItem()
{
	OnClearItem();
}

void FPropertyEditor::OnClearItem()
{
	static const FString None("None");
	PropertyHandle->SetValueFromFormattedString( None );
}

void FPropertyEditor::MakeNewBlueprint()
{
	FProperty* NodeProperty = PropertyNode->GetProperty();
	UClass* Class = UE::Private::GetMetaClass(NodeProperty);

	UClass* RequiredInterface = FEditorClassUtils::GetClassFromString(NodeProperty->GetMetaData("MustImplement"));

	if (Class)
	{
		UBlueprint* Blueprint = FKismetEditorUtilities::CreateBlueprintFromClass(LOCTEXT("CreateNewBlueprint", "Create New Blueprint"), Class, FString::Printf(TEXT("New%s"),*Class->GetName()));

		if(Blueprint != NULL && Blueprint->GeneratedClass)
		{
			if (RequiredInterface != nullptr && FKismetEditorUtilities::CanBlueprintImplementInterface(Blueprint, RequiredInterface))
			{
				FBlueprintEditorUtils::ImplementNewInterface(Blueprint, RequiredInterface->GetClassPathName());
			}
			
			PropertyHandle->SetValueFromFormattedString(Blueprint->GeneratedClass->GetPathName());

			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Blueprint);
		}
	}
}

void FPropertyEditor::EditConfigHierarchy()
{
	FProperty* NodeProperty = PropertyNode->GetProperty();

	IConfigEditorModule& ConfigEditorModule = FModuleManager::LoadModuleChecked<IConfigEditorModule>("ConfigEditor");
	ConfigEditorModule.CreateHierarchyEditor(NodeProperty);
	FGlobalTabmanager::Get()->TryInvokeTab(FName(TEXT("ConfigEditor")));
}

void FPropertyEditor::InsertItem()
{
	// This action must be deferred until next tick so that we avoid accessing invalid data before we have a chance to tick
	PropertyUtilities->EnqueueDeferredAction( FSimpleDelegate::CreateSP( this, &FPropertyEditor::OnInsertItem ) );
}

void FPropertyEditor::OnInsertItem()
{
	TSharedPtr<IPropertyHandleArray> ArrayHandle = PropertyHandle->GetParentHandle()->AsArray();
	check(ArrayHandle.IsValid());

	int32 Index = PropertyNode->GetArrayIndex();
	
	// Insert is only supported on arrays, not maps or sets
	ArrayHandle->Insert(Index);

	//In case the property is show in the favorite category refresh the whole tree
	if (PropertyNode->IsFavorite() || (PropertyNode->GetParentNode() != nullptr && PropertyNode->GetParentNode()->IsFavorite()))
	{
		ForceRefresh();
	}
}

void FPropertyEditor::DeleteItem()
{
	// This action must be deferred until next tick so that we avoid accessing invalid data before we have a chance to tick
	PropertyUtilities->EnqueueDeferredAction( FSimpleDelegate::CreateSP( this, &FPropertyEditor::OnDeleteItem ) );
}

void FPropertyEditor::OnDeleteItem()
{
	TSharedPtr<IPropertyHandleArray> ArrayHandle = PropertyHandle->GetParentHandle()->AsArray();
	TSharedPtr<IPropertyHandleSet> SetHandle = PropertyHandle->GetParentHandle()->AsSet();
	TSharedPtr<IPropertyHandleMap> MapHandle = PropertyHandle->GetParentHandle()->AsMap();

	check(ArrayHandle.IsValid() || SetHandle.IsValid() || MapHandle.IsValid());

	int32 Index = PropertyNode->GetArrayIndex();

	if (ArrayHandle.IsValid())
	{
		ArrayHandle->DeleteItem(Index);
	}
	else if (SetHandle.IsValid())
	{
		SetHandle->DeleteItem(Index);
	}
	else if (MapHandle.IsValid())
	{
		MapHandle->DeleteItem(Index);
	}

	//In case the property is show in the favorite category refresh the whole tree
	if (PropertyNode->IsFavorite() || (PropertyNode->GetParentNode() != nullptr && PropertyNode->GetParentNode()->IsFavorite()))
	{
		ForceRefresh();
	}
}

void FPropertyEditor::DuplicateItem()
{
	// This action must be deferred until next tick so that we avoid accessing invalid data before we have a chance to tick
	PropertyUtilities->EnqueueDeferredAction( FSimpleDelegate::CreateSP( this, &FPropertyEditor::OnDuplicateItem ) );
}

void FPropertyEditor::OnDuplicateItem()
{
	TSharedPtr<IPropertyHandleArray> ArrayHandle = PropertyHandle->GetParentHandle()->AsArray();
	check(ArrayHandle.IsValid());

	int32 Index = PropertyNode->GetArrayIndex();
	
	ArrayHandle->DuplicateItem(Index);

	//In case the property is show in the favorite category refresh the whole tree
	if (PropertyNode->IsFavorite() || (PropertyNode->GetParentNode() != nullptr && PropertyNode->GetParentNode()->IsFavorite()))
	{
		ForceRefresh();
	}
}

void FPropertyEditor::BrowseTo()
{
	OnBrowseTo();
}

void FPropertyEditor::OnBrowseTo()
{
	// Sync the content browser or level editor viewport to the object(s) specified by the given property.
	SyncToObjectsInNode( PropertyNode );
}

void FPropertyEditor::EmptyArray()
{
	// This action must be deferred until next tick so that we avoid accessing invalid data before we have a chance to tick
	PropertyUtilities->EnqueueDeferredAction( FSimpleDelegate::CreateSP( this, &FPropertyEditor::OnEmptyArray ) );
}

void FPropertyEditor::OnEmptyArray()
{
	TSharedPtr<IPropertyHandleArray> ArrayHandle = PropertyHandle->AsArray();
	TSharedPtr<IPropertyHandleSet> SetHandle = PropertyHandle->AsSet();
	TSharedPtr<IPropertyHandleMap> MapHandle = PropertyHandle->AsMap();

	check(ArrayHandle.IsValid() || SetHandle.IsValid() || MapHandle.IsValid());

	if (ArrayHandle.IsValid())
	{
		ArrayHandle->EmptyArray();
	}
	else if (SetHandle.IsValid())
	{
		SetHandle->Empty();
	}
	else if (MapHandle.IsValid())
	{
		MapHandle->Empty();
	}

	//In case the property is show in the favorite category refresh the whole tree
	if (PropertyNode->IsFavorite())
	{
		ForceRefresh();
	}
}

bool FPropertyEditor::DoesPassFilterRestrictions() const
{
	return PropertyNode->HasNodeFlags( EPropertyNodeFlags::IsSeenDueToFiltering ) != 0;
}

bool FPropertyEditor::IsEditConst(const bool bIncludeEditCondition) const
{
	return PropertyNode->IsEditConst(bIncludeEditCondition);
}

bool FPropertyEditor::SupportsEditConditionToggle() const
{
	return PropertyNode->SupportsEditConditionToggle();
}

bool FPropertyEditor::HasEditCondition() const
{
	return PropertyNode->HasEditCondition();
}

bool FPropertyEditor::IsEditConditionMet() const
{
	return PropertyNode->IsEditConditionMet();
}

bool FPropertyEditor::IsOnlyVisibleWhenEditConditionMet() const
{
	return PropertyNode->IsOnlyVisibleWhenEditConditionMet();
}

void FPropertyEditor::ToggleEditConditionState()
{
	const FScopedTransaction Transaction(FText::Format(LOCTEXT("SetEditConditionState", "Set {0} edit condition state "), PropertyNode->GetDisplayName()));

	PropertyNode->NotifyPreChange( PropertyNode->GetProperty(), PropertyUtilities->GetNotifyHook() );

	PropertyNode->ToggleEditConditionState();
	
	TArray<UObject*> InstanceObjects;

	const FPropertyNode* Cur = &PropertyNode.Get();
	const FComplexPropertyNode* Found = nullptr;

	TArray<TMap<FString, int32>> ArrayIndicesPerObject;
	while( InstanceObjects.Num() == 0 )
	{
		Found = Cur->AsComplexNode();
		if( Found )
		{
			ArrayIndicesPerObject.AddDefaulted(Found->GetInstancesNum());
			for (int32 ObjectIndex = 0; ObjectIndex < Found->GetInstancesNum(); ++ObjectIndex)
			{
				FPropertyValueImpl::GenerateArrayIndexMapToObjectNode(ArrayIndicesPerObject[ObjectIndex], &PropertyNode.Get());
				if (TWeakObjectPtr<UObject> Object = Found->GetInstanceAsUObject(ObjectIndex); Object.IsValid())
				{
					InstanceObjects.Add(Object.Get());
				}
			}
		}
		Cur = Cur->GetParentNode();
		if( !Cur )
		{
			// There is a break in the parent chain
			break;
		}
	}
	
	FPropertyChangedEvent ChangeEvent(PropertyNode->GetProperty(), EPropertyChangeType::ToggleEditable, InstanceObjects);
	ChangeEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);
	PropertyNode->NotifyPostChange( ChangeEvent, PropertyUtilities->GetNotifyHook() );
	PropertyUtilities->NotifyFinishedChangingProperties(ChangeEvent);
}

void FPropertyEditor::OnGetClassesForAssetPicker( TArray<const UClass*>& OutClasses )
{
	FProperty* NodeProperty = GetPropertyNode()->GetProperty();

	FObjectPropertyBase* ObjProp = CastField<FObjectPropertyBase>( NodeProperty );

	// This class and its children are the classes that we can show objects for
	UClass* AllowedClass = ObjProp ? ToRawPtr(ObjProp->PropertyClass) : ToRawPtr(UObject::StaticClass());

	OutClasses.Add( AllowedClass );
}

void FPropertyEditor::OnAssetSelected( const FAssetData& AssetData )
{
	// Set the object found from the asset picker
	GetPropertyHandle()->SetValueFromFormattedString( AssetData.IsValid() ? AssetData.GetAsset()->GetPathName() : TEXT("None") );
}

void FPropertyEditor::OnActorSelected( AActor* InActor )
{
	// Update the name like we would a picked asset
	OnAssetSelected(InActor);
}

void FPropertyEditor::OnGetActorFiltersForSceneOutliner( TSharedPtr<FSceneOutlinerFilters>& OutFilters )
{
	struct Local
	{
		static bool IsFilteredActor( const AActor* Actor, TSharedRef<FPropertyEditor> PropertyEditor )
		{
			const TSharedRef<FPropertyNode> PropertyNode = PropertyEditor->GetPropertyNode();
			FProperty* NodeProperty = PropertyNode->GetProperty();

			FObjectPropertyBase* ObjProp = CastField<FObjectPropertyBase>( NodeProperty );

			// This class and its children are the classes that we can show objects for
			UClass* AllowedClass = ObjProp ? ToRawPtr(ObjProp->PropertyClass) : ToRawPtr(AActor::StaticClass());

			return Actor->IsA( AllowedClass );
		}
	};

	OutFilters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateStatic( &Local::IsFilteredActor, AsShared() ) );
}

bool FPropertyEditor::IsPropertyEditingEnabled() const
{
	return PropertyUtilities->IsPropertyEditingEnabled() && (!PropertyNode->HasEditCondition() || PropertyNode->IsEditConditionMet());
}

void FPropertyEditor::ForceRefresh()
{
	PropertyUtilities->ForceRefresh();
}

void FPropertyEditor::RequestRefresh()
{
	PropertyUtilities->RequestRefresh();
}

TSharedRef< FPropertyNode > FPropertyEditor::GetPropertyNode() const
{
	return PropertyNode;
}

const FProperty* FPropertyEditor::GetProperty() const
{
	return PropertyNode->GetProperty();
}

TSharedRef< IPropertyHandle > FPropertyEditor::GetPropertyHandle() const
{
	return PropertyHandle.ToSharedRef();
}

void FPropertyEditor::SyncToObjectsInNode( const TWeakPtr< FPropertyNode >& WeakPropertyNode )
{
#if WITH_EDITOR

	if ( !GUnrealEd )
	{
		return;
	}

	TSharedPtr< FPropertyNode > PropertyNode = WeakPropertyNode.Pin();
	check(PropertyNode.IsValid());
	FProperty* NodeProperty = PropertyNode->GetProperty();

	FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>( NodeProperty );
	FInterfaceProperty* IntProp = CastField<FInterfaceProperty>( NodeProperty );
	{
		UClass* PropertyClass = UObject::StaticClass();
		if( ObjectProperty )
		{
			PropertyClass = ObjectProperty->PropertyClass;
		}
		else if( IntProp )
		{
			// Note: this should be IntProp->InterfaceClass but we're using UObject as the class  to work around InterfaceClass not working with FindObject
			PropertyClass = UObject::StaticClass();
		}

		// Get a list of addresses for objects handled by the property window.
		FReadAddressList ReadAddresses;
		PropertyNode->GetReadAddress( !!PropertyNode->HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ReadAddresses, false );

		// GetReadAddresses will only provide a list of addresses if the property was properly formed, objects were selected, and only one object was selected if the node has the SingleSelectOnly flag.
		// If a list of addresses is provided, GetReadAddress may still return false but we can operate on the property addresses even if they have different values.
		check( ReadAddresses.Num() > 0 );

		// Create a list of object names.
		TArray<FString> ObjectNames;
		ObjectNames.Empty( ReadAddresses.Num() );

		// Copy each object's object property name off into the name list.
		for ( int32 AddrIndex = 0 ; AddrIndex < ReadAddresses.Num() ; ++AddrIndex )
		{
			new( ObjectNames ) FString();
			uint8* Address = ReadAddresses.GetAddress(AddrIndex);
			if( Address )
			{
				NodeProperty->ExportText_Direct(ObjectNames[AddrIndex], Address, Address, NULL, PPF_None );
			}
		}


		// Create a list of objects to sync the generic browser to.
		TArray<UObject*> Objects;
		for ( int32 ObjectIndex = 0 ; ObjectIndex < ObjectNames.Num() ; ++ObjectIndex )
		{
			UObject* Object = nullptr;
			if( ObjectNames[ObjectIndex].Contains( TEXT(".")) )
			{
				Object = StaticFindObject(PropertyClass, nullptr, *ObjectNames[ObjectIndex]);
				if (!Object)
				{
					Object = StaticLoadObject(PropertyClass, nullptr, *ObjectNames[ObjectIndex]);
				}
			}
			else
			{
				Object = StaticFindFirstObject(PropertyClass, *ObjectNames[ObjectIndex], EFindFirstObjectOptions::EnsureIfAmbiguous);
			}

			if ( Object )
			{
				// If the selected object is a blueprint generated class, then browsing to it in the content browser should instead point to the blueprint
				// Note: This code needs to change once classes are the top level asset in the content browser and/or blueprint classes are displayed in the content browser
				if (UClass* ObjectAsClass = Cast<UClass>(Object))
				{
					if (ObjectAsClass->ClassGeneratedBy != NULL)
					{
						Object = ObjectAsClass->ClassGeneratedBy;
					}
				}

				Objects.Add( Object );
			}
		}

		// If a single actor is selected, sync to its location in the level editor viewport instead of the content browser.
		if( Objects.Num() == 1 && Objects[0]->IsA<AActor>() )
		{
			AActor* Actor = CastChecked<AActor>(Objects[0]);

			if (Actor->GetLevel())
			{
				TArray<AActor*> Actors;
				Actors.Add(Actor);

				GEditor->SelectNone(/*bNoteSelectionChange=*/false, /*bDeselectBSPSurfs=*/true);
				GEditor->SelectActor(Actor, /*InSelected=*/true, /*bNotify=*/true, /*bSelectEvenIfHidden=*/true);

				// Jump to the location of the actor
				GEditor->MoveViewportCamerasToActor(Actors, /*bActiveViewportOnly=*/false);
			}
		}
		else if ( Objects.Num() > 0 )
		{
			const FModifierKeysState KeyState = FSlateApplication::Get().GetModifierKeys();
			if (KeyState.IsAltDown( ))
			{
				for (UObject *Obj : Objects)
				{
					GEditor->EditObject(Obj);
				}
			}
			else
			{
				GEditor->SyncBrowserToObjects(Objects);
			}
		}
	}

#endif
}

#undef LOCTEXT_NAMESPACE
