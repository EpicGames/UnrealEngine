// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PropertyHandle.h"
#include "SceneOutlinerFwd.h"

class FEditConditionExpression;
class FEditConditionContext;
class FPropertyNode;
class IPropertyHandle;
class IPropertyUtilities;

class FPropertyEditor : public TSharedFromThis<FPropertyEditor>	
{
public:
	static TSharedRef<FPropertyEditor> Create(const TSharedRef<FPropertyNode>& InPropertyNode, const TSharedRef<IPropertyUtilities>& InPropertyUtilities);

	/** @return The String containing the value of the property */
	FString GetValueAsString() const;

	/** @return The String containing the value of the property, possibly using an alternate form more suitable for display in the UI */
	FString GetValueAsDisplayString() const;

	/** @return The String containing the value of the property as Text */
	FText GetValueAsText() const;

	/** @return The String containing the value of the property as Text, possibly using an alternate form more suitable for display in the UI */
	FText GetValueAsDisplayText() const;

	bool PropertyIsA(const FFieldClass* InClass) const;

	bool IsFavorite() const;

	bool IsChildOfFavorite() const;

	void ToggleFavorite();

	bool IsPropertyEditingEnabled() const;

	void ForceRefresh();

	void RequestRefresh();

	/**	@return Whether the property is editconst. If bIncludeEditCondition is false,
	 *	this will return true if the property is ever editable (as though it always passes any specified EditCondition). */
	bool IsEditConst(const bool bIncludeEditCondition = true) const;

	/** @return Whether this property should have an edit condition toggle. */
	bool SupportsEditConditionToggle() const;

	/** Toggle the current state of the edit condition if this SupportsEditConditionToggle() */
	void ToggleEditConditionState();

	/**	@return Whether the property has a condition which must be met before allowing editing of it's value */
	bool HasEditCondition() const;

	/**	@return Whether the condition has been met to allow editing of this property's value */
	bool IsEditConditionMet() const;

	/**	@return Whether this property derives its visibility from its edit condition */
	bool IsOnlyVisibleWhenEditConditionMet() const;

	/**	@return The tooltip for this property editor */
	FText GetToolTipText() const;

	/** @return The documentation link for this property */
	FString GetDocumentationLink() const;

	/** @return The documentation excerpt name to use from this properties documentation link */
	FString GetDocumentationExcerptName() const;

	/**	@return The display name to be used for the property */
	FText GetDisplayName() const;

	/**	@return Whether the property passes the current filter restrictions. If no there are no filter restrictions false will be returned. */
	bool DoesPassFilterRestrictions() const;

	void UseSelected();
	void AddItem();
	void AddGivenItem(const FString& InGivenItem);
	void ClearItem();
	void InsertItem();
	void DeleteItem();
	void DuplicateItem();
	void MakeNewBlueprint();
	void BrowseTo();
	void EmptyArray();
	void OnGetClassesForAssetPicker( TArray<const UClass*>& OutClasses );
	void OnAssetSelected( const FAssetData& AssetData );
	void OnActorSelected( AActor* InActor );
	void OnGetActorFiltersForSceneOutliner( TSharedPtr<FSceneOutlinerFilters>& OutFilters );
	void EditConfigHierarchy();
	
	/**	
	* If this is an editor of an optional, set that optional to the passed in value.
	* If nullptr is passed in, the optional is set and default-initialized.
	*/
	void SetOptionalItem(FProperty* NewProperty, const UClass* NewObjectClass);

	/**	If this is an editor of an optional, unset that optional. */
	void ClearOptionalItem();

	/**	In an ideal world we wouldn't expose these */
	TSharedRef<FPropertyNode> GetPropertyNode() const;
	const FProperty* GetProperty() const;
	TSharedRef<IPropertyHandle> GetPropertyHandle() const;

	static void SyncToObjectsInNode( const TWeakPtr<FPropertyNode>& WeakPropertyNode );

	static const FString MultipleValuesDisplayName;
private:
	FPropertyEditor( const TSharedRef<FPropertyNode>& InPropertyNode, const TSharedRef<IPropertyUtilities>& InPropertyUtilities );

	void OnUseSelected();
	void OnAddItem();
	void OnAddGivenItem(const FString InGivenItem);
	void OnClearItem();
	void OnInsertItem();
	void OnDeleteItem();
	void OnDuplicateItem();
	void OnBrowseTo();
	void OnEmptyArray();
	void OnSetOptionalValue(FProperty* NewProperty, const UClass* NewObjectClass);
	void OnClearOptionalValue();

private:

	/** Property handle for actually reading/writing the value of a property */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** pointer to the property node. */
	TSharedRef<FPropertyNode> PropertyNode;
	 
	/** The property view where this widget resides */
	TSharedRef<IPropertyUtilities> PropertyUtilities;
};
