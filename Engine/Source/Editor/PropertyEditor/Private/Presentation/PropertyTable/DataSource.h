// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PropertyPath.h"
#include "IPropertyTable.h"

class UObjectDataSource : public IDataSource
{
public:

	UObjectDataSource( const TWeakObjectPtr< UObject >& InObject )
		: Object( InObject )
	{
	}

	virtual ~UObjectDataSource() = default;

	virtual TWeakObjectPtr< UObject > AsUObject() const override { return Object; }
	virtual TSharedPtr< FPropertyPath > AsPropertyPath() const override { return NULL; }

	virtual bool IsValid() const override { return Object.IsValid(); }

private:

	TWeakObjectPtr< UObject > Object;
};

class PropertyPathDataSource : public IDataSource
{
public:

	PropertyPathDataSource( const TSharedRef< FPropertyPath >& InPath )
		: Path( InPath )
	{
	}

	virtual ~PropertyPathDataSource() = default;

	virtual TWeakObjectPtr< UObject > AsUObject() const override { return NULL; }
	virtual TSharedPtr< FPropertyPath > AsPropertyPath() const override { return Path; }

	virtual bool IsValid() const override { return true; }

private:

	TSharedRef< FPropertyPath > Path;
};

class NoDataSource : public IDataSource
{
public:

	virtual ~NoDataSource() = default;

	virtual TWeakObjectPtr< UObject > AsUObject() const override { return NULL; }
	virtual TSharedPtr< FPropertyPath > AsPropertyPath() const override { return NULL; }

	virtual bool IsValid() const override { return false; }
};
