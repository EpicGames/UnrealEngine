// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presentation/PropertyTable/PropertyTable.h"
#include "Misc/FeedbackContext.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Editor.h"

#include "Presentation/PropertyTable/PropertyTableColumn.h"
#include "Presentation/PropertyTable/PropertyTableRow.h"
#include "Presentation/PropertyTable/PropertyTableRowHeaderColumn.h"
#include "Presentation/PropertyTable/PropertyTableObjectNameColumn.h"
#include "Presentation/PropertyTable/PropertyTablePropertyNameColumn.h"
#include "PropertyPermissionList.h"

#include "EditConditionParser.h"

#define LOCTEXT_NAMESPACE "PropertyTable"

FPropertyTable::FPropertyTable()
	: SourceObjects()
	, Columns()
	, Rows()
	, CurrentRow( NULL )
	, CurrentCell( NULL )
	, CurrentColumn( NULL )
	, RootPath( FPropertyPath::CreateEmpty() )
	, SelectionUnit( EPropertyTableSelectionUnit::Cell )
	, SelectionMode( ESelectionMode::Multi )
	, ShowRowHeader( true )
	, ShowObjectName( true )
	, ItemHeight( 20 )
	, PrimarySortMode( EColumnSortMode::None )
	, SecondarySortMode( EColumnSortMode::None )
	, AllowUserToChangeRoot( true )
	, bRefreshRequested( false )
	, Orientation( EPropertyTableOrientation::AlignPropertiesInColumns )
	, EditConditionParser( new FEditConditionParser )
{
}

FPropertyTable::~FPropertyTable()
{
	if (ObjectsReplacedHandle.IsValid())
	{
		FCoreUObjectDelegates::OnObjectsReplaced.Remove(ObjectsReplacedHandle);
	}
}

void FPropertyTable::Tick()
{
	// Execute any deferred actions
	for( int32 ActionIndex = 0; ActionIndex < DeferredActions.Num(); ++ActionIndex )
	{
		DeferredActions[ActionIndex].ExecuteIfBound();
	}

	DeferredActions.Empty();

	for( int32 ColumnIdx = 0; ColumnIdx < Columns.Num(); ColumnIdx++ )
	{
		Columns[ColumnIdx]->Tick();
	}

	if( bRefreshRequested )
	{
		bRefreshRequested = false;
		PurgeInvalidObjectNodes();
		UpdateRows();
		
		if( Orientation == EPropertyTableOrientation::AlignPropertiesInRows )
		{
			UpdateColumns();
		}
	}
}

void FPropertyTable::ForceRefresh()
{
	RequestRefresh();
}

void FPropertyTable::RequestRefresh()
{
	bRefreshRequested = true;
}

void FPropertyTable::RequestForceRefresh()
{
	// RequestRefresh is already a deferred ForceRefresh
	RequestRefresh();
}

class FNotifyHook* FPropertyTable::GetNotifyHook() const
{
	return NULL;
	//return View.GetNotifyHook();
}

bool FPropertyTable::AreFavoritesEnabled() const
{
	return false;
}

void FPropertyTable::ToggleFavorite( const TSharedRef< class FPropertyEditor >& PropertyEditor ) const
{
}

void FPropertyTable::CreateColorPickerWindow( const TSharedRef< class FPropertyEditor >& PropertyEditor, bool bUseAlpha ) const
{
	//TableView.CreateColorPickerWindow( PropertyEditor, bUseAlpha );
}

void FPropertyTable::EnqueueDeferredAction( FSimpleDelegate DeferredAction )
{
	DeferredActions.Add( DeferredAction );
}

TSharedPtr<class FAssetThumbnailPool> FPropertyTable::GetThumbnailPool() const
{
	return NULL;
}

const TArray<TSharedRef<class IClassViewerFilter>>& FPropertyTable::GetClassViewerFilters() const
{
	// not implemented
	static TArray<TSharedRef<class IClassViewerFilter>> NotImplemented;
	return NotImplemented;
}

bool FPropertyTable::GetIsUserAllowedToChangeRoot()
{
	return AllowUserToChangeRoot;
}

void FPropertyTable::SetIsUserAllowedToChangeRoot( bool InAllowUserToChangeRoot )
{
	AllowUserToChangeRoot = InAllowUserToChangeRoot;
}

void FPropertyTable::AddColumn( const TWeakObjectPtr< UObject >& Object )
{
	AddColumn( CreateColumn( Object ) );
}

void FPropertyTable::AddColumn( const TWeakFieldPtr< FProperty >& Property )
{
	AddColumn( CreateColumn( Property ) );
}

void FPropertyTable::AddColumn( const TSharedRef< FPropertyPath >& PropertyPath )
{
	AddColumn( CreateColumn( PropertyPath ) );
}

void FPropertyTable::AddColumn( const TSharedRef< class IPropertyTableColumn >& Column )
{
	Columns.AddUnique( Column );
	ColumnsChanged.Broadcast();
}

void FPropertyTable::AddRow( const TWeakObjectPtr< UObject >& Object )
{
	AddRow( CreateRow( Object ) );
}

void FPropertyTable::AddRow( const TWeakFieldPtr< FProperty >& Property )
{
	AddRow( CreateRow( Property ) );
}

void FPropertyTable::AddRow( const TSharedRef< FPropertyPath >& PropertyPath )
{
	AddRow( CreateRow( PropertyPath ) );
}

void FPropertyTable::AddRow( const TSharedRef< class IPropertyTableRow >& Row )
{
	Rows.AddUnique( Row );
	RowsChanged.Broadcast();
}

void FPropertyTable::RemoveColumn( const TSharedRef< class IPropertyTableColumn >& Column )
{
	// Update the selection to exclude cells in the column we are removing
	TSet<TSharedRef<IPropertyTableCell>> NewSelectedCells;
	for(const TSharedRef<IPropertyTableCell>& CurrentSelectedCell : SelectedCells)
	{
		if(CurrentSelectedCell->GetColumn() != Column)
		{
			NewSelectedCells.Add(CurrentSelectedCell);
		}
	}

	Columns.Remove( Column );
	ColumnsChanged.Broadcast();

	if(NewSelectedCells.Num() != SelectedCells.Num())
	{
		SetSelectedCells(NewSelectedCells);
	}
}

void FPropertyTable::RemoveRow( const TSharedRef< class IPropertyTableRow >& Row )
{
	//@todo Consider encapsulating the logic for this check [12/7/2012 Justin.Sargent]
	if ( !Row->HasChildren() && !Row->GetDataSource()->AsPropertyPath().IsValid() )
	{
		const TWeakObjectPtr< UObject > Object = Row->GetDataSource()->AsUObject();
		SourceObjectPropertyNodes.Remove( Object );

		if ( !Object.IsValid() )
		{
			PurgeInvalidObjectNodes();
		}
	}

	// Update the selection to exclude cells in the row we are removing
	TSet<TSharedRef<IPropertyTableCell>> NewSelectedCells;
	for(const TSharedRef<IPropertyTableCell>& CurrentSelectedCell : SelectedCells)
	{
		if(CurrentSelectedCell->GetRow() != Row)
		{
			NewSelectedCells.Add(CurrentSelectedCell);
		}
	}

	Rows.Remove( Row );
	RowsChanged.Broadcast();

	for( const TSharedRef< IPropertyTableColumn >& Column : Columns )
	{
		Column->RemoveCellsForRow( Row );
	}

	if(NewSelectedCells.Num() != SelectedCells.Num())
	{
		SetSelectedCells(NewSelectedCells);
	}
}

void FPropertyTable::ResetTable()
{
	Rows.Reset();
	Columns.Reset();
	SourceObjectPropertyNodes.Reset();
	SelectedColumns.Reset();
	SelectedRows.Reset();
	SelectedCells.Reset();
	StartingCellSelectionRange = nullptr;
	EndingCellSelectionRange = nullptr;
	CurrentRow = nullptr;
	CurrentCell = nullptr;
	CurrentColumn = nullptr;
	LastClickedCell = nullptr;
}

void FPropertyTable::PurgeInvalidObjectNodes()
{
	TArray< TSharedRef< FObjectPropertyNode > > ValidNodes;
	for( auto NodeIter = SourceObjectPropertyNodes.CreateIterator(); NodeIter; ++NodeIter )
	{
		const TWeakObjectPtr< UObject > Object = NodeIter.Key();

		if ( Object.IsValid() )
		{
			ValidNodes.Add( NodeIter.Value() );
		}
	}

	SourceObjectPropertyNodes.Empty();
	for( auto NodeIter = ValidNodes.CreateIterator(); NodeIter; ++NodeIter )
	{
		const TSharedRef< FObjectPropertyNode >& CurrentNode = *NodeIter;
		SourceObjectPropertyNodes.Add( CurrentNode->GetUObject( 0 ), CurrentNode );
	}
}

void FPropertyTable::SetRootPath( const TSharedPtr< FPropertyPath >& Path )
{
	if ( Path.IsValid() )
	{
		RootPath = Path.ToSharedRef();
	}
	else
	{
		RootPath = FPropertyPath::CreateEmpty();
	}

	RootPathChanged.Broadcast();

	UpdateRows();
	UpdateColumns();
}

TArray< FPropertyInfo > FPropertyTable::GetPossibleExtensionsForPath( const TSharedRef< FPropertyPath >& Path ) const
{
	TArray< FPropertyInfo > ValidExtensions;

	for( auto ObjectNodeIter = SourceObjectPropertyNodes.CreateConstIterator(); ObjectNodeIter; ++ObjectNodeIter )
	{
		TArray< FPropertyInfo > Extensions = FPropertyNode::GetPossibleExtensionsForPath( Path, ObjectNodeIter.Value() );

		for( auto ExtensionIter = Extensions.CreateIterator(); ExtensionIter; ++ExtensionIter )
		{
			const FPropertyInfo& Info = *ExtensionIter;

			if ( Info.ArrayIndex == INDEX_NONE && Info.Property.IsValid() && 
				 ( Info.Property->IsA( FStructProperty::StaticClass() ) ||
				   Info.Property->IsA( FArrayProperty::StaticClass() ) ) )
			{
				bool AlreadyExists = false;
				for( auto ValidExtensionIter = ValidExtensions.CreateConstIterator(); ValidExtensionIter; ++ValidExtensionIter )
				{
					if ( *ValidExtensionIter == Info )
					{
						AlreadyExists = true;
						break;
					}
				}

				if ( !AlreadyExists )
				{
					ValidExtensions.Add( Info );
				}
			}
		}
	}

	return ValidExtensions;
}

void FPropertyTable::SetOrientation( EPropertyTableOrientation::Type InOrientation )
{
	Orientation = InOrientation;

	UpdateColumns();
	UpdateRows();
}

bool FPropertyTable::CanSelectCells() const
{
	return ( SelectionUnit & EPropertyTableSelectionUnit::Cell ) != 0;
}

bool FPropertyTable::CanSelectRows() const
{
	return ( SelectionUnit & EPropertyTableSelectionUnit::Row ) != 0;
}

void FPropertyTable::SetSelectionMode( const ESelectionMode::Type Mode )
{
	SelectionMode = Mode;

	if ( !CanSelectCells() )
	{
		SetCurrentCell( NULL );
		SetCurrentColumn( NULL );
	}

	if ( !CanSelectRows() )
	{
		SetCurrentRow( NULL );
	}
}

EColumnSortPriority::Type FPropertyTable::GetColumnSortPriority(const TSharedRef< class IPropertyTableColumn > Column) const
{
	if (Column == PrimarySortedByColumn.Pin())
	{
		return EColumnSortPriority::Primary;
	}
	else if (Column == SecondarySortedByColumn.Pin())
	{
		return EColumnSortPriority::Secondary;
	}

	return EColumnSortPriority::None;
}

EColumnSortMode::Type FPropertyTable::GetColumnSortMode( const TSharedRef< class IPropertyTableColumn > Column ) const
{
	if ( Column == PrimarySortedByColumn.Pin() )
	{
		return PrimarySortMode;
	}
	else if (Column == SecondarySortedByColumn.Pin())
	{
		return SecondarySortMode;
	}

	return EColumnSortMode::None;
}

void FPropertyTable::SortByColumnWithId( const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type SortMode)
{
	for( auto ColumnIter = Columns.CreateIterator(); ColumnIter; ++ColumnIter )
	{
		const TSharedRef< IPropertyTableColumn > Column = *ColumnIter;
		if ( Column->GetId() == ColumnId )
		{
			SortByColumn( Column, SortMode, SortPriority );
			break;
		}
	}
}

void FPropertyTable::SortByColumn( const TSharedRef< class IPropertyTableColumn >& Column, EColumnSortMode::Type SortMode, const EColumnSortPriority::Type SortPriority)
{
	if ( !Column->CanSortBy() )
	{
		/*PrimarySortedByColumn = nullptr;
		SecondarySortedByColumn = nullptr;
		PrimarySortMode = EColumnSortMode::None;
		PrimarySortMode = EColumnSortMode::None;*/
		return;
	}

	if (SortPriority == EColumnSortPriority::Primary)
	{
		PrimarySortedByColumn = Column;
		PrimarySortMode = SortMode;
	}
	else
	{
		SecondarySortedByColumn = Column;
		SecondarySortMode = SortMode;
	}

	// If the primary and secondary columns are the same eliminate the secondary sort
	if (PrimarySortedByColumn == SecondarySortedByColumn)
	{
		SecondarySortedByColumn = nullptr;
		SecondarySortMode = EColumnSortMode::None;
	}

	if (SortMode == EColumnSortMode::None )
	{
		return;
	}

	if (ensure(PrimarySortedByColumn.IsValid()))
	{
		PrimarySortedByColumn.Pin()->Sort(Rows, PrimarySortMode, SecondarySortedByColumn.Pin(), SecondarySortMode);
	}

	RowsChanged.Broadcast();
}

void SetCellValue( const TSharedRef< IPropertyTableCell >& Cell, FString Value )
{
	if ( Cell->IsReadOnly() )
	{
		return;
	}

	// We need to sanitize property name strings
	TSharedPtr<FPropertyNode> PropertyNode = Cell->GetNode();
	if (PropertyNode.IsValid())
	{
		FProperty* NodeProperty = PropertyNode->GetProperty();
		if (NodeProperty->IsA(FNameProperty::StaticClass()))
		{
			// Remove any pre-existing return characters
			Value = Value.TrimQuotes().Replace(LINE_TERMINATOR, TEXT(""));
		}
	}

	FString CurrentValue = Cell->GetValueAsString();
	if ( CurrentValue != Value )
	{
		// Set value
		Cell->SetValueFromString( Value );
	}
}

void FPropertyTable::PasteTextAtCell( const FString& Text, const TSharedRef< IPropertyTableCell >& Cell )
{
	if ( !SelectedCells.Contains( Cell ) )
	{
		return;
	}

	int32 CurrentRowIdx = 0;
	int32 CurrentColumnIdx = 0;
	TSharedPtr< IPropertyTableCell > FirstCellInRow = Cell;
	TSharedPtr< IPropertyTableCell > TargetCell = Cell;

	// Parse into row strings
	TArray<FString> RowStrings;
	Text.ParseIntoArray(RowStrings,LINE_TERMINATOR, false);
	if (RowStrings.Num() == 0)
	{
		// Pasted an empty string, ParseIntoArray didn't create empty entry, so create that now
		RowStrings.Emplace(TEXT(""));
	}

	// Parse row strings into individual cell strings
	TArray<FString> CellStrings;
	RowStrings[CurrentRowIdx++].ParseIntoArray(CellStrings, TEXT("\t"), false);
	if (CellStrings.Num() == 0)
	{
		// Pasted an empty string, ParseIntoArray didn't create empty entry, so create that now
		CellStrings.Emplace(TEXT(""));
	}

	// Get the maximum paste operations before displaying the slow task
	int32 NumPasteOperationsBeforeWarning = GetDefault<UEditorPerProjectUserSettings>()->PropertyMatrix_NumberOfPasteOperationsBeforeWarning;
	
	const bool bShowCancelButton = false;
	const bool bShowProgressDialog = SelectedCells.Num() > NumPasteOperationsBeforeWarning;
	GWarn->BeginSlowTask(LOCTEXT("UpdatingPropertiesSlowTaskLabel", "Updating properties..."), bShowProgressDialog, bShowCancelButton);

	if ( RowStrings.Num() == 1 && CellStrings.Num() == 1 )
	{
		int32 CurrentCellIndex = 0;
		for( auto CellIter = SelectedCells.CreateIterator(); CellIter; ++CellIter )
		{
			SetCellValue( *CellIter, CellStrings[0] );
			
			if ( bShowProgressDialog )
			{
				GWarn->UpdateProgress(CurrentCellIndex, SelectedCells.Num());
				CurrentCellIndex++;
			}
		}
	}
	else
	{
		// Paste values into cells
		while ( TargetCell.IsValid() && CurrentColumnIdx < CellStrings.Num() )
		{
			SetCellValue( TargetCell.ToSharedRef(), CellStrings[CurrentColumnIdx++] );

			// Move to next column
			TargetCell = GetNextCellInRow(TargetCell.ToSharedRef());

			if ( ( !TargetCell.IsValid() || CurrentColumnIdx == CellStrings.Num() ) && CurrentRowIdx < RowStrings.Num() )
			{
				// Move to next row
				TargetCell = GetNextCellInColumn(FirstCellInRow.ToSharedRef());

				if ( TargetCell.IsValid() )
				{
					// Prepare data to operate on next row
					CurrentColumnIdx = 0;
					FirstCellInRow = TargetCell;
					RowStrings[CurrentRowIdx++].ParseIntoArray(CellStrings, TEXT("\t"), false);
					if (CellStrings.Num() == 0)
					{
						// Tried to parse an empty row string, create an empty cell string
						CellStrings.Emplace(TEXT(""));
					}
				
					if ( bShowProgressDialog )
					{
						GWarn->UpdateProgress(CurrentRowIdx, RowStrings.Num());
					}
				}
			}
		}
	}
	
	GWarn->EndSlowTask();

}

TSharedPtr< IPropertyTableColumn > FPropertyTable::ScanForColumnWithSelectableCells( const int32 StartIndex, const int32 Step ) const
{
	TSharedPtr< IPropertyTableColumn > Column;
	for (int ColumnIndex = StartIndex; ColumnIndex >= 0 && ColumnIndex < Columns.Num() && ( !Column.IsValid() || !Column->CanSelectCells() ); ColumnIndex += Step )
	{
		Column = Columns[ ColumnIndex ];
	}

	if ( !Column.IsValid() || !Column->CanSelectCells() )
	{
		return NULL;
	}

	return Column;
}

TSharedPtr< IPropertyTableRow > FPropertyTable::ScanForRowWithCells( const int32 StartIndex, const int32 Step ) const
{
	TSharedPtr< IPropertyTableRow > Row;
	for (int RowIndex = StartIndex; RowIndex >= 0 && RowIndex < Rows.Num() && ( !Row.IsValid() || !Row->HasCells() ); RowIndex += Step )
	{
		Row = Rows[ RowIndex ];
	}

	if ( !Row.IsValid() || !Row->HasCells() )
	{
		return NULL;
	}

	return Row;
}

TSharedPtr< class IPropertyTableCell > FPropertyTable::GetFirstCellInSelection() 
{
	return StartingCellSelectionRange;
}

TSharedPtr< class IPropertyTableCell > FPropertyTable::GetLastCellInSelection()
{
	return EndingCellSelectionRange;
}

TSharedPtr< class IPropertyTableCell > FPropertyTable::GetNextCellInRow( const TSharedRef< class IPropertyTableCell >& Cell ) 
{
	const int32 FoundIndex = Columns.Find( Cell->GetColumn() );

	if ( FoundIndex == INDEX_NONE )
	{
		return NULL;
	}

	const int32 StartIndex = FoundIndex + 1;
	const int32 Step = 1;
	TSharedPtr< IPropertyTableColumn > NextColumn = ScanForColumnWithSelectableCells( StartIndex, Step );

	if ( !NextColumn.IsValid() )
	{
		return NULL;
	}

	return NextColumn->GetCell( Cell->GetRow() );
}

TSharedPtr< class IPropertyTableCell > FPropertyTable::GetPreviousCellInRow( const TSharedRef< class IPropertyTableCell >& Cell ) 
{
	const int32 FoundIndex = Columns.Find( Cell->GetColumn() );

	if ( FoundIndex == INDEX_NONE )
	{
		return NULL;
	}

	const int32 StartIndex = FoundIndex - 1;
	const int32 Step = -1;
	TSharedPtr< IPropertyTableColumn > PreviousColumn = ScanForColumnWithSelectableCells( StartIndex, Step );

	if ( !PreviousColumn.IsValid() )
	{
		return NULL;
	}

	return PreviousColumn->GetCell( Cell->GetRow() );
}

TSharedPtr< class IPropertyTableCell > FPropertyTable::GetNextCellInColumn( const TSharedRef< class IPropertyTableCell >& Cell ) 
{
	const int32 FoundIndex = Rows.Find( Cell->GetRow() );

	if ( FoundIndex == INDEX_NONE )
	{
		return NULL;
	}

	const int32 StartIndex = FoundIndex + 1;
	const int32 Step = 1;
	TSharedPtr< IPropertyTableRow > NextRow = ScanForRowWithCells( StartIndex, Step );

	if ( !NextRow.IsValid() )
	{
		return NULL;
	}

	return Cell->GetColumn()->GetCell( NextRow.ToSharedRef() );
}

TSharedPtr< class IPropertyTableCell > FPropertyTable::GetPreviousCellInColumn( const TSharedRef< class IPropertyTableCell >& Cell ) 
{
	const int32 FoundIndex = Rows.Find( Cell->GetRow() );

	if ( FoundIndex == INDEX_NONE )
	{
		return NULL;
	}

	const int32 StartIndex = FoundIndex - 1;
	const int32 Step = -1;
	TSharedPtr< IPropertyTableRow > NextRow = ScanForRowWithCells( StartIndex, Step );

	if ( !NextRow.IsValid() )
	{
		return NULL;
	}

	return Cell->GetColumn()->GetCell( NextRow.ToSharedRef() );
}

TSharedPtr< class IPropertyTableCell > FPropertyTable::GetFirstCellInRow( const TSharedRef< class IPropertyTableRow >& Row )
{
	const int32 StartIndex = 0;
	const int32 Step = 1;
	TSharedPtr< IPropertyTableColumn > FirstColumn = ScanForColumnWithSelectableCells( StartIndex, Step );

	if ( !FirstColumn.IsValid() )
	{
		return NULL;
	}

	return FirstColumn->GetCell( Row );
}

TSharedPtr< class IPropertyTableCell > FPropertyTable::GetLastCellInRow( const TSharedRef< class IPropertyTableRow >& Row )
{
	const int32 StartIndex = Columns.Num() - 1;
	const int32 Step = -1;
	TSharedPtr< IPropertyTableColumn > LastColumn = ScanForColumnWithSelectableCells( StartIndex, Step );

	if ( !LastColumn.IsValid() )
	{
		return NULL;
	}

	return LastColumn->GetCell( Row );
}

TSharedPtr< class IPropertyTableCell > FPropertyTable::GetFirstCellInColumn( const TSharedRef< class IPropertyTableColumn >& Column )
{
	const int32 StartIndex = 0;
	const int32 Step = 1;
	TSharedPtr< IPropertyTableRow > FirstRow = ScanForRowWithCells( StartIndex, Step );

	if ( !FirstRow.IsValid() )
	{
		return NULL;
	}

	return Column->GetCell( FirstRow.ToSharedRef() );
}

TSharedPtr< class IPropertyTableCell > FPropertyTable::GetLastCellInColumn( const TSharedRef< class IPropertyTableColumn >& Column )
{
	const int32 StartIndex = Rows.Num() - 1;
	const int32 Step = -1;
	TSharedPtr< IPropertyTableRow > LastRow = ScanForRowWithCells( StartIndex, Step );

	if ( !LastRow.IsValid() )
	{
		return NULL;
	}

	return Column->GetCell( LastRow.ToSharedRef() );
}

TSharedPtr< class IPropertyTableCell > FPropertyTable::GetFirstCellInTable()
{
	const int32 StartIndex = 0;
	const int32 Step = 1;
	
	TSharedPtr< IPropertyTableRow > FirstRow = ScanForRowWithCells( StartIndex, Step );

	if ( !FirstRow.IsValid() )
	{
		return NULL;
	}

	TSharedPtr< IPropertyTableColumn > FirstColumn = ScanForColumnWithSelectableCells( StartIndex, Step );

	if ( !FirstColumn.IsValid() )
	{
		return NULL;
	}

	return FirstColumn->GetCell( FirstRow.ToSharedRef() );
}

TSharedPtr< class IPropertyTableCell > FPropertyTable::GetLastCellInTable() 
{
	const int32 Step = -1;

	const int32 RowStartIndex = Rows.Num() - 1;
	TSharedPtr< IPropertyTableRow > LastRow = ScanForRowWithCells( RowStartIndex, Step );

	if ( !LastRow.IsValid() )
	{
		return NULL;
	}

	const int32 ColumnStartIndex = Columns.Num() - 1;
	TSharedPtr< IPropertyTableColumn > LastColumn = ScanForColumnWithSelectableCells( ColumnStartIndex, Step );

	if ( !LastColumn.IsValid() )
	{
		return NULL;
	}

	return LastColumn->GetCell( LastRow.ToSharedRef() );
}

void FPropertyTable::SetCurrentCell( const TSharedPtr< class IPropertyTableCell >& Cell ) 
{
	if ( CurrentCell == Cell || !CanSelectCells() )
	{
		return;
	}

	if ( CurrentCell.IsValid() )
	{
		CurrentCell->ExitEditMode();
	}

	if ( Cell.IsValid() )
	{
		CurrentRow = Cell->GetRow();
		CurrentColumn = Cell->GetColumn();
	}

	CurrentCell = Cell;

	bool NotifySelectionChanged = false;
	if ( CurrentRow.IsValid() && ( !SelectedRows.Contains( CurrentRow.ToSharedRef() ) || ( CurrentCell.IsValid() && !SelectedCells.Contains( CurrentCell.ToSharedRef() ) ) ) )
	{
		SelectedRows.Empty();
		SelectedRows.Add( CurrentRow.ToSharedRef() );
		NotifySelectionChanged = true;
	}

	if ( CurrentCell.IsValid() && !SelectedCells.Contains( CurrentCell.ToSharedRef() ) )
	{
		SelectedCells.Empty();
		SelectedCells.Add( CurrentCell.ToSharedRef() );

		StartingCellSelectionRange = CurrentCell;
		EndingCellSelectionRange = CurrentCell;

		NotifySelectionChanged = true;
	}

	if ( NotifySelectionChanged )
	{
		SelectionChanged.Broadcast();
	}
}

void FPropertyTable::SetCurrentColumn( const TSharedPtr< class IPropertyTableColumn >& Column )
{
	if ( CurrentColumn == Column || !CanSelectCells() )
	{
		return;
	}

	CurrentColumn = Column;

	if ( CurrentCell.IsValid() && CurrentCell->GetColumn() != Column )
	{
		CurrentCell->ExitEditMode();
		CurrentCell = NULL;
	}
}

void FPropertyTable::SetCurrentRow( const TSharedPtr< class IPropertyTableRow >& Row ) 
{
	if ( CurrentRow == Row || !CanSelectRows() )
	{
		return;
	}

	CurrentRow = Row;

	if ( !CurrentRow.IsValid() || ( CurrentCell.IsValid() && CurrentCell->GetRow() != CurrentRow ) )
	{
		CurrentCell->ExitEditMode();
		CurrentCell = NULL;
	}

	if ( CurrentRow.IsValid() && !SelectedRows.Contains( CurrentRow.ToSharedRef() ) )
	{
		SelectedRows.Empty();
		SelectedRows.Add( CurrentRow.ToSharedRef() );
		SelectionChanged.Broadcast();
	}
}

void FPropertyTable::SetSelectedRows( const TSet< TSharedRef< IPropertyTableRow > >& InSelectedRows )
{
	SelectedColumns.Empty();
	SelectedRows.Empty(); 

	if ( !CanSelectRows() )
	{
		return;
	}

	SelectedRows.Append( InSelectedRows ); 

	TSet< TSharedRef< IPropertyTableCell > > PreviouslySelectedCells( SelectedCells );

	bool RemovedCells = false;
	for( auto CellIter = PreviouslySelectedCells.CreateConstIterator(); CellIter; ++CellIter )
	{
		const TSharedRef< IPropertyTableCell > Cell = *CellIter;
		if ( !SelectedRows.Contains( Cell->GetRow() ) )
		{
			SelectedCells.Remove( Cell );
			SelectedColumns.Add( Cell->GetColumn() );
			RemovedCells = true;
		}
	}

	if ( RemovedCells )
	{
		StartingCellSelectionRange = NULL;
		EndingCellSelectionRange = NULL;
	}

	if ( CurrentRow.IsValid() && !SelectedRows.Contains( CurrentRow.ToSharedRef() ) )
	{
		CurrentRow = NULL;
	}

	if ( CurrentColumn.IsValid() && !SelectedColumns.Contains( CurrentColumn.ToSharedRef() ) )
	{
		CurrentColumn = NULL;
	}

	if ( CurrentCell.IsValid() && !SelectedCells.Contains( CurrentCell.ToSharedRef() ) )
	{
		CurrentCell->ExitEditMode();
		CurrentCell = NULL;
	}

	SelectionChanged.Broadcast();
}

void FPropertyTable::SetSelectedCells( const TSet< TSharedRef< class IPropertyTableCell > >& InSelectedCells )
{
	SelectedCells.Empty();

	if ( !CanSelectCells() )
	{
		return;
	}

	SelectedCells.Append( InSelectedCells );

	SelectedColumns.Empty();
	SelectedRows.Empty();
	TSharedPtr< IPropertyTableCell > LastCellInSet;

	for( auto CellIter = SelectedCells.CreateConstIterator(); CellIter; ++CellIter )
	{
		LastCellInSet = *CellIter;
		SelectedRows.Add( LastCellInSet->GetRow() );
		SelectedColumns.Add( LastCellInSet->GetColumn() );
	}

	if ( CurrentRow.IsValid() && !SelectedRows.Contains( CurrentRow.ToSharedRef() ) )
	{
		CurrentRow = NULL;
	}

	if ( CurrentColumn.IsValid() && !SelectedColumns.Contains( CurrentColumn.ToSharedRef() ) )
	{
		CurrentColumn = NULL;
	}

	if ( LastCellInSet.IsValid() && SelectedCells.Num() == 1 )
	{
		StartingCellSelectionRange = LastCellInSet;
		EndingCellSelectionRange = LastCellInSet;
	}
	else
	{
		StartingCellSelectionRange = NULL;
		EndingCellSelectionRange = NULL;
	}

	if ( CurrentCell.IsValid() && !SelectedCells.Contains( CurrentCell.ToSharedRef() ) )
	{
		CurrentCell->ExitEditMode();
		CurrentCell = NULL;
	}

	SelectionChanged.Broadcast();
}

void FPropertyTable::SelectCellRange( const TSharedRef< class IPropertyTableCell >& StartingCell, const TSharedRef< class IPropertyTableCell >& EndingCell )
{
	SelectedColumns.Empty();
	SelectedRows.Empty();
	SelectedCells.Empty();

	const int32 StartingCellRowIndex = Rows.Find( StartingCell->GetRow() );
	const int32 EndingCellRowIndex = Rows.Find( EndingCell->GetRow() );

	for (int RowIndex = StartingCellRowIndex; RowIndex < Rows.Num() && RowIndex <= EndingCellRowIndex; RowIndex++)
	{
		SelectedRows.Add( Rows[ RowIndex ] );
	}

	const int32 StartingCellColumnIndex = Columns.Find( StartingCell->GetColumn() );
	const int32 EndingCellColumnIndex = Columns.Find( EndingCell->GetColumn() );

	for (int ColumnsIndex = StartingCellColumnIndex; ColumnsIndex < Columns.Num() && ColumnsIndex <= EndingCellColumnIndex; ColumnsIndex++)
	{
		SelectedColumns.Add( Columns[ ColumnsIndex ] );

		for( auto RowIter = SelectedRows.CreateConstIterator(); RowIter; ++RowIter )
		{
			SelectedCells.Add( Columns[ ColumnsIndex ]->GetCell( *RowIter ) );
		}
	}

	if ( CurrentRow.IsValid() && !SelectedRows.Contains( CurrentRow.ToSharedRef() ) )
	{
		CurrentRow = NULL;
	}

	if ( CurrentColumn.IsValid() && !SelectedColumns.Contains( CurrentColumn.ToSharedRef() ) )
	{
		CurrentColumn = NULL;
	}

	StartingCellSelectionRange = StartingCell;
	EndingCellSelectionRange = EndingCell;

	if ( CurrentCell.IsValid() && !SelectedCells.Contains( CurrentCell.ToSharedRef() ) )
	{
		CurrentCell->ExitEditMode();
		CurrentCell = NULL;
	}

	SelectionChanged.Broadcast();
}

void FPropertyTable::GetSelectedTableObjects( TArray< TWeakObjectPtr< UObject > >& OutSelectedObjects) const
{
	for( auto RowIter = SelectedRows.CreateConstIterator(); RowIter; ++RowIter )
	{
		const TSharedRef< IPropertyTableRow > Row = *RowIter;
		const TWeakObjectPtr< UObject > Object = Row->GetDataSource()->AsUObject();

		if ( Object.IsValid() )
		{
			OutSelectedObjects.Add( Object );
		}
	}
}

void FPropertyTable::SetObjects( const TArray< TWeakObjectPtr< UObject > >& Objects )
{
	SourceObjects.Empty();
	SourceObjects.Append( Objects );

	UpdateColumns();
	UpdateRows();

	// Bind to object delegates, we can't do this at construction because the shared pointer isn't set up yet
	if (!ObjectsReplacedHandle.IsValid())
	{
		ObjectsReplacedHandle = FCoreUObjectDelegates::OnObjectsReplaced.AddSP(this, &FPropertyTable::OnObjectsReplaced);
	}
}

void FPropertyTable::SetObjects( const TArray< UObject* >& Objects )
{
	TArray<TWeakObjectPtr<UObject>> WeakObjects;
	
	for(UObject* Object : Objects)
	{
		WeakObjects.Add(Object);
	}

	SetObjects(WeakObjects);
}

void FPropertyTable::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
	bool bChangedAny = false;

	// Fix up any objects in our source list that were replaced, then refresh UI
	for (int32 i = 0; i < SourceObjects.Num(); i++)
	{
		UObject* SourceObject = SourceObjects[i].Get(true);
		UObject* ReplacedObject = ReplacementMap.FindRef(SourceObject);

		if (ReplacedObject && ReplacedObject != SourceObject)
		{
			SourceObjectPropertyNodes.Remove(SourceObject);
			SourceObjects[i] = ReplacedObject;

			bChangedAny = true;
		}
	}

	if (bChangedAny)
	{
		// The update functions do not correctly handle partial reconstruction, so delete everything and reconstruct
		ResetTable();

		UpdateRows();
		UpdateColumns();

		// This reset our selection so broadcast it
		SelectionChanged.Broadcast();
	}
}

void FPropertyTable::SetShowRowHeader( const bool InShowRowHeader ) 
{
	ShowRowHeader = InShowRowHeader; 
	UpdateColumns();
}

void FPropertyTable::SetShowObjectName( const bool InShowObjectName ) 
{
	ShowObjectName = InShowObjectName; 
	UpdateColumns();
}

TSharedRef< FObjectPropertyNode > FPropertyTable::GetObjectPropertyNode( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableRow >& Row ) 
{
	TWeakObjectPtr< UObject > Object;
	if( Orientation == EPropertyTableOrientation::AlignPropertiesInColumns )
	{
		Object = Row->GetDataSource()->AsUObject();
	}
	else
	{
		Object = Column->GetDataSource()->AsUObject();
	}

	return GetObjectPropertyNode( Object );
}

TSharedRef< FObjectPropertyNode > FPropertyTable::GetObjectPropertyNode( const TWeakObjectPtr< UObject >& Object ) 
{
	TSharedRef< FObjectPropertyNode >* ObjectNodePtr = SourceObjectPropertyNodes.Find( Object );

	if( ObjectNodePtr != NULL )
	{
		return *ObjectNodePtr;
	}

	TSharedRef< FObjectPropertyNode > ObjectPropertyNode( new FObjectPropertyNode() );
	ObjectPropertyNode->AddObject( Object.Get() );

	FPropertyNodeInitParams InitParams;
	InitParams.bCreateCategoryNodes = false;
	ObjectPropertyNode->InitNode( InitParams );

	SourceObjectPropertyNodes.Add( Object, ObjectPropertyNode );

	return ObjectPropertyNode;
}

TSharedRef< IPropertyTableColumn > FPropertyTable::CreateColumn( const TWeakObjectPtr< UObject >& Object )
{
	return MakeShareable( new FPropertyTableColumn( SharedThis( this ), Object ) );
}

TSharedRef< IPropertyTableColumn > FPropertyTable::CreateColumn( const TWeakFieldPtr< FProperty >& Property )
{
	return MakeShareable( new FPropertyTableColumn( SharedThis( this ), FPropertyPath::Create( Property ) ) );
}

TSharedRef< IPropertyTableColumn > FPropertyTable::CreateColumn( const TSharedRef< FPropertyPath >& PropertyPath )
{
	return MakeShareable( new FPropertyTableColumn( SharedThis( this ), PropertyPath ) );
}

TSharedRef< IPropertyTableRow > FPropertyTable::CreateRow( const TWeakObjectPtr< UObject >& Object )
{
	return MakeShareable( new FPropertyTableRow( SharedThis( this ), Object ) );
}

TSharedRef< IPropertyTableRow > FPropertyTable::CreateRow( const TWeakFieldPtr< FProperty >& Property )
{
	return MakeShareable( new FPropertyTableRow( SharedThis( this ), FPropertyPath::Create( Property ) ) );
}

TSharedRef< IPropertyTableRow > FPropertyTable::CreateRow( const TSharedRef< FPropertyPath >& PropertyPath )
{
	return MakeShareable( new FPropertyTableRow( SharedThis( this ), PropertyPath ) );
}

void FPropertyTable::UpdateRows()
{
	if( Orientation == EPropertyTableOrientation::AlignPropertiesInColumns )
	{
		TMultiMap< UObject*, TSharedRef< IPropertyTableRow > > RowsMap;

		for (int RowIdx = 0; RowIdx < Rows.Num(); ++RowIdx)
		{
			RowsMap.Add(Rows[RowIdx]->GetDataSource()->AsUObject().Get(), Rows[RowIdx]);
		}

		Rows.Empty();
		for( auto ObjectIter = SourceObjects.CreateConstIterator(); ObjectIter; ++ObjectIter )
		{
			const TWeakObjectPtr< UObject >& Object = *ObjectIter;

			if( Object.IsValid() )
			{
				const TSharedRef< FObjectPropertyNode > ObjectNode = GetObjectPropertyNode( Object );
				const TSharedPtr< FPropertyNode > PropertyNode = FPropertyNode::FindPropertyNodeByPath( RootPath, ObjectNode );

				//@todo This system will need to change in order to properly support arrays [11/30/2012 Justin.Sargent]
				if ( PropertyNode.IsValid() )
				{
					FProperty* Property = PropertyNode->GetProperty();

					if ( Property != NULL && Property->IsA( FArrayProperty::StaticClass() ) )
					{
						for (int ChildIdx = 0; ChildIdx < PropertyNode->GetNumChildNodes(); ChildIdx++)
						{
							TSharedPtr< FPropertyNode > ChildNode = PropertyNode->GetChildNode( ChildIdx );

							FPropertyInfo Extension;
							Extension.Property = ChildNode->GetProperty();
							Extension.ArrayIndex = ChildNode->GetArrayIndex();
							TSharedRef< FPropertyPath > Path = FPropertyPath::CreateEmpty()->ExtendPath( Extension );
							TArray< TSharedRef< IPropertyTableRow > > MapResults;
							bool Found = false;

							RowsMap.MultiFind(Object.Get(), MapResults);

							for (int MapIdx = 0; MapIdx < MapResults.Num(); ++MapIdx)
							{
								if (FPropertyPath::AreEqual(Path, MapResults[MapIdx]->GetPartialPath()))
								{
									Found = true;
									Rows.Add(MapResults[MapIdx]);
									break;
								}
							}

							if (!Found)
							{
								Rows.Add( MakeShareable( new FPropertyTableRow( SharedThis( this ), Object, Path ) ) );
							}
						}
					}
					else
					{
						TArray< TSharedRef< IPropertyTableRow > > MapResults;
						bool Found = false;

						RowsMap.MultiFind(Object.Get(), MapResults);

						for (int MapIdx = 0; MapIdx < MapResults.Num(); ++MapIdx)
						{
							if (MapResults[MapIdx]->GetPartialPath()->GetNumProperties() == 0)
							{
								Found = true;
								Rows.Add(MapResults[MapIdx]);
								break;
							}
						}

						if (!Found)
						{
							Rows.Add( MakeShareable( new FPropertyTableRow( SharedThis( this ), Object ) ) );
						}
					}
				}
			}
		}
	}

	const TSharedPtr< IPropertyTableColumn > Column = PrimarySortedByColumn.Pin();
	if ( Column.IsValid() && PrimarySortMode != EColumnSortMode::None )
	{
		Column->Sort( Rows, PrimarySortMode, SecondarySortedByColumn.Pin(), SecondarySortMode);
	}

	RowsChanged.Broadcast();
}

void FPropertyTable::UpdateColumns()
{
	if( Orientation == EPropertyTableOrientation::AlignPropertiesInColumns)
	{
		TMultiMap< const FProperty*, TSharedRef< IPropertyTableColumn > > ColumnsMap;
		for (int ColumnIdx = 0; ColumnIdx < Columns.Num(); ++ColumnIdx)
		{
			TSharedRef< IDataSource > DataSource = Columns[ColumnIdx]->GetDataSource();
			TSharedPtr< FPropertyPath > PropertyPath = DataSource->AsPropertyPath();
			if( PropertyPath.IsValid() && PropertyPath->GetNumProperties() > 0 )
			{
				ColumnsMap.Add(PropertyPath->GetRootProperty().Property.Get(), Columns[ColumnIdx]);
			}
		}

		Columns.Empty();

		if ( ShowRowHeader )
		{
			TSharedRef< IPropertyTableColumn > NewColumn = MakeShareable( new FPropertyTableRowHeaderColumn( SharedThis( this ) ) );
			Columns.Add( NewColumn );
		}

		if ( ShowObjectName )
		{
			TSharedRef< IPropertyTableColumn > NewColumn = MakeShareable( new FPropertyTableObjectNameColumn( SharedThis( this ) ) );
			NewColumn->SetFrozen( true );
			Columns.Add( NewColumn );
		}

		TArray< TWeakObjectPtr< UStruct > > UniqueTypes;
		TArray< int > TypeCounter;

		for( auto ObjectIter = SourceObjects.CreateConstIterator(); ObjectIter; ++ObjectIter )
		{
			auto Object = *ObjectIter;

			if( !Object.IsValid() )
			{
				continue;
			}

			const TSharedRef< FObjectPropertyNode > RootObjectNode = GetObjectPropertyNode( Object );

			TWeakObjectPtr< UStruct > Type;
			if ( RootPath->GetNumProperties() == 0 )
			{
				Type = RootObjectNode->GetObjectBaseClass();
			}
			else
			{
				const TSharedPtr< FPropertyNode > PropertyNode = FPropertyNode::FindPropertyNodeByPath( RootPath, RootObjectNode );

				if ( !PropertyNode.IsValid() )
				{
					continue;
				}

				const TWeakFieldPtr< FProperty > Property = PropertyNode->GetProperty();

				if ( !Property.IsValid() || !Property->IsA( FStructProperty::StaticClass() ) )
				{
					continue;
				}

				FStructProperty* StructProperty = CastField< FStructProperty >( Property.Get() );
				Type = StructProperty->Struct;
			}

			if ( !Type.IsValid() )
			{
				continue;
			}

			int FoundIndex = -1;
			if ( UniqueTypes.Find( Type, /*OUT*/FoundIndex ) )
			{
				TypeCounter[ FoundIndex ] = TypeCounter[ FoundIndex ] + 1;
				continue;
			}

			UniqueTypes.Add( Type );
			TypeCounter.Add( 1 );
		}

		if ( UniqueTypes.Num() > 0 )
		{
			int HighestCountIndex = 0;
			int HighestCount = 0;
			for (int Index = 0; Index < TypeCounter.Num(); Index++)
			{
				if ( TypeCounter[Index] > HighestCount )
				{
					HighestCountIndex = Index;
					HighestCount = TypeCounter[Index];
				}
			}

			TWeakObjectPtr< UStruct > PrimaryType = UniqueTypes[ HighestCountIndex ];

			for (TFieldIterator<FProperty> PropertyIter( PrimaryType.Get(), EFieldIteratorFlags::IncludeSuper); PropertyIter; ++PropertyIter)
			{
				TWeakFieldPtr< FProperty > Property = *PropertyIter;

				// Don't expose CPF_NativeAccessSpecifierProtected and CPF_NativeAccessSpecifierPrivate properties
				if ( PropertyIter->HasAllPropertyFlags(CPF_NativeAccessSpecifierPublic | CPF_AssetRegistrySearchable) && FPropertyEditorPermissionList::Get().DoesPropertyPassFilter(PrimaryType.Get(), Property->GetFName()))
				{
					TArray< TSharedRef< IPropertyTableColumn > > MapResults;

					ColumnsMap.MultiFind(Property.Get(), MapResults);
					if(MapResults.Num() > 0)
					{
						for (int MapIdx = 0; MapIdx < MapResults.Num(); ++MapIdx)
						{
							Columns.Add(MapResults[MapIdx]);
						}
					}
					else
					{
						TSharedRef< IPropertyTableColumn > NewColumn = CreateColumn( Property );
						Columns.Add( NewColumn );
					}
				}
			}
		}
	}
	else
	{
		Columns.Empty();

		if( SourceObjects.Num() > 0 )
		{
			UClass* ObjectClass = SourceObjects[0]->GetClass();
			TSharedRef< IPropertyTableColumn > HeadingColumn = MakeShareable( new FPropertyTablePropertyNameColumn( SharedThis( this ) ) );

			Columns.Add( HeadingColumn );

			for( auto ObjectIter = SourceObjects.CreateConstIterator(); ObjectIter; ++ObjectIter )
			{
				auto Object = *ObjectIter;

				if( Object.IsValid() )
				{
					const TSharedRef< FObjectPropertyNode > ObjectNode = GetObjectPropertyNode( Object );
					const TSharedPtr< FPropertyNode > PropertyNode = FPropertyNode::FindPropertyNodeByPath( RootPath, ObjectNode );

					FProperty* Property = PropertyNode->GetProperty();
					if ( Property != NULL && Property->IsA( FArrayProperty::StaticClass() ) )
					{
						for (int ChildIdx = 0; ChildIdx < PropertyNode->GetNumChildNodes(); ChildIdx++)
						{
							TSharedPtr< FPropertyNode > ChildNode = PropertyNode->GetChildNode( ChildIdx );
							FPropertyInfo Extension;
							Extension.Property = ChildNode->GetProperty();
							Extension.ArrayIndex = ChildNode->GetArrayIndex();
							TSharedRef< FPropertyPath > Path = FPropertyPath::CreateEmpty()->ExtendPath( Extension );
							TSharedRef< IPropertyTableColumn > NewColumn = MakeShareable( new FPropertyTableColumn( SharedThis( this ), Object, Path ) );
							Columns.Add( NewColumn );
						}
					}
					else if (  Property != NULL )
					{
						TSharedRef< IPropertyTableColumn > NewColumn = MakeShareable( new FPropertyTableColumn( SharedThis( this ), Object ) );
						Columns.Add( NewColumn );
					}
				}

			}
		}
	}

	ColumnsChanged.Broadcast();
}
 
#undef LOCTEXT_NAMESPACE 
 
