// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PropertyPath.h"
#include "AssetThumbnail.h"
#include "ObjectPropertyNode.h"
#include "IPropertyTableColumn.h"
#include "IPropertyTable.h"
#include "IPropertyTableCell.h"
#include "IPropertyTableRow.h"

class FPropertyTable : public TSharedFromThis< FPropertyTable >, public IPropertyTable
{
public: 

	FPropertyTable();
	virtual ~FPropertyTable();

	virtual void Tick() override;

	virtual void ForceRefresh() override;
	virtual void RequestRefresh() override;
	virtual void RequestForceRefresh() override;

	virtual class FNotifyHook* GetNotifyHook() const override;
	virtual bool AreFavoritesEnabled() const override;
	virtual void ToggleFavorite( const TSharedRef< class FPropertyEditor >& PropertyEditor ) const override;
	virtual void CreateColorPickerWindow( const TSharedRef< class FPropertyEditor >& PropertyEditor, bool bUseAlpha ) const override;
	virtual void EnqueueDeferredAction( FSimpleDelegate DeferredAction ) override;
	virtual TSharedPtr<class FAssetThumbnailPool> GetThumbnailPool() const override;
	virtual void NotifyStartedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) override {}
	virtual void NotifyFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) override {}
	virtual const TArray<TSharedRef<class IClassViewerFilter>>& GetClassViewerFilters() const override;

	virtual bool GetIsUserAllowedToChangeRoot() override;
	virtual void SetIsUserAllowedToChangeRoot( bool InAllowUserToChangeRoot ) override;

	virtual void AddColumn( const TWeakObjectPtr< UObject >& Object ) override;
	virtual void AddColumn( const TWeakFieldPtr< FProperty >& Property ) override;
	virtual void AddColumn( const TSharedRef< FPropertyPath >& PropertyPath ) override;
	virtual void AddColumn( const TSharedRef< class IPropertyTableColumn >& Column ) override;
	virtual void RemoveColumn( const TSharedRef< class IPropertyTableColumn >& Column ) override;

	virtual void AddRow( const TWeakObjectPtr< UObject >& Object ) override;
	virtual void AddRow( const TWeakFieldPtr< FProperty >& Property ) override;
	virtual void AddRow( const TSharedRef< FPropertyPath >& PropertyPath ) override;
	virtual void AddRow( const TSharedRef< class IPropertyTableRow >& Row ) override;
	virtual void RemoveRow( const TSharedRef< class IPropertyTableRow >& Row ) override;

	virtual const EPropertyTableOrientation::Type GetOrientation() const override { return Orientation; }
	virtual void SetOrientation( EPropertyTableOrientation::Type InOrientation ) override;

	virtual void SetRootPath( const TSharedPtr< FPropertyPath >& Path ) override;
	virtual TSharedRef< FPropertyPath > GetRootPath() const override { return RootPath; }
	virtual TArray< FPropertyInfo > GetPossibleExtensionsForPath( const TSharedRef< FPropertyPath >& Path ) const override;

	virtual void GetSelectedTableObjects( TArray< TWeakObjectPtr< UObject > >& OutSelectedObjects) const override;

	virtual const TArray<TWeakObjectPtr<UObject>>& GetSelectedObjects() const { return SourceObjects; }
	virtual void SetObjects( const TArray< TWeakObjectPtr< UObject > >& Objects ) override;
	virtual void SetObjects( const TArray< UObject* >& Objects ) override;

	virtual TSharedRef< class FObjectPropertyNode > GetObjectPropertyNode( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableRow >& Row ) override;
	virtual TSharedRef< class FObjectPropertyNode > GetObjectPropertyNode( const TWeakObjectPtr< UObject >& Object ) override;

	virtual bool GetShowRowHeader() const override { return ShowRowHeader; }
	virtual void SetShowRowHeader( const bool InShowRowHeader ) override;

	virtual bool GetShowObjectName() const override { return ShowObjectName; }
	virtual void SetShowObjectName( const bool ShowObjectName ) override;

	virtual const TArray< TSharedRef< class IPropertyTableColumn > >& GetColumns() override { return Columns; }
	virtual TArray< TSharedRef< class IPropertyTableRow > >& GetRows() override { return Rows; }

	virtual const TSet< TSharedRef< class IPropertyTableRow > >& GetSelectedRows() override { return SelectedRows; }
	virtual void SetSelectedRows( const TSet< TSharedRef< IPropertyTableRow > >& InSelectedRows ) override;

	virtual const TSet< TSharedRef< class IPropertyTableCell > >& GetSelectedCells() override { return SelectedCells; }
	virtual void SetSelectedCells( const TSet< TSharedRef< class IPropertyTableCell > >& InSelectedCells ) override;

	virtual void SelectCellRange( const TSharedRef< class IPropertyTableCell >& StartingCell, const TSharedRef< class IPropertyTableCell >& EndingCell ) override;

	virtual float GetItemHeight() const override { return ItemHeight; }
	virtual void SetItemHeight( float NewItemHeight ) override { ItemHeight = NewItemHeight; }

	virtual TSharedPtr< class IPropertyTableCell > GetLastClickedCell() const override { return LastClickedCell; }
	virtual void SetLastClickedCell( const TSharedPtr< class IPropertyTableCell >& Cell ) override { LastClickedCell = Cell; }

	virtual TSharedPtr< class IPropertyTableCell > GetCurrentCell() const override { return CurrentCell; }
	virtual void SetCurrentCell( const TSharedPtr< class IPropertyTableCell >& Cell ) override;

	virtual TSharedPtr< class IPropertyTableColumn > GetCurrentColumn() const override { return CurrentColumn; }
	virtual void SetCurrentColumn( const TSharedPtr< class IPropertyTableColumn >& Column ) override;

	virtual TSharedPtr< class IPropertyTableRow > GetCurrentRow() const override { return CurrentRow; }
	virtual void SetCurrentRow( const TSharedPtr< class IPropertyTableRow >& Row ) override;

	virtual TSharedPtr< class IPropertyTableCell > GetFirstCellInSelection() override;
	virtual TSharedPtr< class IPropertyTableCell > GetLastCellInSelection() override;

	virtual TSharedPtr< class IPropertyTableCell > GetNextCellInRow( const TSharedRef< class IPropertyTableCell >& Cell ) override;
	virtual TSharedPtr< class IPropertyTableCell > GetPreviousCellInRow( const TSharedRef< class IPropertyTableCell >& Cell ) override;

	virtual TSharedPtr< class IPropertyTableCell > GetNextCellInColumn( const TSharedRef< class IPropertyTableCell >& Cell ) override;
	virtual TSharedPtr< class IPropertyTableCell > GetPreviousCellInColumn( const TSharedRef< class IPropertyTableCell >& Cell ) override;

	virtual TSharedPtr< class IPropertyTableCell > GetFirstCellInRow( const TSharedRef< class IPropertyTableRow >& Row ) override;
	virtual TSharedPtr< class IPropertyTableCell > GetLastCellInRow( const TSharedRef< class IPropertyTableRow >& Row ) override;

	virtual TSharedPtr< class IPropertyTableCell > GetFirstCellInColumn( const TSharedRef< class IPropertyTableColumn >& Column ) override;
	virtual TSharedPtr< class IPropertyTableCell > GetLastCellInColumn( const TSharedRef< class IPropertyTableColumn >& Column ) override;

	virtual TSharedPtr< class IPropertyTableCell > GetFirstCellInTable() override;
	virtual TSharedPtr< class IPropertyTableCell > GetLastCellInTable() override;

	virtual EPropertyTableSelectionUnit::Type GetSelectionUnit() const override { return SelectionUnit; }
	virtual void SetSelectionUnit( const EPropertyTableSelectionUnit::Type Unit ) override { SelectionUnit = Unit; }

	virtual ESelectionMode::Type GetSelectionMode() const override { return SelectionMode; }
	virtual void SetSelectionMode( const ESelectionMode::Type Mode ) override;

	virtual EColumnSortPriority::Type GetColumnSortPriority(const TSharedRef< class IPropertyTableColumn > Column) const override;
	virtual EColumnSortMode::Type GetColumnSortMode( const TSharedRef< class IPropertyTableColumn > Column ) const override;
	virtual void SortByColumnWithId( const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type SortMode) override;
	virtual void SortByColumn( const TSharedRef< class IPropertyTableColumn >& Column, EColumnSortMode::Type SortMode, const EColumnSortPriority::Type SortPriority ) override;

	virtual void PasteTextAtCell( const FString& Text, const TSharedRef< class IPropertyTableCell >& Cell ) override;

	DECLARE_DERIVED_EVENT( FPropertyTable, IPropertyTable::FSelectionChanged, FSelectionChanged );
	FSelectionChanged* OnSelectionChanged() override { return &SelectionChanged; }

	DECLARE_DERIVED_EVENT( FPropertyTable, IPropertyTable::FColumnsChanged, FColumnsChanged );
	FColumnsChanged* OnColumnsChanged() override { return &ColumnsChanged; }

	DECLARE_DERIVED_EVENT( FPropertyTable, IPropertyTable::FRowsChanged, FRowsChanged );
	FRowsChanged* OnRowsChanged() override { return &RowsChanged; }

	DECLARE_DERIVED_EVENT( FPropertyTable, IPropertyTable::FRootPathChanged, FRootPathChanged );
	virtual FRootPathChanged* OnRootPathChanged() override { return &RootPathChanged; }

	virtual bool IsPropertyEditingEnabled() const override { return true; }

	virtual bool DontUpdateValueWhileEditing() const override
	{
		return false;
	}

	virtual bool HasClassDefaultObject() const
	{
		return false;
	}
private:

	void UpdateColumns();
	void UpdateRows();

	bool CanSelectCells() const;
	bool CanSelectRows() const;

	TSharedPtr< IPropertyTableColumn > ScanForColumnWithSelectableCells( const int32 StartIndex, const int32 Step ) const;
	TSharedPtr< IPropertyTableRow > ScanForRowWithCells( const int32 StartIndex, const int32 Step ) const;

	TSharedRef< IPropertyTableRow > CreateRow( const TWeakObjectPtr< UObject >& Object );
	TSharedRef< IPropertyTableRow > CreateRow( const TWeakFieldPtr< FProperty >& Property );
	TSharedRef< IPropertyTableRow > CreateRow( const TSharedRef< FPropertyPath >& PropertyPath );

	TSharedRef< IPropertyTableColumn > CreateColumn( const TWeakObjectPtr< UObject >& Object );
	TSharedRef< IPropertyTableColumn > CreateColumn( const TWeakFieldPtr< FProperty >& Property );
	TSharedRef< IPropertyTableColumn > CreateColumn( const TSharedRef< FPropertyPath >& PropertyPath );

	void PurgeInvalidObjectNodes();

	void ResetTable();

	void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);

private:

	TArray< TWeakObjectPtr< UObject > > SourceObjects;
	TMap< TWeakObjectPtr< UObject >, TSharedRef< FObjectPropertyNode > > SourceObjectPropertyNodes;
	
	TArray< TSharedRef< class IPropertyTableColumn > > Columns;
	TArray< TSharedRef< class IPropertyTableRow > > Rows;

	TSet< TSharedRef< class IPropertyTableColumn > > SelectedColumns;
	TSet< TSharedRef< class IPropertyTableRow > > SelectedRows;
	TSet< TSharedRef< class IPropertyTableCell > > SelectedCells;
	TSharedPtr< class IPropertyTableCell > StartingCellSelectionRange;
	TSharedPtr< class IPropertyTableCell > EndingCellSelectionRange;

	TSharedPtr< class IPropertyTableRow > CurrentRow;
	TSharedPtr< class IPropertyTableCell > CurrentCell;
	TSharedPtr< class IPropertyTableColumn > CurrentColumn;

	TSharedRef< FPropertyPath > RootPath;

	EPropertyTableSelectionUnit::Type SelectionUnit;
	ESelectionMode::Type SelectionMode;

	bool ShowRowHeader;
	bool ShowObjectName;

	float ItemHeight;
	TSharedPtr< class IPropertyTableCell > LastClickedCell;

	/** Actions that should be executed next tick */
	TArray<FSimpleDelegate> DeferredActions;

	FSelectionChanged SelectionChanged;
	FColumnsChanged ColumnsChanged;
	FRowsChanged RowsChanged;
	FRootPathChanged RootPathChanged;

	TWeakPtr< class IPropertyTableColumn > PrimarySortedByColumn;
	TWeakPtr< class IPropertyTableColumn > SecondarySortedByColumn;

	EColumnSortMode::Type PrimarySortMode;
	EColumnSortMode::Type SecondarySortMode;

	bool AllowUserToChangeRoot;

	/** Refresh the table contents? */
	bool bRefreshRequested;

	/** The Orientation of this table, I.e. do we swap columns and rows */
	EPropertyTableOrientation::Type Orientation;

	FDelegateHandle ObjectsReplacedHandle;

	TSharedRef<FEditConditionParser> EditConditionParser;
};

