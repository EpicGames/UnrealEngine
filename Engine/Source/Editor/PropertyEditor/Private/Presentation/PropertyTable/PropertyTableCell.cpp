// Copyright Epic Games, Inc. All Rights Reserved.
#include "Presentation/PropertyTable/PropertyTableCell.h"

#include "HAL/PlatformCrt.h"
#include "IPropertyTable.h"
#include "ObjectPropertyNode.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "PropertyHandle.h"
#include "PropertyNode.h"
#include "PropertyPermissionList.h"
#include "UObject/WeakObjectPtr.h"

class FPropertyPath;
class UObject;

FPropertyTableCell::FPropertyTableCell( const TSharedRef< class IPropertyTableColumn >& InColumn, const TSharedRef< class IPropertyTableRow >& InRow )
	: bIsBound( false )
	, bInEditMode( false )
	, bIsPropertyAllowed(true)
	, EnteredEditModeEvent()
	, ExitedEditModeEvent()
	, Column( InColumn )
	, Row( InRow )
	, PropertyNode( NULL )
	, ObjectNode( NULL )
	, PropertyEditor( NULL )
{
	Refresh();
}

void FPropertyTableCell::Refresh()
{
	const TSharedRef< IPropertyTableColumn > ColumnRef = Column.Pin().ToSharedRef();
	const TSharedRef< IPropertyTableRow > RowRef = Row.Pin().ToSharedRef();

	bIsBound = false;
	ObjectNode = GetTable()->GetObjectPropertyNode( ColumnRef, RowRef );

	if ( !ObjectNode.IsValid() )
	{
		return;
	}

	TSharedRef< IDataSource > ColumnBoundData = ColumnRef->GetDataSource();
	TSharedRef< IDataSource > RowBoundData = RowRef->GetDataSource();

	if ( !ColumnBoundData->IsValid() || !RowBoundData->IsValid() )
	{
		return;
	}

	TSharedPtr< FPropertyPath > PropertyPath;
	TWeakObjectPtr< UObject > Item;

	Item = ColumnBoundData->AsUObject();
	if ( !Item.IsValid() )
	{
		Item = RowBoundData->AsUObject();
	}

	if ( !Item.IsValid() )
	{
		// Must have a valid non-FProperty UObject bound to the column or row
		return;
	}

	PropertyPath = ColumnBoundData->AsPropertyPath();
	if ( !PropertyPath.IsValid() )
	{
		PropertyPath = RowBoundData->AsPropertyPath();
	}

	if ( !PropertyPath.IsValid() )
	{
		// By this point a valid PropertyPath must have been found or created
		return;
	}

	PropertyNode = FPropertyNode::FindPropertyNodeByPath( GetTable()->GetRootPath(), ObjectNode.ToSharedRef() );

	if ( !PropertyNode.IsValid() )
	{
		return;
	}

	PropertyNode = FPropertyNode::FindPropertyNodeByPath( RowRef->GetPartialPath(), PropertyNode.ToSharedRef() );

	if ( !PropertyNode.IsValid() )
	{
		return;
	}

	PropertyNode = FPropertyNode::FindPropertyNodeByPath( ColumnRef->GetPartialPath(), PropertyNode.ToSharedRef() );

	if ( !PropertyNode.IsValid() )
	{
		return;
	}

	PropertyNode = FPropertyNode::FindPropertyNodeByPath( PropertyPath, PropertyNode.ToSharedRef() );

	if ( PropertyNode.IsValid() )
	{
		bIsBound = true;
		PropertyEditor = FPropertyEditor::Create( PropertyNode.ToSharedRef(), GetTable() );
	}

	bIsPropertyAllowed = DoesPropertyPassPermissionList();
}

bool FPropertyTableCell::IsReadOnly() const 
{ 
	return !bIsPropertyAllowed || !IsBound() || PropertyEditor->IsEditConst() || ( PropertyEditor->HasEditCondition() && !PropertyEditor->IsEditConditionMet() );
}

bool FPropertyTableCell::IsValid() const
{
	return (!IsBound() || (PropertyEditor->GetPropertyHandle()->GetProperty() != NULL));
}

FString FPropertyTableCell::GetValueAsString() const
{
	return IsBound() ? PropertyEditor->GetValueAsString() : FString();
}

FText FPropertyTableCell::GetValueAsText() const
{
	return IsBound() ? PropertyEditor->GetValueAsText() : FText::GetEmpty();
}

void FPropertyTableCell::SetValueFromString( const FString& InString )
{
	if ( IsReadOnly() )
	{
		return;
	}

	TSharedRef< IPropertyHandle > Handle = PropertyEditor->GetPropertyHandle();
	Handle->SetValueFromFormattedString(InString, EPropertyValueSetFlags::InstanceObjects);
}

TWeakObjectPtr< UObject > FPropertyTableCell::GetObject() const
{
	if ( !ObjectNode.IsValid() )
	{
		return NULL;
	}

	return ObjectNode->GetUObject( 0 );
}

TSharedRef< class IPropertyTable > FPropertyTableCell::GetTable() const
{
	return Column.Pin()->GetTable();
}

TSharedPtr< class IPropertyHandle > FPropertyTableCell::GetPropertyHandle() const 
{ 
	if( IsBound() ) 
	{
		return PropertyEditor->GetPropertyHandle(); 
	}
	return NULL;
}

bool FPropertyTableCell::DoesPropertyPassPermissionList()
{
	// If we have a permission list, make sure the object for this cell allows the property to be displayed
	if(FPropertyEditorPermissionList::Get().IsEnabled())
	{
		// Get the Object and the Property
		UObject* Object = GetObject().Get();
		TSharedPtr< class FPropertyNode > Node = GetNode();
		
		if(Object && Node)
		{
			// Extract the FProperty and UClass from them respectively
			const FProperty* Property = Node->GetProperty();
			const UClass* ObjClass = Cast<UClass>(Object);
			if (ObjClass == nullptr)
			{
				ObjClass = Object->GetClass();
			}
			
			if(ObjClass && Property)
			{
				if(!FPropertyEditorPermissionList::Get().DoesPropertyPassFilter(ObjClass, Property->GetFName()))
				{
					return false;
				}
			}
		}
	}

	return true;
}

void FPropertyTableCell::EnterEditMode() 
{
	// We don't let the cell enter edit mode if it doesn't pass the permission list, but we do allow ExitEditMode()
	// in case the permission list changes when you are in edit mode
	if(!bIsPropertyAllowed)
	{
		return;
	}
	
	if ( !bInEditMode )
	{
		TSharedRef< IPropertyTable > Table = GetTable();
		Table->SetCurrentCell( SharedThis( this ) );
		bInEditMode = true;
		EnteredEditModeEvent.Broadcast();
	}
}

void FPropertyTableCell::ExitEditMode()
{
	if ( bInEditMode )
	{
		bInEditMode = false;
		ExitedEditModeEvent.Broadcast();
	}
}
