// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "IPropertyTableCell.h"
#include "IPropertyTableColumn.h"
#include "IPropertyTableRow.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FPropertyEditor;
class UObject;

class FPropertyTableCell : public TSharedFromThis< FPropertyTableCell >, public IPropertyTableCell
{
public:

	FPropertyTableCell( const TSharedRef< class IPropertyTableColumn >& InColumn, const TSharedRef< class IPropertyTableRow >& InRow );

	virtual ~FPropertyTableCell() = default;

	virtual void Refresh() override;

	virtual bool IsReadOnly() const override;
	virtual bool IsBound() const override { return bIsBound; }
	virtual bool InEditMode() const override { return bInEditMode; }
	virtual bool IsValid() const override;
	virtual bool PassesPermissionList() const override { return bIsPropertyAllowed; }

	virtual FString GetValueAsString() const override;
	virtual FText GetValueAsText() const override;
	virtual void SetValueFromString( const FString& InString ) override;

	virtual TWeakObjectPtr< UObject > GetObject() const override;
	virtual TSharedPtr< class FPropertyNode > GetNode() const override { return PropertyNode; }
	virtual TSharedRef< class IPropertyTableColumn > GetColumn() const override { return Column.Pin().ToSharedRef(); }
	virtual TSharedRef< class IPropertyTableRow > GetRow() const override { return Row.Pin().ToSharedRef(); }
	virtual TSharedRef< class IPropertyTable > GetTable() const override;
	virtual TSharedPtr< class IPropertyHandle > GetPropertyHandle() const override;

	virtual void EnterEditMode() override;
	virtual void ExitEditMode() override;

	DECLARE_DERIVED_EVENT( FPropertyTableCell, IPropertyTableCell::FEnteredEditModeEvent, FEnteredEditModeEvent );
	virtual FEnteredEditModeEvent& OnEnteredEditMode() override { return EnteredEditModeEvent; }

	DECLARE_DERIVED_EVENT( FPropertyTableCell, IPropertyTableCell::FExitedEditModeEvent, FExitedEditModeEvent  );
	virtual FExitedEditModeEvent& OnExitedEditMode() override { return ExitedEditModeEvent; }

private:
	// Check if the property for this cell passes the permission list for the object we are viewing
	bool DoesPropertyPassPermissionList();
	
private:

	bool bIsBound;
	bool bInEditMode;
	bool bIsPropertyAllowed;

	FEnteredEditModeEvent EnteredEditModeEvent;
	FExitedEditModeEvent ExitedEditModeEvent;

	TWeakPtr< class IPropertyTableColumn > Column;
	TWeakPtr< class IPropertyTableRow > Row;

	TSharedPtr< class FPropertyNode > PropertyNode;
	TSharedPtr< class FObjectPropertyNode > ObjectNode;

	TSharedPtr< FPropertyEditor > PropertyEditor;
};
