// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presentation/PropertyTable/PropertyTableColumn.h"
#include "Editor/EditorEngine.h"
#include "IPropertyTableCell.h"
#include "UObject/TextProperty.h"
#include "ObjectPropertyNode.h"
#include "Presentation/PropertyTable/PropertyTableCell.h"
#include "Presentation/PropertyTable/DataSource.h"
#include "PropertyEditorHelpers.h"

#define LOCTEXT_NAMESPACE "PropertyTableColumn"

struct FCompareRowByColumnBase
{
	virtual int32 Compare(const TSharedRef< IPropertyTableRow >& Lhs, const TSharedRef< IPropertyTableRow >& Rhs) const = 0;
	virtual ~FCompareRowByColumnBase() {}
};

struct FCompareRowPrimaryAndSecondary
{
	FCompareRowPrimaryAndSecondary(FCompareRowByColumnBase* InPrimarySort, FCompareRowByColumnBase* InSecondarySort)
		: PrimarySort(InPrimarySort)
		, SecondarySort(InSecondarySort)
	{}
	
	bool operator()(const TSharedRef< IPropertyTableRow >& Lhs, const TSharedRef< IPropertyTableRow >& Rhs) const
	{
		const int32 PrimaryResult = PrimarySort->Compare(Lhs, Rhs);
		if (PrimaryResult != 0 || !SecondarySort)
		{
			return PrimaryResult < 0;
		}
		else
		{
			return SecondarySort->Compare(Lhs, Rhs) < 0;
		}
	}
private:
	FCompareRowByColumnBase* PrimarySort;
	FCompareRowByColumnBase* SecondarySort;

};



template< typename FPropertyType >
struct FCompareRowByColumnAscending : public FCompareRowByColumnBase
{
public:
	FCompareRowByColumnAscending( const TSharedRef< IPropertyTableColumn >& InColumn, const FPropertyType* InFProperty )
		: Property( InFProperty )
		, Column( InColumn )
	{

	}

	int32 Compare( const TSharedRef< IPropertyTableRow >& Lhs, const TSharedRef< IPropertyTableRow >& Rhs ) const
	{
		const TSharedRef< IPropertyTableCell > LhsCell = Column->GetCell( Lhs );
		const TSharedRef< IPropertyTableCell > RhsCell = Column->GetCell( Rhs );

		const TSharedPtr< FPropertyNode > LhsPropertyNode = LhsCell->GetNode();
		if ( !LhsPropertyNode.IsValid() )
		{
			return 1;
		}

		const TSharedPtr< FPropertyNode > RhsPropertyNode = RhsCell->GetNode();
		if ( !RhsPropertyNode.IsValid() )
		{
			return -1;
		}

		const TSharedPtr< IPropertyHandle > LhsPropertyHandle = PropertyEditorHelpers::GetPropertyHandle( LhsPropertyNode.ToSharedRef(), NULL, NULL );
		if ( !LhsPropertyHandle.IsValid() )
		{
			return 1;
		}

		const TSharedPtr< IPropertyHandle > RhsPropertyHandle = PropertyEditorHelpers::GetPropertyHandle( RhsPropertyNode.ToSharedRef(), NULL, NULL );
		if ( !RhsPropertyHandle.IsValid() )
		{
			return -1;
		}

		return ComparePropertyValue( LhsPropertyHandle, RhsPropertyHandle );
	}

private:

	int32 ComparePropertyValue( const TSharedPtr< IPropertyHandle >& LhsPropertyHandle, const TSharedPtr< IPropertyHandle >& RhsPropertyHandle ) const
	{
		typename FPropertyType::TCppType LhsValue; 
		LhsPropertyHandle->GetValue( LhsValue );

		typename FPropertyType::TCppType RhsValue; 
		RhsPropertyHandle->GetValue( RhsValue );

		if (LhsValue < RhsValue)
		{
			return -1;
		}
		else if (LhsValue > RhsValue)
		{
			return 1;
		}

		return 0;
	}

private:

	const FPropertyType* Property;
	TSharedRef< IPropertyTableColumn > Column;
};

template< typename FPropertyType >
struct FCompareRowByColumnDescending : public FCompareRowByColumnBase
{
public:
	FCompareRowByColumnDescending( const TSharedRef< IPropertyTableColumn >& InColumn, const FPropertyType* InFProperty )
		: Comparer( InColumn, InFProperty )
	{

	}

	int32 Compare( const TSharedRef< IPropertyTableRow >& Lhs, const TSharedRef< IPropertyTableRow >& Rhs ) const override
	{
		return Comparer.Compare(Rhs, Lhs);
	}


private:

	const FCompareRowByColumnAscending< FPropertyType > Comparer;
};

struct FCompareRowByColumnUsingExportTextLexicographic : public FCompareRowByColumnBase
{
public:
	FCompareRowByColumnUsingExportTextLexicographic( const TSharedRef< IPropertyTableColumn >& InColumn, const FProperty* InFProperty, bool InAscendingOrder )
		: Property( InFProperty )
		, Column( InColumn )
		, bAscending( InAscendingOrder )
	{

	}

	int32 Compare( const TSharedRef< IPropertyTableRow >& Lhs, const TSharedRef< IPropertyTableRow >& Rhs ) const
	{
		const TSharedRef< IPropertyTableCell > LhsCell = Column->GetCell( Lhs );
		const TSharedRef< IPropertyTableCell > RhsCell = Column->GetCell( Rhs );

		const TSharedPtr< FPropertyNode > LhsPropertyNode = LhsCell->GetNode();
		if ( !LhsPropertyNode.IsValid() )
		{
			return 1;
		}

		const TSharedPtr< FPropertyNode > RhsPropertyNode = RhsCell->GetNode();
		if ( !RhsPropertyNode.IsValid() )
		{
			return -1;
		}

		const TSharedPtr< IPropertyHandle > LhsPropertyHandle = PropertyEditorHelpers::GetPropertyHandle( LhsPropertyNode.ToSharedRef(), NULL, NULL );
		if ( !LhsPropertyHandle.IsValid() )
		{
			return 1;
		}

		const TSharedPtr< IPropertyHandle > RhsPropertyHandle = PropertyEditorHelpers::GetPropertyHandle( RhsPropertyNode.ToSharedRef(), NULL, NULL );
		if ( !RhsPropertyHandle.IsValid() )
		{
			return -1;
		}

		return ComparePropertyValue( LhsPropertyHandle, RhsPropertyHandle );
	}

private:

	int32 ComparePropertyValue( const TSharedPtr< IPropertyHandle >& LhsPropertyHandle, const TSharedPtr< IPropertyHandle >& RhsPropertyHandle ) const
	{
		FString LhsValue; 
		LhsPropertyHandle->GetValueAsDisplayString( LhsValue );

		FString RhsValue; 
		RhsPropertyHandle->GetValueAsDisplayString( RhsValue );

		if (LhsValue < RhsValue)
		{
			return bAscending ? -1 : 1;
		}
		else if (LhsValue > RhsValue)
		{
			return bAscending ? 1: -1;
		}

		return 0;
	}

private:

	const FProperty* Property;
	TSharedRef< IPropertyTableColumn > Column;
	bool bAscending;
};



template<>
FORCEINLINE int32 FCompareRowByColumnAscending<FEnumProperty>::ComparePropertyValue( const TSharedPtr< IPropertyHandle >& LhsPropertyHandle, const TSharedPtr< IPropertyHandle >& RhsPropertyHandle ) const
{
	// Only Bytes work right now

	// Get the basic uint8 values
	uint8 LhsValue; 
	LhsPropertyHandle->GetValue( LhsValue );

	uint8 RhsValue; 
	RhsPropertyHandle->GetValue( RhsValue );

	// Bytes are trivially sorted numerically
	UEnum* PropertyEnum = Property->GetEnum();

	// Enums are sorted alphabetically based on the full enum entry name - must be sure that values are within Enum bounds!
	int32 LhsIndex = PropertyEnum->GetIndexByValue(LhsValue);
	int32 RhsIndex = PropertyEnum->GetIndexByValue(RhsValue);
	bool bLhsEnumValid = LhsIndex != INDEX_NONE;
	bool bRhsEnumValid = RhsIndex != INDEX_NONE;
	if (bLhsEnumValid && bRhsEnumValid)
	{
		FName LhsEnumName(PropertyEnum->GetNameByIndex(LhsIndex));
		FName RhsEnumName(PropertyEnum->GetNameByIndex(RhsIndex));
		return LhsEnumName.Compare(RhsEnumName);
	}
	else if(bLhsEnumValid)
	{
		return -1;
	}
	else if(bRhsEnumValid)
	{
		return 1;
	}
	else
	{
		return RhsValue - LhsValue;
	}
}

// FByteProperty objects may in fact represent Enums - so they need special handling for alphabetic Enum vs. numerical Byte sorting.
template<>
FORCEINLINE int32 FCompareRowByColumnAscending<FByteProperty>::ComparePropertyValue( const TSharedPtr< IPropertyHandle >& LhsPropertyHandle, const TSharedPtr< IPropertyHandle >& RhsPropertyHandle ) const
{
	// Get the basic uint8 values
	uint8 LhsValue; 
	LhsPropertyHandle->GetValue( LhsValue );

	uint8 RhsValue; 
	RhsPropertyHandle->GetValue( RhsValue );

	// Bytes are trivially sorted numerically
	UEnum* PropertyEnum = Property->GetIntPropertyEnum();
	if(PropertyEnum == nullptr)
	{
		return LhsValue < RhsValue;
	}
	else
	{
		int32 LhsIndex = PropertyEnum->GetIndexByValue(LhsValue);
		int32 RhsIndex = PropertyEnum->GetIndexByValue(RhsValue);
		// But Enums are sorted alphabetically based on the full enum entry name - must be sure that values are within Enum bounds!
		bool bLhsEnumValid = LhsIndex != INDEX_NONE;
		bool bRhsEnumValid = RhsIndex != INDEX_NONE;
		if(bLhsEnumValid && bRhsEnumValid)
		{
			FName LhsEnumName(PropertyEnum->GetNameByIndex(LhsIndex));
			FName RhsEnumName(PropertyEnum->GetNameByIndex(RhsIndex));
			return LhsEnumName.Compare(RhsEnumName);
		}
		else if(bLhsEnumValid)
		{
			return true;
		}
		else if(bRhsEnumValid)
		{
			return false;
		}
		else
		{
			return RhsValue - LhsValue;
		}
	}
}

template<>
FORCEINLINE int32 FCompareRowByColumnAscending<FNameProperty>::ComparePropertyValue( const TSharedPtr< IPropertyHandle >& LhsPropertyHandle, const TSharedPtr< IPropertyHandle >& RhsPropertyHandle ) const
{
	FName LhsValue; 
	LhsPropertyHandle->GetValue( LhsValue );

	FName RhsValue; 
	RhsPropertyHandle->GetValue( RhsValue );

	return LhsValue.Compare(RhsValue);
}

template<>
FORCEINLINE int32 FCompareRowByColumnAscending<FObjectPropertyBase>::ComparePropertyValue( const TSharedPtr< IPropertyHandle >& LhsPropertyHandle, const TSharedPtr< IPropertyHandle >& RhsPropertyHandle ) const
{
	UObject* LhsValue; 
	LhsPropertyHandle->GetValue( LhsValue );

	if ( LhsValue == NULL )
	{
		return 1;
	}

	UObject* RhsValue; 
	RhsPropertyHandle->GetValue( RhsValue );

	if ( RhsValue == NULL )
	{
		return -1;
	}

	return FCString::Stricmp(*LhsValue->GetName(), *RhsValue->GetName());
}

template<>
FORCEINLINE int32 FCompareRowByColumnAscending<FStructProperty>::ComparePropertyValue( const TSharedPtr< IPropertyHandle >& LhsPropertyHandle, const TSharedPtr< IPropertyHandle >& RhsPropertyHandle ) const
{
	if ( !FPropertyTableColumn::IsSupportedStructProperty(LhsPropertyHandle->GetProperty() ) )
	{
		return 1;
	}

	if ( !FPropertyTableColumn::IsSupportedStructProperty(RhsPropertyHandle->GetProperty() ) )
	{
		return -1;
	}

	{
		FVector LhsVector;
		FVector RhsVector;

		if ( LhsPropertyHandle->GetValue(LhsVector) != FPropertyAccess::Fail && RhsPropertyHandle->GetValue(RhsVector) != FPropertyAccess::Fail )
		{
			return static_cast<int32>(RhsVector.SizeSquared() - LhsVector.SizeSquared());
		}

		FVector2D LhsVector2D;
		FVector2D RhsVector2D;

		if ( LhsPropertyHandle->GetValue(LhsVector2D) != FPropertyAccess::Fail && RhsPropertyHandle->GetValue(RhsVector2D) != FPropertyAccess::Fail )
		{
			return static_cast<int32>(RhsVector2D.SizeSquared() - LhsVector2D.SizeSquared());
		}

		FVector4 LhsVector4;
		FVector4 RhsVector4;

		if ( LhsPropertyHandle->GetValue(LhsVector4) != FPropertyAccess::Fail && RhsPropertyHandle->GetValue(RhsVector4) != FPropertyAccess::Fail )
		{
			return static_cast<int32>(RhsVector4.SizeSquared() - LhsVector4.SizeSquared());
		}
	}

	ensureMsgf(false, TEXT("A supported struct property does not have a defined implementation for sorting a property column."));
	return 0;
}


FPropertyTableColumn::FPropertyTableColumn( const TSharedRef< IPropertyTable >& InTable, const TWeakObjectPtr< UObject >& InObject )
	: Cells()
	, DataSource( MakeShared<UObjectDataSource>(InObject.Get() ) )
	, Table( InTable )
	, Id( NAME_None )
	, DisplayName()
	, Width( 1.0f )
	, bIsHidden( false )
	, bIsFrozen( false )
	, PartialPath( FPropertyPath::CreateEmpty() )
	, SizeMode(EPropertyTableColumnSizeMode::Fill)
{
	GenerateColumnId();
	GenerateColumnDisplayName();
}

FPropertyTableColumn::FPropertyTableColumn( const TSharedRef< IPropertyTable >& InTable, const TSharedRef< FPropertyPath >& InPropertyPath )
	: Cells()
	, DataSource( MakeShareable( new PropertyPathDataSource( InPropertyPath ) ) )
	, Table( InTable )
	, Id( NAME_None )
	, DisplayName()
	, Width( 1.0f )
	, bIsHidden( false )
	, bIsFrozen( false )
	, PartialPath( FPropertyPath::CreateEmpty() )
	, SizeMode(EPropertyTableColumnSizeMode::Fill)
{
	GenerateColumnId();
	GenerateColumnDisplayName();
}

FPropertyTableColumn::FPropertyTableColumn( const TSharedRef< class IPropertyTable >& InTable, const TWeakObjectPtr< UObject >& InObject, const TSharedRef< FPropertyPath >& InPartialPropertyPath )
	: Cells()
	, DataSource( MakeShareable( new UObjectDataSource( InObject.Get() ) ) )
	, Table( InTable )
	, Id( NAME_None )
	, DisplayName()
	, Width( 1.0f )
	, bIsHidden( false )
	, bIsFrozen( false )
	, PartialPath( InPartialPropertyPath )
	, SizeMode(EPropertyTableColumnSizeMode::Fill)
{
	GenerateColumnId();
}


void FPropertyTableColumn::GenerateColumnId()
{
	TWeakObjectPtr< UObject > Object = DataSource->AsUObject();
	TSharedPtr< FPropertyPath > PropertyPath = DataSource->AsPropertyPath();

	// Use partial path for a valid column ID if we have one. We are pointing to a container with an array, but all columns must be unique
	if ( PartialPath->GetNumProperties() > 0 )
	{
		Id = FName( *PartialPath->ToString());
	}
	else if ( Object.IsValid() )
	{
		Id = Object->GetFName();
	}
	else if ( PropertyPath.IsValid() )
	{
		Id = FName( *PropertyPath->ToString() );
	}
	else
	{
		Id = NAME_None;
	}
}

void FPropertyTableColumn::GenerateColumnDisplayName()
{
	TWeakObjectPtr< UObject > Object = DataSource->AsUObject();
	TSharedPtr< FPropertyPath > PropertyPath = DataSource->AsPropertyPath();

	if ( Object.IsValid() )
	{
		DisplayName = FText::FromString(Object->GetFName().ToString());
	}
	else if ( PropertyPath.IsValid() )
	{
		//@todo unify this logic with all the property editors [12/11/2012 Justin.Sargent]
		FString NewName;
		bool FirstAddition = true;
		const FPropertyInfo* PreviousPropInfo = NULL;
		for (int PropertyIndex = 0; PropertyIndex < PropertyPath->GetNumProperties(); PropertyIndex++)
		{
			const FPropertyInfo& PropInfo = PropertyPath->GetPropertyInfo( PropertyIndex );

			if ( !(PropInfo.Property->IsA( FArrayProperty::StaticClass() ) && PropertyIndex != PropertyPath->GetNumProperties() - 1 ) )
			{
				if ( !FirstAddition )
				{
					NewName += TEXT( "->" );
				}

				FString PropertyName = PropInfo.Property->GetDisplayNameText().ToString();

				if ( PropertyName.IsEmpty() )
				{
					PropertyName = PropInfo.Property->GetName();

					const bool bIsBoolProperty = CastField<const FBoolProperty>( PropInfo.Property.Get() ) != NULL;

					if ( PreviousPropInfo != NULL )
					{
						const FStructProperty* ParentStructProperty = CastField<const FStructProperty>( PreviousPropInfo->Property.Get() );
						if( ParentStructProperty && ParentStructProperty->Struct->GetFName() == NAME_Rotator )
						{
							if( PropInfo.Property->GetFName() == "Roll" )
							{
								PropertyName = TEXT("X");
							}
							else if( PropInfo.Property->GetFName() == "Pitch" )
							{
								PropertyName = TEXT("Y");
							}
							else if( PropInfo.Property->GetFName() == "Yaw" )
							{
								PropertyName = TEXT("Z");
							}
							else
							{
								check(0);
							}
						}
					}

					PropertyName = FName::NameToDisplayString( PropertyName, bIsBoolProperty );
				}

				NewName += PropertyName;

				if ( PropInfo.ArrayIndex != INDEX_NONE )
				{
					NewName += FString::Printf( TEXT( "[%d]" ), PropInfo.ArrayIndex );
				}

				PreviousPropInfo = &PropInfo;
				FirstAddition = false;
			}
		}

		DisplayName = FText::FromString(*NewName);
	}
	else
	{
		DisplayName = LOCTEXT( "InvalidColumnName", "Invalid Property" );
	}
}

FName FPropertyTableColumn::GetId() const 
{ 
	return Id;
}

FText FPropertyTableColumn::GetDisplayName() const 
{ 
	return DisplayName;
}

TSharedRef< IPropertyTableCell > FPropertyTableColumn::GetCell( const TSharedRef< class IPropertyTableRow >& Row ) 
{
	//@todo Clean Cells cache when rows get updated [11/27/2012 Justin.Sargent]
	TSharedRef< IPropertyTableCell >* CellPtr = Cells.Find( Row );

	if( CellPtr != NULL )
	{
		return *CellPtr;
	}

	TSharedRef< IPropertyTableCell > Cell = MakeShareable( new FPropertyTableCell( SharedThis( this ), Row ) );
	Cells.Add( Row, Cell );

	return Cell;
}

void FPropertyTableColumn::RemoveCellsForRow( const TSharedRef< class IPropertyTableRow >& Row )
{
	Cells.Remove( Row );
}

TSharedRef< class IPropertyTable > FPropertyTableColumn::GetTable() const
{
	return Table.Pin().ToSharedRef();
}

bool FPropertyTableColumn::CanSortBy() const
{
	TWeakObjectPtr< UObject > Object = DataSource->AsUObject();
	const FProperty* Property = nullptr;

	TSharedPtr< FPropertyPath > Path = DataSource->AsPropertyPath();
	if ( Path.IsValid() )
	{
		Property = Path->GetLeafMostProperty().Property.Get();
	}

	return ( Property != nullptr );
}

TSharedPtr<FCompareRowByColumnBase> FPropertyTableColumn::GetPropertySorter(const FProperty* Property, EColumnSortMode::Type SortMode)
{
	if (Property->IsA(FEnumProperty::StaticClass()))
	{
		const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property);

		if (SortMode == EColumnSortMode::Ascending)
		{
			return MakeShareable<FCompareRowByColumnBase>(new FCompareRowByColumnAscending<FEnumProperty>(SharedThis(this), EnumProperty));
		}
		else
		{
			return MakeShareable<FCompareRowByColumnBase>(new FCompareRowByColumnDescending<FEnumProperty>(SharedThis(this), EnumProperty));
		}
	}
	else if (Property->IsA(FByteProperty::StaticClass()))
	{
		const FByteProperty* ByteProperty = CastField<FByteProperty>(Property);

		if (SortMode == EColumnSortMode::Ascending)
		{
			return MakeShareable<FCompareRowByColumnBase>(new FCompareRowByColumnAscending<FByteProperty>(SharedThis(this), ByteProperty));
		}
		else
		{
			return MakeShareable<FCompareRowByColumnBase>(new FCompareRowByColumnDescending<FByteProperty>(SharedThis(this), ByteProperty));
		}
	}
	else if (Property->IsA(FIntProperty::StaticClass()))
	{
		const FIntProperty* IntProperty = CastField<FIntProperty>(Property);

		if (SortMode == EColumnSortMode::Ascending)
		{
			return MakeShareable<FCompareRowByColumnBase>(new FCompareRowByColumnAscending<FIntProperty>(SharedThis(this), IntProperty));
		}
		else
		{
			return MakeShareable<FCompareRowByColumnBase>(new FCompareRowByColumnDescending<FIntProperty>(SharedThis(this), IntProperty));
		}
	}
	else if (Property->IsA(FBoolProperty::StaticClass()))
	{
		const FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property);

		if (SortMode == EColumnSortMode::Ascending)
		{
			return MakeShareable<FCompareRowByColumnBase>(new FCompareRowByColumnAscending<FBoolProperty>(SharedThis(this), BoolProperty));
		}
		else
		{
			return MakeShareable<FCompareRowByColumnBase>(new FCompareRowByColumnDescending<FBoolProperty >(SharedThis(this), BoolProperty));
		}
	}
	else if (Property->IsA(FFloatProperty::StaticClass()))
	{
		const FFloatProperty* FloatProperty(CastField< FFloatProperty >(Property));

		if (SortMode == EColumnSortMode::Ascending)
		{
			return MakeShareable<FCompareRowByColumnBase>(new FCompareRowByColumnAscending<FFloatProperty>(SharedThis(this), FloatProperty));
		}
		else
		{
			return MakeShareable<FCompareRowByColumnBase>(new FCompareRowByColumnDescending<FFloatProperty>(SharedThis(this), FloatProperty));
		}
	}
	else if (Property->IsA(FDoubleProperty::StaticClass()))
	{
		const FDoubleProperty* DoubleProperty(CastField< FDoubleProperty >(Property));

		if (SortMode == EColumnSortMode::Ascending)
		{
			return MakeShareable<FCompareRowByColumnBase>(new FCompareRowByColumnAscending<FDoubleProperty>(SharedThis(this), DoubleProperty));
		}
		else
		{
			return MakeShareable<FCompareRowByColumnBase>(new FCompareRowByColumnDescending<FDoubleProperty>(SharedThis(this), DoubleProperty));
		}
	}
	else if (Property->IsA(FNameProperty::StaticClass()))
	{
		const FNameProperty* NameProperty = CastField<FNameProperty>(Property);

		if (SortMode == EColumnSortMode::Ascending)
		{
			return MakeShareable<FCompareRowByColumnBase>(new FCompareRowByColumnAscending<FNameProperty>(SharedThis(this), NameProperty));
		}
		else
		{
			return MakeShareable<FCompareRowByColumnBase>(new FCompareRowByColumnDescending<FNameProperty>(SharedThis(this), NameProperty));
		}
	}
	else if (Property->IsA(FStrProperty::StaticClass()))
	{
		const FStrProperty* StrProperty = CastField<FStrProperty>(Property);

		if (SortMode == EColumnSortMode::Ascending)
		{
			return MakeShareable<FCompareRowByColumnBase>(new FCompareRowByColumnAscending<FStrProperty>(SharedThis(this), StrProperty));
		}
		else
		{
			return MakeShareable<FCompareRowByColumnBase>(new FCompareRowByColumnDescending<FStrProperty>(SharedThis(this), StrProperty));
		}
	}
	else if (Property->IsA(FObjectPropertyBase::StaticClass()) && !Property->HasAnyPropertyFlags(CPF_InstancedReference))
	{
		const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property);

		if (SortMode == EColumnSortMode::Ascending)
		{
			return MakeShareable<FCompareRowByColumnBase>(new FCompareRowByColumnAscending<FObjectPropertyBase>(SharedThis(this), ObjectProperty));
		}
		else
		{
			return MakeShareable<FCompareRowByColumnBase>(new FCompareRowByColumnDescending<FObjectPropertyBase>(SharedThis(this), ObjectProperty));
		}
	}
	else if (IsSupportedStructProperty(Property))
	{
		const FStructProperty* StructProperty = CastField<FStructProperty>(Property);

		if (SortMode == EColumnSortMode::Ascending)
		{
			return MakeShareable<FCompareRowByColumnBase>(new FCompareRowByColumnAscending<FStructProperty>(SharedThis(this), StructProperty));
		}
		else
		{
			return MakeShareable<FCompareRowByColumnBase>(new FCompareRowByColumnDescending<FStructProperty>(SharedThis(this), StructProperty));
		}
	}
	//else if ( Property->IsA( FTextProperty::StaticClass() ) )
	//{
	//	if ( SortMode == EColumnSortMode::Ascending )
	//	{
	//		Rows.Sort( FCompareRowByColumnAscending< FTextProperty >( SharedThis( this ) ) );
	//	}
	//	else
	//	{
	//		Rows.Sort( FCompareRowByColumnDescending< FTextProperty >( SharedThis( this ) ) );
	//	}
	//}
	else
	{
		return MakeShareable<FCompareRowByColumnUsingExportTextLexicographic>(new FCompareRowByColumnUsingExportTextLexicographic(SharedThis(this), Property, (SortMode == EColumnSortMode::Ascending)));
	}
}

void FPropertyTableColumn::Sort( TArray< TSharedRef< class IPropertyTableRow > >& Rows, const EColumnSortMode::Type PrimarySortMode, const TSharedPtr<IPropertyTableColumn>& SecondarySortColumn, const EColumnSortMode::Type SecondarySortMode )
{
	if (PrimarySortMode == EColumnSortMode::None )
	{
		return;
	}

	UObject* PrimaryObject = DataSource->AsUObject().Get();
	const FProperty* PrimaryProperty = nullptr;
	TSharedPtr< FPropertyPath > PrimaryPath = DataSource->AsPropertyPath();
	if (PrimaryPath.IsValid())
	{
		PrimaryProperty = PrimaryPath->GetLeafMostProperty().Property.Get();
	}

	UObject* SecondaryObject = nullptr;
	const FProperty* SecondaryProperty = nullptr;
	if(SecondarySortColumn.IsValid())
	{
		SecondaryObject = SecondarySortColumn->GetDataSource()->AsUObject().Get();
		SecondaryProperty = nullptr;
		TSharedPtr< FPropertyPath > SecondaryPath = SecondarySortColumn->GetDataSource()->AsPropertyPath();
		if (SecondaryPath.IsValid())
		{
			SecondaryProperty = SecondaryPath->GetLeafMostProperty().Property.Get();
		}
	}


	if (!PrimaryProperty)
	{
		return;
	}


	TSharedPtr<FCompareRowByColumnBase> SecondarySorter = nullptr;
	if(SecondaryProperty && SecondarySortMode != EColumnSortMode::None)
	{
		SecondarySorter = SecondarySortColumn->GetPropertySorter(SecondaryProperty, SecondarySortMode);
	}

	// if we had a secondary sort we need to make sure the primary sort is stable to not break the secondary results
	TSharedPtr<FCompareRowByColumnBase> PrimarySorter = GetPropertySorter(PrimaryProperty, PrimarySortMode);
	Rows.Sort(FCompareRowPrimaryAndSecondary(PrimarySorter.Get(), SecondarySorter.Get()));

}

void FPropertyTableColumn::Tick()
{
	if ( !DataSource->AsPropertyPath().IsValid() )
	{
		const TSharedRef< IPropertyTable > TableRef = GetTable();
		const TWeakObjectPtr< UObject > Object = DataSource->AsUObject();

		if ( !Object.IsValid() )
		{
			TableRef->RemoveColumn( SharedThis( this ) );
		}
		else
		{
			const TSharedRef< FObjectPropertyNode > Node = TableRef->GetObjectPropertyNode( Object );
			EPropertyDataValidationResult Result = Node->EnsureDataIsValid();

			if ( Result == EPropertyDataValidationResult::ObjectInvalid )
			{
				TableRef->RemoveColumn( SharedThis( this ) );
			}
			else if ( Result == EPropertyDataValidationResult::ArraySizeChanged )
			{
				TableRef->RequestRefresh();
			}
		}
	}
}

void FPropertyTableColumn::SetFrozen(bool InIsFrozen)
{
	bIsFrozen = InIsFrozen;
	FrozenStateChanged.Broadcast( SharedThis(this) );
}

bool FPropertyTableColumn::IsSupportedStructProperty(const FProperty* InProperty)
{
	if ( InProperty != nullptr && CastField<FStructProperty>(InProperty) != nullptr)
	{
		const FStructProperty* StructProp = CastField<FStructProperty>(InProperty);
		FName StructName = StructProp->Struct->GetFName();

		return StructName == NAME_Vector ||
			StructName == NAME_Vector2D	 ||
			StructName == NAME_Vector4;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
