// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PropertyPath.h"
#include "IPropertyTableColumn.h"
#include "IPropertyTable.h"
#include "IPropertyTableRow.h"

class FPropertyTableColumn : public TSharedFromThis< FPropertyTableColumn >, public IPropertyTableColumn
{
public:

	FPropertyTableColumn( const TSharedRef< IPropertyTable >& InTable, const TWeakObjectPtr< UObject >& InObject );

	FPropertyTableColumn( const TSharedRef< IPropertyTable >& InTable, const TSharedRef< FPropertyPath >& InPropertyPath );

	FPropertyTableColumn( const TSharedRef< IPropertyTable >& InTable, const TWeakObjectPtr< UObject >& InObject, const TSharedRef< FPropertyPath >& InPartialPropertyPath );

	virtual ~FPropertyTableColumn() = default;

	//~ Begin IPropertyTable Interface

	virtual FName GetId() const override;

	virtual FText GetDisplayName() const override;

	virtual TSharedRef< IDataSource > GetDataSource() const override { return DataSource; }

	virtual TSharedRef< class FPropertyPath > GetPartialPath() const override { return PartialPath; }

	virtual TSharedRef< class IPropertyTableCell > GetCell( const TSharedRef< class IPropertyTableRow >& Row ) override;

	virtual void RemoveCellsForRow( const TSharedRef< class IPropertyTableRow >& Row ) override;

	virtual TSharedRef< class IPropertyTable > GetTable() const override;

	virtual bool CanSelectCells() const override { return !IsHidden(); }

	virtual EPropertyTableColumnSizeMode::Type GetSizeMode() const override { return SizeMode; }

	virtual void SetSizeMode(EPropertyTableColumnSizeMode::Type InSizeMode) override{ SizeMode = InSizeMode; }

	virtual float GetWidth() const override { return Width; } 

	virtual void SetWidth( float InWidth ) override { Width = InWidth; }

	virtual bool IsHidden() const override { return bIsHidden; }

	virtual void SetHidden( bool InIsHidden ) override { bIsHidden = InIsHidden; }

	virtual bool IsFrozen() const override { return bIsFrozen; }

	virtual void SetFrozen( bool InIsFrozen ) override;

	virtual bool CanSortBy() const override;

	virtual void Sort(TArray< TSharedRef< class IPropertyTableRow > >& Rows, const EColumnSortMode::Type PrimarySortMode, const TSharedPtr<IPropertyTableColumn>& SecondarySortColumn, const EColumnSortMode::Type SecondarySortMode) override;

	virtual void Tick() override;

	virtual TSharedPtr<struct FCompareRowByColumnBase> GetPropertySorter(const FProperty* Property, EColumnSortMode::Type SortMode) override;

	DECLARE_DERIVED_EVENT( FPropertyTableColumn, IPropertyTableColumn::FFrozenStateChanged, FFrozenStateChanged );
	FFrozenStateChanged* OnFrozenStateChanged() override { return &FrozenStateChanged; }

	//~ End IPropertyTable Interface

	static bool IsSupportedStructProperty(const FProperty* InProperty);

private:

	void GenerateColumnId();

	void GenerateColumnDisplayName();


private:

	TMap< TSharedRef< IPropertyTableRow >, TSharedRef< class IPropertyTableCell > > Cells;

	TSharedRef< IDataSource > DataSource;
	TWeakPtr< IPropertyTable > Table;

	FName Id;
	FText DisplayName;

	float Width;

	bool bIsHidden;
	bool bIsFrozen;

	FFrozenStateChanged FrozenStateChanged;

	TSharedRef< class FPropertyPath > PartialPath;

	EPropertyTableColumnSizeMode::Type SizeMode;
};
