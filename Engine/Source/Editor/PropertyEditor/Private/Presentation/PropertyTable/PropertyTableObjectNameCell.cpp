// Copyright Epic Games, Inc. All Rights Reserved.
#include "Presentation/PropertyTable/PropertyTableObjectNameCell.h"

#include "IPropertyTable.h"
#include "IPropertyTableRow.h"
#include "ObjectPropertyNode.h"
#include "Presentation/PropertyTable/PropertyTableObjectNameColumn.h"
#include "Templates/TypeHash.h"
#include "UObject/WeakObjectPtr.h"

class IPropertyTableColumn;
class UObject;

FPropertyTableObjectNameCell::FPropertyTableObjectNameCell( const TSharedRef< class FPropertyTableObjectNameColumn >& InColumn, const TSharedRef< class IPropertyTableRow >& InRow )
	: EnteredEditModeEvent()
	, ExitedEditModeEvent()
	, Column( InColumn )
	, Row( InRow )
	, ObjectNode( NULL )
	, bIsBound( false )
	, bInEditMode( false )
{
	Refresh();
}

void FPropertyTableObjectNameCell::Refresh()
{
	ObjectName.Empty();
	bIsBound = false;
	const TSharedRef< IPropertyTableColumn > ColumnRef = Column.Pin().ToSharedRef();
	const TSharedRef< IPropertyTableRow > RowRef = Row.Pin().ToSharedRef();

	ObjectNode = GetTable()->GetObjectPropertyNode( ColumnRef, RowRef );

	if ( !ObjectNode.IsValid() || ObjectNode->GetNumObjects() != 1 )
	{
		return;
	}

	bIsBound = true;
}

bool FPropertyTableObjectNameCell::IsReadOnly() const 
{ 
	return true;
}

bool FPropertyTableObjectNameCell::IsValid() const
{
	return !IsBound() || (ObjectNode.IsValid() && (ObjectNode->GetNumObjects() == 1));
}

FString FPropertyTableObjectNameCell::GetValueAsString() const
{
	return Column.Pin()->GetObjectNameAsString( Row.Pin().ToSharedRef() );
}

FText FPropertyTableObjectNameCell::GetValueAsText() const
{
	return FText::FromString(GetValueAsString());
}

TWeakObjectPtr< UObject > FPropertyTableObjectNameCell::GetObject() const
{
	if ( !ObjectNode.IsValid() )
	{
		return NULL;
	}

	return ObjectNode->GetUObject( 0 );
}

TSharedRef< class IPropertyTableColumn > FPropertyTableObjectNameCell::GetColumn() const
{ 
	return Column.Pin().ToSharedRef(); 
}

TSharedRef< class IPropertyTableRow > FPropertyTableObjectNameCell::GetRow() const
{
	return Row.Pin().ToSharedRef(); 
}

TSharedRef< class IPropertyTable > FPropertyTableObjectNameCell::GetTable() const
{
	return Column.Pin()->GetTable();
}

void FPropertyTableObjectNameCell::EnterEditMode() 
{
	if ( !bInEditMode )
	{
		TSharedRef< IPropertyTable > Table = GetTable();
		Table->SetCurrentCell( SharedThis( this ) );
		bInEditMode = true;
		EnteredEditModeEvent.Broadcast();
	}
}

void FPropertyTableObjectNameCell::ExitEditMode()
{
	if ( bInEditMode )
	{
		bInEditMode = false;
		ExitedEditModeEvent.Broadcast();
	}
}
