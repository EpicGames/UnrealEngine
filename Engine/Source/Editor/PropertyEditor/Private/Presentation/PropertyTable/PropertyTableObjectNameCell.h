// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "IPropertyTableCell.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UObject;

class FPropertyTableObjectNameCell : public TSharedFromThis< FPropertyTableObjectNameCell >, public IPropertyTableCell
{
public:

	FPropertyTableObjectNameCell( const TSharedRef< class FPropertyTableObjectNameColumn >& InColumn, const TSharedRef< class IPropertyTableRow >& InRow );
	virtual ~FPropertyTableObjectNameCell() = default;

	virtual void Refresh() override;

	virtual bool IsReadOnly() const override;
	virtual bool IsBound() const override { return bIsBound; }
	virtual bool InEditMode() const override { return bInEditMode; }
	virtual bool IsValid() const override;

	virtual FString GetValueAsString() const override;
	virtual FText GetValueAsText() const override;
	virtual void SetValueFromString( const FString& InString ) override {}

	virtual TWeakObjectPtr< UObject > GetObject() const override;

	virtual TSharedPtr< class FPropertyNode > GetNode() const override { return NULL; }
	virtual TSharedRef< class IPropertyTableColumn > GetColumn() const override;
	virtual TSharedRef< class IPropertyTableRow > GetRow() const override;
	virtual TSharedRef< class IPropertyTable > GetTable() const override;
	virtual TSharedPtr< class IPropertyHandle > GetPropertyHandle() const override { return NULL; }

	virtual void EnterEditMode() override;
	virtual void ExitEditMode() override;

	DECLARE_DERIVED_EVENT( FPropertyTableObjectNameCell, IPropertyTableCell::FEnteredEditModeEvent, FEnteredEditModeEvent );
	virtual FEnteredEditModeEvent& OnEnteredEditMode() override { return EnteredEditModeEvent; }

	DECLARE_DERIVED_EVENT( FPropertyTableObjectNameCell, IPropertyTableCell::FExitedEditModeEvent, FExitedEditModeEvent  );
	virtual FExitedEditModeEvent& OnExitedEditMode() override { return ExitedEditModeEvent; }


private:

	FEnteredEditModeEvent EnteredEditModeEvent;
	FExitedEditModeEvent ExitedEditModeEvent;

	TWeakPtr< class FPropertyTableObjectNameColumn > Column;
	TWeakPtr< class IPropertyTableRow > Row;

	TSharedPtr< class FObjectPropertyNode > ObjectNode;
	FString ObjectName;

	bool bIsBound;
	bool bInEditMode;
};
