// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "ObjectPropertyNode.h"
#include "PropertyPath.h"
#include "IPropertyTableColumn.h"
#include "IPropertyTable.h"
#include "IPropertyTableCell.h"
#include "IPropertyTableRow.h"
#include "Presentation/PropertyTable/PropertyTableColumn.h"
#include "Presentation/PropertyTable/DataSource.h"
#include "Presentation/PropertyTable/PropertyTableObjectNameCell.h"
#include "Engine/BlueprintGeneratedClass.h"

#define LOCTEXT_NAMESPACE "ObjectNameColumnHeader"

class FPropertyTableObjectNameColumn : public TSharedFromThis< FPropertyTableObjectNameColumn >, public IPropertyTableColumn
{
public:

	FPropertyTableObjectNameColumn( const TSharedRef< IPropertyTable >& InTable )
		: Table( InTable )
		, Cells()
		, Width( 2.0f )
		, bIsHidden( false )
	{

	}

	virtual ~FPropertyTableObjectNameColumn() = default;

	//  Begin IPropertyTable Interface

	virtual FName GetId() const override { return FName( TEXT("ObjectName") ); }

	virtual FText GetDisplayName() const override { return LOCTEXT( "DisplayName", "Name" ); }

	virtual TSharedRef< IDataSource > GetDataSource() const override { return MakeShareable( new NoDataSource() ); }

	virtual TSharedRef< class FPropertyPath > GetPartialPath() const override { return FPropertyPath::CreateEmpty(); }

	virtual TSharedRef< class IPropertyTableCell > GetCell( const TSharedRef< class IPropertyTableRow >& Row ) override
	{
		//@todo Clean Cells cache when rows get updated [11/27/2012 Justin.Sargent]
		TSharedRef< IPropertyTableCell >* CellPtr = Cells.Find( Row );

		if( CellPtr != NULL )
		{
			return *CellPtr;
		}

		TSharedRef< IPropertyTableCell > Cell = MakeShareable( new FPropertyTableObjectNameCell( SharedThis( this ), Row ) );
		Cells.Add( Row, Cell );

		return Cell;
	}

	virtual void RemoveCellsForRow( const TSharedRef< class IPropertyTableRow >& Row ) override
	{
		Cells.Remove( Row );
	}

	virtual TSharedRef< class IPropertyTable > GetTable() const override { return Table.Pin().ToSharedRef(); }

	virtual bool CanSelectCells() const override { return true; }

	virtual EPropertyTableColumnSizeMode::Type GetSizeMode() const override { return EPropertyTableColumnSizeMode::Fill; }

	virtual void SetSizeMode(EPropertyTableColumnSizeMode::Type InSizeMode) override {}

	virtual float GetWidth() const override { return Width; } 

	virtual void SetWidth( float InWidth ) override { Width = InWidth; }

	virtual bool IsHidden() const override { return bIsHidden; }

	virtual void SetHidden( bool InIsHidden ) override { bIsHidden = InIsHidden; }

	virtual bool IsFrozen() const override { return true; }

	virtual void SetFrozen( bool InIsFrozen ) override {}

	virtual bool CanSortBy() const override { return true; }

	virtual void Sort( TArray< TSharedRef< class IPropertyTableRow > >& Rows, const EColumnSortMode::Type PrimarySortMode, const TSharedPtr<IPropertyTableColumn>& SecondarySortColumn, const EColumnSortMode::Type SecondarySortMode ) override
	{
		struct FCompareRowByObjectNameAscending
		{
		public:
			FCompareRowByObjectNameAscending(const TSharedRef< FPropertyTableObjectNameColumn >& Column)
				: NameColumn(Column)
			{ }

			FORCEINLINE bool operator()(const TSharedRef< IPropertyTableRow >& Lhs, const TSharedRef< IPropertyTableRow >& Rhs) const
			{
				return NameColumn->GetObjectNameAsString(Lhs) < NameColumn->GetObjectNameAsString(Rhs);
			}

			TSharedRef< FPropertyTableObjectNameColumn > NameColumn;
		};

		struct FCompareRowByObjectNameDescending
		{
		public:
			FCompareRowByObjectNameDescending(const TSharedRef< FPropertyTableObjectNameColumn >& Column)
				: Comparer(Column)
			{ }

			FORCEINLINE bool operator()(const TSharedRef< IPropertyTableRow >& Lhs, const TSharedRef< IPropertyTableRow >& Rhs) const
			{
				return !Comparer(Lhs, Rhs);
			}

		private:

			const FCompareRowByObjectNameAscending Comparer;
		};

		if (PrimarySortMode == EColumnSortMode::None)
		{
			return;
		}

		if (PrimarySortMode == EColumnSortMode::Ascending )
		{
			Rows.Sort( FCompareRowByObjectNameAscending( SharedThis( this ) ) );
		}
		else
		{
			Rows.Sort( FCompareRowByObjectNameDescending( SharedThis( this ) ) );
		}
	}

	virtual TSharedPtr<struct FCompareRowByColumnBase> GetPropertySorter(const FProperty* Property, EColumnSortMode::Type SortMode) override
	{
		// Does not sort properties
		return nullptr;
	}

	virtual void Tick() override {}

	DECLARE_DERIVED_EVENT( FPropertyTableColumn, IPropertyTableColumn::FFrozenStateChanged, FFrozenStateChanged );
	FFrozenStateChanged* OnFrozenStateChanged() override { return &FrozenStateChanged; }

	// End IPropertyTable Interface

public:

	FString GetObjectNameAsString( const TSharedRef< IPropertyTableRow >& Row )
	{
		FString ObjectName;
		FString Suffix = GetTable()->GetRootPath()->ExtendPath( Row->GetPartialPath() )->ToString();
		if ( !Suffix.IsEmpty() )
		{
			Suffix = FString( TEXT("->") ) + Suffix;
		}

		const UObject* const Object = GetTable()->GetObjectPropertyNode( SharedThis( this ), Row )->GetUObject( 0 );
		if ( Object != NULL )
		{
			UClass* ObjectClass = Object->GetClass();
			const UBlueprintGeneratedClass* const BlueprintGeneratedClass = Cast<UBlueprintGeneratedClass>( ObjectClass ); 
			const bool IsCDO = Object->HasAnyFlags(RF_ClassDefaultObject);
			if ( BlueprintGeneratedClass != NULL && IsCDO )
			{
				ObjectName = BlueprintGeneratedClass->ClassGeneratedBy->GetFName().ToString();
			}
			else if ( Object->IsA( AActor::StaticClass() ) )
			{
				ObjectName = Cast<const  AActor >( Object )->GetActorLabel();
			}
			else
			{
				ObjectName = Object->GetFName().ToString();
			}
		}

		return ObjectName + Suffix;
	}


private:

	TWeakPtr< IPropertyTable > Table;
	TMap< TSharedRef< IPropertyTableRow >, TSharedRef< class IPropertyTableCell > > Cells;
	float Width;
	bool bIsHidden;

	FFrozenStateChanged FrozenStateChanged;
};

#undef LOCTEXT_NAMESPACE
