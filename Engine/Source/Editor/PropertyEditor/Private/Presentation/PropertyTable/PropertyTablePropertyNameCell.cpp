// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presentation/PropertyTable/PropertyTablePropertyNameCell.h"

#include "IPropertyTable.h"
#include "IPropertyTableRow.h"
#include "ObjectPropertyNode.h"
#include "Presentation/PropertyTable/PropertyTablePropertyNameColumn.h"
#include "PropertyPath.h"
#include "Templates/TypeHash.h"
#include "UObject/WeakObjectPtr.h"

class IPropertyTableColumn;
class UObject;


FPropertyTablePropertyNameCell::FPropertyTablePropertyNameCell( const TSharedRef< class FPropertyTablePropertyNameColumn >& InColumn, const TSharedRef< class IPropertyTableRow >& InRow )
	: bInEditMode( false )
	, bIsBound( true )
	, Column( InColumn )
	, EnteredEditModeEvent()
	, ExitedEditModeEvent()
	, Row( InRow )
{
	Refresh();
}

void FPropertyTablePropertyNameCell::Refresh()
{
	const TSharedRef< IPropertyTableColumn > ColumnRef = Column.Pin().ToSharedRef();
	const TSharedRef< IPropertyTableRow > RowRef = Row.Pin().ToSharedRef();

	ObjectNode = GetTable()->GetObjectPropertyNode( ColumnRef, RowRef );

	bIsBound = ObjectNode.IsValid();
}

FString FPropertyTablePropertyNameCell::GetValueAsString() const
{
	return Row.Pin()->GetDataSource()->AsPropertyPath()->ToString();
}

FText FPropertyTablePropertyNameCell::GetValueAsText() const
{
	return FText::FromString(GetValueAsString());
}

TSharedRef< class IPropertyTable > FPropertyTablePropertyNameCell::GetTable() const
{
	return Column.Pin()->GetTable();
}

void FPropertyTablePropertyNameCell::EnterEditMode() 
{
}

TSharedRef< class IPropertyTableColumn > FPropertyTablePropertyNameCell::GetColumn() const
{ 
	return Column.Pin().ToSharedRef(); 
}

TWeakObjectPtr< UObject > FPropertyTablePropertyNameCell::GetObject() const
{
	if ( !ObjectNode.IsValid() )
	{
		return NULL;
	}

	return ObjectNode->GetUObject( 0 );
}

TSharedRef< class IPropertyTableRow > FPropertyTablePropertyNameCell::GetRow() const
{ 
	return Row.Pin().ToSharedRef(); 
}

void FPropertyTablePropertyNameCell::ExitEditMode()
{
}
