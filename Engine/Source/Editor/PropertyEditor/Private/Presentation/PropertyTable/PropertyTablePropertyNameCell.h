// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "IPropertyTableCell.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UObject;

class FPropertyTablePropertyNameCell : public TSharedFromThis< FPropertyTablePropertyNameCell >, public IPropertyTableCell
{
public:

	FPropertyTablePropertyNameCell( const TSharedRef< class FPropertyTablePropertyNameColumn >& InColumn, const TSharedRef< class IPropertyTableRow >& InRow );

	//~ Begin IPropertyTableCell Interface

	virtual void EnterEditMode() override;

	virtual void ExitEditMode() override;

	virtual TSharedRef< class IPropertyTableColumn > GetColumn() const override;

	virtual TSharedPtr< class FPropertyNode > GetNode() const override { return NULL; }

	virtual TWeakObjectPtr< UObject > GetObject() const override;

	virtual TSharedRef< class IPropertyTableRow > GetRow() const override;

	virtual TSharedRef< class IPropertyTable > GetTable() const override;

	virtual FString GetValueAsString() const override;

	virtual FText GetValueAsText() const override;

	virtual bool InEditMode() const override { return bInEditMode; }

	virtual bool IsReadOnly() const override { return true; }

	virtual bool IsBound() const override { return bIsBound; }

	virtual bool IsValid() const override  { return true; }

	DECLARE_DERIVED_EVENT( FPropertyTablePropertyNameCell, IPropertyTableCell::FEnteredEditModeEvent, FEnteredEditModeEvent );
	virtual FEnteredEditModeEvent& OnEnteredEditMode() override { return EnteredEditModeEvent; }

	DECLARE_DERIVED_EVENT( FPropertyTablePropertyNameCell, IPropertyTableCell::FExitedEditModeEvent, FExitedEditModeEvent  );
	virtual FExitedEditModeEvent& OnExitedEditMode() override { return ExitedEditModeEvent; }

	virtual void Refresh() override;

	virtual void SetValueFromString( const FString& InString ) override {}

	//~ End IPropertyTableCell Interface

private:

	/** Is this cell being edited? */
	bool bInEditMode;

	/** Is this cell valid? */
	bool bIsBound;

	/** The column this cell is in */
	TWeakPtr< class FPropertyTablePropertyNameColumn > Column;

	/** Delegate to execute when we enter edit mode */
	FEnteredEditModeEvent EnteredEditModeEvent;

	/** Delegate to execute when we exit edit mode */
	FExitedEditModeEvent ExitedEditModeEvent;

	/** The object node which is associated with this cell */
	TSharedPtr< class FObjectPropertyNode > ObjectNode;

	/** The row this cell is in */
	TWeakPtr< class IPropertyTableRow > Row;
};
