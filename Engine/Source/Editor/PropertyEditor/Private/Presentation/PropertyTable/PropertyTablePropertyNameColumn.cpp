// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presentation/PropertyTable/PropertyTablePropertyNameColumn.h"
#include "Editor/EditorEngine.h"
#include "IPropertyTableCell.h"
#include "Presentation/PropertyTable/PropertyTableCell.h"
#include "Presentation/PropertyTable/DataSource.h"


#define LOCTEXT_NAMESPACE "PropertyNameColumnHeader"


FPropertyTablePropertyNameColumn::FPropertyTablePropertyNameColumn( const TSharedRef< IPropertyTable >& InTable )
	: bIsHidden( false )
	, Cells()
	, DataSource( MakeShareable( new NoDataSource() ) )
	, Table( InTable )
	, Width( 2.0f )
{

}


TSharedRef< class IPropertyTableCell > FPropertyTablePropertyNameColumn::GetCell( const TSharedRef< class IPropertyTableRow >& Row )
{
	TSharedRef< IPropertyTableCell >* CellPtr = Cells.Find( Row );

	if( CellPtr != NULL )
	{
		return *CellPtr;
	}

	TSharedRef< IPropertyTableCell > Cell = MakeShareable( new FPropertyTableCell( SharedThis( this ), Row ) );
	Cells.Add( Row, Cell );

	return Cell;
}


void FPropertyTablePropertyNameColumn::Sort( TArray<TSharedRef<class IPropertyTableRow>>& Rows, const EColumnSortMode::Type PrimarySortMode, const TSharedPtr<IPropertyTableColumn>& SecondarySortColumn, const EColumnSortMode::Type SecondarySortMode )
{
	struct FCompareRowByPropertyNameAscending
	{
	public:
		FCompareRowByPropertyNameAscending( const TSharedRef< FPropertyTablePropertyNameColumn >& Column )
			: NameColumn( Column )
		{ }

		FORCEINLINE bool operator()( const TSharedRef< IPropertyTableRow >& Lhs, const TSharedRef< IPropertyTableRow >& Rhs ) const
		{
			return NameColumn->GetPropertyNameAsString( Lhs ) < NameColumn->GetPropertyNameAsString( Rhs );
		}

		TSharedRef< FPropertyTablePropertyNameColumn > NameColumn;
	};

	struct FCompareRowByPropertyNameDescending
	{
	public:
		FCompareRowByPropertyNameDescending( const TSharedRef< FPropertyTablePropertyNameColumn >& Column )
			: Comparer( Column )
		{ }

		FORCEINLINE bool operator()( const TSharedRef< IPropertyTableRow >& Lhs, const TSharedRef< IPropertyTableRow >& Rhs ) const
		{
			return !Comparer( Lhs, Rhs ); 
		}

	private:

		const FCompareRowByPropertyNameAscending Comparer;
	};

	if (PrimarySortMode == EColumnSortMode::None )
	{
		return;
	}

	if (PrimarySortMode == EColumnSortMode::Ascending )
	{
		Rows.Sort( FCompareRowByPropertyNameAscending( SharedThis( this ) ) );
	}
	else
	{
		Rows.Sort( FCompareRowByPropertyNameDescending( SharedThis( this ) ) );
	}
}


TSharedPtr<struct FCompareRowByColumnBase> FPropertyTablePropertyNameColumn::GetPropertySorter(const FProperty* Property, EColumnSortMode::Type SortMode)
{
	// Does not sort properties
	return nullptr;
}

FString FPropertyTablePropertyNameColumn::GetPropertyNameAsString( const TSharedRef< IPropertyTableRow >& Row )
{
	FString PropertyName;
	if( Row->GetDataSource()->AsPropertyPath().IsValid() )
	{
		const TWeakFieldPtr< const FProperty > Property = Row->GetDataSource()->AsPropertyPath()->GetLeafMostProperty().Property;
		PropertyName = UEditorEngine::GetFriendlyName( Property.Get() );
	}
	return PropertyName;
}

#undef LOCTEXT_NAMESPACE
