// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PropertyPath.h"
#include "IPropertyTableColumn.h"
#include "IPropertyTable.h"
#include "IPropertyTableRow.h"
#include "Presentation/PropertyTable/PropertyTableColumn.h"

#define LOCTEXT_NAMESPACE "PropertyNameColumnHeader"

class FPropertyTablePropertyNameColumn : public TSharedFromThis< FPropertyTablePropertyNameColumn >, public IPropertyTableColumn
{
public:

	FPropertyTablePropertyNameColumn( const TSharedRef< IPropertyTable >& InTable );

	virtual ~FPropertyTablePropertyNameColumn() = default;

	//~ Begin IPropertyTableColumn Interface

	virtual bool CanSelectCells() const override { return true; }

	virtual bool CanSortBy() const override { return true; }

	virtual TSharedRef< class IPropertyTableCell > GetCell( const TSharedRef< class IPropertyTableRow >& Row ) override;

	virtual TSharedRef< IDataSource > GetDataSource() const override { return DataSource; }

	virtual TSharedRef< class FPropertyPath > GetPartialPath() const override { return FPropertyPath::CreateEmpty(); }

	virtual FText GetDisplayName() const override { return LOCTEXT( "DisplayName", "Name" ); }

	virtual FName GetId() const override { return FName( TEXT("PropertyName") ); }

	virtual EPropertyTableColumnSizeMode::Type GetSizeMode() const override { return EPropertyTableColumnSizeMode::Fill; }

	virtual void SetSizeMode(EPropertyTableColumnSizeMode::Type InSizeMode) override {}

	virtual TSharedRef< class IPropertyTable > GetTable() const override { return Table.Pin().ToSharedRef(); }

	virtual float GetWidth() const override { return Width; }

	virtual bool IsFrozen() const override { return false; }

	virtual bool IsHidden() const override { return bIsHidden; }

	virtual void RemoveCellsForRow( const TSharedRef< class IPropertyTableRow >& Row ) override
	{
		Cells.Remove( Row );
	}

	virtual void SetFrozen( bool InIsFrozen ) override {}

	virtual void SetHidden( bool InIsHidden ) override { bIsHidden = InIsHidden; }

	virtual void SetWidth( float InWidth ) override { Width = InWidth; }

	virtual void Sort( TArray< TSharedRef< class IPropertyTableRow > >& Rows, const EColumnSortMode::Type PrimarySortMode, const TSharedPtr<IPropertyTableColumn>& SecondarySortColumn, const EColumnSortMode::Type SecondarySortMode ) override;

	virtual TSharedPtr<struct FCompareRowByColumnBase> GetPropertySorter(const FProperty* Property, EColumnSortMode::Type SortMode) override;

	virtual void Tick() override {}

	DECLARE_DERIVED_EVENT( FPropertyTableColumn, IPropertyTableColumn::FFrozenStateChanged, FFrozenStateChanged );
	FFrozenStateChanged* OnFrozenStateChanged() override { return &FrozenStateChanged; }

	//~ End IPropertyTableColumn Interface

private:

	FString GetPropertyNameAsString( const TSharedRef< IPropertyTableRow >& Row );

private:

	/** Has this column been hidden? */
	bool bIsHidden;

	/** A map of all cells in this column */
	TMap< TSharedRef< IPropertyTableRow >, TSharedRef< class IPropertyTableCell > > Cells;

	/** The data source for this column */
	TSharedRef< IDataSource > DataSource;

	/** A reference to the owner table */
	TWeakPtr< IPropertyTable > Table;

	/** The width of the column */
	float Width;

	FFrozenStateChanged FrozenStateChanged;
};

#undef LOCTEXT_NAMESPACE
