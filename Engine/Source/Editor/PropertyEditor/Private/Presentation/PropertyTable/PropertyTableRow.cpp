// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presentation/PropertyTable/PropertyTableRow.h"

#include "HAL/PlatformCrt.h"
#include "IPropertyTable.h"
#include "IPropertyTableCell.h"
#include "IPropertyTableColumn.h"
#include "ObjectPropertyNode.h"
#include "Presentation/PropertyTable/DataSource.h"
#include "PropertyNode.h"
#include "PropertyPath.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakFieldPtr.h"
#include "UObject/WeakObjectPtr.h"

class UObject;

FPropertyTableRow::FPropertyTableRow( const TSharedRef< class IPropertyTable >& InTable, const TWeakObjectPtr< UObject >& InObject )
	: DataSource( MakeShared<UObjectDataSource>( InObject ) )
	, Table( InTable )
	, Children()
	, PartialPath( FPropertyPath::CreateEmpty() )
{
	GenerateChildren();
}

FPropertyTableRow::FPropertyTableRow( const TSharedRef< class IPropertyTable >& InTable, const TSharedRef< FPropertyPath >& InPropertyPath )
	: DataSource( MakeShared<PropertyPathDataSource>( InPropertyPath ) )
	, Table( InTable )
	, Children()
	, PartialPath( FPropertyPath::CreateEmpty() )
{
	GenerateChildren();
}

FPropertyTableRow::FPropertyTableRow( const TSharedRef< class IPropertyTable >& InTable, const TWeakObjectPtr< UObject >& InObject, const TSharedRef< FPropertyPath >& InPartialPropertyPath )
	: DataSource( MakeShared<UObjectDataSource>( InObject ) )
	, Table( InTable )
	, Children()
	, PartialPath( InPartialPropertyPath )
{
	GenerateChildren();
}

FPropertyTableRow::FPropertyTableRow( const TSharedRef< class IPropertyTable >& InTable, const TSharedRef< FPropertyPath >& InPropertyPath, const TSharedRef< FPropertyPath >& InPartialPropertyPath )
	: DataSource( MakeShared<PropertyPathDataSource>( InPropertyPath ) )
	, Table( InTable )
	, Children()
	, PartialPath( InPartialPropertyPath )
{
	GenerateChildren();
}

bool FPropertyTableRow::HasChildren() const
{
	return Children.Num() > 0;
}

void FPropertyTableRow::GetChildRows( TArray< TSharedRef< IPropertyTableRow > >& OutChildren ) const 
{
	OutChildren.Append( Children );
}

void FPropertyTableRow::GenerateChildren()
{
	if ( !DataSource->IsValid() || DataSource->AsPropertyPath().IsValid() )
	{
		return;
	}

	const TSharedRef< class IPropertyTable > TableRef = GetTable();
	TWeakObjectPtr< UObject > Object = DataSource->AsUObject();
	TSharedRef< FObjectPropertyNode > ObjectNode = TableRef->GetObjectPropertyNode( Object );
	TSharedRef< FPropertyPath > RootPath = TableRef->GetRootPath();

	TSharedPtr< FPropertyNode > PropertyNode = FPropertyNode::FindPropertyNodeByPath( RootPath, ObjectNode );

	if ( !PropertyNode.IsValid() )
	{
		return;
	}

	PropertyNode = FPropertyNode::FindPropertyNodeByPath( PartialPath, PropertyNode.ToSharedRef() );

	if ( !PropertyNode.IsValid() )
	{
		return;
	}

	FProperty* Property = PropertyNode->GetProperty();

	if ( Property != NULL && Property->IsA( FArrayProperty::StaticClass() ) )
	{
		for (int Index = 0; Index < PropertyNode->GetNumChildNodes(); Index++)
		{
			TSharedPtr< FPropertyNode > ChildNode = PropertyNode->GetChildNode( Index );

			FPropertyInfo Extension;
			Extension.Property = ChildNode->GetProperty();
			Extension.ArrayIndex = ChildNode->GetArrayIndex();

			Children.Add( MakeShared<FPropertyTableRow>( TableRef, Object, RootPath->ExtendPath( Extension ) ) );
		}
	}
}

TSharedRef< class IPropertyTable > FPropertyTableRow::GetTable() const 
{
	return Table.Pin().ToSharedRef();
}

void FPropertyTableRow::Tick() 
{
	//@todo Create similar logic to handle if the column is the object and row is the property [12/7/2012 Justin.Sargent]
	if ( !HasChildren() && !DataSource->AsPropertyPath().IsValid() )
	{
		const TSharedRef< IPropertyTable > TableRef = GetTable();
		const TWeakObjectPtr< UObject > Object = DataSource->AsUObject();

		if ( !Object.IsValid() )
		{
			TableRef->RemoveRow( SharedThis( this ) );
		}
		else
		{
			TSharedPtr< FPropertyNode > Node = TableRef->GetObjectPropertyNode( Object );
			if(!Node.IsValid())
			{
				TableRef->RemoveRow( SharedThis( this ) );
				return;
			}

			Node = FPropertyNode::FindPropertyNodeByPath( GetTable()->GetRootPath(), Node.ToSharedRef() );
			if ( !Node.IsValid() )
			{
				TableRef->RemoveRow( SharedThis( this ) );
				return;
			}

			Node = FPropertyNode::FindPropertyNodeByPath( GetPartialPath(), Node.ToSharedRef() );

			if ( !Node.IsValid() )
			{
				TableRef->RemoveRow( SharedThis( this ) );
				return;
			}

			EPropertyDataValidationResult Result = Node->EnsureDataIsValid();

			if ( Result == EPropertyDataValidationResult::ObjectInvalid )
			{
				TableRef->RemoveRow( SharedThis( this ) );
			}
			else if ( Result == EPropertyDataValidationResult::ArraySizeChanged )
			{
				TableRef->RequestRefresh();
			}
			else if ( Result != EPropertyDataValidationResult::DataValid )
			{
				Refresh();
			}
		}
	}
}

void FPropertyTableRow::Refresh()
{
	const TSharedRef< IPropertyTable > TableRef = GetTable();
	const TSharedRef< IPropertyTableRow > SelfRef = SharedThis( this );
	for( auto ColumnIter = TableRef->GetColumns().CreateConstIterator(); ColumnIter; ++ColumnIter )
	{
		const TSharedRef< IPropertyTableColumn >& CurrentColumn = *ColumnIter;

		CurrentColumn->GetCell( SelfRef )->Refresh();
	}

	Refreshed.Broadcast();
}
