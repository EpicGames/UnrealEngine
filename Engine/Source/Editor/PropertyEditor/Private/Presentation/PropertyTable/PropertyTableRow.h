// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "IPropertyTableRow.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FPropertyPath;
class UObject;

class FPropertyTableRow : public TSharedFromThis< FPropertyTableRow >, public IPropertyTableRow
{
public:
	
	FPropertyTableRow( const TSharedRef< class IPropertyTable >& InTable, const TWeakObjectPtr< UObject >& InObject  );

	FPropertyTableRow( const TSharedRef< class IPropertyTable >& InTable, const TSharedRef< FPropertyPath >& InPropertyPath );

	FPropertyTableRow( const TSharedRef< class IPropertyTable >& InTable, const TWeakObjectPtr< UObject >& InObject, const TSharedRef< FPropertyPath >& InPartialPropertyPath );

	FPropertyTableRow( const TSharedRef< class IPropertyTable >& InTable, const TSharedRef< FPropertyPath >& InPropertyPath, const TSharedRef< FPropertyPath >& InPartialPropertyPath );

	virtual ~FPropertyTableRow() = default;

	virtual TSharedRef< class IDataSource > GetDataSource() const override { return DataSource; }

	virtual bool HasChildren() const override;

	virtual void GetChildRows( TArray< TSharedRef< class IPropertyTableRow > >& OutChildren ) const override;

	virtual TSharedRef< class IPropertyTable > GetTable() const override;

	virtual bool HasCells() const override { return true; }

	virtual TSharedRef< class FPropertyPath > GetPartialPath() const override { return PartialPath; }

	virtual void Tick() override;

	virtual void Refresh() override;

	DECLARE_DERIVED_EVENT( FPropertyTableRow, IPropertyTableRow::FRefreshed, FRefreshed );
	virtual FRefreshed* OnRefresh() override { return &Refreshed; }


private:

	void GenerateChildren();


private:

	TSharedRef< class IDataSource > DataSource;

	TWeakPtr< class IPropertyTable > Table;

	TArray< TSharedRef< class IPropertyTableRow > > Children;

	TSharedRef< class FPropertyPath > PartialPath;

	FRefreshed Refreshed;
};
