// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PropertyPath.h"
#include "IPropertyTableColumn.h"
#include "IPropertyTable.h"
#include "IPropertyTableCell.h"
#include "IPropertyTableRow.h"
#include "Presentation/PropertyTable/PropertyTableColumn.h"
#include "Presentation/PropertyTable/PropertyTableCell.h"
#include "Presentation/PropertyTable/DataSource.h"

class FPropertyTableRowHeaderColumn : public TSharedFromThis< FPropertyTableRowHeaderColumn >, public IPropertyTableColumn
{
public:

	FPropertyTableRowHeaderColumn( const TSharedRef< IPropertyTable >& InTable )
		: Table( InTable )
		, Cells()
		, bIsHidden( false )
		, DataSource( MakeShareable( new NoDataSource() ) )
	{

	}

	virtual ~FPropertyTableRowHeaderColumn() = default;

	virtual FName GetId() const override { return FName( TEXT("RowHeader") ); }
	virtual FText GetDisplayName() const override { return FText::GetEmpty(); }
	virtual TSharedRef< IDataSource > GetDataSource() const override { return MakeShareable( new PropertyPathDataSource( FPropertyPath::CreateEmpty() ) ); }
	virtual TSharedRef< class FPropertyPath > GetPartialPath() const override { return FPropertyPath::CreateEmpty(); }

	virtual TSharedRef< class IPropertyTableCell > GetCell( const TSharedRef< class IPropertyTableRow >& Row ) override
	{
		//@todo Clean Cells cache when rows get updated [11/27/2012 Justin.Sargent]
		TSharedRef< IPropertyTableCell >* CellPtr = Cells.Find( Row );

		if( CellPtr != NULL )
		{
			return *CellPtr;
		}

		TSharedRef< IPropertyTableCell > Cell = MakeShareable( new FPropertyTableCell( SharedThis( this ), Row ) );
		Cells.Add( Row, Cell );

		return Cell;
	}

	virtual void RemoveCellsForRow( const TSharedRef< class IPropertyTableRow >& Row ) override
	{
		Cells.Remove( Row );
	}

	virtual TSharedRef< class IPropertyTable > GetTable() const override { return Table.Pin().ToSharedRef(); }

	virtual bool CanSelectCells() const override { return false; }

	virtual EPropertyTableColumnSizeMode::Type GetSizeMode() const override { return EPropertyTableColumnSizeMode::Fixed; }

	virtual void SetSizeMode(EPropertyTableColumnSizeMode::Type InSizeMode) override {}

	virtual float GetWidth() const override { return 24.0f; } 

	virtual void SetWidth( float InWidth ) override {  }

	virtual bool IsHidden() const override { return bIsHidden; }

	virtual void SetHidden( bool InIsHidden ) override { bIsHidden = InIsHidden; }

	virtual bool IsFrozen() const override { return true; }

	virtual void SetFrozen( bool InIsFrozen ) override {}

	virtual bool CanSortBy() const override { return false; }

	virtual void Sort( TArray< TSharedRef< class IPropertyTableRow > >& Rows, const EColumnSortMode::Type PrimarySortMode, const TSharedPtr<IPropertyTableColumn>& SecondarySortColumn, const EColumnSortMode::Type SecondarySortMode ) override {  }

	virtual TSharedPtr<struct FCompareRowByColumnBase> GetPropertySorter(const FProperty* Property, EColumnSortMode::Type SortMode) override { return nullptr;  }

	virtual void Tick() override {}

	DECLARE_DERIVED_EVENT( FPropertyTableColumn, IPropertyTableColumn::FFrozenStateChanged, FFrozenStateChanged );
	FFrozenStateChanged* OnFrozenStateChanged() override { return &FrozenStateChanged; }

private:

	TWeakPtr< IPropertyTable > Table;
	TMap< TSharedRef< IPropertyTableRow >, TSharedRef< class IPropertyTableCell > > Cells;
	bool bIsHidden;
	TSharedRef< IDataSource > DataSource;
	
	FFrozenStateChanged FrozenStateChanged;
};
