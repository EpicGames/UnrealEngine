// Copyright Epic Games, Inc. All Rights Reserved.
#include "PropertyChangeListener.h"
#include "UObject/UnrealType.h"
#include "PropertyNode.h"
#include "ObjectPropertyNode.h"
#include "PropertyEditorHelpers.h"

void FPropertyChangeListener::SetObject( UObject& Object, const FPropertyListenerSettings& InPropertyListenerSettings  )
{
	PropertyListenerSettings = InPropertyListenerSettings;

	bool bInitNode = false;
	if( !RootPropertyNode.IsValid() )
	{
		RootPropertyNode = MakeShareable( new FObjectPropertyNode );

	}

	RootPropertyNode->AddObject( &Object );

	FPropertyNodeInitParams InitParams;
	InitParams.ParentNode = NULL;
	InitParams.Property = NULL;
	InitParams.ArrayOffset = 0;
	InitParams.ArrayIndex = INDEX_NONE;
	InitParams.bAllowChildren = true;
	InitParams.bForceHiddenPropertyVisibility = false;
	InitParams.bCreateCategoryNodes = false;

	RootPropertyNode->InitNode( InitParams );

	TSharedRef<FPropertyNode> Start = RootPropertyNode.ToSharedRef();
	CreatePropertyCaches( Start, &Object );
}

/**
 * Caches a single property value
 */
class FValueCache
{
public:

	FValueCache( TSharedRef<FPropertyNode> InPropertyNode, UObject* InOwnerObject )
		: PropertyNode( InPropertyNode )
	{
		OwnerObject = InOwnerObject;

		FProperty* Property = InPropertyNode->GetProperty();

		check(Property);
		check(OwnerObject);

		FPropertyNode* ParentNode = InPropertyNode->GetParentNode();
	}

	/**
	 * @return The property node we are inspecting
	 */
	TSharedPtr<FPropertyNode> GetPropertyNode() const { return PropertyNode.Pin(); }

	/**
	 * Caches the property value
	 */
	void CacheValue()
	{
		Data.Reset();

		TSharedPtr<FPropertyNode> PropertyNodePtr = PropertyNode.Pin();
		if(FProperty* Property = PropertyNodePtr ? PropertyNodePtr->GetProperty() : nullptr)
		{
			FPropertyValueAddresses PropertyValueAddresses = GetPropertyValueAddresses();

			// Not supported yet
			check( !Property->IsA( FArrayProperty::StaticClass() ) );
			check( !Property->IsA( FSetProperty::StaticClass() ) );
			check( !Property->IsA( FMapProperty::StaticClass() ) );

			if(PropertyNodePtr->GetArrayIndex() == INDEX_NONE && Property->ArrayDim > 1 )
			{
				// Check static arrays
				Data.AddZeroed( Property->ArrayDim * Property->GetElementSize() );
				Property->CopyCompleteValue( Data.GetData(), PropertyValueAddresses.Address);
			}
			else
			{
				// Regular properties
				Data.AddZeroed( Property->GetElementSize() );
				Property->CopySingleValue( Data.GetData(), PropertyValueAddresses.Address );
			}
		}
	}

	/**
	 * Scans for changes to the value
	 *
	 * @param bRecacheNewValues	If true, recaches new values found
	 * @return true if any changes were found
	 */
	bool ScanForChanges( bool bRecacheNewValues )
	{
		TSharedPtr<FPropertyNode> PropertyNodePtr = PropertyNode.Pin();
		if (FProperty* Property = PropertyNodePtr ? PropertyNodePtr->GetProperty() : nullptr)
		{
			FPropertyValueAddresses PropertyValueAddresses = GetPropertyValueAddresses();

			bool bPropertyValid = true;
			bool bChanged = false;

			if (PropertyValueAddresses.BaseAddress != nullptr && PropertyValueAddresses.Address != nullptr)
			{
				FArrayProperty* OuterArrayProperty = Property->GetOwner<FArrayProperty>();
				FSetProperty* OuterSetProperty = Property->GetOwner<FSetProperty>();
				FMapProperty* OuterMapProperty = Property->GetOwner<FMapProperty>();

				if (OuterArrayProperty != NULL)
				{
					// make sure we're not trying to compare against an element that doesn't exist
					FScriptArrayHelper ArrayHelper(OuterArrayProperty, PropertyValueAddresses.BaseAddress);
					if (PropertyNodePtr->GetArrayIndex() >= ArrayHelper.Num())
					{
						bPropertyValid = false;
					}
				}
				else if (OuterSetProperty != NULL)
				{
					FScriptSetHelper SetHelper(OuterSetProperty, PropertyValueAddresses.BaseAddress);

					bool bIsValidIndex = PropertyNodePtr->GetArrayIndex() >= 0 && PropertyNodePtr->GetArrayIndex() < SetHelper.Num();
					if (!bIsValidIndex)
					{
						bPropertyValid = false;
					}
				}
				else if (OuterMapProperty != NULL)
				{
					FScriptMapHelper MapHelper(OuterMapProperty, PropertyValueAddresses.BaseAddress);

					bool bIsValidIndex = PropertyNodePtr->GetArrayIndex() >= 0 && PropertyNodePtr->GetArrayIndex() < MapHelper.Num();
					if (!bIsValidIndex)
					{
						bPropertyValid = false;
					}
				}
			}
			else
			{
				bPropertyValid = false;
			}

			if (bPropertyValid)
			{
				bChanged = !Property->Identical(PropertyValueAddresses.Address, Data.GetData());
				if (bRecacheNewValues)
				{
					CacheValue();
				}
			}

			return bChanged;
		}

		return false;
	}

private:
	struct FPropertyValueAddresses
	{
		/**
		* The address of the property's value.
		*/
		uint8* Address = nullptr;
		/**
		* The base address of this property's value.  i.e. for dynamic arrays, the location of the FScriptArray which
		* contains the array property's value
		*/
		uint8* BaseAddress = nullptr;
	};

	/**
	 * Gets the addresses for the property value.  These addresses must be retrieved before access every time because
	 * array item addresses will change if the array is resized and it's storage reallocated.
	 */
	FPropertyValueAddresses GetPropertyValueAddresses()
	{
		TSharedPtr<FPropertyNode> PropertyNodePtr = PropertyNode.Pin();
		if (FProperty* Property = PropertyNodePtr ? PropertyNodePtr->GetProperty() : nullptr)
		{
			FPropertyNode* ParentNode = PropertyNodePtr->GetParentNode();
			FArrayProperty* OuterArrayProp = Property->GetOwner<FArrayProperty>();
			FSetProperty* OuterSetProp = Property->GetOwner<FSetProperty>();
			FMapProperty* OuterMapProp = Property->GetOwner<FMapProperty>();

			FPropertyValueAddresses ValueAddresses;

			ValueAddresses.BaseAddress = (OuterArrayProp == NULL && OuterSetProp == NULL && OuterMapProp == NULL)
				? PropertyNodePtr->GetValueBaseAddressFromObject(OwnerObject)
				: (ParentNode ? ParentNode->GetValueBaseAddressFromObject(OwnerObject) : nullptr);

			ValueAddresses.Address = PropertyNodePtr->GetValueAddressFromObject(OwnerObject);

			return ValueAddresses;
		}

		return FPropertyValueAddresses();
	}

private:
	/** Cached value data */
	TArray<uint8> Data;

	/** The property node we are inspecting */
	TWeakPtr<FPropertyNode> PropertyNode;

	/** The top level object owning the value (e.g if the value is from a property on a component, the top level object is most likely the actor owner of the component) */
	TWeakObjectPtr<UObject> CachedTopLevelObject;

	/** The address of the owning object */
	UObject*	OwnerObject = nullptr;
};


bool FPropertyChangeListener::ScanForChanges( bool bRecacheNewValues )
{
	bool bChangesFound = false;

	// Check each value to see if it has been changed
	for( int32 ValueIndex = 0; ValueIndex < CachedValues.Num(); ++ValueIndex )
	{
		FValueCache& ValueCache = *CachedValues[ValueIndex];
		
		bool bChanged = ValueCache.ScanForChanges( bRecacheNewValues );
		if( bChanged )
		{
			// The value has changed.  Call a delegate to notify users
			UE_LOGF( LogPropertyNode, Verbose, "Property changed: %ls", *ValueCache.GetPropertyNode()->GetProperty()->GetName() );
			TSharedPtr<FPropertyNode> PropertyNode = ValueCache.GetPropertyNode();

			// Find the root most object parent as that contains the main object(s) being changed
			FObjectPropertyNode* ObjectNode = PropertyNode->FindRootObjectItemParent();

			// Get each object that changed
			TArray<UObject*> ObjectsThatChanged;
			if (ObjectNode)
			{
				ObjectNode->PurgeKilledObjects();

				for (auto It = ObjectNode->ObjectConstIterator(); It; ++It)
				{
					UObject* Obj = It->Get();
					if (ensure(Obj != nullptr))
					{
						ObjectsThatChanged.Add(Obj);
					}
				}
			}

			if( ObjectsThatChanged.Num() > 0 )
			{
				bChangesFound = true;
				TSharedPtr<IPropertyHandle> Handle = PropertyEditorHelpers::GetPropertyHandle( PropertyNode.ToSharedRef(), NULL, NULL );
				OnPropertyChangedDelegate.Broadcast( ObjectsThatChanged, *Handle );
			}
		}
		
	}

	return bChangesFound;
}

void FPropertyChangeListener::TriggerAllPropertiesChangedDelegate()
{
	// Check each value to see if it has been changed
	for( int32 ValueIndex = 0; ValueIndex < CachedValues.Num(); ++ValueIndex )
	{
		FValueCache& ValueCache = *CachedValues[ValueIndex];
		
		TSharedPtr<FPropertyNode> PropertyNode = ValueCache.GetPropertyNode();

		// Find the root most object parent as that contains the main object(s) being changed
		FObjectPropertyNode* ObjectNode = PropertyNode->FindRootObjectItemParent();

		// Get each object that changed
		TArray<UObject*> ObjectsThatChanged;
		if (ObjectNode)
		{
			ObjectNode->PurgeKilledObjects();

			for (auto It = ObjectNode->ObjectConstIterator(); It; ++It)
			{
				UObject* Obj = It->Get();
				if (ensure(Obj != nullptr))
				{
					ObjectsThatChanged.Add(Obj);
				}
			}
		}

		if( ObjectsThatChanged.Num() > 0 )
		{
			TSharedPtr<IPropertyHandle> Handle = PropertyEditorHelpers::GetPropertyHandle( PropertyNode.ToSharedRef(), NULL, NULL );
			OnPropertyChangedDelegate.Broadcast( ObjectsThatChanged, *Handle );
		}
	}
}

void FPropertyChangeListener::CreatePropertyCaches( TSharedRef<FPropertyNode>& PropertyNode, UObject* ParentObject )
{
	FObjectPropertyNode* ObjectNode = PropertyNode->AsObjectNode();
	FProperty* Property = PropertyNode->GetProperty();

	bool bIsBuiltInStructProp = PropertyEditorHelpers::IsBuiltInStructProperty( Property );

	if( PropertyNode->AsItemPropertyNode() && Property )
	{
		// Check whether or not we should ignore object properties
		bool bValidProperty = ( !PropertyListenerSettings.bIgnoreObjectProperties || !Property->IsA( FObjectPropertyBase::StaticClass() ) );
		// Check whether or not we should ignore array properties
		bValidProperty &= ( !PropertyListenerSettings.bIgnoreArrayProperties || ! ( Property->IsA(FArrayProperty::StaticClass()) || Property->IsA(FSetProperty::StaticClass()) || Property->IsA(FMapProperty::StaticClass()) ) );
		// Check whether or not the required property flags are set
		bValidProperty &= ( PropertyListenerSettings.RequiredPropertyFlags == 0 || Property->HasAllPropertyFlags( PropertyListenerSettings.RequiredPropertyFlags ) );
		// Check to make sure the disallowed property flags are not set
		bValidProperty &= (PropertyListenerSettings.DisallowedPropertyFlags == 0 || !Property->HasAnyPropertyFlags( PropertyListenerSettings.DisallowedPropertyFlags ) );
		// Only examine struct properties if they are built in (they are treated as whole units).  Otherwise just examine the children
		bValidProperty &= ( bIsBuiltInStructProp || !Property->IsA( FStructProperty::StaticClass() ) );

		if( bValidProperty )
		{
			TSharedPtr<FValueCache> ValueCache( new FValueCache( PropertyNode, ParentObject ) );
			ValueCache->CacheValue();
			CachedValues.Add( ValueCache );
		}
	}

	// Built in struct types (Vector, rotator, etc) are stored as a single value 
	if( !bIsBuiltInStructProp )
	{
		// Only one object supported
		UObject* NewParent = ObjectNode ? ObjectNode->GetUObject(0) : ParentObject;

		// Cache each child value
		for( int32 ChildIndex = 0; ChildIndex < PropertyNode->GetNumChildNodes(); ++ChildIndex )
		{
			TSharedRef<FPropertyNode> ChildNode = PropertyNode->GetChildNode( ChildIndex ).ToSharedRef();
			CreatePropertyCaches( ChildNode, NewParent );
		}
	}
}
