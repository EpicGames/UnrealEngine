// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyChangeListener.h"

class FObjectPropertyNode;
class FPropertyNode;
class FValueCache;

/**
 * Implementation of an IPropertyChangeListener
 */
class FPropertyChangeListener : public IPropertyChangeListener
{
public:
	virtual ~FPropertyChangeListener() = default;
	/** IPropertyChangeListener interface */
	virtual void SetObject( UObject& Object, const FPropertyListenerSettings& InPropertyListenerSettings ) override;
	virtual bool ScanForChanges( bool bRecacheNewValues = true ) override;
	virtual void TriggerAllPropertiesChangedDelegate() override;
	virtual FOnPropertyChanged& GetOnPropertyChangedDelegate() override { return OnPropertyChangedDelegate; }
private:
	/**
	 * Recursively creates a property value cache for each property node in the tree starting with the passed in property node
	 * 
	 * @param PropertyNode	The starting propertynode
	 * @param ParentObject	The current parent object to the property in the property node
	 */
	void CreatePropertyCaches( TSharedRef<FPropertyNode>& PropertyNode, UObject* ParentObject );
private:
	/** Settings for how to listen to properties */
	FPropertyListenerSettings PropertyListenerSettings;
	/** The root of the property tree */
	TSharedPtr<FObjectPropertyNode> RootPropertyNode;
	/** List of all cached values */
	TArray< TSharedPtr<class FValueCache> > CachedValues;
	/** Delegate to call when a property has changed */
	FOnPropertyChanged OnPropertyChangedDelegate;

};
