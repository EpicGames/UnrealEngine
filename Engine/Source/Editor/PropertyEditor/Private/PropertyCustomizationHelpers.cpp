// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyCustomizationHelpers.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Engine/Texture.h"
#include "Factories/Factory.h"
#include "Editor.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "DetailLayoutBuilder.h"
#include "UserInterface/PropertyEditor/SPropertyAssetPicker.h"
#include "UserInterface/PropertyEditor/SPropertyMenuAssetPicker.h"
#include "UserInterface/PropertyEditor/SPropertyMenuComponentPicker.h"
#include "UserInterface/PropertyEditor/SPropertyMenuActorPicker.h"
#include "UserInterface/PropertyEditor/SPropertySceneOutliner.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "UserInterface/PropertyEditor/SPropertyEditorAsset.h"
#include "UserInterface/PropertyEditor/SPropertyEditorCombo.h"
#include "UserInterface/PropertyEditor/SPropertyEditorClass.h"
#include "UserInterface/PropertyEditor/SPropertyEditorStruct.h"
#include "UserInterface/PropertyEditor/SPropertyEditorInteractiveActorPicker.h"
#include "UserInterface/PropertyEditor/SPropertyEditorSceneDepthPicker.h"
#include "UserInterface/PropertyEditor/SPropertyEditorEditInline.h"
#include "Widgets/Input/SHyperlink.h"
#include "IDocumentation.h"
#include "EditorFontGlyphs.h"
#include "DetailCategoryBuilder.h"
#include "IDetailGroup.h"
#include "AssetToolsModule.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "ObjectTools.h"
#include "PropertyPermissionList.h"
#include "PropertyNode.h"
#include "Reflection/FunctionUtils.h"
#include "ScopedTransaction.h"
#include "Settings/BlueprintEditorProjectSettings.h"
#include "Settings/EditorStyleSettings.h"
#include "Widgets/Layout/SWrapBox.h"

#define LOCTEXT_NAMESPACE "PropertyCustomizationHelpers"

namespace PropertyCustomizationHelpers
{
	class SPropertyEditorButton : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS( SPropertyEditorButton ) 
			: _Text( )
			, _Image( FAppStyle::GetBrush("Default") )
			, _IsFocusable( true )
		{}
			SLATE_ATTRIBUTE( FText, Text )
			SLATE_ATTRIBUTE( const FSlateBrush*, Image )
			SLATE_EVENT( FSimpleDelegate, OnClickAction )

			/** Sometimes a button should only be mouse-clickable and never keyboard focusable. */
			SLATE_ARGUMENT( bool, IsFocusable )
		SLATE_END_ARGS()

		void Construct( const FArguments& InArgs )
		{
			OnClickAction = InArgs._OnClickAction;

			ChildSlot
			[
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.WidthOverride(22.0f)
				.HeightOverride(22.0f)
				.ToolTipText(InArgs._Text)
				[
					SNew(SButton)
					.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
					.OnClicked( this, &SPropertyEditorButton::OnClick )
					.ContentPadding(0.0f)
					.IsFocusable(InArgs._IsFocusable)
					[ 
						SNew( SImage )
						.Image( InArgs._Image )
						.ColorAndOpacity( FSlateColor::UseForeground() )
					]
				]
			]; 
		}


	private:
		FReply OnClick()
		{
			OnClickAction.ExecuteIfBound();
			return FReply::Handled();
		}
	private:
		FSimpleDelegate OnClickAction;
	};

	TSharedRef<SWidget> MakeCustomButton(const FSlateBrush* IconBrush, FSimpleDelegate OnClicked,
		TAttribute<FText> ToolTipText, TAttribute<bool> IsEnabled, TAttribute<EVisibility> IsVisible)
	{
		return
			SNew(SPropertyEditorButton)
			.Text(ToolTipText)
			.Image(IconBrush)
			.OnClickAction(OnClicked)
			.IsEnabled(IsEnabled)
			.Visibility(IsVisible)
			.IsFocusable(false);
	}

	TSharedRef<SWidget> MakeResetButton(FSimpleDelegate OnResetClicked, TAttribute<FText> OptionalToolTipText /*= FText()*/, TAttribute<bool> IsEnabled /*= true*/)
	{
		return
			SNew(SPropertyEditorButton)
			.Text(OptionalToolTipText.Get().IsEmpty() ? LOCTEXT("ResetButtonToolTipText", "Reset Element to Default Value") : OptionalToolTipText)
			.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
			.OnClickAction(OnResetClicked)
			.IsEnabled(IsEnabled)
			.Visibility(IsEnabled.Get() ? EVisibility::Visible : EVisibility::Collapsed)
			.IsFocusable(false);
	}


	TSharedRef<SWidget> MakeAddButton( FSimpleDelegate OnAddClicked, TAttribute<FText> OptionalToolTipText, TAttribute<bool> IsEnabled )
	{
		return	
			SNew( SPropertyEditorButton )
			.Text( OptionalToolTipText.Get().IsEmpty() ? LOCTEXT( "AddButtonToolTipText", "Add Element") : OptionalToolTipText )
			.Image( FAppStyle::GetBrush("Icons.PlusCircle") )
			.OnClickAction( OnAddClicked )
			.IsEnabled(IsEnabled)
			.IsFocusable( false );
	}

	TSharedRef<SWidget> MakeRemoveButton( FSimpleDelegate OnRemoveClicked, TAttribute<FText> OptionalToolTipText, TAttribute<bool> IsEnabled )
	{
		return	
			SNew( SPropertyEditorButton )
			.Text( OptionalToolTipText.Get().IsEmpty() ? LOCTEXT( "RemoveButtonToolTipText", "Remove Element") : OptionalToolTipText )
			.Image( FAppStyle::GetBrush("Icons.Minus") )
			.OnClickAction( OnRemoveClicked )
			.IsEnabled(IsEnabled)
			.IsFocusable( false );
	}

	TSharedRef<SWidget> MakeEditButton( FSimpleDelegate OnEditClicked, TAttribute<FText> OptionalToolTipText, TAttribute<bool> IsEnabled )
	{
		return
			SNew( SPropertyEditorButton )
			.Text( OptionalToolTipText.Get().IsEmpty() ? LOCTEXT( "EditButtonToolTipText", "Edit Element") : OptionalToolTipText )
			.Image( FAppStyle::GetBrush("Icons.Edit") )
			.OnClickAction( OnEditClicked )
			.IsEnabled( IsEnabled )
			.IsFocusable( false );
	}

	TSharedRef<SWidget> MakeEmptyButton( FSimpleDelegate OnEmptyClicked, TAttribute<FText> OptionalToolTipText, TAttribute<bool> IsEnabled )
	{
		return
			SNew( SPropertyEditorButton )
			.Text( OptionalToolTipText.Get().IsEmpty() ? LOCTEXT( "EmptyButtonToolTipText", "Remove All Elements") : OptionalToolTipText )
			.Image( FAppStyle::GetBrush("Icons.Delete") )
			.OnClickAction( OnEmptyClicked )
			.IsEnabled(IsEnabled)
			.IsFocusable( false );
	}

	TSharedRef<SWidget> MakeUseSelectedButton( FSimpleDelegate OnUseSelectedClicked, TAttribute<FText> OptionalToolTipText, TAttribute<bool> IsEnabled, const bool IsActor )
	{
		return
			SNew( SPropertyEditorButton )
			.Text(
				!OptionalToolTipText.Get().IsEmpty() ? OptionalToolTipText
				: IsActor ? LOCTEXT( "UseActorButtonToolTipText", "Use Selected Actor from the Level Editor")
				: LOCTEXT( "UseButtonToolTipText", "Use Selected Asset from Content Browser")
			)
			.Image( FAppStyle::GetBrush("Icons.Use") )
			.OnClickAction( OnUseSelectedClicked )
			.IsEnabled(IsEnabled)
			.IsFocusable( false );
	}

	TSharedRef<SWidget> MakeDeleteButton( FSimpleDelegate OnDeleteClicked, TAttribute<FText> OptionalToolTipText, TAttribute<bool> IsEnabled )
	{
		return
			SNew( SPropertyEditorButton )
			.Text( OptionalToolTipText.Get().IsEmpty() ? LOCTEXT( "DeleteButtonToolTipText", "Delete") : OptionalToolTipText )
			.Image( FAppStyle::GetBrush("Icons.Delete") )
			.OnClickAction( OnDeleteClicked )
			.IsEnabled(IsEnabled)
			.IsFocusable( false );
	}

	TSharedRef<SWidget> MakeClearButton( FSimpleDelegate OnClearClicked, TAttribute<FText> OptionalToolTipText, TAttribute<bool> IsEnabled )
	{
		return
			SNew( SPropertyEditorButton )
			.Text( OptionalToolTipText.Get().IsEmpty() ? LOCTEXT( "ClearButtonToolTipText", "Clear") : OptionalToolTipText )
			.Image(FAppStyle::Get().GetBrush("Icons.X"))
			.OnClickAction( OnClearClicked )
			.IsEnabled(IsEnabled)
			.IsFocusable( false );
	}

	TSharedRef<SWidget> MakeSetOptionalButton(FOnSetOptional OnSetOptional, TAttribute<FText> OptionalToolTipText, TAttribute<bool> IsEnabled)
	{
		// Custom widget for this button as it has no image and should fill a larger space
		return SNew(SBox)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.IsEnabled(IsEnabled)
			.ToolTipText(OptionalToolTipText.Get().IsEmpty() ? LOCTEXT("SetOptionalButtonToolTipText", "Set Optional to default value.") : OptionalToolTipText)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "Button")
				.OnClicked_Lambda([OnSetOptional](){
					OnSetOptional.ExecuteIfBound(nullptr, nullptr);
					return FReply::Handled();
				})
				.Text(LOCTEXT("SetButtonText", "Set to Value"))
				.ContentPadding(0.0f)
				.IsFocusable(false)
			];
	}

	TSharedRef<SWidget> MakePickOptionalButton(FOnSetOptional OnSetOptional, FSimpleDelegate OnClearOptional, const TSharedRef<FPropertyNode>& PropertyNode, TAttribute<FText> OptionalToolTipText, TAttribute<bool> IsEnabled)
	{
		const FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(PropertyNode->GetProperty());
		const FProperty* ValueProperty = OptionalProperty->GetValueProperty();

		const UClass* MetaClass = UObject::StaticClass();
		const UClass* RequiredInterface = nullptr;
		const UClass* SelectedClass = nullptr;

		if(const FInterfaceProperty* InterfaceProperty = CastField<FInterfaceProperty>(ValueProperty))
		{
			RequiredInterface = InterfaceProperty->InterfaceClass;
		}
		else if (ValueProperty->IsA(FClassProperty::StaticClass()) || ValueProperty->IsA(FSoftClassProperty::StaticClass()))
		{
			MetaClass = ValueProperty->IsA(FClassProperty::StaticClass())
				? CastFieldChecked<FClassProperty>(ValueProperty)->MetaClass
				: CastFieldChecked<FSoftClassProperty>(ValueProperty)->MetaClass;
			RequiredInterface = ValueProperty->GetOwnerProperty()->GetClassMetaData(TEXT("MustImplement"));
		}
		else if (ValueProperty->IsA(FObjectProperty::StaticClass()) || ValueProperty->IsA(FSoftObjectProperty::StaticClass()))
		{
			MetaClass = ValueProperty->IsA(FObjectProperty::StaticClass())
				? CastFieldChecked<FObjectProperty>(ValueProperty)->PropertyClass
				: CastFieldChecked<FSoftObjectProperty>(ValueProperty)->PropertyClass;

			// Verse generated interfaces are not fully supported yet, for now they are stored in FObjectProperty metadata
			RequiredInterface = ValueProperty->GetOwnerProperty()->GetClassMetaData(TEXT("ObjectMustImplement"));
		}

		TWeakPtr<FPropertyNode> WeakPropertyNode = PropertyNode;

		uint8* ValueAddress = nullptr;
		const bool bMultipleValues = PropertyNode->GetSingleReadAddress(ValueAddress) == FPropertyAccess::MultipleValues;

		return SNew(SPropertyEditorClass)
			.MetaClass(MetaClass)
			.RequiredInterface(RequiredInterface)
			.AllowAbstract(false)
			.AllowNone(bMultipleValues) // selecting `None` is how we clear multiple-values
			.ShowTree(false)
			.SelectedClass(SelectedClass)
			.IsEnabled(IsEnabled)
			.ToolTipText(OptionalToolTipText.Get().IsEmpty() ? LOCTEXT("PickOptionalButtonToolTipText", "Pick a value to set this Optional to.") : OptionalToolTipText)
			.OnSetClass_Lambda([OnSetOptional, OnClearOptional, WeakPropertyNode] (const UClass* SelectedClass){
					if (WeakPropertyNode.IsValid())
					{
						if (SelectedClass)
						{
							OnSetOptional.ExecuteIfBound(nullptr, SelectedClass);
						}
						else
						{
							OnClearOptional.ExecuteIfBound();
						}
					}
				})
			.InvalidObjectDisplayText("Set to Value")
			.bIsMultiOptionalSetter(true);
	}

	TSharedRef<SWidget> MakeClearOptionalButton(FSimpleDelegate OnClearOptional, TAttribute<FText> OptionalToolTipText, TAttribute<bool> IsEnabled)
	{
		return
			SNew(SPropertyEditorButton)
			.Text(OptionalToolTipText.Get().IsEmpty() ? LOCTEXT("ClearOptionalButtonToolTipText", "Clear Optional") : OptionalToolTipText)
			.Image(FAppStyle::Get().GetBrush("Icons.X"))
			.OnClickAction(OnClearOptional)
			.IsEnabled(IsEnabled)
			.IsFocusable(false);
	}

	TSharedRef<SWidget> MakeFunctionCallButton(const FPropertyFunctionCallArgs& InArgs)
	{
		TSharedRef<SWidget> Widget = SNullWidget::NullWidget;
		if (const UFunction* Function = InArgs.Function.Get())
		{
			const FText Label = InArgs.LabelOverride.Get(Function->GetDisplayNameText());
			FText ToolTipText = InArgs.ToolTipTextOverride.Get(Function->GetToolTipText());
			if (ToolTipText.IsEmpty())
			{
				ToolTipText = Label;
			}

			TAttribute<bool> IsEnabled;
			if (InArgs.OnCanExecute.IsBound())
			{
				IsEnabled.Bind(
					TAttribute<bool>::FGetter::CreateLambda(
					[WeakFunctionPtr = InArgs.Function, CanExecute = InArgs.OnCanExecute]()
					{
						return CanExecute.Execute(WeakFunctionPtr);
					})
				);
			}
			else
			{
				IsEnabled = true;
			}

			Widget = SNew(SButton)
				.Text(Label)
				.OnClicked_Lambda([WeakFunctionPtr = InArgs.Function, OnExecute = InArgs.OnExecute]()
				{
					return OnExecute.Execute(WeakFunctionPtr);
				})
				.IsEnabled(IsEnabled)
				.ToolTipText(ToolTipText.IsEmptyOrWhitespace() ? LOCTEXT("CallInEditorTooltip", "Call an event on the selected object(s)") : ToolTipText);

			if (InArgs.SearchText != nullptr)
			{
				InArgs.SearchText->AppendLine(Label);
				InArgs.SearchText->AppendLine(ToolTipText);

				if (Label.ToString() != Function->GetName())
				{
					InArgs.SearchText->AppendLine(FText::FromString(Function->GetName()));
				}
			}
		}

		return Widget;
	}

	struct FCategorizedFunctionCallEntry
	{
		FName CategoryName;
		FName RowTag;
		TSharedPtr<SWrapBox> WrapBox;
		FTextBuilder FunctionSearchText;

		FCategorizedFunctionCallEntry(FName InCategoryName)
			: CategoryName(InCategoryName)
		{
			WrapBox = SNew(SWrapBox)
				// Setting the preferred size here (despite using UseAllottedSize) is a workaround for an issue
				// when contained in a scroll box: prior to the first tick, the wrap box will use preferred size
				// instead of allotted, and if preferred size is set small, it will cause the box to wrap a lot and
				// request too much space from the scroll box. On next tick, SWrapBox is updated but the scroll box
				// does not realize that it needs to show more elements, until it is scrolled.
				// Setting a large value here means that the SWrapBox will request too little space prior to tick,
				// which will cause the scroll box to virtualize more elements at the start, but this is less broken.
				.PreferredSize(2000)
				.UseAllottedSize(true);
		}
	};

	void GetFunctionCallWidgets(const TArrayView<UFunction*>& InCallInEditorFunctions, const FPropertyFunctionCallDelegates& InArgs, TArray<FCategorizedFunctionCallEntry, TInlineAllocator<8>>& OutCategorizedEntries)
	{
		if (InCallInEditorFunctions.IsEmpty())
		{
			return;
		}

		// Build up a set of functions for each category, accumulating search text and buttons in a wrap box
		FName ActiveCategory;

		const bool bUseDisplayNames = GEditor && GetDefault<UEditorStyleSettings>()->bShowFriendlyNames;

		// FBlueprintMetadata::MD_FunctionCategory
		static const FName NAME_FunctionCategory(TEXT("Category"));

		FPropertyFunctionCallDelegates::FOnExecute OnExecute = InArgs.OnExecute;
		FPropertyFunctionCallDelegates::FOnCanExecute OnCanExecute = InArgs.OnCanExecute;

		if (!OnExecute.IsBound() && InArgs.OnGetExecutionContext.IsBound())
		{
			// FBlueprintMetadata::MD_WorldContext
			static const FName NAME_WorldContext(TEXT("WorldContext"));

			OnExecute = FPropertyFunctionCallDelegates::FOnExecute::CreateLambda(
				[OnGetExecutionContext = InArgs.OnGetExecutionContext](const TWeakObjectPtr<UFunction>& InWeakFunction)
				{
					using namespace UE::Reflection;
					if (UFunction* Function = InWeakFunction.Get())
					{
						FText TransactionName = ObjectTools::GetUserFacingFunctionName(Function);
						if (TransactionName.IsEmptyOrWhitespace())
						{
							TransactionName = LOCTEXT("ExecuteCallInEditorMethod", "Call In Editor Action");
						}

						FScopedTransaction Transaction(TransactionName);
						TStrongObjectPtr<UFunction> CallingFunction(Function);

						if (Function->HasMetaData(NAME_WorldContext)
							&& DoesStaticFunctionSignatureMatch<void(TObjectPtr<UObject>)>(Function))
						{
							FEditorScriptExecutionGuard ScriptGuard;
							UEditorEngine* EditorEngine = Cast<UEditorEngine>(GEngine);
							UObject* WorldContextObject = EditorEngine->GetEditorWorldContext().World();
							TStrongObjectPtr<UObject> CDO(Function->GetOwnerClass()->GetDefaultObject(false));
							CDO->ProcessEvent(Function, &WorldContextObject);
						}
						else
						{
							TArray<TWeakObjectPtr<UObject>> WeakExecutionObjects = OnGetExecutionContext.Execute(InWeakFunction);
							if (!WeakExecutionObjects.IsEmpty())
							{
								FEditorScriptExecutionGuard ScriptGuard;
								for (const TWeakObjectPtr<UObject>& WeakExecutionObject : WeakExecutionObjects)
								{
									if (UObject* ExecutionObject = WeakExecutionObject.Get())
									{
										ensure(Function->ParmsSize == 0);
										TStrongObjectPtr<UObject> StrongExecutionObject(ExecutionObject); // Prevent GC during call
										ExecutionObject->ProcessEvent(Function, nullptr);
									}
								}
							}
						}
					}

					return FReply::Handled();
				});
		}

		for (UFunction* Function : InCallInEditorFunctions)
		{
			if (!Function)
			{
				continue;
			}

			FName FunctionCategoryName(NAME_Default);
			if (Function->HasMetaData(NAME_FunctionCategory))
			{
				FunctionCategoryName = FName(*Function->GetMetaData(NAME_FunctionCategory));
			}

			if (FunctionCategoryName != ActiveCategory)
			{
				ActiveCategory = FunctionCategoryName;
				OutCategorizedEntries.Emplace(FunctionCategoryName);
			}

			FCategorizedFunctionCallEntry& CategoryEntry = OutCategorizedEntries.Last();

			FText ButtonLabel = ObjectTools::GetUserFacingFunctionName(Function);
			FText ButtonToolTip = Function->GetToolTipText();
			if (ButtonToolTip.IsEmpty())
			{
				ButtonToolTip = ButtonLabel;
			}

			CategoryEntry.WrapBox->AddSlot()
			.Padding(0.0f, 0.0f, 5.0f, 3.0f)
			[
				PropertyCustomizationHelpers::MakeFunctionCallButton(
					FPropertyFunctionCallArgs(
						Function,
						OnExecute,
						OnCanExecute,
						ButtonLabel,
						ButtonToolTip,
						&CategoryEntry.FunctionSearchText)
					)
			];

			CategoryEntry.RowTag = Function->GetFName();
		}
	}

	void AddFunctionCallWidgets(IDetailGroup& RootGroup, const TArrayView<UFunction*>& InCallInEditorFunctions, const FPropertyFunctionCallDelegates& InArgs)
	{
		TArray<FCategorizedFunctionCallEntry, TInlineAllocator<8>> CategorizedEntries;
		GetFunctionCallWidgets(InCallInEditorFunctions, InArgs, CategorizedEntries);

		TMap<FName, IDetailGroup*> Groups;

		// Now edit the categories, adding the button strips to the details panel
		for (FCategorizedFunctionCallEntry& CategoryEntry : CategorizedEntries)
		{
			IDetailGroup* Group = nullptr;
			if (CategoryEntry.CategoryName == NAME_Default)
			{
				Group = &RootGroup;
			}
			else if (IDetailGroup** ExistingGroup = Groups.Find(CategoryEntry.CategoryName))
			{
				Group = *ExistingGroup;
			}
			else
			{
				Group = Groups.Emplace(
					CategoryEntry.CategoryName,
					&RootGroup.AddGroup(
						CategoryEntry.CategoryName,
						FText::FromName(CategoryEntry.CategoryName)));
			}

			Group->AddWidgetRow()
			.FilterString(CategoryEntry.FunctionSearchText.ToText())
			.ShouldAutoExpand(true)
			.RowTag(CategoryEntry.RowTag)
			[
				CategoryEntry.WrapBox.ToSharedRef()
			];
		}
	}

	void AddFunctionCallWidgets(IDetailLayoutBuilder& DetailBuilder, const TArrayView<UFunction*>& InCallInEditorFunctions, const FPropertyFunctionCallDelegates& InArgs)
	{
		TArray<FCategorizedFunctionCallEntry, TInlineAllocator<8>> CategorizedEntries;
		GetFunctionCallWidgets(InCallInEditorFunctions, InArgs, CategorizedEntries);

		// Now edit the categories, adding the button strips to the details panel
		for (FCategorizedFunctionCallEntry& CategoryEntry : CategorizedEntries)
		{
			IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(CategoryEntry.CategoryName);
			CategoryBuilder.AddCustomRow(CategoryEntry.FunctionSearchText.ToText())
			.RowTag(CategoryEntry.RowTag)
			[
				CategoryEntry.WrapBox.ToSharedRef()
			];
		}
	}

	void AddCallInEditorFunctionCallWidgetsForClass(IDetailGroup& RootGroup, const UClass* Class, const FPropertyFunctionCallDelegates& InArgs)
	{
		TArray<UFunction*> CallInEditorFunctions;
		PropertyCustomizationHelpers::GetCallInEditorFunctionsForClass(
			Class,
			CallInEditorFunctions);

		if (!CallInEditorFunctions.IsEmpty())
		{
			AddFunctionCallWidgets(RootGroup, CallInEditorFunctions, InArgs);
		}
	}

	void AddCallInEditorFunctionCallWidgetsForClass(IDetailLayoutBuilder& DetailBuilder, const UClass* Class, const FPropertyFunctionCallDelegates& InArgs)
	{
		TArray<UFunction*> CallInEditorFunctions;
		PropertyCustomizationHelpers::GetCallInEditorFunctionsForClass(
			Class,
			CallInEditorFunctions);

		if (!CallInEditorFunctions.IsEmpty())
		{
			AddFunctionCallWidgets(DetailBuilder, CallInEditorFunctions, InArgs);
		}
	}

	FText GetVisibilityDisplay(TAttribute<bool> bEnabled)
	{
		return bEnabled.Get() ? FEditorFontGlyphs::Eye : FEditorFontGlyphs::Eye_Slash;
	}

	TSharedRef<SWidget> MakeVisibilityButton(FOnClicked OnVisibilityClicked, TAttribute<FText> OptionalToolTipText, TAttribute<bool> VisibilityDelegate)
	{
		TAttribute<FText>::FGetter DynamicVisibilityGetter;
		DynamicVisibilityGetter.BindStatic(&GetVisibilityDisplay, VisibilityDelegate);
		TAttribute<FText> DynamicVisibilityAttribute = TAttribute<FText>::Create(DynamicVisibilityGetter);
		return
			SNew( SButton )
			.OnClicked( OnVisibilityClicked )
			.IsEnabled(true)
			.IsFocusable( false )
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ToolTipText(LOCTEXT("ToggleVisibility", "Toggle Visibility"))
			.ContentPadding(2.0f)
			.ForegroundColor(FSlateColor::UseForeground())
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
				.Text(DynamicVisibilityAttribute)
			];
	}

	TSharedRef<SWidget> MakeBrowseButton( FSimpleDelegate OnFindClicked, TAttribute<FText> OptionalToolTipText, TAttribute<bool> IsEnabled, const bool IsActor, const TAttribute<const FSlateBrush*>& InIconOverride)
	{
		TAttribute<const FSlateBrush*> IconAttribute = InIconOverride;

		if (!IconAttribute.IsSet())
		{
			IconAttribute = IsActor ? FAppStyle::Get().GetBrush("Icons.SelectInViewport") : FAppStyle::Get().GetBrush("Icons.BrowseContent");
		}

		return
			SNew( SPropertyEditorButton )
			.Text( OptionalToolTipText.Get().IsEmpty() ? LOCTEXT( "BrowseButtonToolTipText", "Browse to Asset in Content Browser") : OptionalToolTipText )
			.Image( IconAttribute )
			.OnClickAction( OnFindClicked )
			.IsEnabled(IsEnabled)
			.IsFocusable( false );
	}

	TSharedRef<SWidget> MakeNewBlueprintButton( FSimpleDelegate OnNewBlueprintClicked, TAttribute<FText> OptionalToolTipText, TAttribute<bool> IsEnabled )
	{
		return
			SNew( SPropertyEditorButton )
			.Text( OptionalToolTipText.Get().IsEmpty() ? LOCTEXT( "NewBlueprintButtonToolTipText", "Create New Blueprint") : OptionalToolTipText )
			.Image( FAppStyle::GetBrush("Icons.PlusCircle") )
			.OnClickAction( OnNewBlueprintClicked )
			.IsEnabled(IsEnabled)
			.IsFocusable( false );
	}

	TSharedRef<SWidget> MakeInsertDeleteDuplicateButton(FExecuteAction OnInsertClicked, FExecuteAction OnDeleteClicked, FExecuteAction OnDuplicateClicked)
	{
		FMenuBuilder MenuContentBuilder( true, nullptr, nullptr, true );
		{
			if (OnInsertClicked.IsBound())
			{
				FUIAction InsertAction(OnInsertClicked);
				MenuContentBuilder.AddMenuEntry(LOCTEXT("InsertButtonLabel", "Insert"), FText::GetEmpty(), FSlateIcon(), InsertAction);
			}

			if (OnDeleteClicked.IsBound())
			{
				FUIAction DeleteAction(OnDeleteClicked);
				MenuContentBuilder.AddMenuEntry(LOCTEXT("DeleteButtonLabel", "Delete"), FText::GetEmpty(), FSlateIcon(), DeleteAction);
			}

			if (OnDuplicateClicked.IsBound())
			{
				FUIAction DuplicateAction( OnDuplicateClicked );
				MenuContentBuilder.AddMenuEntry( LOCTEXT( "DuplicateButtonLabel", "Duplicate"), FText::GetEmpty(), FSlateIcon(), DuplicateAction );
			}
		}

		return
			SNew(SComboButton)
			.ComboButtonStyle( FAppStyle::Get(), "SimpleComboButton" )
			.ContentPadding(2.0f)
			.ForegroundColor( FSlateColor::UseForeground() )
			.HasDownArrow(true)
			.MenuContent()
			[
				MenuContentBuilder.MakeWidget()
			];
	}

	TSharedRef<SWidget> MakeAssetPickerAnchorButton( FOnGetAllowedClasses OnGetAllowedClasses, FOnAssetSelected OnAssetSelectedFromPicker, const TSharedPtr<IPropertyHandle>& PropertyHandle)
	{
		return 
			SNew( SPropertyAssetPicker )
			.OnGetAllowedClasses( OnGetAllowedClasses )
			.OnAssetSelected( OnAssetSelectedFromPicker )
			.PropertyHandle( PropertyHandle );
	}

	TArray<const UClass*> EmptyClassArray;

	TSharedRef<SWidget> MakeAssetPickerWithMenu(const FAssetData& InitialObject, const bool AllowClear, const TArray<const UClass*>& AllowedClasses, const TArray<UFactory*>& NewAssetFactories, FOnShouldFilterAsset OnShouldFilterAsset, FOnAssetSelected OnSet, FSimpleDelegate OnClose, const TSharedPtr<IPropertyHandle>& PropertyHandle, const TArray<FAssetData>& OwnerAssetArray)
	{
		return MakeAssetPickerWithMenu(InitialObject, AllowClear, true, AllowedClasses, EmptyClassArray, NewAssetFactories, OnShouldFilterAsset, OnSet, OnClose, PropertyHandle, OwnerAssetArray);
	}

	TSharedRef<SWidget> MakeAssetPickerWithMenu(const FAssetData& InitialObject, const bool AllowClear, const TArray<const UClass*>& AllowedClasses, const TArray<const UClass*>& DisallowedClasses, const TArray<UFactory*>& NewAssetFactories, FOnShouldFilterAsset OnShouldFilterAsset, FOnAssetSelected OnSet, FSimpleDelegate OnClose, const TSharedPtr<IPropertyHandle>& PropertyHandle, const TArray<FAssetData>& OwnerAssetArray)
	{
		return MakeAssetPickerWithMenu(InitialObject, AllowClear, true, AllowedClasses, DisallowedClasses, NewAssetFactories, OnShouldFilterAsset, OnSet, OnClose, PropertyHandle, OwnerAssetArray);
	}

	TSharedRef<SWidget> MakeAssetPickerWithMenu(const FAssetData& InitialObject, const bool AllowClear, const bool AllowCopyPaste, const TArray<const UClass*>& AllowedClasses, const TArray<UFactory*>& NewAssetFactories, FOnShouldFilterAsset OnShouldFilterAsset, FOnAssetSelected OnSet, FSimpleDelegate OnClose, const TSharedPtr<IPropertyHandle>& PropertyHandle, const TArray<FAssetData>& OwnerAssetArray)
	{
		return MakeAssetPickerWithMenu(InitialObject, AllowClear, AllowCopyPaste, AllowedClasses, EmptyClassArray, NewAssetFactories, OnShouldFilterAsset, OnSet, OnClose, PropertyHandle, OwnerAssetArray);
	}

	TSharedRef<SWidget> MakeAssetPickerWithMenu( const FAssetData& InitialObject, const bool AllowClear, const bool AllowCopyPaste, const TArray<const UClass*>& AllowedClasses, const TArray<const UClass*>& DisallowedClasses, const TArray<UFactory*>& NewAssetFactories, FOnShouldFilterAsset OnShouldFilterAsset, FOnAssetSelected OnSet, FSimpleDelegate OnClose, const TSharedPtr<IPropertyHandle>& PropertyHandle, const TArray<FAssetData>& OwnerAssetArray)
	{
		return
			SNew(SPropertyMenuAssetPicker)
			.InitialObject(InitialObject)
			.PropertyHandle(PropertyHandle)
			.OwnerAssetArray(OwnerAssetArray)
			.AllowClear(AllowClear)
			.AllowCopyPaste(AllowCopyPaste)
			.AllowedClasses(AllowedClasses)
			.DisallowedClasses(DisallowedClasses)
			.NewAssetFactories(NewAssetFactories)
			.OnShouldFilterAsset(OnShouldFilterAsset)
			.OnSet(OnSet)
			.OnClose(OnClose);
	}

	TSharedRef<SWidget> MakeActorPickerAnchorButton( FOnGetActorFilters OnGetActorFilters, FOnActorSelected OnActorSelectedFromPicker )
	{
		return 
			SNew( SPropertySceneOutliner )
			.OnGetActorFilters( OnGetActorFilters )
			.OnActorSelected( OnActorSelectedFromPicker );
	}

	TSharedRef<SWidget> MakeActorPickerWithMenu( AActor* const InitialActor, const bool AllowClear, FOnShouldFilterActor ActorFilter, FOnActorSelected OnSet, FSimpleDelegate OnClose, FSimpleDelegate OnUseSelected )
	{
		return 
			SNew( SPropertyMenuActorPicker )
			.InitialActor(InitialActor)
			.AllowClear(AllowClear)
			.AllowPickingLevelInstanceContent(false)
			.ActorFilter(ActorFilter)
			.OnSet(OnSet)
			.OnClose(OnClose)
			.OnUseSelected(OnUseSelected);
	}

	TSharedRef<SWidget> MakeActorPickerWithMenu(AActor* const InitialActor, const bool AllowClear, const bool AllowPickingLevelInstanceContent, FOnShouldFilterActor ActorFilter, FOnActorSelected OnSet, FSimpleDelegate OnClose, FSimpleDelegate OnUseSelected, bool bDisplayUseSelected, bool bShowTransient)
	{
		return
			SNew(SPropertyMenuActorPicker)
			.InitialActor(InitialActor)
			.AllowClear(AllowClear)
			.AllowPickingLevelInstanceContent(AllowPickingLevelInstanceContent)
			.ActorFilter(ActorFilter)
			.OnSet(OnSet)
			.OnClose(OnClose)
			.OnUseSelected(OnUseSelected)
			.DisplayUseSelected(bDisplayUseSelected)
			.ShowTransient(bShowTransient);
	}

	TSharedRef<SWidget> MakeComponentPickerWithMenu( UActorComponent* const InitialComponent, const bool AllowClear, FOnShouldFilterActor ActorFilter, FOnShouldFilterComponent ComponentFilter, FOnComponentSelected OnSet, FSimpleDelegate OnClose, UWorld* PickerWorld, bool bShowTransient)
	{
		return
			SNew(SPropertyMenuComponentPicker)
			.InitialComponent(InitialComponent)
			.AllowClear(AllowClear)
			.ShowTransient(bShowTransient)
			.ActorFilter(ActorFilter)
			.ComponentFilter(ComponentFilter)
			.OnSet(OnSet)
			.OnClose(OnClose)
			.PickerWorld(PickerWorld);
	}

	TSharedRef<SWidget> MakeInteractiveActorPicker( FOnGetAllowedClasses OnGetAllowedClasses, FOnShouldFilterActor OnShouldFilterActor, FOnActorSelected OnActorSelectedFromPicker)
	{
		return
			SNew(SPropertyEditorInteractiveActorPicker)
			.ToolTipText(LOCTEXT("PickButtonLabel", "Pick Actor from scene"))
			.OnGetAllowedClasses(OnGetAllowedClasses)
			.OnShouldFilterActor(OnShouldFilterActor)
			.OnActorSelected(OnActorSelectedFromPicker);
	}

	TSharedRef<SWidget> MakeSceneDepthPicker(FOnSceneDepthLocationSelected OnSceneDepthLocationSelected)
	{
		return
			SNew(SPropertyEditorSceneDepthPicker)
			.ToolTipText(LOCTEXT("PickSceneDepthLabel", "Sample Scene Depth from scene"))
			.OnSceneDepthLocationSelected(OnSceneDepthLocationSelected);
	}

	TSharedRef<SWidget> MakeEditConfigHierarchyButton(FSimpleDelegate OnEditConfigClicked, TAttribute<FText> OptionalToolTipText, TAttribute<bool> IsEnabled)
	{
		return
			SNew(SPropertyEditorButton)
			.Text(OptionalToolTipText.Get().IsEmpty() ? LOCTEXT("EditConfigHierarchyButtonToolTipText", "Edit the config values of this property") : OptionalToolTipText)
			.Image(FAppStyle::GetBrush("DetailsView.EditConfigProperties"))
			.OnClickAction(OnEditConfigClicked)
			.IsEnabled(IsEnabled)
			.IsFocusable(false);
	}

	TSharedRef<SWidget> MakeDocumentationButton(const TSharedRef<FPropertyEditor>& InPropertyEditor)
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = InPropertyEditor->GetPropertyHandle();

		FString DocLink;
		FString DocExcerptName;

		if (PropertyHandle.IsValid() && PropertyHandle->HasDocumentation())
		{
			DocLink = PropertyHandle->GetDocumentationLink();
			DocExcerptName = PropertyHandle->GetDocumentationExcerptName();
		}
		else
		{
			DocLink = InPropertyEditor->GetDocumentationLink();
			DocExcerptName = InPropertyEditor->GetDocumentationExcerptName();
		}

		return IDocumentation::Get()->CreateAnchor(DocLink, FString(), DocExcerptName);
	}

	TSharedRef<SWidget> MakeSaveButton(FSimpleDelegate OnSaveClicked, TAttribute<FText> OptionalToolTipText, TAttribute<bool> IsEnabled)
	{
		return
			SNew(SPropertyEditorButton)
			.Text(OptionalToolTipText.Get().IsEmpty() ? LOCTEXT("SaveButtonTooltipText", "Save the currently selected asset.") : OptionalToolTipText)
			.Image(FAppStyle::GetBrush("Icons.Save"))
			.OnClickAction(OnSaveClicked)
			.IsEnabled(IsEnabled)
			.IsFocusable(false);
	}

	FBoolProperty* GetEditConditionProperty(const FProperty* InProperty, bool& bNegate)
	{
		FBoolProperty* EditConditionProperty = NULL;
		bNegate = false;

		if ( InProperty != NULL )
		{
			// find the name of the property that should be used to determine whether this property should be editable
			FString ConditionPropertyName = InProperty->GetMetaData(TEXT("EditCondition"));

			// Support negated edit conditions whose syntax is !BoolProperty
			if ( ConditionPropertyName.StartsWith(FString(TEXT("!"))) )
			{
				bNegate = true;
				// Chop off the negation from the property name
				ConditionPropertyName = ConditionPropertyName.Right(ConditionPropertyName.Len() - 1);
			}

			// for now, only support boolean conditions, and only allow use of another property within the same struct as the conditional property
			if ( ConditionPropertyName.Len() > 0 && !ConditionPropertyName.Contains(TEXT(".")) )
			{
				UStruct* Scope = InProperty->GetOwnerStruct();
				EditConditionProperty = FindFProperty<FBoolProperty>(Scope, *ConditionPropertyName);
			}
		}

		return EditConditionProperty;
	}

	TArray<UFactory*> GetNewAssetFactoriesForClasses(const TArray<const UClass*>& Classes)
	{
		return GetNewAssetFactoriesForClasses(Classes, EmptyClassArray);
	}

	TArray<UFactory*> GetNewAssetFactoriesForClasses(const TArray<const UClass*>& Classes, const TArray<const UClass*>& DisallowedClasses)
	{
		const IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		TArray<UFactory*> AllFactories = AssetTools.GetNewAssetFactories();
		TArray<UFactory*> FilteredFactories;

		for (UFactory* Factory : AllFactories)
		{
			UClass* SupportedClass = Factory->GetSupportedClass();
			auto IsChildOfLambda = [SupportedClass](const UClass* InClass) { return SupportedClass->IsChildOf(InClass); };

			if (SupportedClass != nullptr 
				&& Classes.ContainsByPredicate(IsChildOfLambda)
				&& !DisallowedClasses.ContainsByPredicate(IsChildOfLambda))
			{
				FilteredFactories.Add(Factory);
			}
		}

		FilteredFactories.Sort([](UFactory& A, UFactory& B) -> bool
		{
			return A.GetDisplayName().CompareToCaseIgnored(B.GetDisplayName()) < 0;
		});

		return FilteredFactories;
	}
}

void SObjectPropertyEntryBox::Construct( const FArguments& InArgs )
{
	ObjectPath = InArgs._ObjectPath;
	OnObjectChanged = InArgs._OnObjectChanged;
	OnShouldSetAsset = InArgs._OnShouldSetAsset;
	OnIsEnabled = InArgs._OnIsEnabled;
	OnShouldFilterActor = InArgs._OnShouldFilterActor;

	const TArray<FAssetData>& OwnerAssetDataArray = InArgs._OwnerAssetDataArray;

	bool bDisplayThumbnail = InArgs._DisplayThumbnail;
	FIntPoint ThumbnailSize(48, 48);
	if (InArgs._ThumbnailSizeOverride.IsSet())
	{
		ThumbnailSize = InArgs._ThumbnailSizeOverride.Get();
	}

	if( InArgs._PropertyHandle.IsValid() && InArgs._PropertyHandle->IsValidHandle() )
	{
		PropertyHandle = InArgs._PropertyHandle;

		// check if the property metadata wants us to display a thumbnail
		const FString& DisplayThumbnailString = PropertyHandle->GetMetaData(TEXT("DisplayThumbnail"));
		if(DisplayThumbnailString.Len() > 0)
		{
			bDisplayThumbnail = DisplayThumbnailString == TEXT("true");
		}

		// check if the property metadata has an override to the thumbnail size
		const FString& ThumbnailSizeString = PropertyHandle->GetMetaData(TEXT("ThumbnailSize"));
		if ( ThumbnailSizeString.Len() > 0 )
		{
			FVector2D ParsedVector;
			if ( ParsedVector.InitFromString(ThumbnailSizeString) )
			{
				ThumbnailSize.X = (int32)ParsedVector.X;
				ThumbnailSize.Y = (int32)ParsedVector.Y;
			}
		}

		// if being used with an object property, check the allowed class is valid for the property
		FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(PropertyHandle->GetProperty());
		if (ObjectProperty != NULL)
		{
			checkSlow(InArgs._AllowedClass->IsChildOf(ObjectProperty->PropertyClass));
		}
	}

	ChildSlot
	[	
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1)
		.VAlign(VAlign_Center)	
		[
			SAssignNew(PropertyEditorAsset, SPropertyEditorAsset)
				.ObjectPath( this, &SObjectPropertyEntryBox::OnGetObjectPath )
				.Class( InArgs._AllowedClass )
				.NewAssetFactories( InArgs._NewAssetFactories )
				.IsEnabled(this, &SObjectPropertyEntryBox::IsEnabled)
				.OnSetObject(this, &SObjectPropertyEntryBox::OnSetObject)
				.ThumbnailPool(InArgs._ThumbnailPool)
				.DisplayThumbnail(bDisplayThumbnail)
				.OnShouldFilterAsset(InArgs._OnShouldFilterAsset)
				.AllowClear(InArgs._AllowClear)
				.AllowCreate(InArgs._AllowCreate)
				.DisplayUseSelected(InArgs._DisplayUseSelected)
				.DisplayBrowse(InArgs._DisplayBrowse)
				.OnBrowseOverride(InArgs._OnBrowseOverride)
				.OnBrowseToolTipOverride(InArgs._OnBrowseToolTipOverride)
				.OnBrowseIconOverride(InArgs._OnBrowseIconOverride)
				.EnableContentPicker(InArgs._EnableContentPicker)
				.PropertyHandle(PropertyHandle)
				.OwnerAssetDataArray(OwnerAssetDataArray)
				.ThumbnailSize(ThumbnailSize)
				.DisplayCompactSize(InArgs._DisplayCompactSize)
				.OnShouldFilterActor(InArgs._OnShouldFilterActor)
				.bOnlyRecognizeOnDragEnter(InArgs._bOnlyRecognizeOnDragEnter)
				.CustomContentSlot()
				[
					InArgs._CustomContentSlot.Widget
				]
				.CustomButtonSlot()
				[
					InArgs._CustomButtonSlot.Widget
				]
		]
	];
}

void SObjectPropertyEntryBox::GetDesiredWidth(float& OutMinDesiredWidth, float &OutMaxDesiredWidth)
{
	checkf(PropertyEditorAsset.IsValid(), TEXT("SObjectPropertyEntryBox hasn't been constructed yet."));
	PropertyEditorAsset->GetDesiredWidth(OutMinDesiredWidth, OutMaxDesiredWidth);
}

void SObjectPropertyEntryBox::OpenEntryBox()
{
	if (PropertyEditorAsset.IsValid())
	{
		PropertyEditorAsset->OpenComboButton();
	}
}

FString SObjectPropertyEntryBox::OnGetObjectPath() const
{
	FString StringReference;
	if (ObjectPath.IsSet())
	{
		StringReference = ObjectPath.Get();
	}
	else if( PropertyHandle.IsValid() )
	{
		PropertyHandle->GetValueAsFormattedString( StringReference );
	}
	
	return StringReference;
}

void SObjectPropertyEntryBox::OnSetObject(const FAssetData& AssetData)
{
	if( PropertyHandle.IsValid() && PropertyHandle->IsValidHandle() )
	{
		if (!OnShouldSetAsset.IsBound() || OnShouldSetAsset.Execute(AssetData))
		{
			PropertyHandle->SetValue(AssetData);
		}
	}
	OnObjectChanged.ExecuteIfBound(AssetData);
}

bool SObjectPropertyEntryBox::IsEnabled() const
{
	bool IsEnabled = true;
	if (PropertyHandle.IsValid())
	{
		IsEnabled &= PropertyHandle->IsEditable();
	}

	if (OnIsEnabled.IsBound())
	{
		IsEnabled &= OnIsEnabled.Execute();
	}

	return IsEnabled;
}

void SClassPropertyEntryBox::Construct(const FArguments& InArgs)
{
	ChildSlot
	[	
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SAssignNew(PropertyEditorClass, SPropertyEditorClass)
				.MetaClass(InArgs._MetaClass)
				.RequiredInterface(InArgs._RequiredInterface)
				.AllowedClasses(InArgs._AllowedClasses)
				.DisallowedClasses(InArgs._DisallowedClasses)
				.AllowAbstract(InArgs._AllowAbstract)
				.IsBlueprintBaseOnly(InArgs._IsBlueprintBaseOnly)
				.AllowNone(InArgs._AllowNone)
				.ShowViewOptions(!InArgs._HideViewOptions)
				.ShowDisplayNames(InArgs._ShowDisplayNames)
				.ShowTree(InArgs._ShowTreeView)
				.SelectedClass(InArgs._SelectedClass)
				.OnSetClass(InArgs._OnSetClass)
				.ClassViewerFilters(InArgs._ClassViewerFilters)
		]
	];
}

void SStructPropertyEntryBox::Construct(const FArguments& InArgs)
{
	ChildSlot
	[	
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SAssignNew(PropertyEditorStruct, SPropertyEditorStruct)
				.MetaStruct(InArgs._MetaStruct)
				.AllowNone(InArgs._AllowNone)
				.ShowViewOptions(!InArgs._HideViewOptions)
				.ShowDisplayNames(InArgs._ShowDisplayNames)
				.ShowTree(InArgs._ShowTreeView)
				.SelectedStruct(InArgs._SelectedStruct)
				.OnSetStruct(InArgs._OnSetStruct)
		]
	];
}

void SProperty::Construct( const FArguments& InArgs, TSharedPtr<IPropertyHandle> InPropertyHandle )
{
	TSharedPtr<SWidget> ChildSlotContent;

	const FText& DisplayName = InArgs._DisplayName.Get();

	PropertyHandle = InPropertyHandle;

	if( PropertyHandle->IsValidHandle() )
	{
		InPropertyHandle->MarkHiddenByCustomization();

		if( InArgs._CustomWidget.Widget != SNullWidget::NullWidget )
		{
			TSharedRef<SWidget> CustomWidget = InArgs._CustomWidget.Widget;

			// If the name should be displayed create it now
			if( InArgs._ShouldDisplayName )
			{
				CustomWidget = 
					SNew( SHorizontalBox )
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Right)
					.Padding( 4.0f, 0.0f )
					.FillWidth(1.0f)
					[
						InPropertyHandle->CreatePropertyNameWidget( DisplayName )
					]
					+ SHorizontalBox::Slot()
					.Padding( 0.0f, 0.0f )
					.VAlign(VAlign_Center)
					.FillWidth(1.0f)
					[
						CustomWidget
					];
			}

			ChildSlotContent = CustomWidget;
		}
		else
		{
			if( InArgs._ShouldDisplayName )
			{
				ChildSlotContent = 
					SNew( SHorizontalBox )
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Right)
					.Padding( 3.0f, 0.0f )
					.FillWidth(1.0f)
					[
						InPropertyHandle->CreatePropertyNameWidget( DisplayName )
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.FillWidth(1.0f)
					[
						InPropertyHandle->CreatePropertyValueWidget()
					];
			}
			else
			{
				ChildSlotContent = InPropertyHandle->CreatePropertyValueWidget();
			}
		}
	}
	else
	{
		// The property was not found, just filter out this widget completely
		// Note a spacer widget is used instead of setting the visibility of this widget in the case that a user overrides the visibility of this widget
		ChildSlotContent = 
			SNew( SSpacer )
			.Visibility( EVisibility::Collapsed );
	}
	
	ChildSlot
	[
		ChildSlotContent.ToSharedRef()
	];
}

void SProperty::ResetToDefault()
{
	if( PropertyHandle->IsValidHandle() )
	{
		PropertyHandle->ResetToDefault();
	}
}

FText SProperty::GetResetToDefaultLabel() const
{
	if( PropertyHandle->IsValidHandle() )
	{
		return PropertyHandle->GetResetToDefaultLabel();
	}

	return FText();
}

bool SProperty::ShouldShowResetToDefault() const
{
	return PropertyHandle->IsValidHandle() && !PropertyHandle->IsEditConst() && PropertyHandle->DiffersFromDefault();
}

bool SProperty::IsValidProperty() const
{
	return PropertyHandle.IsValid() && PropertyHandle->IsValidHandle();
}

TSharedRef<SWidget> PropertyCustomizationHelpers::MakePropertyComboBox(const FPropertyComboBoxArgs& InArgs)
{
	return SNew(SPropertyEditorCombo).ComboArgs(InArgs);
}

TSharedRef<SWidget> PropertyCustomizationHelpers::MakePropertyComboBox(const TSharedPtr<IPropertyHandle>& InPropertyHandle, FOnGetPropertyComboBoxStrings OnGetStrings, FOnGetPropertyComboBoxValue OnGetValue, FOnPropertyComboBoxValueSelected OnValueSelected)
{
	return MakePropertyComboBox(FPropertyComboBoxArgs(InPropertyHandle, OnGetStrings, OnGetValue, OnValueSelected));
}

void PropertyCustomizationHelpers::MakeInstancedPropertyCustomUI(TMap<FName, IDetailGroup*>& ExistingGroup, IDetailCategoryBuilder& BaseCategory, TSharedRef<IPropertyHandle>& BaseProperty, FOnInstancedPropertyIteration AddRowDelegate)
{
	uint32 NumChildren = 0;
	BaseProperty->GetNumChildren(NumChildren);
	for (uint32 PropertyIndex = 0; PropertyIndex < NumChildren; ++PropertyIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = BaseProperty->GetChildHandle(PropertyIndex).ToSharedRef();

		if (ChildHandle->GetProperty())
		{
			const FName DefaultCategoryName = ChildHandle->GetDefaultCategoryName();
			const bool DelegateIsBound = AddRowDelegate.IsBound();
			IDetailGroup* DetailGroup = nullptr;

			if (!DefaultCategoryName.IsNone())
			{
				// Custom categories don't work with instanced object properties, so we are using groups instead here.
				IDetailGroup*& DetailGroupPtr = ExistingGroup.FindOrAdd(DefaultCategoryName);
				if (!DetailGroupPtr)
				{
					DetailGroupPtr = &BaseCategory.AddGroup(DefaultCategoryName, ChildHandle->GetDefaultCategoryText());
				}
				DetailGroup = DetailGroupPtr;
			}

			if (DelegateIsBound)
			{
				AddRowDelegate.Execute(BaseCategory, DetailGroup, ChildHandle);
			}
			else if (DetailGroup)
			{
				DetailGroup->AddPropertyRow(ChildHandle);
			}
			else
			{
				BaseCategory.AddProperty(ChildHandle);
			}
		}
		else
		{
			MakeInstancedPropertyCustomUI(ExistingGroup, BaseCategory, ChildHandle, AddRowDelegate);
		}
	}
}

TArray<const UClass*> PropertyCustomizationHelpers::GetClassesFromMetadataString(const FString& MetadataString)
{
	if (MetadataString.IsEmpty())
	{
		return TArray<const UClass*>();
	}

	auto FindClass = [](const FString& InClassName) -> const UClass*
	{
		const UClass* Class = UClass::TryFindTypeSlow<const UClass>(InClassName, EFindFirstObjectOptions::EnsureIfAmbiguous);
		if (!Class)
		{
			Class = LoadObject<const UClass>(nullptr, *InClassName);
		}
		return Class;
	};
		
	TArray<FString> ClassNames;
	MetadataString.ParseIntoArrayWS(ClassNames, TEXT(","), true);

	TArray<const UClass*> Classes;
	Classes.Reserve(ClassNames.Num());

	for (const FString& ClassName : ClassNames)
	{
		const UClass* Class = FindClass(ClassName);
		if (!Class)
		{
			continue;
		}

		// If the class is an interface, expand it to be all classes in memory that implement the class.
		if (Class->HasAnyClassFlags(CLASS_Interface))
		{
			for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
			{
				const UClass* ClassWithInterface = (*ClassIt);
				if (ClassWithInterface->ImplementsInterface(Class))
				{
					Classes.Add(ClassWithInterface);
				}
			}
		}
		else
		{
			Classes.Add(Class);
		}
	}

	return Classes;
}

TArray<const UScriptStruct*> PropertyCustomizationHelpers::GetStructsFromMetadataString(const FString& MetadataString)
{
	if (MetadataString.IsEmpty())
	{
		return TArray<const UScriptStruct*>();
	}

	auto FindClass = [](const FString& InStructName) -> const UScriptStruct*
	{
		const UScriptStruct* Struct = UClass::TryFindTypeSlow<const UScriptStruct>(InStructName, EFindFirstObjectOptions::EnsureIfAmbiguous);
		if (!Struct)
		{
			Struct = LoadObject<const UScriptStruct>(nullptr, *InStructName);
		}
		return Struct;
	};
		
	TArray<FString> StructNames;
	MetadataString.ParseIntoArrayWS(StructNames, TEXT(","), true);

	TArray<const UScriptStruct*> Structs;
	Structs.Reserve(StructNames.Num());

	for (const FString& StructName : StructNames)
	{
		const UScriptStruct* Struct = FindClass(StructName);
		if (!Struct)
		{
			continue;
		}

		Structs.Add(Struct);
	}

	return Structs;
}


TSharedRef<SWidget> PropertyCustomizationHelpers::MakeEditInlineObjectClassPicker(TSharedRef<IPropertyHandle> PropertyHandle, FOnClassPicked OnClassPicked, TSharedPtr<IClassViewerFilter> AdditionalClassFilter)
{
	return SPropertyEditorEditInline::GenerateClassPicker(PropertyHandle, 
	FOnClassPicked::CreateLambda([OnClassPicked, PropertyHandle](UClass* ClassPicked)
		{
			SPropertyEditorEditInline::OnClassPicked(ClassPicked, PropertyHandle);
			OnClassPicked.ExecuteIfBound(ClassPicked);
		}),
		AdditionalClassFilter);
}

void PropertyCustomizationHelpers::CreateNewInstanceOfEditInlineObjectClass(TSharedRef<IPropertyHandle> PropertyHandle, UClass* Class, EPropertyValueSetFlags::Type Flags)
{
	SPropertyEditorEditInline::OnClassPicked(Class, PropertyHandle, Flags);
}

namespace PropertyCustomizationHelpers
{
	namespace Private
	{
		static const FName NAME_CallInEditor(TEXT("CallInEditor"));
		static const FName NAME_WorldContext(TEXT("WorldContext"));

		static bool CanCallFunctionBasedOnParams(const UFunction* TestFunction)
		{
			// If the function only takes a world context object we can use the editor's
			// world context - but only if the blueprint is editor only and the function
			// is static:
			if (UClass* TestFunctionOwnerClass = TestFunction->GetOwnerClass())
			{
				if (UBlueprint* Blueprint = Cast<UBlueprint>(TestFunctionOwnerClass->ClassGeneratedBy))
				{
					if (FBlueprintEditorUtils::IsEditorUtilityBlueprint(Blueprint) && Blueprint->BlueprintType == BPTYPE_FunctionLibrary)
					{
						using namespace UE::Reflection;
						return TestFunction->HasMetaData(NAME_WorldContext) &&
							DoesStaticFunctionSignatureMatch<void(TObjectPtr<UObject>)>(TestFunction);
					}
				}
			}

			const bool bCanCall = TestFunction->GetBoolMetaData(NAME_CallInEditor) && (TestFunction->ParmsSize == 0); // no params required, we can call it!
			return bCanCall;
		}

		void GetCallInEditorFunctionsForClassInternal(const UClass* InClass, TOptional<TFunctionRef<bool(const UFunction*)>> InFunctionFilter, TArray<UFunction*>& OutCallInEditorFunctions, EFieldIterationFlags InIterationFlags)
		{
			// metadata tag for defining sort order of function buttons within a Category
			static const FName NAME_DisplayPriority("DisplayPriority");

			const bool bDisallowEditorUtilityBlueprintFunctions = GetDefault<UBlueprintEditorProjectSettings>()->bDisallowEditorUtilityBlueprintFunctionsInDetailsView;

			// Get all of the functions we need to display (done ahead of time so we can sort them)
			for (TFieldIterator<UFunction> FunctionIter(InClass, InIterationFlags); FunctionIter; ++FunctionIter)
			{
				const UFunction* TestFunction = *FunctionIter;
				if (CanCallFunctionBasedOnParams(TestFunction) && (!InFunctionFilter.IsSet() || InFunctionFilter.GetValue()(TestFunction)))
				{
					const FName FunctionName = TestFunction->GetFName();

					if (const bool bFunctionIsPermissible = FPropertyEditorPermissionList::Get().DoesPropertyPassFilter(TestFunction->GetOwnerClass(), FunctionName);
						!bFunctionIsPermissible)
					{
						continue;
					}

					if (const bool bFunctionAlreadyAdded = OutCallInEditorFunctions.ContainsByPredicate([&FunctionName](UFunction*& Func) { return Func->GetFName() == FunctionName; });
						bFunctionAlreadyAdded)
					{
						continue;
					}

					OutCallInEditorFunctions.Add(*FunctionIter);
				}
			}

			if (OutCallInEditorFunctions.IsEmpty())
			{
				return;
			}


			// FBlueprintMetadata::MD_FunctionCategory
			static const FName NAME_FunctionCategory(TEXT("Category"));

			// Sort the functions by category and then by DisplayPriority meta tag, and then by name
			OutCallInEditorFunctions.Sort([](const UFunction& A, const UFunction& B)
			{
				const int32 CategorySort = A.GetMetaData(NAME_FunctionCategory).Compare(B.GetMetaData(NAME_FunctionCategory));
				if (CategorySort != 0)
				{
					return (CategorySort <= 0);
				}
				else
				{
					const FString DisplayPriorityAStr = A.GetMetaData(NAME_DisplayPriority);
					int32 DisplayPriorityA = (DisplayPriorityAStr.IsEmpty() ? MAX_int32 : FCString::Atoi(*DisplayPriorityAStr));
					if (DisplayPriorityA == 0 && !FCString::IsNumeric(*DisplayPriorityAStr))
					{
						DisplayPriorityA = MAX_int32;
					}

					const FString DisplayPriorityBStr = B.GetMetaData(NAME_DisplayPriority);
					int32 DisplayPriorityB = (DisplayPriorityBStr.IsEmpty() ? MAX_int32 : FCString::Atoi(*DisplayPriorityBStr));
					if (DisplayPriorityB == 0 && !FCString::IsNumeric(*DisplayPriorityBStr))
					{
						DisplayPriorityB = MAX_int32;
					}

					return (DisplayPriorityA == DisplayPriorityB) ? (A.GetName() <= B.GetName()) : (DisplayPriorityA <= DisplayPriorityB);
				}
			});
		}
	}

	void GetCallInEditorFunctionsForClass(const UClass* InClass, TArray<UFunction*>& OutCallInEditorFunctions, EFieldIterationFlags InIterationFlags)
	{
		return Private::GetCallInEditorFunctionsForClassInternal(InClass, {}, OutCallInEditorFunctions, InIterationFlags);
	}

	void GetCallInEditorFunctionsForClass(const UClass* InClass, const TFunctionRef<bool(const UFunction*)>& InFunctionFilter,	TArray<UFunction*>& OutCallInEditorFunctions, EFieldIterationFlags InIterationFlags)
	{
		return Private::GetCallInEditorFunctionsForClassInternal(InClass, InFunctionFilter, OutCallInEditorFunctions, InIterationFlags);
	}
}

//////////////////////////////////////////////////////////////////////////
//
// Sections list

/**
* Builds up a list of unique Sections while creating some information about the Sections
*/
class FSectionListBuilder : public ISectionListBuilder
{
	friend class FSectionList;
public:
	
	FSectionListBuilder(int32 InThumbnailSize)
		:ThumbnailSize(InThumbnailSize)
	{}
	
	/**
	* Adds a new Section to the list
	*
	* @param SlotIndex		The slot (usually mesh element index) where the Section is located on the component
	* @param Section		The Section being used
	* @param bCanBeReplced	Whether or not the Section can be replaced by a user
	*/
	virtual void AddSection(int32 LodIndex, int32 SectionIndex, FName InMaterialSlotName, int32 InMaterialSlotIndex, FName InOriginalMaterialSlotName, const TMap<int32, FName> &InAvailableMaterialSlotName, const UMaterialInterface* Material, bool IsSectionUsingCloth, bool bIsChunkSection, int32 DefaultMaterialIndex) override
	{
		FSectionListItem SectionItem(LodIndex, SectionIndex, InMaterialSlotName, InMaterialSlotIndex, InOriginalMaterialSlotName, InAvailableMaterialSlotName, Material, IsSectionUsingCloth, ThumbnailSize, bIsChunkSection, DefaultMaterialIndex);
		if (!Sections.Contains(SectionItem))
		{
			Sections.Add(SectionItem);
			if (!SectionsByLOD.Contains(SectionItem.LodIndex))
			{
				TArray<FSectionListItem> LodSectionsArray;
				LodSectionsArray.Add(SectionItem);
				SectionsByLOD.Add(SectionItem.LodIndex, LodSectionsArray);
			}
			else
			{
				//Remove old entry
				TArray<FSectionListItem> &ExistingSections = *SectionsByLOD.Find(SectionItem.LodIndex);
				for (int32 ExistingSectionIndex = 0; ExistingSectionIndex < ExistingSections.Num(); ++ExistingSectionIndex)
				{
					const FSectionListItem &ExistingSectionItem = ExistingSections[ExistingSectionIndex];
					if (ExistingSectionItem.LodIndex == LodIndex && ExistingSectionItem.SectionIndex == SectionIndex)
					{
						ExistingSections.RemoveAt(ExistingSectionIndex);
						break;
					}
				}
				ExistingSections.Add(SectionItem);
			}
		}
	}

	/** Empties the list */
	void Empty()
	{
		Sections.Reset();
		SectionsByLOD.Reset();
	}

	/** Sorts the list by lod and section index */
	void Sort()
	{
		struct FSortByIndex
		{
			bool operator()(const FSectionListItem& A, const FSectionListItem& B) const
			{
				return (A.LodIndex == B.LodIndex) ? A.SectionIndex < B.SectionIndex : A.LodIndex < B.LodIndex;
			}
		};

		Sections.Sort(FSortByIndex());
	}

	/** @return The number of Sections in the list */
	uint32 GetNumSections() const
	{
		return Sections.Num();
	}

	uint32 GetNumSections(int32 LodIndex) const
	{
		return SectionsByLOD.Contains(LodIndex) ? SectionsByLOD.Find(LodIndex)->Num() : 0;
	}

private:
	/** All Section items in the list */
	TArray<FSectionListItem> Sections;
	/** All Section items in the list */
	TMap<int32, TArray<FSectionListItem>> SectionsByLOD;

	int32 ThumbnailSize;
};


/**
* A view of a single item in an FSectionList
*/
class FSectionItemView : public TSharedFromThis<FSectionItemView>
{
public:
	/**
	* Creates a new instance of this class
	*
	* @param Section				The Section to view
	* @param InOnSectionChanged	Delegate for when the Section changes
	*/
	static TSharedRef<FSectionItemView> Create(
		const FSectionListItem& Section,
		FOnSectionChanged InOnSectionChanged,
		FOnGenerateWidgetsForSection InOnGenerateNameWidgetsForSection,
		FOnGenerateWidgetsForSection InOnGenerateWidgetsForSection,
		FOnResetSectionToDefaultClicked InOnResetToDefaultClicked,
		int32 InMultipleSectionCount,
		int32 InThumbnailSize)
	{
		return MakeShareable(new FSectionItemView(Section, InOnSectionChanged, InOnGenerateNameWidgetsForSection, InOnGenerateWidgetsForSection, InOnResetToDefaultClicked, InMultipleSectionCount, InThumbnailSize));
	}

	TSharedRef<SWidget> CreateNameContent()
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("SectionIndex"), SectionItem.SectionIndex);
		return
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(FText::Format(LOCTEXT("SectionIndex", "Section {SectionIndex}"), Arguments))
			]
			+ SVerticalBox::Slot()
			.Padding(0.0f, 4.0f)
			.AutoHeight()
			[
				OnGenerateCustomNameWidgets.IsBound() ? OnGenerateCustomNameWidgets.Execute(SectionItem.LodIndex, SectionItem.SectionIndex) : StaticCastSharedRef<SWidget>(SNullWidget::NullWidget)
			];
	}

	TSharedRef<SWidget> CreateValueContent(const TSharedPtr<FAssetThumbnailPool>& ThumbnailPool)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("DefaultMaterialIndex"), SectionItem.DefaultMaterialIndex);
		FText BaseMaterialSlotTooltip = SectionItem.DefaultMaterialIndex != SectionItem.MaterialSlotIndex ? FText::Format(LOCTEXT("SectionIndex_BaseMaterialSlotNameTooltip", "This section material slot was change from the default value [{DefaultMaterialIndex}]."), Arguments) : FText::GetEmpty();
		FText MaterialSlotNameTooltipText = SectionItem.IsSectionUsingCloth ? FText(LOCTEXT("SectionIndex_MaterialSlotNameTooltip", "Cannot change the material slot when the mesh section use the cloth system.")) : BaseMaterialSlotTooltip;
		return
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					SNew(SHorizontalBox)
					.Visibility(SectionItem.bIsChunkSection ? EVisibility::Collapsed : EVisibility::Visible)
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(SPropertyEditorAsset)
						.ObjectPath(SectionItem.Material->GetPathName())
						.Class(UMaterialInterface::StaticClass())
						.DisplayThumbnail(true)
						.ThumbnailSize(FIntPoint(ThumbnailSize, ThumbnailSize))
						.DisplayUseSelected(false)
						.AllowClear(false)
						.DisplayBrowse(false)
						.EnableContentPicker(false)
						.ThumbnailPool(ThumbnailPool)
						.DisplayCompactSize(true)
						.CustomContentSlot()
						[
							SNew( SBox )
							.HAlign(HAlign_Fill)
							[
								SNew(SVerticalBox)
								+SVerticalBox::Slot()
								.AutoHeight()
								[
									SNew(SHorizontalBox)
									+ SHorizontalBox::Slot()
									.Padding(0.0f)
									.VAlign(VAlign_Center)
									.AutoWidth()
									[
										SNew(SBox)
										.HAlign(HAlign_Right)
										.MinDesiredWidth(65.0f)
										[
											SNew(STextBlock)
											.Font(IDetailLayoutBuilder::GetDetailFont())
											.Text(LOCTEXT("SectionListItemMaterialSlotNameLabel", "Material Slot"))
											.ToolTipText(MaterialSlotNameTooltipText)
										]
									]
									+ SHorizontalBox::Slot()
									.VAlign(VAlign_Center)
									.FillWidth(1.0f)
									.Padding(5.0f, 0.0f, 0.0f, 0.0f)
									[
										SNew(SBox)
										.HAlign(HAlign_Fill)
										.VAlign(VAlign_Center)
										.MinDesiredWidth(210.0f)
										[
											//Material Slot Name
											SNew(SComboButton)
											.OnGetMenuContent(this, &FSectionItemView::OnGetMaterialSlotNameMenuForSection)
											.VAlign(VAlign_Center)
											.ContentPadding(2.0f)
											.IsEnabled(!SectionItem.IsSectionUsingCloth)
											.ButtonContent()
											[
												SNew(STextBlock)
												.Font(IDetailLayoutBuilder::GetDetailFont())
												.Text(this, &FSectionItemView::GetCurrentMaterialSlotName)
												.ToolTipText(MaterialSlotNameTooltipText)
											]
										]
									]
								]
								+SVerticalBox::Slot()
								.AutoHeight()
								.VAlign(VAlign_Center)
								[
									OnGenerateCustomSectionWidgets.IsBound() ? OnGenerateCustomSectionWidgets.Execute(SectionItem.LodIndex, SectionItem.SectionIndex) : StaticCastSharedRef<SWidget>(SNullWidget::NullWidget)
								]
							]
						]
					]
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					SNew(SHorizontalBox)
					.Visibility(SectionItem.bIsChunkSection ? EVisibility::Visible : EVisibility::Collapsed)
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(STextBlock)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text(LOCTEXT("SectionListItemChunkSectionValueLabel", "Chunked"))
					]
				]
			];
	}

private:

	FSectionItemView(const FSectionListItem& InSection,
		FOnSectionChanged& InOnSectionChanged,
		FOnGenerateWidgetsForSection& InOnGenerateNameWidgets,
		FOnGenerateWidgetsForSection& InOnGenerateSectionWidgets,
		FOnResetSectionToDefaultClicked& InOnResetToDefaultClicked,
		int32 InMultipleSectionCount,
		int32 InThumbnailSize)
		: SectionItem(InSection)
		, OnSectionChanged(InOnSectionChanged)
		, OnGenerateCustomNameWidgets(InOnGenerateNameWidgets)
		, OnGenerateCustomSectionWidgets(InOnGenerateSectionWidgets)
		, OnResetToDefaultClicked(InOnResetToDefaultClicked)
		, MultipleSectionCount(InMultipleSectionCount)
		, ThumbnailSize(InThumbnailSize)
	{

	}

	TSharedRef<SWidget> OnGetMaterialSlotNameMenuForSection()
	{
		FMenuBuilder MenuBuilder(true, NULL);

		// Add a menu item for each texture.  Clicking on the texture will display it in the content browser
		for (auto kvp : SectionItem.AvailableMaterialSlotName)
		{
			FName AvailableMaterialSlotName = kvp.Value;
			int32 AvailableMaterialSlotIndex = kvp.Key;

			FUIAction Action(FExecuteAction::CreateSP(this, &FSectionItemView::SetMaterialSlotName, AvailableMaterialSlotIndex, AvailableMaterialSlotName));

			FString MaterialSlotDisplayName;
			AvailableMaterialSlotName.ToString(MaterialSlotDisplayName);
			MaterialSlotDisplayName = TEXT("[") + FString::FromInt(kvp.Key) + TEXT("] ") + MaterialSlotDisplayName;
			MenuBuilder.AddMenuEntry(FText::FromString(MaterialSlotDisplayName), LOCTEXT("BrowseAvailableMaterialSlotName_ToolTip", "Set the material slot name for this section"), FSlateIcon(), Action);
		}

		return MenuBuilder.MakeWidget();
	}

	void SetMaterialSlotName(int32 MaterialSlotIndex, FName NewSlotName)
	{
		OnSectionChanged.ExecuteIfBound(SectionItem.LodIndex, SectionItem.SectionIndex, MaterialSlotIndex, NewSlotName);
	}

	FText GetCurrentMaterialSlotName() const
	{
		FString MaterialSlotDisplayName;
		SectionItem.MaterialSlotName.ToString(MaterialSlotDisplayName);
		FString MaterialSlotRemapString = TEXT("");
		if (SectionItem.DefaultMaterialIndex != INDEX_NONE && SectionItem.DefaultMaterialIndex != SectionItem.MaterialSlotIndex)
		{
			MaterialSlotRemapString = TEXT(" (Modified)");
		}
		MaterialSlotDisplayName = TEXT("[") + FString::FromInt(SectionItem.MaterialSlotIndex) + TEXT("] ") + MaterialSlotDisplayName + MaterialSlotRemapString;
		return FText::FromString(MaterialSlotDisplayName);
	}

	/**
	* Called when reset to base is clicked
	*/
	void OnResetToBaseClicked(TSharedRef<IPropertyHandle> PropertyHandle)
	{
		OnResetToDefaultClicked.ExecuteIfBound(SectionItem.LodIndex, SectionItem.SectionIndex);
	}

private:
	FSectionListItem SectionItem;
	FOnSectionChanged OnSectionChanged;
	FOnGenerateWidgetsForSection OnGenerateCustomNameWidgets;
	FOnGenerateWidgetsForSection OnGenerateCustomSectionWidgets;
	FOnResetSectionToDefaultClicked OnResetToDefaultClicked;
	int32 MultipleSectionCount;
	int32 ThumbnailSize;
};


FSectionList::FSectionList(IDetailLayoutBuilder& InDetailLayoutBuilder, FSectionListDelegates& InSectionListDelegates, bool bInInitiallyCollapsed, int32 InThumbnailSize, int32 InSectionsLodIndex, FName InSectionListName)
	: SectionListDelegates(InSectionListDelegates)
	, DetailLayoutBuilder(InDetailLayoutBuilder)
	, SectionListBuilder(new FSectionListBuilder(InThumbnailSize))
	, bInitiallyCollapsed(bInInitiallyCollapsed)
	, SectionListName(InSectionListName)
	, ThumbnailSize(InThumbnailSize)
	, SectionsLodIndex(InSectionsLodIndex)
{
}

void FSectionList::OnDisplaySectionsForLod(int32 LodIndex)
{
	// We now want to display all the materials in the element
	ExpandedSlots.Add(LodIndex);

	SectionListBuilder->Empty();
	SectionListDelegates.OnGetSections.ExecuteIfBound(*SectionListBuilder);

	OnRebuildChildren.ExecuteIfBound();
}

void FSectionList::OnHideSectionsForLod(int32 SlotIndex)
{
	// No longer want to expand the element
	ExpandedSlots.Remove(SlotIndex);

	// regenerate the Sections
	SectionListBuilder->Empty();
	SectionListDelegates.OnGetSections.ExecuteIfBound(*SectionListBuilder);

	OnRebuildChildren.ExecuteIfBound();
}


void FSectionList::Tick(float DeltaTime)
{
	// Check each Section to see if its still valid.  This allows the Section list to stay up to date when Sections are changed out from under us
	if (SectionListDelegates.OnGetSections.IsBound())
	{
		// Whether or not to refresh the Section list
		bool bRefrestSectionList = false;

		// Get the current list of Sections from the user
		SectionListBuilder->Empty();
		SectionListDelegates.OnGetSections.ExecuteIfBound(*SectionListBuilder);

		if (SectionListBuilder->GetNumSections() != DisplayedSections.Num())
		{
			// The array sizes differ so we need to refresh the list
			bRefrestSectionList = true;
		}
		else
		{
			// Compare the new list against the currently displayed list
			for (int32 SectionIndex = 0; SectionIndex < SectionListBuilder->Sections.Num(); ++SectionIndex)
			{
				const FSectionListItem& Item = SectionListBuilder->Sections[SectionIndex];

				// The displayed Sections is out of date if there isn't a 1:1 mapping between the Section sets
				if (!DisplayedSections.IsValidIndex(SectionIndex) || DisplayedSections[SectionIndex] != Item)
				{
					bRefrestSectionList = true;
					break;
				}
			}
		}

		if (bRefrestSectionList)
		{
			OnRebuildChildren.ExecuteIfBound();
		}
	}
}

void FSectionList::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	NodeRow.CopyAction(FUIAction(FExecuteAction::CreateSP(this, &FSectionList::OnCopySectionList), FCanExecuteAction::CreateSP(this, &FSectionList::OnCanCopySectionList)));
	NodeRow.PasteAction(FUIAction(FExecuteAction::CreateSP(this, &FSectionList::OnPasteSectionList)));

	NodeRow.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("SectionHeaderTitle", "Sections"))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	];
}

void FSectionList::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	ViewedSections.Empty();
	DisplayedSections.Empty();
	if (SectionListBuilder->GetNumSections() > 0)
	{
		DisplayedSections = SectionListBuilder->Sections;

		SectionListBuilder->Sort();
		TArray<FSectionListItem>& Sections = SectionListBuilder->Sections;

		int32 CurrentLODIndex = INDEX_NONE;
		bool bDisplayAllSectionsInSlot = true;
		for (auto It = Sections.CreateConstIterator(); It; ++It)
		{
			const FSectionListItem& Section = *It;

			CurrentLODIndex = Section.LodIndex;

			// Display each thumbnail element unless we shouldn't display multiple Sections for one slot
			if (bDisplayAllSectionsInSlot)
			{
				FDetailWidgetRow& ChildRow = ChildrenBuilder.AddCustomRow(Section.Material.IsValid() ? FText::FromString(Section.Material->GetName()) : FText::GetEmpty());
				AddSectionItem(ChildRow, CurrentLODIndex, FSectionListItem(CurrentLODIndex, Section.SectionIndex, Section.MaterialSlotName, Section.MaterialSlotIndex, Section.OriginalMaterialSlotName, Section.AvailableMaterialSlotName, Section.Material.Get(), Section.IsSectionUsingCloth, ThumbnailSize, Section.bIsChunkSection, Section.DefaultMaterialIndex), !bDisplayAllSectionsInSlot);
			}
		}
	}
	else
	{
		FDetailWidgetRow& ChildRow = ChildrenBuilder.AddCustomRow(LOCTEXT("NoSections", "No Sections"));

		ChildRow
			[
				SNew(SBox)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NoSections", "No Sections"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			];
	}
}

bool FSectionList::OnCanCopySectionList() const
{
	if (SectionListDelegates.OnCanCopySectionList.IsBound())
	{
		return SectionListDelegates.OnCanCopySectionList.Execute();
	}

	return false;
}

void FSectionList::OnCopySectionList()
{
	if (SectionListDelegates.OnCopySectionList.IsBound())
	{
		SectionListDelegates.OnCopySectionList.Execute();
	}
}

void FSectionList::OnPasteSectionList()
{
	if (SectionListDelegates.OnPasteSectionList.IsBound())
	{
		SectionListDelegates.OnPasteSectionList.Execute();
	}
}

bool FSectionList::OnCanCopySectionItem(int32 LODIndex, int32 SectionIndex) const
{
	if (SectionListDelegates.OnCanCopySectionItem.IsBound())
	{
		return SectionListDelegates.OnCanCopySectionItem.Execute(LODIndex, SectionIndex);
	}

	return false;
}

void FSectionList::OnCopySectionItem(int32 LODIndex, int32 SectionIndex)
{
	if (SectionListDelegates.OnCopySectionItem.IsBound())
	{
		SectionListDelegates.OnCopySectionItem.Execute(LODIndex, SectionIndex);
	}
}

void FSectionList::OnPasteSectionItem(int32 LODIndex, int32 SectionIndex)
{
	if (SectionListDelegates.OnPasteSectionItem.IsBound())
	{
		SectionListDelegates.OnPasteSectionItem.Execute(LODIndex, SectionIndex);
	}
}

void FSectionList::OnEnableSectionItem(int32 LodIndex, int32 SectionIndex, bool bEnable)
{
	SectionListDelegates.OnEnableSectionItem.ExecuteIfBound(LodIndex, SectionIndex, bEnable);
}

void FSectionList::AddSectionItem(FDetailWidgetRow& Row, int32 LodIndex, const struct FSectionListItem& Item, bool bDisplayLink)
{
	uint32 NumSections = SectionListBuilder->GetNumSections(LodIndex);

	bool bIsChunkSection = Item.bIsChunkSection;
	TSharedRef<FSectionItemView> NewView = FSectionItemView::Create(Item, SectionListDelegates.OnSectionChanged, SectionListDelegates.OnGenerateCustomNameWidgets, SectionListDelegates.OnGenerateCustomSectionWidgets, SectionListDelegates.OnResetSectionToDefaultClicked, NumSections, ThumbnailSize);

	TSharedPtr<SWidget> RightSideContent;
	if (bDisplayLink)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("NumSections"), NumSections);

		RightSideContent =
			SNew(SBox)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Top)
			[
				SNew(SHyperlink)
				.TextStyle(FAppStyle::Get(), "MaterialList.HyperlinkStyle")
				.Text(FText::Format(LOCTEXT("DisplayAllSectionLinkText", "Display {NumSections} Sections"), Arguments))
				.ToolTipText(LOCTEXT("DisplayAllSectionLink_ToolTip", "Display all Sections. Drag and drop a Section here to replace all Sections."))
				.OnNavigate(this, &FSectionList::OnDisplaySectionsForLod, LodIndex)
			];
	}
	else
	{
		RightSideContent = NewView->CreateValueContent(DetailLayoutBuilder.GetThumbnailPool());
		ViewedSections.Add(NewView);
	}

	//Chunk section cannot be copy enable or disable, do the operation on the parent section
	if (!bIsChunkSection)
	{
		Row.CopyAction(FUIAction(FExecuteAction::CreateSP(this, &FSectionList::OnCopySectionItem, LodIndex, Item.SectionIndex), FCanExecuteAction::CreateSP(this, &FSectionList::OnCanCopySectionItem, LodIndex, Item.SectionIndex)));
		Row.PasteAction(FUIAction(FExecuteAction::CreateSP(this, &FSectionList::OnPasteSectionItem, LodIndex, Item.SectionIndex)));

		if (SectionListDelegates.OnEnableSectionItem.IsBound())
		{
			Row.AddCustomContextMenuAction(FUIAction(FExecuteAction::CreateSP(this, &FSectionList::OnEnableSectionItem, LodIndex, Item.SectionIndex, true)), LOCTEXT("SectionItemContexMenu_Enable", "Enable"));
			Row.AddCustomContextMenuAction(FUIAction(FExecuteAction::CreateSP(this, &FSectionList::OnEnableSectionItem, LodIndex, Item.SectionIndex, false)), LOCTEXT("SectionItemContexMenu_Disable", "Disable"));
		}
	}

	Row.RowTag(SectionListName);
	Row.NameContent()
		[
			NewView->CreateNameContent()
		]
	.ValueContent()
		.MinDesiredWidth(250.0f)
		.MaxDesiredWidth(0.0f) // no maximum
		[
			RightSideContent.ToSharedRef()
		];
}


void SMaterialSlotWidget::Construct(const FArguments& InArgs, int32 SlotIndex, bool bIsMaterialUsed)
{
	TSharedPtr<SHorizontalBox> SlotNameBox;

	TSharedRef<SWidget> DeleteButton =
		PropertyCustomizationHelpers::MakeDeleteButton(
			InArgs._OnDeleteMaterialSlot,
			LOCTEXT("CustomNameMaterialNotUsedDeleteTooltip", "Delete this material slot"),
			InArgs._CanDeleteMaterialSlot);

	DeleteButton->SetVisibility(InArgs._DeleteMaterialSlotVisibility);

	ChildSlot
	[
		SAssignNew(SlotNameBox, SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("MaterialArrayNameLabelStringKey", "Slot"))
			]
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(5.0f, 0.0f, 0.0f,0.0f)
		[
			SNew(SBox)
			.VAlign(VAlign_Center)
			.MinDesiredWidth(160.0f)
			[
				SNew(SEditableTextBox)
				.Text(InArgs._MaterialName)
				.IsReadOnly(InArgs._IsMaterialSlotNameReadOnly)
				.OnTextChanged(InArgs._OnMaterialNameChanged)
				.OnTextCommitted(InArgs._OnMaterialNameCommitted)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		]
	];

	
	if (bIsMaterialUsed)
	{
		DeleteButton->SetEnabled(false);
	}
	

	SlotNameBox->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2)
		[
			DeleteButton
		];
}

#undef LOCTEXT_NAMESPACE
