// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyEditorArchetypePolicy.h"
#include "PropertyNode.h"

namespace PropertyEditorPolicy
{
	UObject* GetArchetype(const UObject* Object)
	{
		return FPropertyNode::GetArchetype(Object);
	}

	void RegisterArchetypePolicy(IArchetypePolicy* Policy)
	{
		FPropertyNode::RegisterArchetypePolicy(Policy);
	}

	void UnregisterArchetypePolicy(IArchetypePolicy* Policy)
	{
		FPropertyNode::UnregisterArchetypePolicy(Policy);
	}
}