// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyEditorClipboard.h"

#include "HAL/PlatformApplicationMisc.h"
#include "PropertyNode.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"


namespace UE::PropertyEditor::Internal
{
	constexpr const TCHAR* ClipboardMapKeyName = TEXT("Tagged");

	bool IsJsonString(const FString& InStr)
	{
		if (InStr.IsEmpty())
		{
			return false;
		}

		// Str has contents but isn't necessarily json
		TSharedPtr<FJsonObject> UnusedJsonObject;
		const TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(*InStr);
		return FJsonSerializer::Deserialize(JsonReader, UnusedJsonObject); // This will return false if not parsed
	}

	bool TryWriteClipboard(const TMap<FName, FString>& InTaggedClipboardPairs, FString& OutClipboardStr)
	{
		const TSharedPtr<FJsonObject> RootJsonObject = MakeShared<FJsonObject>();
		TArray<TSharedPtr<FJsonValue>> TaggedValuesJson;
		TaggedValuesJson.Reserve(InTaggedClipboardPairs.Num());
		for (const TPair<FName, FString>& TaggedPair : InTaggedClipboardPairs)
		{
			TArray<TSharedPtr<FJsonValue>> TaggedPairJson;
			TaggedPairJson.Reserve(2);

			TaggedPairJson.Add(MakeShared<FJsonValueString>(TaggedPair.Key.ToString()));
			TaggedPairJson.Add(MakeShared<FJsonValueString>(TaggedPair.Value));

			TaggedValuesJson.Add(MakeShared<FJsonValueArray>(MoveTemp(TaggedPairJson)));
		}
		RootJsonObject->SetArrayField(ClipboardMapKeyName, TaggedValuesJson);

		const TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&OutClipboardStr);
		return FJsonSerializer::Serialize(RootJsonObject.ToSharedRef(), JsonWriter);
	}

	bool TryParseClipboard(const FString& InClipboardStr, TMap<FName, FString>& OutTaggedClipboard)
	{
		TSharedPtr<FJsonObject> RootJsonObject;
		const TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(InClipboardStr);
		if (FJsonSerializer::Deserialize(JsonReader, RootJsonObject))
		{
			const TArray<TSharedPtr<FJsonValue>>* TaggedValueArray = nullptr;
			if (RootJsonObject->TryGetArrayField(ClipboardMapKeyName, TaggedValueArray))
			{
				OutTaggedClipboard.Reserve(TaggedValueArray->Num());				
				for (const TSharedPtr<FJsonValue>& TaggedValueEntry : *TaggedValueArray)
				{
					const TArray<TSharedPtr<FJsonValue>>* TaggedValuePair = nullptr;
					if (TaggedValueEntry->TryGetArray(TaggedValuePair))
					{
						// Expects at least key, value entries
						if (TaggedValuePair->Num() < 2)
						{
							continue;
						}
						
						FString TagName = (*TaggedValuePair)[0]->AsString();
						FString TagValue = (*TaggedValuePair)[1]->AsString();

						OutTaggedClipboard.Add(FName(TagName), TagValue);
					}
				}
				return true;
			}
		}

		return false;
	}
}

TSharedRef<FPropertyEditorClipboard> FPropertyEditorClipboard::Get()
{
	static TSharedRef<FPropertyEditorClipboard> Instance = MakeShared<FPropertyEditorClipboard>();
	return Instance;
}

void FPropertyEditorClipboard::ClipboardCopy(const TCHAR* Str)
{
	FPropertyEditorClipboard::Get()->ClipboardContents.Reset();
	FPlatformApplicationMisc::ClipboardCopy(Str);
}

static bool EncodeClipboardCopy(const TMap<FName, FString>& InTaggedClipboard)
{
	FString ClipboardStr;
	if (UE::PropertyEditor::Internal::TryWriteClipboard(InTaggedClipboard, ClipboardStr))
	{
		// Tagged content written, now encode and write to clipboard
		FPlatformApplicationMisc::ClipboardCopy(*ClipboardStr);

		// Encoding (to json) successful
		return true;
	}
	
	// Encoding failed - silently fail and do regular copy
	UE_LOGF(LogPropertyNode, Warning, "Failed to encode tagged clipboard as json.")

	// Only if default exists (it always should, caller!)
	if (const FString* DefaultClipboardStr = InTaggedClipboard.Find(NAME_None))
	{
		FPlatformApplicationMisc::ClipboardCopy(**DefaultClipboardStr);
	}
	return false;
}

void FPropertyEditorClipboard::ClipboardCopy(
	const TCHAR* Str, 
	FName Tag)
{
	if (Tag == NAME_None)
	{
		return ClipboardCopy(Str);
	}

	TMap<FName, FString>& TaggedContents = FPropertyEditorClipboard::Get()->ClipboardContents;
	TaggedContents.Reset();
	// Populate both default and tagged, makes it easy to check OnPaste if the contents is intended to be "specialized"
	TaggedContents.Add(NAME_None, Str);
	TaggedContents.Add(Tag, Str);

	EncodeClipboardCopy(TaggedContents);
}

void FPropertyEditorClipboard::ClipboardCopy(
	TUniqueFunction<void(TMap<FName, FString>&)>&& TagMappingFunc)
{
	TMap<FName, FString>& TaggedContents = FPropertyEditorClipboard::Get()->ClipboardContents;
	TaggedContents.Reset();

	ClipboardAppend(MoveTemp(TagMappingFunc));
}

void FPropertyEditorClipboard::ClipboardAppend(TUniqueFunction<void(TMap<FName, FString>&)>&& TagMappingFunc)
{
	TMap<FName, FString>& TaggedContents = FPropertyEditorClipboard::Get()->ClipboardContents;
	
	TagMappingFunc(TaggedContents);

    EncodeClipboardCopy(TaggedContents);
}

bool FPropertyEditorClipboard::ClipboardPaste(FString& Dest)
{
	FPlatformApplicationMisc::ClipboardPaste(Dest);
	return true;
}

bool FPropertyEditorClipboard::ClipboardPaste(FString& Dest, FName Tag)
{
	if (Tag != NAME_None)
	{
		// Check for tagged entry, return true if found
		if (FString* ClipboardText = FPropertyEditorClipboard::Get()->ClipboardContents.Find(Tag))
		{
			Dest = *ClipboardText;
			return true;
		}
	}

	// Otherwise perform default operation
	return ClipboardPaste(Dest);
}
