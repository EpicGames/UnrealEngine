// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Templates/Function.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/NameTypes.h"

/** Utilities to encode and decode the clipboard. */
namespace UE::PropertyEditor::Internal
{
	bool IsJsonString(const FString& InStr);
	bool TryWriteClipboard(const TMap<FName, FString>& InTaggedClipboardPairs, FString& OutClipboardStr);
	bool TryParseClipboard(const FString& InClipboardStr, TMap<FName, FString>& OutTaggedClipboard);

	/** Contains parsed clipboard data, used to cache subsequent validity checks. */
	struct FClipboardData
	{
		/** Clipboard content string. */
		TOptional<FString> Content;

		/** Applicability of the clipboard content to this row. */ 
		bool bIsApplicable = false;

		/** Previous PropertyHandle count, used to validate applicability. */
		int32 PreviousPropertyHandleNum = INDEX_NONE;

		/** Deserialized property values (as text). */
		TMap<FName, FString> PropertyValues;

		/** List of unique property names being pasted. */
		TArray<FName> PropertyNames;

		/** Resets/invalidates data. */
		void Reset()
		{
			Content.Reset();
			bIsApplicable = false;
			PreviousPropertyHandleNum = INDEX_NONE;
			PropertyValues.Reset();
			PropertyNames.Reset();
		}

		void Reserve(const int32& InNum)
		{
			PropertyValues.Reserve(InNum);
			PropertyNames.Reserve(InNum);
		}
	};
}
