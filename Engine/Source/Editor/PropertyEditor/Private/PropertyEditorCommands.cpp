// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyEditorCommands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "PropertyEditorCommands"

FPropertyEditorCommands::FPropertyEditorCommands()
	: TCommands<FPropertyEditorCommands>
	(
		TEXT("PropertyEditor"),
		LOCTEXT("PropertyEditorContext", "Property Editor"),
		NAME_None,
		FAppStyle::GetAppStyleSetName()
	)
{
}

void FPropertyEditorCommands::RegisterCommands()
{
	UI_COMMAND(ExpandAll, "Expand All", "Expand all elements in this details view", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::X));
	UI_COMMAND(CollapseAll, "Collapse All", "Collapse all elements in this details view", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::C));
}

#undef LOCTEXT_NAMESPACE
