// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyEditorCopyPaste.h"
#include "PropertyEditorCopyPastePrivate.h"
#include "PropertyHandleImpl.h"
#include "PropertyNode.h"

namespace UE::PropertyEditor
{
	FString GetPropertyPath(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	{
		return Private::GetPropertyPath(
			[InPropertyHandle] { return InPropertyHandle; },
			{});
	}

	bool TagMatchesProperty(const FString& InTag, const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	{
		if (InTag.IsEmpty())
		{
			return true;
		}

		const FString PropertyPath = UE::PropertyEditor::GetPropertyPath(InPropertyHandle);
		
		// If tag is specified, ensure that it matches the property (by path)
		return InTag.Equals(PropertyPath);
	}

	namespace Private
	{
		FString GetPropertyPath(
			TUniqueFunction<const TSharedPtr<IPropertyHandle>()>&& GetPropertyHandle,
			TUniqueFunction<const TSharedPtr<FPropertyNode>()>&& GetPropertyNode)
		{
			const TSharedPtr<IPropertyHandle> PropertyHandle = GetPropertyHandle();
			if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
			{
				FString PropertyPath = PropertyHandle->GeneratePathToProperty();
				if (!PropertyPath.IsEmpty())
				{
					return PropertyPath;
				}
			}

			TSharedPtr<FPropertyNode> PropertyNode = nullptr;
			if (GetPropertyNode)
			{
				PropertyNode = GetPropertyNode();
			}

			if (!PropertyNode && PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
			{
				PropertyNode = StaticCastSharedPtr<FPropertyHandleBase>(PropertyHandle)->GetPropertyNode();			
			}

			const static FString EmptyString = TEXT("");
			return PropertyNode.IsValid() ? PropertyNode->GetPropertyPath() : EmptyString;
		}
	}
}
