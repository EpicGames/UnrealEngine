// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "PropertyHandle.h"
#include "PropertyNode.h"

/** Utilities related to copy/paste. */
namespace UE::PropertyEditor::Private
{
	/** Get property path from handle, or node if the handle isn't valid. */
	[[nodiscard]] PROPERTYEDITOR_API FString GetPropertyPath(
		TUniqueFunction<const TSharedPtr<IPropertyHandle>()>&& GetPropertyHandle,
		TUniqueFunction<const TSharedPtr<FPropertyNode>()>&& GetPropertyNode);
}
