// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyEditorDelegates.h"

#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "ObjectPropertyNode.h"
#include "PropertyHandleImpl.h"
#include "PropertyNode.h"

class FProperty;

FPropertyAndParent::FPropertyAndParent(const TSharedRef<IPropertyHandle>& InPropertyHandle) :
	Property(*InPropertyHandle->GetProperty())
{
	Initialize(StaticCastSharedRef<FPropertyHandleBase>(InPropertyHandle)->GetPropertyNode().ToSharedRef());
}

FPropertyAndParent::FPropertyAndParent(const TSharedRef<FPropertyNode>& InPropertyNode) :
	Property(*InPropertyNode->GetProperty())
{
	Initialize(InPropertyNode);
}

void FPropertyAndParent::Initialize(const TSharedRef<FPropertyNode>& InPropertyNode)
{
	checkf(InPropertyNode->GetProperty() != nullptr, TEXT("Creating an FPropertyAndParent with a null property!"));

	FObjectPropertyNode* ObjectNode = InPropertyNode->FindObjectItemParent();
	if (ObjectNode)
	{
		for (int32 ObjectIndex = 0; ObjectIndex < ObjectNode->GetNumObjects(); ++ObjectIndex)
		{
			Objects.Add(ObjectNode->GetUObject(ObjectIndex));
		}
	}

	ArrayIndex = InPropertyNode->GetArrayIndex();

	TSharedPtr<FPropertyNode> ParentNode = InPropertyNode->GetParentNodeSharedPtr();
	while (ParentNode.IsValid())
	{
		const FProperty* ParentProperty = ParentNode->GetProperty();
		if (ParentProperty != nullptr)
		{
			ParentProperties.Add(ParentProperty);
			ParentArrayIndices.Add(ParentNode->GetArrayIndex());
		}

		ParentNode = ParentNode->GetParentNodeSharedPtr();
	}
}