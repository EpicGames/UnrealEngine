// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyEditorEditConstPolicy.h"
#include "PropertyNode.h"

namespace PropertyEditorPolicy
{
	void RegisterEditConstPolicy(IEditConstPolicy* Policy)
	{
		FPropertyNode::RegisterEditConstPolicy(Policy);
	}

	void UnregisterEditConstPolicy(IEditConstPolicy* Policy)
	{
		FPropertyNode::UnregisterEditConstPolicy(Policy);
	}

	bool IsPropertyEditConst(const FEditPropertyChain& PropertyChain, UObject* Object)
	{
		return FPropertyNode::IsPropertyEditConst(PropertyChain, Object);
	}

	bool IsPropertyEditConst(const FProperty* Property, UObject* Object)
	{
		return FPropertyNode::IsPropertyEditConst(Property, Object);
	}
}