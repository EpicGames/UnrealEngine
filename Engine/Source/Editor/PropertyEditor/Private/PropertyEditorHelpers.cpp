// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyEditorHelpers.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SDeferredToolTip.h"
#include "IDocumentation.h"

#include "PropertyHandleImpl.h"
#include "PropertyNode.h"

#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "UserInterface/PropertyEditor/SPropertyEditor.h"
#include "UserInterface/PropertyEditor/SPropertyEditorNumeric.h"
#include "UserInterface/PropertyEditor/SPropertyEditorArray.h"
#include "UserInterface/PropertyEditor/SPropertyEditorColor.h"
#include "UserInterface/PropertyEditor/SPropertyEditorCombo.h"
#include "UserInterface/PropertyEditor/SPropertyEditorEditInline.h"
#include "UserInterface/PropertyEditor/SPropertyEditorText.h"
#include "UserInterface/PropertyEditor/SPropertyEditorBool.h"
#include "UserInterface/PropertyEditor/SPropertyEditorArrayItem.h"
#include "UserInterface/PropertyEditor/SPropertyEditorTitle.h"
#include "UserInterface/PropertyEditor/SPropertyEditorDateTime.h"
#include "UserInterface/PropertyEditor/SPropertyEditorAsset.h"
#include "UserInterface/PropertyEditor/SPropertyEditorClass.h"
#include "UserInterface/PropertyEditor/SPropertyEditorStruct.h"
#include "UserInterface/PropertyEditor/SPropertyEditorSet.h"
#include "UserInterface/PropertyEditor/SPropertyEditorMap.h"
#include "UserInterface/PropertyEditor/SPropertyEditorOptional.h"

#include "Kismet2/KismetEditorUtilities.h"
#include "EditorClassUtils.h"
#include "Engine/Selection.h"

#include "UObject/PropertyOptional.h"

#define LOCTEXT_NAMESPACE "PropertyEditor"

void SPropertyNameWidget::Construct( const FArguments& InArgs, TSharedPtr<FPropertyEditor> InPropertyEditor )
{
	PropertyEditor = InPropertyEditor;

	TWeakPtr<FPropertyEditor> WeakPropertyEditor = PropertyEditor;
	// Note : ToolTipText has to be captured immediately because FPropertyHandleBase::CreatePropertyNameWidget has a ToolTipOverride that cannot be deferred since 
	//  it's restored to FText::Empty right after SPropertyNameWidget::Construct :
	auto CreateDocumentationToolTipDeferred = [WeakPropertyEditor, ToolTipText = PropertyEditor->GetToolTipText()]() 
	{
		if (TSharedPtr<FPropertyEditor> PropertyEditorPinned = WeakPropertyEditor.Pin())
		{
			return IDocumentation::Get()->CreateToolTip(ToolTipText, NULL, PropertyEditorPinned->GetDocumentationLink(), PropertyEditorPinned->GetDocumentationExcerptName());
		};

		return SNew(SToolTip)
		.Text(LOCTEXT("EditorDestroyedToolTip", "Invalid ToolTip, source editor closed."));
	};
	TSharedRef<SDeferredToolTip> DeferredDocumentationToolTip = MakeShared<SDeferredToolTip>(FOnGetDeferredToolTip::CreateLambda(CreateDocumentationToolTipDeferred));

	TSharedPtr<SHorizontalBox> HorizontalBox;
	ChildSlot
	[
		SAssignNew(HorizontalBox, SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding( FMargin( 0.0f, 1.0f, 0.0f, 1.0f) )
		.FillWidth(1)
		[
			SNew(SBorder)
			.BorderImage_Static( &PropertyEditorConstants::GetOverlayBrush, PropertyEditor.ToSharedRef() )
			.Padding( FMargin( 0.0f, 2.0f ) )
			.VAlign(VAlign_Center)
			[
				SNew( SPropertyEditorTitle, PropertyEditor.ToSharedRef() )
				.OnDoubleClicked( InArgs._OnDoubleClicked )
                .ToolTip(DeferredDocumentationToolTip)
			]
		]
	
	];
}

void SPropertyValueWidget::Construct( const FArguments& InArgs, TSharedPtr<FPropertyEditor> PropertyEditor, TSharedPtr<IPropertyUtilities> InPropertyUtilities )
{
	MinDesiredWidth = 0.0f;
	MaxDesiredWidth = 0.0f;
	if(InArgs._InWidgetRow.IsSet())
	{
		WidgetRow = InArgs._InWidgetRow;
	}

	SetEnabled( TAttribute<bool>( PropertyEditor.ToSharedRef(), &FPropertyEditor::IsPropertyEditingEnabled ) );

	ValueEditorWidget = ConstructPropertyEditorWidget( PropertyEditor, InPropertyUtilities );

	if ( !ValueEditorWidget->GetToolTip().IsValid() )
	{
		ValueEditorWidget->SetToolTipText(TAttribute<FText>(PropertyEditor.ToSharedRef(), &FPropertyEditor::GetValueAsText));
	}


	if( InArgs._ShowPropertyButtons )
	{
		TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

		HorizontalBox->AddSlot()
		.FillWidth(1) // Fill the entire width if possible
		.VAlign(VAlign_Center)	
		[
			ValueEditorWidget.ToSharedRef()
		];

		TArray< TSharedRef<SWidget> > RequiredButtons;
		PropertyEditorHelpers::MakeRequiredPropertyButtons( PropertyEditor.ToSharedRef(), /*OUT*/RequiredButtons );

		for( int32 ButtonIndex = 0; ButtonIndex < RequiredButtons.Num(); ++ButtonIndex )
		{
			HorizontalBox->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding( 2.0f, 0.0f )
				[ 
					RequiredButtons[ButtonIndex]
				];
		}

		ChildSlot
		[
			HorizontalBox
		];

	}
	else
	{
		ChildSlot
		.VAlign(VAlign_Center)	
		[
			ValueEditorWidget.ToSharedRef()
		];
	}


}

TSharedRef<SWidget> SPropertyValueWidget::ConstructPropertyEditorWidget( TSharedPtr<FPropertyEditor>& PropertyEditor, TSharedPtr<IPropertyUtilities> InPropertyUtilities )
{
	const TSharedRef<FPropertyEditor> PropertyEditorRef = PropertyEditor.ToSharedRef();
	//const TSharedRef<IPropertyUtilities> PropertyUtilitiesRef = InPropertyUtilities.ToSharedRef();

	const TSharedRef< FPropertyNode > PropertyNode = PropertyEditorRef->GetPropertyNode();
	FProperty* Property = PropertyNode->GetProperty();
	
	FSlateFontInfo FontStyle = FAppStyle::GetFontStyle( PropertyEditorConstants::PropertyFontStyle );
	TSharedPtr<SWidget> PropertyWidget; 
	if( Property )
	{
		// ORDER MATTERS: first widget type to support the property node wins!
		if ( SPropertyEditorArray::Supports(PropertyEditorRef) )
		{
			TSharedRef<SPropertyEditorArray> ArrayWidget = 
				SAssignNew( PropertyWidget, SPropertyEditorArray, PropertyEditorRef )
				.Font( FontStyle );

			ArrayWidget->GetDesiredWidth( MinDesiredWidth, MaxDesiredWidth );
		}
		else if ( SPropertyEditorSet::Supports(PropertyEditorRef) )
		{
			TSharedRef<SPropertyEditorSet> SetWidget =
				SAssignNew( PropertyWidget, SPropertyEditorSet, PropertyEditorRef )
				.Font( FontStyle );

			SetWidget->GetDesiredWidth( MinDesiredWidth, MaxDesiredWidth );
		}
		else if ( SPropertyEditorMap::Supports(PropertyEditorRef) )
		{
			TSharedRef<SPropertyEditorMap> MapWidget =
				SAssignNew( PropertyWidget, SPropertyEditorMap, PropertyEditorRef )
				.Font( FontStyle );

			MapWidget->GetDesiredWidth( MinDesiredWidth, MaxDesiredWidth );
		}
		else if (SPropertyEditorOptional::Supports(PropertyEditorRef))
		{
			TSharedRef<SPropertyEditorOptional> OptionalWidget =
				SAssignNew(PropertyWidget, SPropertyEditorOptional, PropertyEditorRef, InPropertyUtilities.ToSharedRef())
				.Font(FontStyle);

			OptionalWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if (SPropertyEditorClass::Supports(PropertyEditorRef))
		{
			static TArray<TSharedRef<class IClassViewerFilter>> NullFilters;
			TSharedRef<SPropertyEditorClass> ClassWidget =
				SAssignNew(PropertyWidget, SPropertyEditorClass, PropertyEditorRef)
				.Font(FontStyle)
				.ClassViewerFilters(InPropertyUtilities.IsValid() ? InPropertyUtilities->GetClassViewerFilters() : NullFilters);

			ClassWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if (SPropertyEditorStruct::Supports(PropertyEditorRef))
		{
			TSharedRef<SPropertyEditorStruct> StructWidget =
				SAssignNew(PropertyWidget, SPropertyEditorStruct, PropertyEditorRef)
				.Font(FontStyle);

			StructWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if ( SPropertyEditorAsset::Supports( PropertyEditorRef ) )
		{
			// SPropertyEditorAsset has its own copy & paste that need to be bound to the widget row's
			TSharedRef<SPropertyEditorAsset> AssetWidget = 
				SAssignNew( PropertyWidget, SPropertyEditorAsset, PropertyEditorRef )
				.ThumbnailPool( InPropertyUtilities.IsValid() ? InPropertyUtilities->GetThumbnailPool() : nullptr )
				.InWidgetRow(WidgetRow);
			
			AssetWidget->GetDesiredWidth( MinDesiredWidth, MaxDesiredWidth );
		}
		else if ( SPropertyEditorNumeric<float>::Supports( PropertyEditorRef ) )
		{
			auto NumericWidget = 
				SAssignNew( PropertyWidget, SPropertyEditorNumeric<float>, PropertyEditorRef )
				.Font( FontStyle );

			NumericWidget->GetDesiredWidth( MinDesiredWidth, MaxDesiredWidth );
		}
		else if ( SPropertyEditorNumeric<double>::Supports( PropertyEditorRef ) )
		{
			auto NumericWidget =
				SAssignNew( PropertyWidget, SPropertyEditorNumeric<double>, PropertyEditorRef )
				.Font( FontStyle );

			NumericWidget->GetDesiredWidth( MinDesiredWidth, MaxDesiredWidth );
		}
		else if (SPropertyEditorNumeric<int8>::Supports(PropertyEditorRef))
		{
			auto NumericWidget =
				SAssignNew(PropertyWidget, SPropertyEditorNumeric<int8>, PropertyEditorRef)
				.Font(FontStyle);

			NumericWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if (SPropertyEditorNumeric<int16>::Supports(PropertyEditorRef))
		{
			auto NumericWidget =
				SAssignNew(PropertyWidget, SPropertyEditorNumeric<int16>, PropertyEditorRef)
				.Font(FontStyle);

			NumericWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if ( SPropertyEditorNumeric<int32>::Supports( PropertyEditorRef ) )
		{
			auto NumericWidget = 
				SAssignNew( PropertyWidget, SPropertyEditorNumeric<int32>, PropertyEditorRef )
				.Font( FontStyle );

			NumericWidget->GetDesiredWidth( MinDesiredWidth, MaxDesiredWidth );
		}
		else if (SPropertyEditorNumeric<int64>::Supports(PropertyEditorRef))
		{
			auto NumericWidget =
				SAssignNew(PropertyWidget, SPropertyEditorNumeric<int64>, PropertyEditorRef)
				.Font(FontStyle);

			NumericWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if ( SPropertyEditorNumeric<uint8>::Supports( PropertyEditorRef ) )
		{
			auto NumericWidget = 
				SAssignNew( PropertyWidget, SPropertyEditorNumeric<uint8>, PropertyEditorRef )
				.Font( FontStyle );

			NumericWidget->GetDesiredWidth( MinDesiredWidth, MaxDesiredWidth );
		}
		else if (SPropertyEditorNumeric<uint16>::Supports(PropertyEditorRef))
		{
			auto NumericWidget =
				SAssignNew(PropertyWidget, SPropertyEditorNumeric<uint16>, PropertyEditorRef)
				.Font(FontStyle);

			NumericWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if (SPropertyEditorNumeric<uint32>::Supports(PropertyEditorRef))
		{
			auto NumericWidget =
				SAssignNew(PropertyWidget, SPropertyEditorNumeric<uint32>, PropertyEditorRef)
				.Font(FontStyle);

			NumericWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if (SPropertyEditorNumeric<uint64>::Supports(PropertyEditorRef))
		{
			auto NumericWidget =
				SAssignNew(PropertyWidget, SPropertyEditorNumeric<uint64>, PropertyEditorRef)
				.Font(FontStyle);

			NumericWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if ( SPropertyEditorCombo::Supports( PropertyEditorRef ) )
		{
			FPropertyComboBoxArgs ComboArgs;
			ComboArgs.Font = FontStyle;

			TSharedRef<SPropertyEditorCombo> ComboWidget = 
				SAssignNew( PropertyWidget, SPropertyEditorCombo, PropertyEditorRef )
				.ComboArgs( ComboArgs );

			ComboWidget->GetDesiredWidth( MinDesiredWidth, MaxDesiredWidth );
		}
		else if ( SPropertyEditorEditInline::Supports( PropertyEditorRef ) )
		{
			TSharedRef<SPropertyEditorEditInline> EditInlineWidget = 
				SAssignNew( PropertyWidget, SPropertyEditorEditInline, PropertyEditorRef )
				.Font( FontStyle );

			EditInlineWidget->GetDesiredWidth( MinDesiredWidth, MaxDesiredWidth );
		}
		else if ( SPropertyEditorText::Supports( PropertyEditorRef ) )
		{
			TSharedRef<SPropertyEditorText> TextWidget = 
				SAssignNew( PropertyWidget, SPropertyEditorText, PropertyEditorRef )
				.Font( FontStyle );

			TextWidget->GetDesiredWidth( MinDesiredWidth, MaxDesiredWidth );
		}
		else if ( SPropertyEditorBool::Supports( PropertyEditorRef ) )
		{
			TSharedRef<SPropertyEditorBool> BoolWidget = 
				SAssignNew( PropertyWidget, SPropertyEditorBool, PropertyEditorRef );

			BoolWidget->GetDesiredWidth( MinDesiredWidth, MaxDesiredWidth );

		}
		else if ( SPropertyEditorColor::Supports( PropertyEditorRef ) )
		{
			TSharedRef<SPropertyEditorColor> ColorWidget = 
				SAssignNew( PropertyWidget, SPropertyEditorColor, PropertyEditorRef, InPropertyUtilities.ToSharedRef() );
		}
		else if ( SPropertyEditorArrayItem::Supports( PropertyEditorRef ) )
		{
			TSharedRef<SPropertyEditorArrayItem> ArrayItemWidget = 
				SAssignNew( PropertyWidget, SPropertyEditorArrayItem, PropertyEditorRef )
				.Font( FontStyle );

			ArrayItemWidget->GetDesiredWidth( MinDesiredWidth, MaxDesiredWidth );
		}
		else if ( SPropertyEditorDateTime::Supports( PropertyEditorRef ) )
		{
			TSharedRef<SPropertyEditorDateTime> DateTimeWidget =
				SAssignNew( PropertyWidget, SPropertyEditorDateTime, PropertyEditorRef )
				.Font( FontStyle );
		}
	}

	if( !PropertyWidget.IsValid() )
	{
		TSharedRef<SPropertyEditor> BasePropertyEditorWidget = 
			SAssignNew( PropertyWidget, SPropertyEditor, PropertyEditorRef )
			.Font( FontStyle );

		BasePropertyEditorWidget->GetDesiredWidth( MinDesiredWidth, MaxDesiredWidth );
	}

	return PropertyWidget.ToSharedRef();
}

void SEditConditionWidget::Construct( const FArguments& Args )
{
	EditConditionValue = Args._EditConditionValue;
	OnEditConditionValueChanged = Args._OnEditConditionValueChanged;

	ChildSlot
	[
		// Some properties become irrelevant depending on the value of other properties.
		// We prevent the user from editing those properties by disabling their widgets.
		// This is a shortcut for toggling the property that disables us.
		SNew(SCheckBox)
		.OnCheckStateChanged(this, &SEditConditionWidget::OnEditConditionCheckChanged)
		.IsChecked(this, &SEditConditionWidget::OnGetEditConditionCheckState)
		.Visibility(this, &SEditConditionWidget::GetVisibility)
	];
}

EVisibility SEditConditionWidget::GetVisibility() const
{
	return HasEditConditionToggle() ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SEditConditionWidget::HasEditConditionToggle() const
{
	return OnEditConditionValueChanged.IsBound();
}

void SEditConditionWidget::OnEditConditionCheckChanged( ECheckBoxState CheckState )
{
	checkSlow(HasEditConditionToggle());

	FScopedTransaction EditConditionChangedTransaction(LOCTEXT("UpdatedEditConditionFmt", "Edit Condition Changed"));
	
	OnEditConditionValueChanged.ExecuteIfBound(CheckState == ECheckBoxState::Checked);
}

ECheckBoxState SEditConditionWidget::OnGetEditConditionCheckState() const
{
	return EditConditionValue.Get() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

namespace PropertyEditorHelpers
{
	bool ShouldBeVisible(const FPropertyNode& InParentNode, const FProperty* Property)
	{
		const bool bShouldShowHiddenProperties = !!InParentNode.HasNodeFlags(EPropertyNodeFlags::ShouldShowHiddenProperties);
		if (bShouldShowHiddenProperties)
		{
			return true;
		}

		const bool bShouldShowDisableEditOnInstance = !!InParentNode.HasNodeFlags(EPropertyNodeFlags::ShouldShowDisableEditOnInstance);

		static const FName Name_InlineEditConditionToggle("InlineEditConditionToggle");
		const bool bOnlyShowAsInlineEditCondition = Property->HasMetaData(Name_InlineEditConditionToggle);
		const bool bShowIfEditableProperty = Property->HasAnyPropertyFlags(CPF_Edit);
		const bool bShowIfDisableEditOnInstance = bShouldShowDisableEditOnInstance || !Property->HasAnyPropertyFlags(CPF_DisableEditOnInstance);

		return bShowIfEditableProperty && !bOnlyShowAsInlineEditCondition && bShowIfDisableEditOnInstance;
	}

	bool IsBuiltInStructProperty( const FProperty* Property )
	{
		bool bIsBuiltIn = false;

		const FStructProperty* StructProp = CastField<const FStructProperty>( Property );
		if( StructProp && StructProp->Struct )
		{
			FName StructName = StructProp->Struct->GetFName();

			bIsBuiltIn = StructName == NAME_Rotator || 
						 StructName == NAME_Color ||  
						 StructName == NAME_LinearColor || 
						 StructName == NAME_Vector ||
						 StructName == NAME_Quat ||
						 StructName == NAME_Vector4 ||
						 StructName == NAME_Vector2D ||
						 StructName == NAME_IntPoint;
		}

		return bIsBuiltIn;
	}

	bool IsChildOfArray( const FPropertyNode& InPropertyNode )
	{
		return GetArrayParent( InPropertyNode ) != NULL;
	}

	bool IsChildOfSet(  const FPropertyNode& InPropertyNode )
	{
		return GetSetParent( InPropertyNode ) != NULL;
	}

	bool IsChildOfMap(const FPropertyNode& InPropertyNode)
	{
		return GetMapParent(InPropertyNode) != NULL;
	}

	bool IsChildOfOption(const FPropertyNode& InPropertyNode)
	{
		return GetOptionParent(InPropertyNode) != NULL;
	}

	bool IsStaticArray( const FPropertyNode& InPropertyNode )
	{
		const FProperty* NodeProperty = InPropertyNode.GetProperty();
		return NodeProperty && NodeProperty->ArrayDim != 1 && InPropertyNode.GetArrayIndex() == -1;
	}

	bool IsDynamicArray( const FPropertyNode& InPropertyNode )
	{
		const FProperty* NodeProperty = InPropertyNode.GetProperty();
		return NodeProperty && CastField<const FArrayProperty>(NodeProperty) != NULL;
	}

	bool IsOptionalProperty(const FPropertyNode& InPropertyNode)
	{
		const FProperty* NodeProperty = InPropertyNode.GetProperty();
		return NodeProperty && CastField<const FOptionalProperty>(NodeProperty) != NULL;
	}

	const FProperty* GetArrayParent( const FPropertyNode& InPropertyNode )
	{
		const FProperty* ParentProperty = InPropertyNode.GetParentNode() != NULL ? InPropertyNode.GetParentNode()->GetProperty() : NULL;
		
		if( ParentProperty )
		{
			if( (ParentProperty->IsA<FArrayProperty>()) || // dynamic array
				(InPropertyNode.GetArrayIndex() != INDEX_NONE && ParentProperty->ArrayDim > 0) ) //static array
			{
				return ParentProperty;
			}
		}

		return NULL;
	}

	const FProperty* GetSetParent( const FPropertyNode& InPropertyNode )
	{
		const FProperty* ParentProperty = InPropertyNode.GetParentNode() != NULL ? InPropertyNode.GetParentNode()->GetProperty() : NULL;

		if ( ParentProperty )
		{
			if (ParentProperty->IsA<FSetProperty>())
			{
				return ParentProperty;
			}
		}

		return NULL;
	}

	const FProperty* GetMapParent( const FPropertyNode& InPropertyNode )
	{
		const FProperty* ParentProperty = InPropertyNode.GetParentNode() != NULL ? InPropertyNode.GetParentNode()->GetProperty() : NULL;

		if (ParentProperty)
		{
			if (ParentProperty->IsA<FMapProperty>())
			{
				return ParentProperty;
			}

			//@todo: Also check a key/value node parent property?
		}

		return NULL;
	}

	const FProperty* GetOptionParent(const FPropertyNode& InPropertyNode)
	{
		const FProperty* ParentProperty = InPropertyNode.GetParentNode() != NULL ? InPropertyNode.GetParentNode()->GetProperty() : NULL;

		if (ParentProperty)
		{
			if (ParentProperty->IsA<FOptionalProperty>())
			{
				return ParentProperty;
			}

			//@todo: Also check a key/value node parent property?
		}

		return NULL;
	}

	bool IsEditInlineClassAllowed( UClass* CheckClass, bool bAllowAbstract ) 
	{
		return !CheckClass->HasAnyClassFlags(CLASS_Hidden|CLASS_HideDropDown|CLASS_Deprecated)
			&&	(bAllowAbstract || !CheckClass->HasAnyClassFlags(CLASS_Abstract));
	}

	FText GetToolTipText( const FProperty* const Property )
	{
		if( Property )
		{
			return Property->GetToolTipText();
		}

		return FText::GetEmpty();
	}

	FString GetDocumentationLink( const FProperty* const Property )
	{
		if ( Property != NULL )
		{
			UStruct* OwnerStruct = Property->GetOwnerStruct();

			if ( OwnerStruct != NULL )
			{
				return FString::Printf( TEXT("Shared/Types/%s%s"), OwnerStruct->GetPrefixCPP(), *OwnerStruct->GetName() );
			}
		}

		return TEXT("");
	}

	FString GetEnumDocumentationLink(const FProperty* const Property)
	{
		if(Property != NULL)
		{
			const FByteProperty* ByteProperty = CastField<FByteProperty>(Property);
			const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property);
			if(ByteProperty || EnumProperty || (Property->IsA(FStrProperty::StaticClass()) && Property->HasMetaData(TEXT("Enum"))))
			{
				UEnum* Enum = nullptr;
				if(ByteProperty)
				{
					Enum = ByteProperty->Enum;
				}
				else if (EnumProperty)
				{
					Enum = EnumProperty->GetEnum();
				}
				else
				{

					const FString& EnumName = Property->GetMetaData(TEXT("Enum"));
					Enum = UClass::TryFindTypeSlow<UEnum>(EnumName, EFindFirstObjectOptions::ExactClass);
				}

				if(Enum)
				{
					return FString::Printf(TEXT("Shared/Enums/%s"), *Enum->GetName());
				}
			}
		}

		return TEXT("");
	}

	FString GetDocumentationExcerptName(const FProperty* const Property)
	{
		if ( Property != NULL )
		{
			return Property->GetName();
		}

		return TEXT("");
	}

	TSharedPtr<IPropertyHandle> GetPropertyHandle( TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities )
	{
		TSharedPtr<IPropertyHandle> PropertyHandle;

		// Always check arrays first, many types can be static arrays
		if( FPropertyHandleArray::Supports( PropertyNode ) )
		{
			PropertyHandle = MakeShareable( new FPropertyHandleArray( PropertyNode, NotifyHook, PropertyUtilities ) );
		}
		else if( FPropertyHandleInt::Supports( PropertyNode ) )
		{
			PropertyHandle = MakeShareable( new FPropertyHandleInt( PropertyNode, NotifyHook, PropertyUtilities ) );
		}
		else if( FPropertyHandleFloat::Supports( PropertyNode ) )
		{
			PropertyHandle = MakeShareable( new FPropertyHandleFloat( PropertyNode, NotifyHook, PropertyUtilities ) );
		}
		else if ( FPropertyHandleDouble::Supports( PropertyNode ) )
		{
			PropertyHandle = MakeShareable( new FPropertyHandleDouble( PropertyNode, NotifyHook, PropertyUtilities ) );
		}
		else if( FPropertyHandleBool::Supports( PropertyNode ) )
		{
			PropertyHandle = MakeShareable( new FPropertyHandleBool( PropertyNode, NotifyHook, PropertyUtilities ) ) ;
		}
		else if( FPropertyHandleByte::Supports( PropertyNode ) )
		{
			PropertyHandle = MakeShareable( new FPropertyHandleByte( PropertyNode, NotifyHook, PropertyUtilities ) );
		}
		else if( FPropertyHandleObject::Supports( PropertyNode ) )
		{
			PropertyHandle = MakeShareable( new FPropertyHandleObject( PropertyNode, NotifyHook, PropertyUtilities ) );
		}
		else if( FPropertyHandleString::Supports( PropertyNode ) ) 
		{
			PropertyHandle = MakeShareable( new FPropertyHandleString( PropertyNode, NotifyHook, PropertyUtilities ) );
		}
		else if (FPropertyHandleText::Supports(PropertyNode))
		{
			PropertyHandle = MakeShareable(new FPropertyHandleText(PropertyNode, NotifyHook, PropertyUtilities));
		}
		else if( FPropertyHandleVector::Supports( PropertyNode ) )
		{
			PropertyHandle = MakeShareable( new FPropertyHandleVector( PropertyNode, NotifyHook, PropertyUtilities ) );
		}
		else if( FPropertyHandleRotator::Supports( PropertyNode ) )
		{
			PropertyHandle = MakeShareable( new FPropertyHandleRotator( PropertyNode, NotifyHook, PropertyUtilities ) );
		}
		else if (FPropertyHandleColor::Supports(PropertyNode))
		{
			PropertyHandle = MakeShareable(new FPropertyHandleColor(PropertyNode, NotifyHook, PropertyUtilities));
		}
		else if (FPropertyHandleSet::Supports(PropertyNode))
		{
			PropertyHandle = MakeShareable( new FPropertyHandleSet( PropertyNode, NotifyHook, PropertyUtilities ) );
		}
		else if ( FPropertyHandleMap::Supports(PropertyNode) )
		{
			PropertyHandle = MakeShareable( new FPropertyHandleMap( PropertyNode, NotifyHook, PropertyUtilities ) );
		}
		else if (FPropertyHandleFieldPath::Supports(PropertyNode))
		{
			PropertyHandle = MakeShareable(new FPropertyHandleFieldPath(PropertyNode, NotifyHook, PropertyUtilities));
		}
		else if (FPropertyHandleOptional::Supports(PropertyNode))
		{
			PropertyHandle = MakeShareable(new FPropertyHandleOptional(PropertyNode, NotifyHook, PropertyUtilities));
		}
		// struct should be checked last as there are several specializations of it above
		else if (FPropertyHandleStruct::Supports(PropertyNode))
		{
			PropertyHandle = MakeShareable(new FPropertyHandleStruct(PropertyNode, NotifyHook, PropertyUtilities));
		}
		else
		{
			// Untyped or doesn't support getting the property directly but the property is still valid
			PropertyHandle = MakeShareable( new FPropertyHandleBase( PropertyNode, NotifyHook, PropertyUtilities ) ); 
		}

		return PropertyHandle;
	}

	static bool SupportsObjectPropertyButtons( FProperty* NodeProperty, bool bUsingAssetPicker )
	{
		return (NodeProperty->IsA<FObjectPropertyBase>() || NodeProperty->IsA<FInterfaceProperty>()) && (!bUsingAssetPicker || !SPropertyEditorAsset::Supports(NodeProperty));
	}
	
	bool IsSoftObjectPath( const FProperty* Property )
	{
		const FStructProperty* StructProp = CastField<const FStructProperty>( Property );
		return StructProp && StructProp->Struct == TBaseStructure<FSoftObjectPath>::Get();
	}

	bool IsSoftClassPath( const FProperty* Property )
	{
		const FStructProperty* StructProp = CastField<const FStructProperty>(Property);
		return StructProp && StructProp->Struct == TBaseStructure<FSoftClassPath>::Get();
	}

	void GetRequiredPropertyButtons( TSharedRef<FPropertyNode> PropertyNode, TArray<EPropertyButton::Type>& OutRequiredButtons, bool bUsingAssetPicker )
	{
		FProperty* NodeProperty = PropertyNode->GetProperty();

		// If no property is bound, don't create any buttons.
		if ( !NodeProperty )
		{
			return;
		}

		const FProperty* MetadataProperty = PropertyNode->GetMetaDataProperty();
		check(MetadataProperty); // As NodeProperty is non-null, this should always be valid at this point.

		// If the property is an item of a const container, don't create any buttons.
		const FArrayProperty* OuterArrayProp = NodeProperty->GetOwner<FArrayProperty>();
		const FSetProperty* OuterSetProp = NodeProperty->GetOwner<FSetProperty>();
		const FMapProperty* OuterMapProp = NodeProperty->GetOwner<FMapProperty>();

		// Some buttons should be skipped for statically sized arrays
		bool bStaticSizedArray = (NodeProperty->ArrayDim > 1) && (PropertyNode->GetArrayIndex() == -1);

		//////////////////////////////
		// Handle a container property.
		if( NodeProperty->IsA(FArrayProperty::StaticClass()) || NodeProperty->IsA(FSetProperty::StaticClass()) || NodeProperty->IsA(FMapProperty::StaticClass()) )
		{
			if( !(NodeProperty->PropertyFlags & CPF_EditFixedSize) )
			{
				OutRequiredButtons.Add( EPropertyButton::Add );
				OutRequiredButtons.Add( EPropertyButton::Empty );
			}
		}

		//////////////////////////////
		// Handle a class property.
		FClassProperty* ClassProp = CastField<FClassProperty>(NodeProperty);
		FSoftClassProperty* SoftClassProp = CastField<FSoftClassProperty>(NodeProperty);
		if( ClassProp || SoftClassProp || IsSoftClassPath(NodeProperty))
		{
			if (!bStaticSizedArray)
			{
				OutRequiredButtons.Add(EPropertyButton::Use);
				OutRequiredButtons.Add(EPropertyButton::Browse);

				UClass* Class = nullptr;
				if (ClassProp)
				{
					Class = ClassProp->MetaClass;
				}
				else if (SoftClassProp)
				{
					Class = SoftClassProp->MetaClass;
				}
				else
				{
					Class = NodeProperty->GetOwnerProperty()->GetClassMetaData(TEXT("MetaClass"));
				}

				if (Class && FKismetEditorUtilities::CanCreateBlueprintOfClass(Class) && !MetadataProperty->HasMetaData("DisallowCreateNew"))
				{
					OutRequiredButtons.Add(EPropertyButton::NewBlueprint);
				}

				if (!(NodeProperty->PropertyFlags & CPF_NoClear))
				{
					OutRequiredButtons.Add(EPropertyButton::Clear);
				}
			}
		}

		//////////////////////////////
		// Handle a struct type property.
		if (SPropertyEditorStruct::Supports(NodeProperty))
		{
			OutRequiredButtons.Add(EPropertyButton::Use);

			OutRequiredButtons.Add(EPropertyButton::Browse);

			if (!(NodeProperty->PropertyFlags & CPF_NoClear))
			{
				OutRequiredButtons.Add(EPropertyButton::Clear);
			}
		}

		//////////////////////////////
		// Handle an object property.
		if( SupportsObjectPropertyButtons( NodeProperty, bUsingAssetPicker ) )
		{
			//ignore this node if the consistency check should happen for the children
			if (!bStaticSizedArray)
			{
				if( PropertyNode->HasNodeFlags(EPropertyNodeFlags::EditInlineNew) )
				{
					// hmmm, seems like this code could be removed and the code inside the 'if <FClassProperty>' check
					// below could be moved outside the else....but is there a reason to allow class properties to have the
					// following buttons if the class property is marked 'editinline' (which is effectively what this logic is doing)
					if( !(NodeProperty->PropertyFlags & CPF_NoClear) )
					{
						OutRequiredButtons.Add( EPropertyButton::Clear );
					}
				}
				else
				{
					// ignore class properties
					if( (CastField<const FClassProperty>( NodeProperty ) == NULL) && (CastField<const FSoftClassProperty>( NodeProperty ) == NULL) )
					{
						FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>( NodeProperty );

						if( ObjectProperty && ObjectProperty->PropertyClass->IsChildOf( AActor::StaticClass() ) )
						{
							// add button for picking the actor from the viewport
							OutRequiredButtons.Add( EPropertyButton::PickActorInteractive );
						}
						else
						{
							// add button for filling the value of this item with the selected object from the GB
							OutRequiredButtons.Add( EPropertyButton::Use );
						}

						// add button to display the generic browser
						OutRequiredButtons.Add( EPropertyButton::Browse );

						// reference to object resource that isn't dynamically created (i.e. some content package)
						if( !(NodeProperty->PropertyFlags & CPF_NoClear) )
						{
							// add button to clear the text
							OutRequiredButtons.Add( EPropertyButton::Clear );
						}
							
						// Do not allow actor object properties to show the asset picker
						if( ( ObjectProperty && !ObjectProperty->PropertyClass->IsChildOf( AActor::StaticClass() ) ) || IsSoftObjectPath(NodeProperty) )
						{
							// add button for picking the asset from an asset picker
							OutRequiredButtons.Add( EPropertyButton::PickAsset );
						}
						else if( ObjectProperty && ObjectProperty->PropertyClass->IsChildOf( AActor::StaticClass() ) )
						{
							// add button for picking the actor from the scene outliner
							OutRequiredButtons.Add( EPropertyButton::PickActor );
						}
					}
				}
			}
		}

		if( OuterArrayProp )
		{
			if( PropertyNode->HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly) && !(MetadataProperty->PropertyFlags & CPF_EditFixedSize) )
			{
				if (MetadataProperty->HasMetaData(TEXT("NoElementDuplicate")))
				{
					OutRequiredButtons.Add( EPropertyButton::Insert_Delete );
				}
				else
				{
					OutRequiredButtons.Add( EPropertyButton::Insert_Delete_Duplicate );
				}
			}
		}

		if (OuterSetProp || OuterMapProp)
		{
			FProperty* OuterNodeProperty = NodeProperty->GetOwner<FProperty>();

			if ( PropertyNode->HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly) && !(OuterNodeProperty->PropertyFlags & CPF_EditFixedSize) )
			{
				OutRequiredButtons.Add(EPropertyButton::Delete);
			}
		}

		//////////////////////////////
		// Handle an optional value node.
		if (PropertyEditorHelpers::IsChildOfOption(*PropertyNode))
		{
			// Add optional 'X' button
			OutRequiredButtons.Add(EPropertyButton::OptionalClear);
		}
	}
	
	void MakeRequiredPropertyButtons( const TSharedRef<FPropertyNode>& PropertyNode, const TSharedRef<IPropertyUtilities>& PropertyUtilities,  TArray< TSharedRef<SWidget> >& OutButtons, const TArray<EPropertyButton::Type>& ButtonsToIgnore, bool bUsingAssetPicker  )
	{
		const TSharedRef<FPropertyEditor> PropertyEditor = FPropertyEditor::Create( PropertyNode, PropertyUtilities );
		PropertyEditorHelpers::MakeRequiredPropertyButtons( PropertyEditor, OutButtons, ButtonsToIgnore, bUsingAssetPicker );
	}

	static bool IsPropertyButtonEnabled(TWeakPtr<FPropertyNode> PropertyNode)
	{
		return PropertyNode.IsValid() ? !PropertyNode.Pin()->IsEditConst() : false;
	}

	TSharedRef<SWidget> MakePropertyReorderHandle(TSharedPtr<SDetailSingleItemRow> InParentRow, TAttribute<bool> InEnabledAttr)
	{
		const TWeakPtr<SDetailSingleItemRow> RowPtr = InParentRow.ToWeakPtr();
		TSharedRef<SArrayRowHandle> Handle = SNew(SArrayRowHandle)
			.Content()
			[
				SNew(SBox)
				.Padding(0.0f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.WidthOverride(16.0f)
				[
					SNew(SImage)
					.Image(FCoreStyle::Get().GetBrush("VerticalBoxDragIndicatorShort"))
				]
			]
		.ParentRow(InParentRow)
		.Cursor(EMouseCursor::GrabHand)
		.IsEnabled(InEnabledAttr)
		.Visibility_Lambda([RowPtr]()
		{
			if( const TSharedPtr<SDetailSingleItemRow> Row = RowPtr.Pin())
			{
				return Row->IsHovered() ? EVisibility::Visible : EVisibility::Hidden;
			}
			return EVisibility::Hidden;
		});
		return Handle;
	}

	void MakeRequiredPropertyButtons( const TSharedRef< FPropertyEditor >& PropertyEditor, TArray< TSharedRef<SWidget> >& OutButtons, const TArray<EPropertyButton::Type>& ButtonsToIgnore, bool bUsingAssetPicker )
	{
		TArray< EPropertyButton::Type > RequiredButtons;
		GetRequiredPropertyButtons( PropertyEditor->GetPropertyNode(), RequiredButtons, bUsingAssetPicker );

		for( int32 ButtonIndex = 0; ButtonIndex < RequiredButtons.Num(); ++ButtonIndex )
		{
			if( !ButtonsToIgnore.Contains( RequiredButtons[ButtonIndex] ) )
			{
				OutButtons.Add( MakePropertyButton( RequiredButtons[ButtonIndex], PropertyEditor ) );
			}
		}
	}

	/**
	 * A helper function that retrieves the path name of the currently selected 
	 * item (the value that will be used to set the associated property from the 
	 * "use selection" button)
	 * 
	 * @param  PropertyNode		The associated property that the selection is a candidate for.
	 * @return Empty if the selection isn't compatible with the specified property, else the path-name of the object/class selected in the editor. 
	 */
	static FString GetSelectionPathNameForProperty(TSharedRef<FPropertyNode> PropertyNode)
	{
		FString SelectionPathName;

		FProperty* Property = PropertyNode->GetProperty();
		FClassProperty* ClassProperty = CastField<FClassProperty>(Property);
		FSoftClassProperty* SoftClassProperty = CastField<FSoftClassProperty>(Property);

		if (ClassProperty || SoftClassProperty)
		{
			UClass const* const SelectedClass = GEditor->GetFirstSelectedClass(ClassProperty ? ClassProperty->MetaClass : SoftClassProperty->MetaClass);
			if (SelectedClass != nullptr)
			{
				SelectionPathName = SelectedClass->GetPathName();
			}
		}
		else
		{
			UClass* ObjectClass = UObject::StaticClass();

			bool bMustBeLevelActor = false;
			UClass* RequiredInterface = nullptr;

			if (FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
			{
				ObjectClass = ObjectProperty->PropertyClass;
				bMustBeLevelActor = ObjectProperty->GetOwnerProperty()->GetBoolMetaData(TEXT("MustBeLevelActor"));
				RequiredInterface = ObjectProperty->GetOwnerProperty()->GetClassMetaData(TEXT("MustImplement"));
			}
			else if (FInterfaceProperty* InterfaceProperty = CastField<FInterfaceProperty>(Property))
			{
				ObjectClass = InterfaceProperty->InterfaceClass;
			}

			UObject* SelectedObject = nullptr;
			if (bMustBeLevelActor)
			{
				USelection* const SelectedSet = GEditor->GetSelectedActors();
				SelectedObject = SelectedSet->GetTop(ObjectClass, RequiredInterface);
			}
			else 
			{
				USelection* const SelectedSet = GEditor->GetSelectedSet(ObjectClass);
				SelectedObject = SelectedSet->GetTop(ObjectClass, RequiredInterface);
			}

			if (SelectedObject != nullptr)
			{
				SelectionPathName = SelectedObject->GetPathName();
			}
		}

		return SelectionPathName;
	}

	/**
	 * A helper method that checks to see if the editor's current selection is 
	 * compatible with the specified property.
	 * 
	 * @param  PropertyNode		The property you desire to set from the "use selected" button.
	 * @return False if the currently selected object is restricted for the specified property, true otherwise.
	 */
	static bool IsUseSelectedUnrestricted(TWeakPtr<FPropertyNode> PropertyNode)
	{
		TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
		return ( PropertyNodePin.IsValid() && IsPropertyButtonEnabled(PropertyNode) ) ? !PropertyNodePin->IsRestricted( GetSelectionPathNameForProperty( PropertyNodePin.ToSharedRef() ) ) : false;
	}

	/**
	 * A helper method that checks to see if the editor's current selection is 
	 * restricted, and then returns a tooltip explaining why (otherwise, it 
	 * returns a default explanation of the "use selected" button).
	 * 
	 * @param  PropertyNode		The property that would be set from the "use selected" button.
	 * @return A tooltip for the "use selected" button.
	 */
	static FText GetUseSelectedTooltip(TWeakPtr<FPropertyNode> PropertyNode)
	{
		TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
		FText ToolTip;
		if (PropertyNodePin.IsValid() && !PropertyNodePin->GenerateRestrictionToolTip(GetSelectionPathNameForProperty(PropertyNodePin.ToSharedRef()), ToolTip))
		{
			ToolTip = LOCTEXT("UseButtonToolTipText", "Use Selected Asset from Content Browser");
		}

		return ToolTip;
	}

	TSharedRef<SWidget> MakePropertyButton( const EPropertyButton::Type ButtonType, const TSharedRef< FPropertyEditor >& PropertyEditor )
	{
		TSharedPtr<SWidget> NewButton;

		TWeakPtr<FPropertyNode> WeakPropertyNode = PropertyEditor->GetPropertyNode();

		TAttribute<bool>::FGetter IsPropertyButtonEnabledDelegate = TAttribute<bool>::FGetter::CreateStatic(&IsPropertyButtonEnabled, WeakPropertyNode);
		TAttribute<bool> IsEnabledAttribute = TAttribute<bool>::Create( IsPropertyButtonEnabledDelegate );

		TAttribute<bool> IsAddEnabledAttribute = TAttribute<bool>::Create(
			TAttribute<bool>::FGetter::CreateLambda([WeakPropertyNode, PropertyEditor]() {
			if (WeakPropertyNode.IsValid())
			{
				FProperty* Property = WeakPropertyNode.Pin()->GetProperty();
				// Check for multiple array selections with mismatched values
				FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property);
				if (ArrayProperty)
				{
					void* ValueAddress;
					FPropertyAccess::Result GetValResult = PropertyEditor->GetPropertyHandle()->GetValueData(ValueAddress);
					return IsPropertyButtonEnabled(WeakPropertyNode) && GetValResult == FPropertyAccess::Success;
				}
			}
			return IsPropertyButtonEnabled(WeakPropertyNode);
		}));

		switch( ButtonType )
		{
		case EPropertyButton::Add:
			NewButton = PropertyCustomizationHelpers::MakeAddButton( FSimpleDelegate::CreateSP( PropertyEditor, &FPropertyEditor::AddItem ), FText(), IsAddEnabledAttribute);
			break;

		case EPropertyButton::Empty:
			NewButton = PropertyCustomizationHelpers::MakeEmptyButton( FSimpleDelegate::CreateSP( PropertyEditor, &FPropertyEditor::EmptyArray ), FText(), IsEnabledAttribute );
			break;

		case EPropertyButton::Delete:
		case EPropertyButton::Insert_Delete:
		case EPropertyButton::Insert_Delete_Duplicate:
			{
				FExecuteAction InsertAction; 
				FExecuteAction DeleteAction = FExecuteAction::CreateSP( PropertyEditor, &FPropertyEditor::DeleteItem );
				FExecuteAction DuplicateAction;

				if (ButtonType == EPropertyButton::Insert_Delete || ButtonType == EPropertyButton::Insert_Delete_Duplicate)
				{
					InsertAction = FExecuteAction::CreateSP(PropertyEditor, &FPropertyEditor::InsertItem);
				}

				if (ButtonType == EPropertyButton::Insert_Delete_Duplicate)
				{
					DuplicateAction = FExecuteAction::CreateSP( PropertyEditor, &FPropertyEditor::DuplicateItem );
				}

				NewButton = PropertyCustomizationHelpers::MakeInsertDeleteDuplicateButton( InsertAction, DeleteAction, DuplicateAction );
				NewButton->SetEnabled( IsEnabledAttribute );
				break;
			}

		case EPropertyButton::Browse:
			NewButton = PropertyCustomizationHelpers::MakeBrowseButton( FSimpleDelegate::CreateSP( PropertyEditor, &FPropertyEditor::BrowseTo ) );
			break;

		case EPropertyButton::Clear:
			NewButton = PropertyCustomizationHelpers::MakeClearButton( FSimpleDelegate::CreateSP( PropertyEditor, &FPropertyEditor::ClearItem ), FText(), IsEnabledAttribute );
			break;

		case EPropertyButton::Use:
			{
				FSimpleDelegate OnClickDelegate = FSimpleDelegate::CreateSP(PropertyEditor, &FPropertyEditor::UseSelected);
				TAttribute<bool>::FGetter EnabledDelegate = TAttribute<bool>::FGetter::CreateStatic(&IsUseSelectedUnrestricted, WeakPropertyNode);
				TAttribute<FText>::FGetter TooltipDelegate = TAttribute<FText>::FGetter::CreateStatic(&GetUseSelectedTooltip, WeakPropertyNode);

				NewButton = PropertyCustomizationHelpers::MakeUseSelectedButton(OnClickDelegate, TAttribute<FText>::Create(TooltipDelegate), TAttribute<bool>::Create(EnabledDelegate));
				break;
			}

		case EPropertyButton::PickAsset:
			NewButton = PropertyCustomizationHelpers::MakeAssetPickerAnchorButton( FOnGetAllowedClasses::CreateSP( PropertyEditor, &FPropertyEditor::OnGetClassesForAssetPicker ), FOnAssetSelected::CreateSP( PropertyEditor, &FPropertyEditor::OnAssetSelected ), PropertyEditor->GetPropertyHandle());
			break;

		case EPropertyButton::PickActor:
			NewButton = PropertyCustomizationHelpers::MakeActorPickerAnchorButton( FOnGetActorFilters::CreateSP( PropertyEditor, &FPropertyEditor::OnGetActorFiltersForSceneOutliner ), FOnActorSelected::CreateSP( PropertyEditor, &FPropertyEditor::OnActorSelected ) );
			break;

		case EPropertyButton::PickActorInteractive:
			NewButton = PropertyCustomizationHelpers::MakeInteractiveActorPicker( FOnGetAllowedClasses::CreateSP( PropertyEditor, &FPropertyEditor::OnGetClassesForAssetPicker ), FOnShouldFilterActor(), FOnActorSelected::CreateSP( PropertyEditor, &FPropertyEditor::OnActorSelected ) );
			break;

		case EPropertyButton::NewBlueprint:
			NewButton = PropertyCustomizationHelpers::MakeNewBlueprintButton( FSimpleDelegate::CreateSP( PropertyEditor, &FPropertyEditor::MakeNewBlueprint )  );
			break;

		case EPropertyButton::EditConfigHierarchy:
			NewButton = PropertyCustomizationHelpers::MakeEditConfigHierarchyButton(FSimpleDelegate::CreateSP(PropertyEditor, &FPropertyEditor::EditConfigHierarchy));
			break;

		case EPropertyButton::Documentation:
			NewButton = PropertyCustomizationHelpers::MakeDocumentationButton(PropertyEditor);
			break;

		case EPropertyButton::OptionalSet:
			NewButton = PropertyCustomizationHelpers::MakeSetOptionalButton(FOnSetOptional::CreateSP(PropertyEditor, &FPropertyEditor::SetOptionalItem), FText(), IsEnabledAttribute);
			break;

		case EPropertyButton::OptionalPick:
			NewButton = PropertyCustomizationHelpers::MakePickOptionalButton(
				FOnSetOptional::CreateSP(PropertyEditor, &FPropertyEditor::SetOptionalItem), 
				FSimpleDelegate::CreateSP(PropertyEditor, &FPropertyEditor::ClearOptionalItem),
				PropertyEditor->GetPropertyNode(), FText(), IsEnabledAttribute);
			break;

		case EPropertyButton::OptionalClear:
			NewButton = PropertyCustomizationHelpers::MakeClearOptionalButton(FSimpleDelegate::CreateSP(PropertyEditor, &FPropertyEditor::ClearOptionalItem), FText(), IsEnabledAttribute);
			break;

		default:
			checkf( 0, TEXT( "Unknown button type" ) );
			break;
		}

		return NewButton.ToSharedRef();
	}

	void CollectObjectNodes( TSharedPtr<FPropertyNode> StartNode, TArray<FObjectPropertyNode*>& OutObjectNodes )
	{
		if( StartNode->AsObjectNode() != nullptr )
		{
			OutObjectNodes.Add( StartNode->AsObjectNode() );
		}
			
		for( int32 ChildIndex = 0; ChildIndex < StartNode->GetNumChildNodes(); ++ChildIndex )
		{
			CollectObjectNodes( StartNode->GetChildNode( ChildIndex ), OutObjectNodes );
		}
	}

	TArray<FName> GetValidEnumsFromPropertyOverride(const FProperty* Property, const UEnum* InEnum)
	{
		TArray<FName> ValidEnumValues;

		static const FName ValidEnumValuesName("ValidEnumValues");

		const FProperty* OwnerProperty = Property->GetOwnerProperty();
		if (OwnerProperty->HasMetaData(ValidEnumValuesName))
		{
			TArray<FString> ValidEnumValuesAsString;
			OwnerProperty->GetMetaData(ValidEnumValuesName).ParseIntoArray(ValidEnumValuesAsString, TEXT(","));
			
			for (FString& Value : ValidEnumValuesAsString)
			{
				Value.TrimStartInline();
				ValidEnumValues.Add(*InEnum->GenerateFullEnumName(*Value));
			}
		}

		return ValidEnumValues;
	}

	TArray<FName> GetInvalidEnumsFromPropertyOverride(const FProperty* Property, const UEnum* InEnum)
	{
		TArray<FName> InvalidEnumValues;

		static const FName InvalidEnumValuesName("InvalidEnumValues");

		const FProperty* OwnerProperty = Property->GetOwnerProperty();
		if (OwnerProperty->HasMetaData(InvalidEnumValuesName))
		{
			TArray<FString> InvalidEnumValuesAsString;

			OwnerProperty->GetMetaData(InvalidEnumValuesName).ParseIntoArray(InvalidEnumValuesAsString, TEXT(","));
			for (FString& Value : InvalidEnumValuesAsString)
			{
				Value.TrimStartInline();
				InvalidEnumValues.Add(*InEnum->GenerateFullEnumName(*Value));
			}
		}

		return InvalidEnumValues;
	}
	
	TArray<FName> GetRestrictedEnumsFromPropertyOverride(TArrayView<UObject*> ObjectList, const FProperty* Property, const UEnum* InEnum)
	{
		TArray<FName> RestrictedEnumValues;

		static const FName GetRestrictedEnumValuesName("GetRestrictedEnumValues");
		TArray<FString> ValidEnumValuesAsString;

		const FProperty* OwnerProperty = Property->GetOwnerProperty();
		if (OwnerProperty->HasMetaData(GetRestrictedEnumValuesName))
		{
			const FString GetRestrictedEnumValuesNameFunctionName = OwnerProperty->GetMetaData(GetRestrictedEnumValuesName);
			if (!GetRestrictedEnumValuesNameFunctionName.IsEmpty())
			{
				for (UObject* Object : ObjectList)
				{
					const UFunction* GetRestrictedEnumValuesNameFunction = Object ? Object->FindFunction(*GetRestrictedEnumValuesNameFunctionName) : nullptr;
					if (GetRestrictedEnumValuesNameFunction)
					{
						DECLARE_DELEGATE_RetVal(TArray<FString>, FGetValidEnumValuesNameF);

						ValidEnumValuesAsString.Append(FGetValidEnumValuesNameF::CreateUFunction(Object, GetRestrictedEnumValuesNameFunction->GetFName()).Execute());
					}
				}
			}

			for (FString& Value : ValidEnumValuesAsString)
			{
				Value.TrimStartInline();
				RestrictedEnumValues.AddUnique(*InEnum->GenerateFullEnumName(*Value));
			}
		}
		
		return RestrictedEnumValues;
	}

	TMap<FName, FText> GetEnumValueDisplayNamesFromPropertyOverride(const FProperty* Property, const UEnum* InEnum)
	{
		TMap<FName, FText> DisplayNameOverrides;

		static const FName NAME_EnumValueDisplayNameOverrides = "EnumValueDisplayNameOverrides";

		const FProperty* OwnerProperty = Property->GetOwnerProperty();
		const FString& DisplayNameOverridesStr = OwnerProperty->GetMetaData(NAME_EnumValueDisplayNameOverrides);
		if (DisplayNameOverridesStr.Len() > 0)
		{
			TArray<FString> DisplayNameOverridePairs;
			DisplayNameOverridesStr.ParseIntoArray(DisplayNameOverridePairs, TEXT(";"));

			for (const FString& DisplayNameOverridePair : DisplayNameOverridePairs)
			{
				FString DisplayNameKey;
				FString DisplayNameValue;
				if (DisplayNameOverridePair.Split(TEXT("="), &DisplayNameKey, &DisplayNameValue))
				{
					DisplayNameOverrides.Add(*InEnum->GenerateFullEnumName(*DisplayNameKey), FTextStringHelper::CreateFromBuffer(*DisplayNameValue));
				}
			}
		}

		return DisplayNameOverrides;
	}

	bool IsCategoryHiddenByClass(const TSharedPtr<FComplexPropertyNode>& InRootNode, FName CategoryName)
	{
		return InRootNode->AsObjectNode() && InRootNode->AsObjectNode()->GetHiddenCategories().Contains(CategoryName);
	}

	/**
	* Determines whether or not a property should be visible in the default generated detail layout
	*
	* @param PropertyNode	The property node to check
	* @param ParentNode	The parent property node to check
	* @return true if the property should be visible
	*/
	bool IsVisibleStandaloneProperty(const FPropertyNode& PropertyNode, const FPropertyNode& ParentNode)
	{
		const FProperty* Property = PropertyNode.GetProperty();
		const FArrayProperty* ParentArrayProperty = CastField<const FArrayProperty>(ParentNode.GetProperty());

		bool bIsVisibleStandalone = false;
		if (Property)
		{
			if (Property->IsA(FObjectPropertyBase::StaticClass()))
			{
				// Do not add this child node to the current map if its a single object property in a category (serves no purpose for UI)
				bIsVisibleStandalone = !ParentArrayProperty && (PropertyNode.GetNumChildNodes() == 0 || PropertyNode.GetNumChildNodes() > 1);
			}
			else if (Property->IsA(FArrayProperty::StaticClass()) || (Property->ArrayDim > 1 && PropertyNode.GetArrayIndex() == INDEX_NONE))
			{
				// Base array properties are always visible
				bIsVisibleStandalone = true;
			}
			else
			{
				bIsVisibleStandalone = true;
			}

		}

		return bIsVisibleStandalone;
	}

	static const FName NAME_DisplayAfter("DisplayAfter");
	static const FName NAME_DisplayPriority("DisplayPriority");

	void OrderPropertiesFromMetadata(TArray<FProperty*>& Properties)
	{
		TMap<FName, TArray<TTuple<FProperty*, int32>>> DisplayAfterPropertyMap;
		TArray<TTuple<FProperty*, int32>> OrderedProperties;
		OrderedProperties.Reserve(Properties.Num());

		// First establish the properties that are not dependent on another property in display priority order
		// At the same time build a display priority sorted list of order after properties for each property name
		for (FProperty* Prop : Properties)
		{
			const FString& DisplayPriorityStr = Prop->GetMetaData(NAME_DisplayPriority);
			int32 DisplayPriority = (DisplayPriorityStr.IsEmpty() ? MAX_int32 : FCString::Atoi(*DisplayPriorityStr));
			if (DisplayPriority == 0 && !FCString::IsNumeric(*DisplayPriorityStr))
			{
				// If there was a malformed display priority str Atoi will say it is 0, but we want to treat it as unset
				DisplayPriority = MAX_int32;
			}

			auto InsertProperty = [Prop, DisplayPriority](TArray<TTuple<FProperty*, int32>>& InsertToArray)
			{
				bool bInserted = false;
				if (DisplayPriority != MAX_int32)
				{
					for (int32 InsertIndex = 0; InsertIndex < InsertToArray.Num(); ++InsertIndex)
					{
						const int32 PriorityAtIndex = InsertToArray[InsertIndex].Get<1>();
						if (DisplayPriority < PriorityAtIndex)
						{
							InsertToArray.Insert(MakeTuple(Prop, DisplayPriority), InsertIndex);
							bInserted = true;
							break;
						}
					}
				}

				if (!bInserted)
				{
					InsertToArray.Emplace(MakeTuple(Prop, DisplayPriority));
				}
			};

			const FString& DisplayAfterPropertyName = Prop->GetMetaData(NAME_DisplayAfter);
			if (DisplayAfterPropertyName.IsEmpty())
			{
				InsertProperty(OrderedProperties);
			}
			else
			{
				TArray<TPair<FProperty*, int32>>& DisplayAfterProperties = DisplayAfterPropertyMap.FindOrAdd(FName(*DisplayAfterPropertyName));
				InsertProperty(DisplayAfterProperties);
			}
		}

		// While there are still properties that need insertion seek out the property they should be listed after and insert them in their pre-display priority sorted order
		int32 RemainingDisplayAfterNodes = -1; // avoid infinite loop caused by cycles or missing dependencies by tracking that the map shrunk each iteration
		while (DisplayAfterPropertyMap.Num() > 0 && DisplayAfterPropertyMap.Num() != RemainingDisplayAfterNodes)
		{
			RemainingDisplayAfterNodes = DisplayAfterPropertyMap.Num();

			for (int32 InsertIndex = 0; InsertIndex < OrderedProperties.Num(); ++InsertIndex)
			{
				FProperty* Prop = OrderedProperties[InsertIndex].Get<0>();

				if (TArray<TTuple<FProperty*, int32>>* DisplayAfterProperties = DisplayAfterPropertyMap.Find(Prop->GetFName()))
				{
					OrderedProperties.Insert(MoveTemp(*DisplayAfterProperties), InsertIndex + 1);
					DisplayAfterPropertyMap.Remove(Prop->GetFName());
					if (DisplayAfterPropertyMap.Num() == 0)
					{
						break;
					}
				}
			}
		}

		// Copy the sorted properties back in to the original array
		Properties.Reset();
		for (const TTuple<FProperty*, int32>& Property : OrderedProperties)
		{
			Properties.Add(Property.Get<0>());
		}

		if (DisplayAfterPropertyMap.Num() != 0)
		{
			// If we hit this there is either a cycle or a dependency on something that doesn't exist, so just put them at the end of the list
			// TODO: Some kind of warning?
			for (TPair<FName, TArray<TTuple<FProperty*, int32>>>& DisplayAfterProperties : DisplayAfterPropertyMap)
			{
				for (const TTuple<FProperty*, int32>& Property : DisplayAfterProperties.Value)
				{
					Properties.Add(Property.Get<0>());
				}
			}
		}
	}

	FName GetPropertyOptionsMetaDataKey(const FProperty* Property)
	{
		// Only string and name properties can have options
		if (Property->IsA(FStrProperty::StaticClass()) || Property->IsA(FNameProperty::StaticClass()))
		{
			const FProperty* OwnerProperty = Property->GetOwnerProperty();
			static const FName GetOptionsName("GetOptions");
			if (OwnerProperty->HasMetaData(GetOptionsName))
			{
				return GetOptionsName;
			}

			// Map properties can have separate options for keys and values
			const FMapProperty* MapProperty = CastField<FMapProperty>(OwnerProperty);
			if (MapProperty)
			{
				static const FName GetKeyOptionsName("GetKeyOptions");
				if (MapProperty->HasMetaData(GetKeyOptionsName) && MapProperty->GetKeyProperty() == Property)
				{
					return GetKeyOptionsName;
				}

				static const FName GetValueOptionsName("GetValueOptions");
				if (MapProperty->HasMetaData(GetValueOptionsName) && MapProperty->GetValueProperty() == Property)
				{
					return GetValueOptionsName;
				}
			}
		}

		return NAME_None;
	}

	const UClass* FindOrLoadClass(const FString& ClassPath)
	{
		if (const UObject* Object = StaticLoadObject(nullptr, nullptr, ClassPath))
		{
			if (const UClass* Class = Cast<UClass>(Object))
			{
				return Class;
			}
			else
			{
				UE_LOGF(LogPropertyNode, Error, "Found object '%ls' instead of a class.", *Object->GetFullName());
			}
		}
		else
		{
			UE_LOGF(LogPropertyNode, Error, "Unable to load class '%ls'.", *ClassPath);
		}

		return nullptr;
	}

	FText GetNumElementsText(const FText& InContainerText, const FPropertyEditor& InPropertyEditor)
	{
		void* DummyValueAddress;
		if (InPropertyEditor.GetPropertyHandle()->GetValueData(DummyValueAddress) == FPropertyAccess::MultipleValues)
		{
			return PropertyEditorConstants::DefaultUndeterminedText;
		}

		return FText::Format(LOCTEXT("NumItemsFmt", "{0} {1} {0}|plural(one=element,other=elements)"), FText::AsNumber(InPropertyEditor.GetPropertyNode()->GetNumChildNodes()), InContainerText);
	}

}

#undef LOCTEXT_NAMESPACE
