// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/SWidget.h"
#include "IPropertyUtilities.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "PropertyNode.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "SDetailSingleItemRow.h"

class FNotifyHook;
class FObjectPropertyNode;
class FDetailWidgetRow;

namespace UE::Editor::ContentBrowser
{
	static bool IsNewStyleEnabled()
	{
		static bool bIsNewStyleEnabled = [&]()
		{
			if (const IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("ContentBrowser.EnableNewStyle")))
			{
				ensureAlwaysMsgf(!EnumHasAnyFlags(CVar->GetFlags(), ECVF_Default), TEXT("The CVar should have already been set from commandline, @see: UnrealEdGlobals.cpp, UE::Editor::ContentBrowser::EnableContentBrowserNewStyleCVarRegistration."));
				return CVar->GetBool();
			}
			return false;
		}();

		return bIsNewStyleEnabled;
	}
}

/** Property button enums. */
namespace EPropertyButton
{
	enum Type
	{
		Add,
		Empty,
		Insert_Delete_Duplicate,
		Insert_Delete,
		Insert,
		Delete,
		Duplicate,
		Browse,
		PickAsset,
		PickActor,
		PickActorInteractive,
		Clear,
		Use,
		NewBlueprint,
		EditConfigHierarchy,
		Documentation,
		OptionalSet,
		OptionalPick,
		OptionalClear,
	};
}

class SPropertyNameWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SPropertyNameWidget )
	{}
		SLATE_EVENT( FOnClicked, OnDoubleClicked )
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, TSharedPtr<FPropertyEditor> PropertyEditor );

private:
	TSharedPtr< class FPropertyEditor > PropertyEditor;
};

class SPropertyValueWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SPropertyValueWidget )
		: _ShowPropertyButtons( true )
	{}
		SLATE_ARGUMENT( bool, ShowPropertyButtons )
		SLATE_ARGUMENT(TOptional<FDetailWidgetRow*>, InWidgetRow)
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, TSharedPtr<FPropertyEditor> InPropertyEditor, TSharedPtr<IPropertyUtilities> InPropertyUtilities );

	/** @return The minimum desired with if this property value */
	float GetMinDesiredWidth() const { return MinDesiredWidth; }

	/** @return The maximum desired with if this property value */
	float GetMaxDesiredWidth() const { return MaxDesiredWidth; }

private:
	TSharedRef<SWidget> ConstructPropertyEditorWidget( TSharedPtr<FPropertyEditor>& PropertyEditor, TSharedPtr<IPropertyUtilities> InPropertyUtilities );
private:
	TSharedPtr< SWidget > ValueEditorWidget;
	/** The minimum desired with if this property value */
	float MinDesiredWidth;
	/** The maximum desired with if this property value */
	float MaxDesiredWidth;
	/** The widget row this value widget is part of */
	TOptional<FDetailWidgetRow*> WidgetRow;

};

class SEditConditionWidget : public SCompoundWidget
{
	SLATE_BEGIN_ARGS( SEditConditionWidget )
	{}
		SLATE_ATTRIBUTE( bool, EditConditionValue )
		SLATE_EVENT( FOnBooleanValueChanged, OnEditConditionValueChanged )
	SLATE_END_ARGS()

	void Construct( const FArguments& Args );

private:
	void OnEditConditionCheckChanged( ECheckBoxState CheckState );
	bool HasEditConditionToggle() const;
	ECheckBoxState OnGetEditConditionCheckState() const;
	EVisibility GetVisibility() const;

private:
	TAttribute<bool> EditConditionValue;
	FOnBooleanValueChanged OnEditConditionValueChanged;
};

namespace PropertyEditorHelpers
{
	/** 
	 * Perform common checks from metadata to determine if a given Property
	 * should be visible when displayed underneath the given property node.
	 */
	bool ShouldBeVisible(const FPropertyNode& InParentNode, const FProperty* Property);

	/**
	 * Returns whether or not a property is a built in struct property like a vector or color
	 *
	 * @return true if the property is built in type, false otherwise
	 */
	bool IsBuiltInStructProperty( const FProperty* Property );

	/**
	 * Returns whether or not a property is a child of an array (static or dynamic)
	 *
	 * @return true if the property is in an array, false otherwise
	 */
	bool IsChildOfArray( const FPropertyNode& InPropertyNode );

	/**
	 * @return true if the property is a child (within) a set, false otherwise
	 */
	bool IsChildOfSet( const FPropertyNode& InPropertyNode );

	/**
	 * @return true if the property is a child (within) a map, false otherwise
	 */
	bool IsChildOfMap( const FPropertyNode& InPropertyNode );

	/**
	 * @return true if the property is a child (within) a option, false otherwise
	 */
	bool IsChildOfOption(const FPropertyNode& InPropertyNode);
	
	/**
	 * Returns whether or not a property a static array
	 *
	 * @param InPropertyNode	The property node containing the property to check
	 */
	bool IsStaticArray( const FPropertyNode& InPropertyNode );
	
	/**
	 * Returns whether or not a property a static array
	 *
	 * @param InPropertyNode	The property node containing the property to check
	 */
	bool IsDynamicArray( const FPropertyNode& InPropertyNode );

	/**
	 * Returns whether or not a property is an optional wrapper
	 *
	 * @param InPropertyNode	The property node containing the property to check
	 */
	bool IsOptionalProperty(const FPropertyNode& InPropertyNode);

	/**
	 * Returns true if this is an FSoftObjectPath and should be treated like a TSoftObjectPtr
	 */
	bool IsSoftObjectPath(const FProperty* Property);

	/**
	 * Returns true if this is an FSoftClassPath and should be treated like a TSoftClassPtr
	 */
	bool IsSoftClassPath(const FProperty* Property);

	/**
	 * Gets the array parent of a property if it is in a dynamic or static array
	 */
	const FProperty* GetArrayParent( const FPropertyNode& InPropertyNode );

	/**
	 * Gets the set parent of a property if it is in a set
	 */
	const FProperty* GetSetParent( const FPropertyNode& InPropertyNode );

	/**
	 * Gets the map parent of a property if it is in a map
	 */
	const FProperty* GetMapParent(const FPropertyNode& InPropertyNode);

	/**
	 * Gets the option parent of a property if it is in an option
	 */
	const FProperty* GetOptionParent(const FPropertyNode& InPropertyNode);

	/**
	 * Returns if a class is acceptable for edit inline
	 *
	 * @param CheckClass		The class to check
	 * @param bAllowAbstract	True if abstract classes are allowed
	 */
	bool IsEditInlineClassAllowed( UClass* CheckClass, bool bAllowAbstract );

	/**
	 * @return The text that represents the specified properties tooltip
	 */
	FText GetToolTipText( const FProperty* const Property );

	/**
	 * @return The link to the documentation that describes this property in detail
	 */
	FString GetDocumentationLink( const FProperty* const Property );

	/**
	* @return The link to the documentation that describes this enum property in detail
	*/
	FString GetEnumDocumentationLink(const FProperty* const Property);

	/**
	 * @return The name of the excerpt that describes this property in detail in the documentation file linked to this property
	 */
	FString GetDocumentationExcerptName( const FProperty* const Property );

	/**
	 * Gets a property handle for the specified property node
	 * 
	 * @param PropertyNode		The property node to get the handle from
	 * @param NotifyHook		Optional notify hook for the handle to use
	 */
	TSharedPtr<IPropertyHandle> GetPropertyHandle( TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities );

	/**
	 * Generates a list of required button types for the property
	 *
	 * @param PropertyNode			The property node to get required buttons from
	 * @param OutRequiredbuttons	The list of required buttons 
	 * @param bUsingAssetPicker		Whether or not the property editor is using the asset picker
	 */
	void GetRequiredPropertyButtons( TSharedRef<FPropertyNode> PropertyNode, TArray<EPropertyButton::Type>& OutRequiredButtons, bool bUsingAssetPicker = true );

	/**
	 * Makes property buttons widgets that accompany a property
	 *
	 * @param PropertyNode				The property node to make buttons for
	 * @param ButtonTypeToDelegateMap	A mapping of button type to the delegate to call when the button is clicked
	 * @param OutButtons				The populated list of button widgets
	 * @param ButtonsToIgnore			Specify those buttons which should be ignored and not added
	 * @param bUsingAssetPicker			Whether or not the property editor is using the asset picker
	 */
	void MakeRequiredPropertyButtons( const TSharedRef<FPropertyNode>& PropertyNode, const TSharedRef<IPropertyUtilities>& PropertyUtilities, TArray< TSharedRef<SWidget> >& OutButtons, const TArray<EPropertyButton::Type>& ButtonsToIgnore = TArray<EPropertyButton::Type>(), bool bUsingAssetPicker = true );
	void MakeRequiredPropertyButtons( const TSharedRef< FPropertyEditor >& PropertyEditor, TArray< TSharedRef<SWidget> >& OutButtons, const TArray<EPropertyButton::Type>& ButtonsToIgnore = TArray<EPropertyButton::Type>(), bool bUsingAssetPicker = true );

	TSharedRef<SWidget> MakePropertyButton( const EPropertyButton::Type ButtonType, const TSharedRef< FPropertyEditor >& PropertyEditor );
	TSharedRef<SWidget> MakePropertyReorderHandle(TSharedPtr<SDetailSingleItemRow> InParentRow, TAttribute<bool> InEnabledAttr);
	/**
	 * Recursively finds all object property nodes in a property tree
	 *
	 * @param StartNode	The node to start from
	 * @param OutObjectNodes	List of all object nodes found
	 */
	void CollectObjectNodes( TSharedPtr<FPropertyNode> StartNode, TArray<FObjectPropertyNode*>& OutObjectNodes );

	/**
	 * Returns any enums that are explicitly allowed by the "ValidEnumValues" metadata on FProperty using the specified enum.
	 *
	 * @param Property	The property which may contain the "ValidEnumValues" metadata
	 * @param InEnum	The enum to search
	 * @return The array of allowed enums.  NOTE: If an empty array is returned all enum values are allowed. It is an error for a property to hide all enum values so that state is undefined here.
	 */
	TArray<FName> GetValidEnumsFromPropertyOverride(const FProperty* Property, const UEnum* InEnum);

	/**
	 * Returns any enums that are explicitly Disallowed by the "InvalidEnumValues" metadata on FProperty using the specified enum.
	 *
	 * @param Property	The property which may contain the "InvalidEnumValues" metadata
	 * @param InEnum	The enum to search
	 * @return The array of disallowed enums.
	 */
	TArray<FName> GetInvalidEnumsFromPropertyOverride(const FProperty* Property, const UEnum* InEnum);

	/**
	 * Returns any enums that are explicitly restricted by the "GetRestrictedEnumValues" metadata on FProperty using the specified enum.
	 *
	 * @param ObjectList The list of objects currently edited
	 * @param Property	The property which may contain the "GetRestrictedEnumValues" metadata
	 * @param InEnum	The enum to search
	 * @return The array of restricted enums.  
	 */
	TArray<FName> GetRestrictedEnumsFromPropertyOverride(TArrayView<UObject*> ObjectList, const FProperty* Property, const UEnum* InEnum);

	/**
	 * Returns any enums that are have an overridden display name from the "EnumValueDisplayNameOverrides" metadata on FProperty using the specified enum.
	 *
	 * @param Property	The property which may contain the "EnumValueDisplayNameOverrides" metadata
	 * @param InEnum	The enum to search
	 * @return The map of display name overrides.
	 */
	TMap<FName, FText> GetEnumValueDisplayNamesFromPropertyOverride(const FProperty* Property, const UEnum* InEnum);
	
	/**
	 * Whether or not a category is hidden by a given root object
	 * @param InRootNode	The root node that for the objects we are customizing
	 * @param CategoryName	The name of the category to check
	 * @return true if a category is hidden, false otherwise
	 */
	bool IsCategoryHiddenByClass(const TSharedPtr<FComplexPropertyNode>& InRootNode, FName CategoryName);
	
	/**
	 * Determines whether or not a property should be visible in the default generated detail layout
	 *
	 * @param PropertyNode	The property node to check
	 * @param ParentNode	The parent property node to check
	 * @return true if the property should be visible
	 */
	bool IsVisibleStandaloneProperty(const FPropertyNode& PropertyNode, const FPropertyNode& ParentNode);

	void OrderPropertiesFromMetadata(TArray<FProperty*>& Properties);

	/**
	* For properties that support options lists, returns the metadata key which holds the name of the UFunction to call.
	* Returns NAME_None if the property doesn't support, or doesn't have, options.
	*/
	FName GetPropertyOptionsMetaDataKey(const FProperty* Property);

	/**
	 * Attempts to find the class with the given object path.
	 * @param ClassPath  The asset path of the class to find or load (eg: /Game/Blueprints/BP_Example.BP_Example_C)
	 * 
	 * @return nullptr if the class could not be found, or if the path doesn't correspond to a class object
	 */
	const UClass* FindOrLoadClass(const FString& ClassPath);

	/** Returns the text to inform the users of the number of elements in a given container property 
	 * @param InContainerText the name of the container to use in the text (e.g. "Map" for a TMap)
	 * @param InPropertyEditor container to use in the text (e.g. "Map" for a TMap)
	 * 
	 * @return the localized text
	 */
	FText GetNumElementsText(const FText& InContainerText, const FPropertyEditor& InPropertyEditor);
}

