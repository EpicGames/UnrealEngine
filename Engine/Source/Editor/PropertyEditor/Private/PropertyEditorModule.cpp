// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyEditorModule.h"

#include "AssetToolsModule.h"
#include "DetailRowMenuContextPrivate.h"
#include "DetailsViewStyle.h"
#include "Engine/UserDefinedEnum.h"
#include "Framework/Application/SlateApplication.h"
#include "IAssetTools.h"
#include "IDetailsView.h"
#include "Interfaces/IMainFrameModule.h"
#include "IPropertyChangeListener.h"
#include "IPropertyTable.h"
#include "IPropertyTableCellPresenter.h"
#include "IPropertyTableWidgetHandle.h"
#include "IStructureDataProvider.h"
#include "Modules/ModuleManager.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "Presentation/PropertyTable/PropertyTable.h"
#include "PropertyChangeListener.h"
#include "PropertyEditorCommands.h"
#include "PropertyEditorToolkit.h"
#include "PropertyRowGenerator.h"
#include "SDetailsView.h"
#include "SDetailsViewBase.h"
#include "SPropertyTreeViewImpl.h"
#include "SSingleProperty.h"
#include "SStructureDetailsView.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "ToolMenus.h"
#include "UObject/UnrealType.h"
#include "UserInterface/PropertyTable/PropertyTableWidgetHandle.h"
#include "UserInterface/PropertyTable/SPropertyTable.h"
#include "UserInterface/PropertyTable/TextPropertyTableCellPresenter.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Layout/SBorder.h"

IMPLEMENT_MODULE(FPropertyEditorModule, PropertyEditor);

TAutoConsoleVariable<bool> FPropertyEditorModule::CVarAllowPropertyMatrixForNonActors = TAutoConsoleVariable<bool>(
	TEXT("PropertyEditor.AllowPropertyMatrixForNonActors"),
	false,
	TEXT("Specifies whether the property matrix can be launched when selected items are not actors or actor components.")
);

TSharedRef<IPropertyTypeCustomization> FPropertyTypeLayoutCallback::GetCustomizationInstance() const
{
	return PropertyTypeLayoutDelegate.Execute();
}

void FPropertyTypeLayoutCallbackList::Add( const FPropertyTypeLayoutCallback& NewCallback )
{
	if( !NewCallback.PropertyTypeIdentifier.IsValid() )
	{
		BaseCallback = NewCallback;
	}
	else
	{
		IdentifierList.Add( NewCallback );
	}
}

void FPropertyTypeLayoutCallbackList::Remove( const TSharedPtr<IPropertyTypeIdentifier>& InIdentifier )
{
	if( !InIdentifier.IsValid() )
	{
		BaseCallback = FPropertyTypeLayoutCallback();
	}
	else
	{
		IdentifierList.RemoveAllSwap( [&InIdentifier]( FPropertyTypeLayoutCallback& Callback) { return Callback.PropertyTypeIdentifier == InIdentifier; } );
	}
}

const FPropertyTypeLayoutCallback& FPropertyTypeLayoutCallbackList::Find( const IPropertyHandle& PropertyHandle ) const 
{
	if( IdentifierList.Num() > 0 )
	{
		const FPropertyTypeLayoutCallback* Callback =
			IdentifierList.FindByPredicate
			(
				[&]( const FPropertyTypeLayoutCallback& InCallback )
				{
					return InCallback.PropertyTypeIdentifier->IsPropertyTypeCustomized( PropertyHandle );
				}
			);

		if( Callback )
		{
			return *Callback;
		}
	}

	return BaseCallback;
}

void FPropertyEditorModule::StartupModule()
{
	StructOnScopePropertyOwner = nullptr;

	FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &FPropertyEditorModule::ReplaceViewedObjects);

	FDetailsViewStyle::InitializeDetailsViewStyles();

	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FPropertyEditorModule::RegisterMenus));

	FCoreDelegates::GetOnPostEngineInit().Add(FSimpleMulticastDelegate::FDelegate::CreateStatic(SDetailsViewBase::ResetTransientPerViewDetailExpansion));
}

void FPropertyEditorModule::ShutdownModule()
{
	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner(this);

	FPropertyEditorCommands::Unregister();
	
	// No need to remove this object from root since the final GC pass doesn't care about root flags
	StructOnScopePropertyOwner = nullptr;

	// NOTE: It's vital that we clean up everything created by this DLL here!  We need to make sure there
	//       are no outstanding references to objects as the compiled code for this module's class will
	//       literally be unloaded from memory after this function exits.  This even includes instantiated
	//       templates, such as delegate wrapper objects that are allocated by the module!
	DestroyColorPicker();

	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);

	AllDetailViews.Empty();
	AllSinglePropertyViews.Empty();
}

void FPropertyEditorModule::NotifyCustomizationModuleChanged()
{
	if (FSlateApplication::IsInitialized())
	{
		// The module was changed (loaded or unloaded), force a refresh.  Note it is assumed the module unregisters all customization delegates before this
		for (int32 ViewIndex = 0; ViewIndex < AllDetailViews.Num(); ++ViewIndex)
		{
			if (AllDetailViews[ViewIndex].IsValid())
			{
				TSharedPtr<SDetailsView> DetailViewPin = AllDetailViews[ViewIndex].Pin();

				DetailViewPin->ForceRefresh();
			}
		}
	}
}

void FPropertyEditorModule::RequestRefreshOfAllDetailsPanels()
{
	if (FSlateApplication::IsInitialized())
	{
		for (int32 ViewIndex = 0; ViewIndex < AllDetailViews.Num(); ++ViewIndex)
		{
			if (AllDetailViews[ViewIndex].IsValid())
			{
				TSharedPtr<SDetailsView> DetailViewPin = AllDetailViews[ViewIndex].Pin();
				DetailViewPin->RequestForceRefresh();
			}
		}
	}
}

static bool ShouldShowProperty(const FPropertyAndParent& PropertyAndParent, bool bHaveTemplate)
{
	const FProperty& Property = PropertyAndParent.Property;

	if ( bHaveTemplate )
	{
		const UClass* PropertyOwnerClass = Property.GetOwner<const UClass>();
		const bool bDisableEditOnTemplate = PropertyOwnerClass 
			&& PropertyOwnerClass->IsNative()
			&& Property.HasAnyPropertyFlags(CPF_DisableEditOnTemplate);
		if(bDisableEditOnTemplate)
		{
			return false;
		}
	}
	return true;
}

TSharedRef<SWindow> FPropertyEditorModule::CreateFloatingDetailsView( const TArray< UObject* >& InObjects, bool bIsLockable )
{

	TSharedRef<SWindow> NewSlateWindow = SNew(SWindow)
		.Title( NSLOCTEXT("PropertyEditor", "WindowTitle", "Property Editor") )
		.ClientSize( FVector2D(400, 550) );

	// If the main frame exists parent the window to it
	TSharedPtr< SWindow > ParentWindow;
	if( FModuleManager::Get().IsModuleLoaded( "MainFrame" ) )
	{
		IMainFrameModule& MainFrame = FModuleManager::GetModuleChecked<IMainFrameModule>( "MainFrame" );
		ParentWindow = MainFrame.GetParentWindow();
	}

	if( ParentWindow.IsValid() )
	{
		// Parent the window to the main frame 
		FSlateApplication::Get().AddWindowAsNativeChild( NewSlateWindow, ParentWindow.ToSharedRef() );
	}
	else
	{
		FSlateApplication::Get().AddWindow( NewSlateWindow );
	}
	
	FDetailsViewArgs Args;
	Args.bHideSelectionTip = true;
	Args.bLockable = bIsLockable;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedRef<IDetailsView> DetailView = PropertyEditorModule.CreateDetailView( Args );

	bool bHaveTemplate = false;
	for(int32 i=0; i<InObjects.Num(); i++)
	{
		if(InObjects[i] != NULL && InObjects[i]->IsTemplate())
		{
			bHaveTemplate = true;
			break;
		}
	}

	DetailView->SetIsPropertyVisibleDelegate( FIsPropertyVisible::CreateStatic(&ShouldShowProperty, bHaveTemplate) );

	DetailView->SetObjects( InObjects );

	NewSlateWindow->SetContent(
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush(TEXT("PropertyWindow.WindowBorder")) )
		[
			DetailView
		]
	);

	return NewSlateWindow;
}

TSharedRef<SPropertyTreeViewImpl> FPropertyEditorModule::CreatePropertyView( 
	UObject* InObject,
	bool bAllowFavorites, 
	bool bIsLockable, 
	bool bHiddenPropertyVisibility, 
	bool bAllowSearch, 
	bool ShowTopLevelNodes,
	FNotifyHook* InNotifyHook, 
	float InNameColumnWidth,
	FOnPropertySelectionChanged OnPropertySelectionChanged,
	FOnPropertyClicked OnPropertyMiddleClicked,
	FConstructExternalColumnHeaders ConstructExternalColumnHeaders,
	FConstructExternalColumnCell ConstructExternalColumnCell)
{
	TSharedRef<SPropertyTreeViewImpl> PropertyView = 
		SNew( SPropertyTreeViewImpl )
		.IsLockable( bIsLockable )
		.AllowFavorites( bAllowFavorites )
		.HiddenPropertyVis( bHiddenPropertyVisibility )
		.NotifyHook( InNotifyHook )
		.AllowSearch( bAllowSearch )
		.ShowTopLevelNodes( ShowTopLevelNodes )
		.NameColumnWidth( InNameColumnWidth )
		.OnPropertySelectionChanged( OnPropertySelectionChanged )
		.OnPropertyMiddleClicked( OnPropertyMiddleClicked )
		.ConstructExternalColumnHeaders( ConstructExternalColumnHeaders )
		.ConstructExternalColumnCell( ConstructExternalColumnCell );

	if( InObject )
	{
		TArray<UObject*> Objects;
		Objects.Add( InObject );
		PropertyView->SetObjectArray( Objects );
	}

	return PropertyView;
}

TSharedPtr<FAssetThumbnailPool> FPropertyEditorModule::GetThumbnailPool()
{
	if (!GlobalThumbnailPool.IsValid())
	{
		// Create a thumbnail pool for the view if it doesn't exist.  This does not use resources of no thumbnails are used
		GlobalThumbnailPool = MakeShareable(new FAssetThumbnailPool(50));
	}

	return GlobalThumbnailPool;
}

TSharedRef<IDetailsView> FPropertyEditorModule::CreateDetailView( const FDetailsViewArgs& DetailsViewArgs )
{
	LLM_SCOPE(ELLMTag::UI);

	// Compact the list of detail view instances
	for( int32 ViewIndex = 0; ViewIndex < AllDetailViews.Num(); ++ViewIndex )
	{
		if ( !AllDetailViews[ViewIndex].IsValid() )
		{
			AllDetailViews.RemoveAtSwap( ViewIndex );
			--ViewIndex;
		}
	}

	TSharedRef<SDetailsView> DetailView = SNew(SDetailsView, DetailsViewArgs);

	AllDetailViews.Add( DetailView );

	PropertyEditorOpened.Broadcast();
	return DetailView;
}

TSharedPtr<IDetailsView> FPropertyEditorModule::FindDetailView( const FName ViewIdentifier ) const
{
	if(ViewIdentifier.IsNone())
	{
		return nullptr;
	}

	for(auto It = AllDetailViews.CreateConstIterator(); It; ++It)
	{
		TSharedPtr<SDetailsView> DetailsView = It->Pin();
		if(DetailsView.IsValid() && DetailsView->GetIdentifier() == ViewIdentifier)
		{
			return DetailsView;
		}
	}

	return nullptr;
}

TSharedPtr<ISinglePropertyView> FPropertyEditorModule::CreateSingleProperty( UObject* InObject, FName InPropertyName, const FSinglePropertyParams& InitParams )
{
	return CreateSinglePropertyImpl(InObject, TSharedPtr<IStructureDataProvider>(), InPropertyName, InitParams);
}

TSharedPtr<class ISinglePropertyView> FPropertyEditorModule::CreateSingleProperty( const TSharedPtr<IStructureDataProvider>& InStruct, FName InPropertyName, const struct FSinglePropertyParams& InitParams )
{
	return CreateSinglePropertyImpl(nullptr, InStruct, InPropertyName, InitParams);
}

TSharedPtr<class ISinglePropertyView> FPropertyEditorModule::CreateSinglePropertyImpl(UObject* InObject, const TSharedPtr<IStructureDataProvider>& InStruct, FName InPropertyName, const struct FSinglePropertyParams& InitParams)
{
	// Compact the list of detail view instances
	CompactSinglePropertyViewArray();

	TSharedRef<SSingleProperty> Property =
		SNew(SSingleProperty)
		.Object(InObject)
		.StructData(InStruct)
		.PropertyName(InPropertyName)
		.NamePlacement(InitParams.NamePlacement)
		.NameOverride(InitParams.NameOverride)
		.NotifyHook(InitParams.NotifyHook)
		.PropertyFont(InitParams.Font)
		.bShouldHideAssetThumbnail(InitParams.bHideAssetThumbnail)
		.bShouldHideResetToDefault(InitParams.bHideResetToDefault);

	if (Property->HasValidProperty())
	{
		AllSinglePropertyViews.Add(Property);

		return Property;
	}

	return nullptr;
}

void FPropertyEditorModule::CompactSinglePropertyViewArray()
{
	for( int32 ViewIndex = 0; ViewIndex < AllSinglePropertyViews.Num(); ++ViewIndex )
	{
		if (!AllSinglePropertyViews[ViewIndex].IsValid())
		{
			AllSinglePropertyViews.RemoveAtSwap(ViewIndex);
			--ViewIndex;
		}
	}
}

TSharedRef< IPropertyTable > FPropertyEditorModule::CreatePropertyTable()
{
	return MakeShareable( new FPropertyTable() );
}

TSharedRef< SWidget > FPropertyEditorModule::CreatePropertyTableWidget( const TSharedRef< class IPropertyTable >& PropertyTable )
{
	return SNew( SPropertyTable, PropertyTable );
}

TSharedRef< SWidget > FPropertyEditorModule::CreatePropertyTableWidget( const TSharedRef< IPropertyTable >& PropertyTable, const TArray< TSharedRef< class IPropertyTableCustomColumn > >& Customizations )
{
	return SNew( SPropertyTable, PropertyTable )
		.ColumnCustomizations( Customizations );
}

TSharedRef< IPropertyTableWidgetHandle > FPropertyEditorModule::CreatePropertyTableWidgetHandle( const TSharedRef< IPropertyTable >& PropertyTable )
{
	TSharedRef< FPropertyTableWidgetHandle > FWidgetHandle = MakeShareable( new FPropertyTableWidgetHandle( SNew( SPropertyTable, PropertyTable ) ) );

	TSharedRef< IPropertyTableWidgetHandle > IWidgetHandle = StaticCastSharedRef<IPropertyTableWidgetHandle>(FWidgetHandle);

	 return IWidgetHandle;
}

TSharedRef< IPropertyTableWidgetHandle > FPropertyEditorModule::CreatePropertyTableWidgetHandle( const TSharedRef< IPropertyTable >& PropertyTable, const TArray< TSharedRef< class IPropertyTableCustomColumn > >& Customizations )
{
	TSharedRef< FPropertyTableWidgetHandle > FWidgetHandle = MakeShareable( new FPropertyTableWidgetHandle( SNew( SPropertyTable, PropertyTable )
		.ColumnCustomizations( Customizations )));

	TSharedRef< IPropertyTableWidgetHandle > IWidgetHandle = StaticCastSharedRef<IPropertyTableWidgetHandle>(FWidgetHandle);

	 return IWidgetHandle;
}

TSharedRef< IPropertyTableCellPresenter > FPropertyEditorModule::CreateTextPropertyCellPresenter(const TSharedRef< class FPropertyNode >& InPropertyNode, const TSharedRef< class IPropertyTableUtilities >& InPropertyUtilities, 
																								 const FSlateFontInfo* InFontPtr /* = NULL */ , const TSharedPtr< IPropertyTableCell >& InCell /* = nullptr */)
{
	FSlateFontInfo InFont;

	if (InFontPtr == NULL)
	{
		// Encapsulating reference to Private file PropertyTableConstants.h
		InFont = FAppStyle::GetFontStyle( PropertyTableConstants::NormalFontStyle );
	}
	else
	{
		InFont = *InFontPtr;
	}

	TSharedRef< FPropertyEditor > PropertyEditor = FPropertyEditor::Create( InPropertyNode, InPropertyUtilities );
	return MakeShareable( new FTextPropertyTableCellPresenter( PropertyEditor, InPropertyUtilities, InFont, InCell) );
}

FStructProperty* FPropertyEditorModule::RegisterStructProperty(const UStruct* StructClass)
{
	check(StructClass);

	const FName StructName = StructClass->GetFName();
	FStructProperty* StructProperty = RegisteredStructToProxyMap.FindRef(StructName);

	if(!StructProperty)
	{
		if (!StructOnScopePropertyOwner)
		{
			// Create a container for all StructOnScope property objects.
			// It's important that this container is the owner of these properties and maintains a linked list 
			// to all of the properties created here. This is automatically handled by the specialized property constructor.
			StructOnScopePropertyOwner = NewObject<UStruct>(GetTransientPackage(), TEXT("StructOnScope"), RF_Transient);
			StructOnScopePropertyOwner->AddToRoot();
		}
		UScriptStruct* InnerStruct = CastChecked<UScriptStruct>(const_cast<UStruct*>(StructClass));
		StructProperty = new FStructProperty(StructOnScopePropertyOwner, *MakeUniqueObjectName(StructOnScopePropertyOwner, UField::StaticClass(), InnerStruct->GetFName()).ToString());
		StructProperty->Struct = InnerStruct;
		StructProperty->SetElementSize(StructClass->GetStructureSize());
		StructOnScopePropertyOwner->AddCppProperty(StructProperty);

		RegisteredStructToProxyMap.Add(StructName, StructProperty);
	}

	return StructProperty;
}

TSharedRef< FAssetEditorToolkit > FPropertyEditorModule::CreatePropertyEditorToolkit(const TSharedPtr< class IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit )
{
	return FPropertyEditorToolkit::CreateEditor(EToolkitMode::Standalone, InitToolkitHost, ObjectToEdit);
}


TSharedRef< FAssetEditorToolkit > FPropertyEditorModule::CreatePropertyEditorToolkit( const TSharedPtr< IToolkitHost >& InitToolkitHost, const TArray< UObject* >& ObjectsToEdit )
{
	return FPropertyEditorToolkit::CreateEditor(EToolkitMode::Standalone, InitToolkitHost, ObjectsToEdit);
}

TSharedRef< FAssetEditorToolkit > FPropertyEditorModule::CreatePropertyEditorToolkit(const TSharedPtr< IToolkitHost >& InitToolkitHost, const TArray< TWeakObjectPtr< UObject > >& ObjectsToEdit )
{
	TArray< UObject* > RawObjectsToEdit;
	for( auto ObjectIter = ObjectsToEdit.CreateConstIterator(); ObjectIter; ++ObjectIter )
	{
		RawObjectsToEdit.Add( ObjectIter->Get() );
	}

	return FPropertyEditorToolkit::CreateEditor(EToolkitMode::Standalone, InitToolkitHost, RawObjectsToEdit );
}

TSharedRef<IPropertyChangeListener> FPropertyEditorModule::CreatePropertyChangeListener()
{
	return MakeShareable( new FPropertyChangeListener );
}

void FPropertyEditorModule::RegisterCustomClassLayout( FName ClassName, FOnGetDetailCustomizationInstance DetailLayoutDelegate, FRegisterCustomClassLayoutParams Params )
{
	if (ClassName != NAME_None)
	{
		FDetailLayoutCallback Callback;
		Callback.DetailLayoutDelegate = DetailLayoutDelegate;
		Callback.Order = Params.OptionalOrder.Get(ClassNameToDetailLayoutNameMap.Num());

		ClassNameToDetailLayoutNameMap.Add(ClassName, Callback);
	}
}


void FPropertyEditorModule::UnregisterCustomClassLayout( FName ClassName )
{
	if (ClassName.IsValid() && (ClassName != NAME_None))
	{
		ClassNameToDetailLayoutNameMap.Remove(ClassName);
	}
}


void FPropertyEditorModule::RegisterCustomPropertyTypeLayout( FName PropertyTypeName, FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate, TSharedPtr<IPropertyTypeIdentifier> Identifier)
{
	if( PropertyTypeName != NAME_None )
	{
		FPropertyTypeLayoutCallback Callback;
		Callback.PropertyTypeLayoutDelegate = PropertyTypeLayoutDelegate;
		Callback.PropertyTypeIdentifier = Identifier;

		FPropertyTypeLayoutCallbackList* LayoutCallbacks = GlobalPropertyTypeToLayoutMap.Find(PropertyTypeName);
		if (LayoutCallbacks)
		{
			LayoutCallbacks->Add(Callback);
		}
		else
		{
			FPropertyTypeLayoutCallbackList NewLayoutCallbacks;
			NewLayoutCallbacks.Add(Callback);
			GlobalPropertyTypeToLayoutMap.Add(PropertyTypeName, NewLayoutCallbacks);
		}
	}
}

void FPropertyEditorModule::UnregisterCustomPropertyTypeLayout( FName PropertyTypeName, TSharedPtr<IPropertyTypeIdentifier> Identifier)
{
	if (!PropertyTypeName.IsValid() || (PropertyTypeName == NAME_None))
	{
		return;
	}

	FPropertyTypeLayoutCallbackList* LayoutCallbacks = GlobalPropertyTypeToLayoutMap.Find(PropertyTypeName);

	if (LayoutCallbacks)
	{
		LayoutCallbacks->Remove(Identifier);
	}
}

FDelegateHandle FPropertyEditorModule::RegisterPropertyHandleLayoutOverride(const FName PropertyTypeName,
	const FPropertyHandleLayoutOverride& Delegate)
{
	PropertyHandleLayoutOverrides.Add(PropertyTypeName, Delegate);
	return Delegate.GetHandle();
}

void FPropertyEditorModule::UnregisterPropertyHandleLayoutOverride(const FDelegateHandle DelegateHandle)
{
	for (auto It = PropertyHandleLayoutOverrides.CreateIterator(); It; ++It)
	{
		if (It->Value.GetHandle() == DelegateHandle)
		{
			It.RemoveCurrent();
		}
	}
}

void FPropertySection::AddCategory(FName CategoryName)
{
	checkf(!CategoryName.ToString().Contains(TEXT("|")), TEXT("Cannnot register a section mapping for a subcategory. Section: '%s', Category: '%s'"), *Name.ToString(), *CategoryName.ToString());

	// Remove all spaces - customization authors will probably write "Static Mesh", but internally it's stored as "StaticMesh"
	FString CategoryString = CategoryName.ToString();
	CategoryString.RemoveSpacesInline();
	CategoryName = FName(*CategoryString);

	AddedCategories.Add(CategoryName);
	RemovedCategories.Remove(CategoryName);
}

void FPropertySection::RemoveCategory(FName CategoryName)
{
	checkf(!CategoryName.ToString().Contains(TEXT("|")), TEXT("Cannnot register a section mapping for a subcategory. Section: '%s', Category: '%s'"), *Name.ToString(), *CategoryName.ToString());

	// Remove all spaces - customization authors will probably write "Static Mesh", but internally it's stored as "StaticMesh"
	FString CategoryString = CategoryName.ToString();
	CategoryString.RemoveSpacesInline();
	CategoryName = FName(*CategoryString);

	RemovedCategories.Add(CategoryName);
	AddedCategories.Remove(CategoryName);
}

bool FPropertySection::HasAddedCategory(FName CategoryName) const
{
	FString CategoryString = CategoryName.ToString();
	CategoryString.RemoveSpacesInline();

	return AddedCategories.Contains(*CategoryString);
}

bool FPropertySection::HasRemovedCategory(FName CategoryName) const
{
	FString CategoryString = CategoryName.ToString();
	CategoryString.RemoveSpacesInline();

	return RemovedCategories.Contains(*CategoryString);
}

FClassSectionMapping::FClassSectionMapping(FName InClassName) :
	ClassName(InClassName)
{

}

TSharedPtr<FPropertySection> FClassSectionMapping::FindSection(FName SectionName) const
{
	const TSharedPtr<FPropertySection>* Section = DefinedSections.Find(SectionName);
	if (Section == nullptr)
	{
		return TSharedPtr<FPropertySection>();
	}

	return *Section;
}

TSharedRef<FPropertySection> FClassSectionMapping::FindOrAddSection(FName SectionName, FText DisplayName, const int32 Order)
{
	TSharedPtr<FPropertySection> Section = FindSection(SectionName);
	if (!Section.IsValid())
	{
		Section = MakeShared<FPropertySection>(SectionName, DisplayName, Order);
		DefinedSections.Add(SectionName, Section);
	}

	return Section.ToSharedRef();
}

void FClassSectionMapping::RemoveSection(FName SectionName)
{
	DefinedSections.Remove(SectionName);
}

bool FClassSectionMapping::GetSectionsForCategory(FName CategoryName, TArray<TSharedPtr<FPropertySection>>& OutSections) const 
{
	bool bModified = false;

	for (const TPair<FName, TSharedPtr<FPropertySection>>& Pair : DefinedSections)
	{
		if (Pair.Value->HasAddedCategory(CategoryName))
		{
			OutSections.Add(Pair.Value);
			bModified = true;
		}

		if (Pair.Value->HasRemovedCategory(CategoryName))
		{
			// if this class removes a category, then we need to remove all previously-added sections of the same name
			// this is because superstructs are added before inherited structs, and you can remove categories from sections in more-derived classes
			for (int32 Idx = 0; Idx < OutSections.Num(); ++Idx)
			{
				if (OutSections[Idx]->GetName() == Pair.Key)
				{
					OutSections.RemoveAt(Idx);
					--Idx;
				}
			}

			bModified = true;
		}
	}

	return bModified;
}


void FPropertyEditorModule::RemoveSection(FName ClassName, FName SectionName)
{
	TSharedPtr<FClassSectionMapping>* ClassMapping = ClassSectionMappings.Find(ClassName);
	if (ClassMapping != nullptr)
	{
		(*ClassMapping)->RemoveSection(SectionName);
	}
}


TSharedRef<FPropertySection> FPropertyEditorModule::FindOrCreateSection(FName ClassName, FName SectionName, FText DisplayName, const int32 Order)
{
	checkf(!ClassName.IsNone(), TEXT("Invalid class name given."));
	checkf(!SectionName.IsNone(), TEXT("Invalid section name given."));

	TSharedPtr<FClassSectionMapping> ClassMapping;

	TSharedPtr<FClassSectionMapping>* ExistingMapping = ClassSectionMappings.Find(ClassName);
	if (ExistingMapping == nullptr)
	{
		ClassMapping = ClassSectionMappings.Add(ClassName, MakeShared<FClassSectionMapping>(ClassName));
	}
	else
	{
		ClassMapping = *ExistingMapping;
	}

	return ClassMapping->FindOrAddSection(SectionName, DisplayName, Order);
}

TArray<TSharedPtr<FPropertySection>> FPropertyEditorModule::FindSectionsForCategory(const UStruct* Struct, FName CategoryName) const
{
	TArray<TSharedPtr<FPropertySection>> Sections;
	
	// remove all spaces - customization authors will probably write "Static Mesh", but internally it's stored as "StaticMesh"
	FString CategoryString = CategoryName.ToString();
	CategoryString.RemoveSpacesInline();
	CategoryName = FName(*CategoryString);

	if (Struct != nullptr)
	{
		TSet<const UStruct*> SearchedStructs;
		FindSectionsForCategoryHelper(Struct, CategoryName, Sections, SearchedStructs);
	}

	return MoveTemp(Sections);
}

void FPropertyEditorModule::FindSectionsForCategoryHelper(const UStruct* Struct, FName CategoryName, TArray<TSharedPtr<FPropertySection>>& OutSections, TSet<const UStruct*>& SearchedStructs) const
{
	if (Struct == nullptr)
	{
		return;
	}

	bool bAlreadySearched = false;
	SearchedStructs.Add(Struct, &bAlreadySearched);

	if (bAlreadySearched)
	{
		return;
	}

	// check this struct's super
	FindSectionsForCategoryHelper(Struct->GetSuperStruct(), CategoryName, OutSections, SearchedStructs);

	// check all inline object properties' sections
	for (TFieldIterator<FObjectPropertyBase> It(Struct); It; ++It)
	{
		const FObjectPropertyBase* Property = *It;
		if (Property->HasAnyPropertyFlags(CPF_InstancedReference | CPF_ContainsInstancedReference))
		{
			FindSectionsForCategoryHelper(Property->PropertyClass, CategoryName, OutSections, SearchedStructs);
		}
	}

	// check this class' sections
	const TSharedPtr<FClassSectionMapping>* SectionMapping = ClassSectionMappings.Find(Struct->GetFName());
	if (SectionMapping != nullptr)
	{
		(*SectionMapping)->GetSectionsForCategory(CategoryName, OutSections);
	}
}

void FPropertyEditorModule::RegisterMenus()
{
	// Owner will be used for cleanup in call to UToolMenus::UnregisterOwner
	FToolMenuOwnerScoped OwnerScoped(this);

	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return;		
	}

	// Property row context menu
	{
		static FName MenuName = UE::PropertyEditor::RowContextMenuName;
		if (!ToolMenus->IsMenuRegistered(MenuName))
		{
			UToolMenu* Menu = ToolMenus->RegisterMenu(MenuName, NAME_None, EMultiBoxType::Menu, false);
			Menu->AddSection(
				TEXT("Expansion"),
				NSLOCTEXT("PropertyView", "ExpansionHeading", "Expansion"),
				FToolMenuInsert(TEXT("Edit"), EToolMenuInsertType::Before));
		
			Menu->AddSection(TEXT("Edit"), NSLOCTEXT("PropertyView", "EditHeading", "Edit"));

			Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateStatic(&FPropertyEditorModule::PopulateRowContextMenu));
		}
	}
}

void FPropertyEditorModule::PopulateRowContextMenu(UToolMenu* InToolMenu)
{
	if (!InToolMenu)
	{
		return;
	}
				
	const UDetailRowMenuContextPrivate* MenuContext = InToolMenu->FindContext<UDetailRowMenuContextPrivate>();
	if (!MenuContext)
	{
		return;
	}

	const TSharedPtr<SDetailTableRowBase> RowContext = MenuContext->GetRowWidget<SDetailTableRowBase>();
	if (!RowContext.IsValid())
	{
		return;
	}

	RowContext->PopulateContextMenu(InToolMenu);
}

void FPropertyEditorModule::GetAllSections(const UStruct* Struct, TArray<TSharedPtr<FPropertySection>>& OutSections) const
{
	if (Struct == nullptr)
	{
		return;
	}

	TSet<const UStruct*> ProcessedStructs;
	GetAllSectionsHelper(Struct, OutSections, ProcessedStructs);
}

void FPropertyEditorModule::GetAllSectionsHelper(const UStruct* Struct, TArray<TSharedPtr<FPropertySection>>& OutSections, TSet<const UStruct*>& ProcessedStructs) const
{
	if (Struct == nullptr)
	{
		return;
	}

	bool bAlreadyProcessed = false;
	ProcessedStructs.Add(Struct, &bAlreadyProcessed);

	if (bAlreadyProcessed)
	{
		return;
	}

	// add all sections for this class' super-struct
	GetAllSectionsHelper(Struct->GetSuperStruct(), OutSections, ProcessedStructs);

	// add all sections from struct properties
	for (TFieldIterator<FStructProperty> It(Struct); It; ++It)
	{
		const FStructProperty* Property = *It;
		GetAllSectionsHelper(Property->Struct, OutSections, ProcessedStructs);
	}

	// add all sections from inline object properties
	for (TFieldIterator<FObjectPropertyBase> It(Struct); It; ++It)
	{
		const FObjectPropertyBase* Property = *It;
		if (Property->HasAnyPropertyFlags(CPF_InstancedReference | CPF_ContainsInstancedReference))
		{
			GetAllSectionsHelper(Property->PropertyClass, OutSections, ProcessedStructs);
		}
	}

	// add all sections defined for this class
	const TSharedPtr<FClassSectionMapping>* SectionMapping = ClassSectionMappings.Find(Struct->GetFName());
	if (SectionMapping != nullptr)
	{
		for (const TPair<FName, TSharedPtr<FPropertySection>>& Pair : (*SectionMapping)->DefinedSections)
		{
			OutSections.Add(Pair.Value);
		}
	}
}

bool FPropertyEditorModule::HasUnlockedDetailViews() const
{
	uint32 NumUnlockedViews = 0;
	for( int32 ViewIndex = 0; ViewIndex < AllDetailViews.Num(); ++ViewIndex )
	{
		const TWeakPtr<SDetailsView>& DetailView = AllDetailViews[ ViewIndex ];
		if( DetailView.IsValid() )
		{
			TSharedPtr<SDetailsView> DetailViewPin = DetailView.Pin();
			if( DetailViewPin->IsUpdatable() &&  !DetailViewPin->IsLocked() )
			{
				return true;
			}
		}
	}

	return false;
}

/**
 * Refreshes property windows with a new list of objects to view
 * 
 * @param NewObjectList	The list of objects each property window should view
 */
void FPropertyEditorModule::UpdatePropertyViews( const TArray<UObject*>& NewObjectList )
{
	DestroyColorPicker();

	TSet<AActor*> ValidObjects;
	
	for( int32 ViewIndex = 0; ViewIndex < AllDetailViews.Num(); ++ViewIndex )
	{
		if( AllDetailViews[ ViewIndex ].IsValid() )
		{
			TSharedPtr<SDetailsView> DetailViewPin = AllDetailViews[ ViewIndex ].Pin();
			if( DetailViewPin->IsUpdatable() )
			{
				if( !DetailViewPin->IsLocked() )
				{
					DetailViewPin->SetObjects(NewObjectList, true);
				}
				else
				{
					DetailViewPin->ForceRefresh();
				}
			}
		}
	}
}

void FPropertyEditorModule::ReplaceViewedObjects( const TMap<UObject*, UObject*>& OldToNewObjectMap )
{
	// Replace objects from detail views
	for( int32 ViewIndex = 0; ViewIndex < AllDetailViews.Num(); ++ViewIndex )
	{
		if( AllDetailViews[ ViewIndex ].IsValid() )
		{
			TSharedPtr<SDetailsView> DetailViewPin = AllDetailViews[ ViewIndex ].Pin();

			DetailViewPin->ReplaceObjects( OldToNewObjectMap );
		}
	}

	// Replace objects from single views
	for( int32 ViewIndex = 0; ViewIndex < AllSinglePropertyViews.Num(); ++ViewIndex )
	{
		if( AllSinglePropertyViews[ ViewIndex ].IsValid() )
		{
			TSharedPtr<SSingleProperty> SinglePropPin = AllSinglePropertyViews[ ViewIndex ].Pin();

			SinglePropPin->ReplaceObjects( OldToNewObjectMap );
		}
	}
}

void FPropertyEditorModule::RemoveDeletedObjects( TArray<UObject*>& DeletedObjects )
{
	DestroyColorPicker();

	// remove deleted objects from detail views
	for( int32 ViewIndex = 0; ViewIndex < AllDetailViews.Num(); ++ViewIndex )
	{
		if( AllDetailViews[ ViewIndex ].IsValid() )
		{
			TSharedPtr<SDetailsView> DetailViewPin = AllDetailViews[ ViewIndex ].Pin();

			DetailViewPin->RemoveDeletedObjects( DeletedObjects );
		}
	}

	// remove deleted object from single property views
	for( int32 ViewIndex = 0; ViewIndex < AllSinglePropertyViews.Num(); ++ViewIndex )
	{
		if( AllSinglePropertyViews[ ViewIndex ].IsValid() )
		{
			TSharedPtr<SSingleProperty> SinglePropPin = AllSinglePropertyViews[ ViewIndex ].Pin();

			SinglePropPin->RemoveDeletedObjects( DeletedObjects );
		}
	}
}

bool FPropertyEditorModule::IsCustomizedStruct(const UStruct* Struct, const FCustomPropertyTypeLayoutMap& InstancePropertyTypeLayoutMap) const
{
	bool bFound = false;
	if (Struct && !Struct->IsA<UUserDefinedStruct>())
	{
		bFound = InstancePropertyTypeLayoutMap.Contains( Struct->GetFName() );
		if( !bFound )
		{
			bFound = GlobalPropertyTypeToLayoutMap.Contains( Struct->GetFName() );
		}
		
		if( !bFound )
		{
			static const FName NAME_PresentAsTypeMetadata(TEXT("PresentAsType"));
			if (const FString* DisplayType = Struct->FindMetaData(NAME_PresentAsTypeMetadata))
			{
				// try finding DisplayType instead
				bFound = InstancePropertyTypeLayoutMap.Contains( FName(*DisplayType) );
				if( !bFound )
				{
					bFound = GlobalPropertyTypeToLayoutMap.Contains( FName(*DisplayType) );
				}
			}
		}
	}
	
	return bFound;
}

FPropertyTypeLayoutCallback FPropertyEditorModule::GetPropertyTypeCustomization(const FProperty* Property, const IPropertyHandle& PropertyHandle, const FCustomPropertyTypeLayoutMap& InstancedPropertyTypeLayoutMap)
{
	if( Property )
	{
		const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
		bool bStructProperty = StructProperty && StructProperty->Struct;
		const bool bUserDefinedStruct = bStructProperty && StructProperty->Struct->IsA<UUserDefinedStruct>();
		bStructProperty &= !bUserDefinedStruct;

		const UEnum* Enum = nullptr;

		if (const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
		{
			Enum = ByteProperty->Enum;
		}
		else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			Enum = EnumProperty->GetEnum();
		}

		if (Enum && Enum->IsA<UUserDefinedEnum>())
		{
			Enum = nullptr;
		}

		const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property);
		const bool bObjectProperty = ObjectProperty != NULL && ObjectProperty->PropertyClass != NULL;

		FName PropertyTypeName;
		if( bStructProperty )
		{
			PropertyTypeName = StructProperty->Struct->GetFName();
		}
		else if( Enum )
		{
			PropertyTypeName = Enum->GetFName();
		}
		else if ( bObjectProperty )
		{
			// Try to find the PropertyClass as common base class of the current instances if possible.
			// That way we show the selected class' customization, not the based class customization.
			// If no instances are present, use the base class from the property. 
			UClass* InstanceBaseClass = nullptr;
			PropertyHandle.EnumerateConstRawData([&InstanceBaseClass, ObjectProperty](const void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
			{
				if (!RawData)
				{
					return true;
				}

				if (const UObject* Object = ObjectProperty->GetObjectPropertyValue(RawData))
				{
					UClass* Class = Object->GetClass();
					InstanceBaseClass = InstanceBaseClass ? UClass::FindCommonBase(InstanceBaseClass, Class) : Class;
				}
				return true;
			});
			const UClass* PropertyClass = InstanceBaseClass ? InstanceBaseClass : ObjectProperty->PropertyClass.Get();
			
			while (PropertyClass)
			{
				const FPropertyTypeLayoutCallback& Callback = FindPropertyTypeLayoutCallback(PropertyClass->GetFName(), PropertyHandle, InstancedPropertyTypeLayoutMap);
				if (Callback.IsValid())
				{
					return Callback;
				}

				PropertyClass = PropertyClass->GetSuperClass();
			}
		}
		else
		{
			PropertyTypeName = Property->GetClass()->GetFName();
		}

		return FindPropertyTypeLayoutCallback(PropertyTypeName, PropertyHandle, InstancedPropertyTypeLayoutMap);
	}

	return FPropertyTypeLayoutCallback();
}

FPropertyTypeLayoutCallback FPropertyEditorModule::FindPropertyTypeLayoutCallback(FName PropertyTypeName, const IPropertyHandle& PropertyHandle, const FCustomPropertyTypeLayoutMap& InstancedPropertyTypeLayoutMap)
{
	if (PropertyTypeName != NAME_None)
	{
		for (auto It = PropertyHandleLayoutOverrides.CreateConstKeyIterator(PropertyTypeName); It; ++It)
		{
			const FName TypeOverrideName = It->Value.Execute(PropertyHandle);
			if (TypeOverrideName != NAME_None)
			{
				PropertyTypeName = TypeOverrideName;
				break;
			}
		}

		const FPropertyTypeLayoutCallbackList* LayoutCallbacks = InstancedPropertyTypeLayoutMap.Find( PropertyTypeName );
	
		if( !LayoutCallbacks )
		{
			LayoutCallbacks = GlobalPropertyTypeToLayoutMap.Find(PropertyTypeName);
		}
		
		if( !LayoutCallbacks )
		{
			if (const FStructProperty* AsStructProperty = CastField<FStructProperty>(PropertyHandle.GetProperty()))
			{
				static const FName NAME_PresentAsTypeMetadata(TEXT("PresentAsType"));
				if (const FString* DisplayType = AsStructProperty->Struct->FindMetaData(NAME_PresentAsTypeMetadata))
				{
					if (UE::FPropertyTypeNameBuilder Type; Type.TryParse(*DisplayType))
					{
						const FName DisplayStruct = Type.Build().GetName();
						// try finding DisplayType instead
						LayoutCallbacks = InstancedPropertyTypeLayoutMap.Find(DisplayStruct);
		
						if( !LayoutCallbacks )
						{
							LayoutCallbacks = GlobalPropertyTypeToLayoutMap.Find(DisplayStruct);
						}
					}
					
				}
			}
		}

		if ( LayoutCallbacks )
		{
			const FPropertyTypeLayoutCallback& Callback = LayoutCallbacks->Find(PropertyHandle);
			return Callback;
		}
	}

	return FPropertyTypeLayoutCallback();
}

TSharedRef<class IStructureDetailsView> FPropertyEditorModule::CreateStructureDetailView(const FDetailsViewArgs& DetailsViewArgs, const FStructureDetailsViewArgs& StructureDetailsViewArgs, const FText& CustomName)
{
	TSharedRef<SStructureDetailsView> DetailView =
		SNew(SStructureDetailsView)
		.DetailsViewArgs(DetailsViewArgs)
		.CustomName(CustomName);

	struct FStructureDetailsViewFilter
	{
		static bool HasFilter( const FStructureDetailsViewArgs InStructureDetailsViewArgs )
		{
			const bool bShowEverything = InStructureDetailsViewArgs.bShowObjects 
				&& InStructureDetailsViewArgs.bShowAssets 
				&& InStructureDetailsViewArgs.bShowClasses 
				&& InStructureDetailsViewArgs.bShowInterfaces;
			return !bShowEverything;
		}

		static bool PassesFilter( const FPropertyAndParent& PropertyAndParent, const FStructureDetailsViewArgs InStructureDetailsViewArgs )
		{
			const auto ArrayProperty = CastField<FArrayProperty>(&PropertyAndParent.Property);
			const auto SetProperty = CastField<FSetProperty>(&PropertyAndParent.Property);
			const auto MapProperty = CastField<FMapProperty>(&PropertyAndParent.Property);

			// If the property is a container type, the filter should test against the type of the container's contents
			const FProperty* PropertyToTest = ArrayProperty ? ArrayProperty->Inner : &PropertyAndParent.Property;
			PropertyToTest = SetProperty ? SetProperty->ElementProp : PropertyToTest;
			PropertyToTest = MapProperty ? MapProperty->ValueProp : PropertyToTest;

			// Meta-data should always be queried off the top-level property, as the inner (for container types) is generated by UHT
			const FProperty* MetaDataProperty = &PropertyAndParent.Property;

			if( InStructureDetailsViewArgs.bShowClasses && (PropertyToTest->IsA<FClassProperty>() || PropertyToTest->IsA<FSoftClassProperty>()) )
			{
				return true;
			}

			if( InStructureDetailsViewArgs.bShowInterfaces && PropertyToTest->IsA<FInterfaceProperty>() )
			{
				return true;
			}

			const auto ObjectProperty = CastField<FObjectPropertyBase>(PropertyToTest);
			if( ObjectProperty )
			{
				if (InStructureDetailsViewArgs.bShowClasses && ObjectProperty->PropertyClass && ObjectProperty->PropertyClass->IsChildOf(UScriptStruct::StaticClass()))
				{
					return true;
				}

				if( InStructureDetailsViewArgs.bShowAssets )
				{
					// Is this an "asset" property?
					if( PropertyToTest->IsA<FSoftObjectProperty>())
					{
						return true;
					}

					// Not an "asset" property, but it may still be a property using an asset class type (such as a raw pointer)
					if( ObjectProperty->PropertyClass )
					{
						if (ensure(MetaDataProperty) && MetaDataProperty->HasMetaData(TEXT("AllowedClasses")))
						{
							return true;
						}
						else
						{
							// We can use the asset tools module to see whether this type has asset actions (which likely means it's an asset class type)
							FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
							return AssetToolsModule.Get().GetAssetTypeActionsForClass(ObjectProperty->PropertyClass).IsValid();
						}
					}
				}

				return InStructureDetailsViewArgs.bShowObjects;
			}

			return true;
		}
	};

	// Only add the filter if we need to exclude things
	if (FStructureDetailsViewFilter::HasFilter(StructureDetailsViewArgs))
	{
		DetailView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateStatic(&FStructureDetailsViewFilter::PassesFilter, StructureDetailsViewArgs));
	}

	return DetailView;
}

TSharedRef<IStructureDetailsView> FPropertyEditorModule::CreateStructureDetailView(
	const FDetailsViewArgs& DetailsViewArgs, const FStructureDetailsViewArgs& StructureDetailsViewArgs,
	TSharedPtr<FStructOnScope> StructData, const FText& CustomName)
{
	TSharedRef<IStructureDetailsView> DetailView = CreateStructureDetailView(DetailsViewArgs, StructureDetailsViewArgs, CustomName);
	DetailView->SetStructureData(StructData);
	return DetailView;
}

TSharedRef<IStructureDetailsView> FPropertyEditorModule::CreateStructureProviderDetailView(
	const FDetailsViewArgs& DetailsViewArgs, const FStructureDetailsViewArgs& StructureDetailsViewArgs,
	TSharedPtr<IStructureDataProvider> StructProvider, const FText& CustomName)
{
	TSharedRef<IStructureDetailsView> DetailView = CreateStructureDetailView(DetailsViewArgs, StructureDetailsViewArgs, CustomName);
	DetailView->SetStructureProvider(StructProvider);
	return DetailView;
}

TSharedRef<class IPropertyRowGenerator> FPropertyEditorModule::CreatePropertyRowGenerator(const struct FPropertyRowGeneratorArgs& InArgs)
{
	return MakeShared<FPropertyRowGenerator>(InArgs);
}
