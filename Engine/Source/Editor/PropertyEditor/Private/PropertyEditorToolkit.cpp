// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyEditorToolkit.h"
#include "Engine/Blueprint.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "IPropertyTableColumn.h"
#include "IPropertyTreeRow.h"
#include "IPropertyTableRow.h"
#include "SSimpleButton.h"

#include "Widgets/Docking/SDockTab.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "PropertyEditorToolkit"

const FName FPropertyEditorToolkit::ToolkitFName( TEXT( "PropertyEditor" ) );
const FName FPropertyEditorToolkit::ApplicationId( TEXT( "PropertyEditorToolkitApp" ) );
const FName FPropertyEditorToolkit::TreeTabId( TEXT( "PropertyEditorToolkit_PropertyTree" ) );
const FName FPropertyEditorToolkit::GridTabId( TEXT( "PropertyEditorToolkit_PropertyTable" ) );
const FName FPropertyEditorToolkit::TreePinAsColumnHeaderId( TEXT( "PropertyEditorToolkit_PinAsColumnHeader" ) );
const FName FPropertyEditorToolkit::DetailsTabId(TEXT("PropertyEditorToolkit_DetailsPanel"));

void FPropertyEditorToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_PropertyEditorToolkit", "Property Editor"));

	InTabManager->RegisterTabSpawner( GridTabId, FOnSpawnTab::CreateSP(this, &FPropertyEditorToolkit::SpawnTab_PropertyTable) )
		.SetDisplayName( LOCTEXT("PropertyTableTab", "Grid") )
		.SetGroup( WorkspaceMenuCategory.ToSharedRef() )
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	InTabManager->RegisterTabSpawner( TreeTabId, FOnSpawnTab::CreateSP(this, &FPropertyEditorToolkit::SpawnTab_PropertyTree) )
		.SetDisplayName( LOCTEXT("PropertiesTab", "Pinned Columns") )
		.SetGroup( WorkspaceMenuCategory.ToSharedRef() )
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "PropertyEditor.Grid.TabIcon"));

	InTabManager->RegisterTabSpawner(DetailsTabId, FOnSpawnTab::CreateSP(this, &FPropertyEditorToolkit::SpawnTab_DetailsPanel))
		.SetDisplayName(LOCTEXT("DetailsTab", "Details"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
}

void FPropertyEditorToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	InTabManager->UnregisterTabSpawner( GridTabId );
	InTabManager->UnregisterTabSpawner( TreeTabId );
	InTabManager->UnregisterTabSpawner( DetailsTabId );
}

FPropertyEditorToolkit::FPropertyEditorToolkit()
	: PropertyTree()
	, PropertyTable()
	, PathToRoot()
	, DetailsView()
{
}


TSharedPtr<FPropertyEditorToolkit> FPropertyEditorToolkit::FindExistingEditor( UObject* Object )
{
	// Find any existing property editor instances for this asset
	const TArray<IAssetEditorInstance*> Editors = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorsForAsset( Object );

	IAssetEditorInstance* const * ExistingInstance = Editors.FindByPredicate( [&]( IAssetEditorInstance* Editor ){
		return Editor->GetEditorName() == ToolkitFName;
	} );

	if( ExistingInstance )
	{
		auto* PropertyEditor = static_cast<FPropertyEditorToolkit*>( *ExistingInstance );
		return PropertyEditor->SharedThis( PropertyEditor );
	}

	return nullptr;
}

TSharedRef<FPropertyEditorToolkit> FPropertyEditorToolkit::CreateEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit )
{
	auto ExistingEditor = FindExistingEditor( ObjectToEdit );
	if( ExistingEditor.IsValid() )
	{
		ExistingEditor->FocusWindow();
		return ExistingEditor.ToSharedRef();
	}

	TSharedRef< FPropertyEditorToolkit > NewEditor( new FPropertyEditorToolkit() );

	TArray<UObject*> ObjectsToEdit;
	ObjectsToEdit.Add( ObjectToEdit );
	NewEditor->Initialize( Mode, InitToolkitHost, ObjectsToEdit );

	return NewEditor;
}


TSharedRef<FPropertyEditorToolkit> FPropertyEditorToolkit::CreateEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, const TArray<UObject*>& ObjectsToEdit )
{
	if( ObjectsToEdit.Num() == 1 )
	{
		auto ExistingEditor = FindExistingEditor( ObjectsToEdit[0] );
		if( ExistingEditor.IsValid() )
		{
			ExistingEditor->FocusWindow();
			return ExistingEditor.ToSharedRef();
		}
	}

	TSharedRef< FPropertyEditorToolkit > NewEditor( new FPropertyEditorToolkit() );
	NewEditor->Initialize( Mode, InitToolkitHost, ObjectsToEdit );

	return NewEditor;
}


void FPropertyEditorToolkit::Initialize( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, const TArray<UObject*>& ObjectsToEdit )
{
	TArray< UObject* > AdjustedObjectsToEdit;
	for( auto ObjectIter = ObjectsToEdit.CreateConstIterator(); ObjectIter; ++ObjectIter )
	{
		//@todo Remove this and instead extend the blueprints Edit Defaults editor to use a Property Table as well [12/6/2012 Justin.Sargent]
		UObject* Object = *ObjectIter;
		if ( Object->IsA( UBlueprint::StaticClass() ) )
		{
			UBlueprint* Blueprint = Cast<UBlueprint>( Object );

			// Make sure that the generated class is valid, in case the super has been removed, and this class can't be loaded.
			if( ensureMsgf(Blueprint->GeneratedClass != NULL, TEXT("Blueprint %s has no generated class"), *Blueprint->GetName()) )
			{
				AdjustedObjectsToEdit.Add( Blueprint->GeneratedClass->GetDefaultObject() );
			}
		}
		else
		{
			AdjustedObjectsToEdit.Add( Object );
		}
	}

	if(AdjustedObjectsToEdit.Num() > 0)
	{

		CreatePropertyTree();
		CreatePropertyTable();
		CreateDetailsPanel();

		PropertyTable->SetObjects(AdjustedObjectsToEdit);
		TableColumnsChanged();

		TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_PropertyEditorToolkit_Layout_v1.1")
			->AddArea
			(
				FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.65f)
					->SetHideTabWell(true)
					->AddTab(GridTabId, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.35f)
					->AddTab(TreeTabId, ETabState::OpenedTab)
					->AddTab(DetailsTabId, ETabState::OpenedTab)
					->SetForegroundTab(TreeTabId)
				)
			);

		const bool bCreateDefaultStandaloneMenu = true;
		const bool bCreateDefaultToolbar = true;
		FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, ApplicationId, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, AdjustedObjectsToEdit);

		TArray<UObject*> AdjustedObjectsToEditWeak;
		for (auto ObjectIter = AdjustedObjectsToEdit.CreateConstIterator(); ObjectIter; ++ObjectIter)
		{
			AdjustedObjectsToEditWeak.Add(*ObjectIter);
		}
		PropertyTree->SetObjectArray(AdjustedObjectsToEditWeak);

		DetailsView->SetObjects(AdjustedObjectsToEditWeak, true);

	}
}


TSharedRef<SDockTab> FPropertyEditorToolkit::SpawnTab_PropertyTree( const FSpawnTabArgs& Args ) 
{
	check( Args.GetTabId() == TreeTabId );

	TSharedRef<SDockTab> TreeToolkitTab = SNew(SDockTab)
		.Label( LOCTEXT("GenericDetailsTitle", "Pinned Columns") )
		.TabColorScale( GetTabColorScale() )
		.Content()
		[
			SNew(SBorder)
			.Padding(4.0f)
			.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
			.Content()
			[
				PropertyTree.ToSharedRef()
			]
		];	
	
	return TreeToolkitTab;
}


TSharedRef<SDockTab> FPropertyEditorToolkit::SpawnTab_PropertyTable( const FSpawnTabArgs& Args ) 
{
	check( Args.GetTabId() == GridTabId );

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
	TSharedRef<SDockTab> GridToolkitTab = SNew(SDockTab)
		.Label( LOCTEXT("GenericGridTitle", "Grid") )
		.TabColorScale( GetTabColorScale() )
		.Content()
		[
			PropertyEditorModule.CreatePropertyTableWidget( PropertyTable.ToSharedRef() )
		];	

	return GridToolkitTab;
}

TSharedRef<SDockTab> FPropertyEditorToolkit::SpawnTab_DetailsPanel(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == DetailsTabId);

	TSharedPtr<SDockTab> DetailsTab = SNew(SDockTab)
		.Label(LOCTEXT("GenericDetailsPanel", "Details"))
		[
			DetailsView.ToSharedRef()
		];

	return DetailsTab.ToSharedRef();
}


void FPropertyEditorToolkit::CreatePropertyTree()
{
	PropertyTree = SNew( SPropertyTreeViewImpl )
		.AllowFavorites( false )
		.ShowTopLevelNodes(false)
		.OnPropertyMiddleClicked( this, &FPropertyEditorToolkit::ToggleColumnForProperty )
		.ConstructExternalColumnHeaders( this, &FPropertyEditorToolkit::ConstructTreeColumns )
		.ConstructExternalColumnCell( this, &FPropertyEditorToolkit::ConstructTreeCell )
		.NameColumnWidth( 0.5f );
}


void FPropertyEditorToolkit::CreatePropertyTable()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
	PropertyTable = PropertyEditorModule.CreatePropertyTable();

	PropertyTable->OnSelectionChanged()->AddSP( this, &FPropertyEditorToolkit::GridSelectionChanged );
	PropertyTable->OnColumnsChanged()->AddSP( this, &FPropertyEditorToolkit::TableColumnsChanged );
	PropertyTable->OnRootPathChanged()->AddSP( this, &FPropertyEditorToolkit::GridRootPathChanged );
}

void FPropertyEditorToolkit::CreateDetailsPanel()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
}

void FPropertyEditorToolkit::ConstructTreeColumns( const TSharedRef< class SHeaderRow >& HeaderRow )
{
	SHeaderRow::FColumn::FArguments ColumnArgs;
	ColumnArgs
	.ColumnId( TreePinAsColumnHeaderId )
	.FixedWidth(24)
	[
		SNew(SBorder)
		.Padding( 0.0f )
		.BorderImage( FAppStyle::GetBrush( "NoBorder" ) )
		.HAlign( HAlign_Center )
		.VAlign( VAlign_Center )
		.ToolTipText( LOCTEXT("AddColumnLabel", "Push Pins to Add Columns") )
		[
			SNew( SImage )
			.Image( FAppStyle::GetBrush(TEXT("Icons.Unpinned")) )
			.ColorAndOpacity(FStyleColors::Foreground)
		]
	];

	HeaderRow->InsertColumn( ColumnArgs, 0 );
}

TSharedRef< SWidget > FPropertyEditorToolkit::ConstructTreeCell( const FName& ColumnName, const TSharedRef< IPropertyTreeRow >& Row )
{
	if ( ColumnName == TreePinAsColumnHeaderId )
	{
		const TWeakPtr<IPropertyTreeRow> RowPtr = Row;
		PinRows.Add( Row );

		return
			SNew(SSimpleButton)
			.ToolTipText(NSLOCTEXT("PropertyEditor", "ToggleColumnButtonToolTip", "Toggle Column"))
			.OnClicked( this, &FPropertyEditorToolkit::OnToggleColumnClicked, RowPtr )
			.Icon(this, &FPropertyEditorToolkit::GetToggleColumnButtonImageBrush, RowPtr);

	}

	return SNullWidget::NullWidget;
}

void FPropertyEditorToolkit::ToggleColumnForProperty( const TSharedPtr< FPropertyPath >& PropertyPath )
{
	if ( !PropertyPath.IsValid() )
	{
		return;
	}

	TSharedRef< FPropertyPath > NewPath = PropertyPath->TrimRoot( PropertyTable->GetRootPath()->GetNumProperties() );
	const TSet< TSharedRef< IPropertyTableRow > > SelectedRows = PropertyTable->GetSelectedRows();
	
	if (auto RowIter = SelectedRows.CreateConstIterator(); RowIter)
	{
		NewPath = NewPath->TrimRoot( (*RowIter)->GetPartialPath()->GetNumProperties() );
	}

	if ( NewPath->GetNumProperties() == 0 )
	{
		return;
	}

	TSharedPtr< IPropertyTableColumn > ExistingColumn;
	for( auto ColumnIter = PropertyTable->GetColumns().CreateConstIterator(); ColumnIter; ++ColumnIter )
	{
		TSharedRef< IPropertyTableColumn > Column = *ColumnIter;
		const TSharedPtr< FPropertyPath > Path = Column->GetDataSource()->AsPropertyPath();

		if ( Path.IsValid() && FPropertyPath::AreEqual( Path.ToSharedRef(), NewPath ) )
		{
			ExistingColumn = Column;
		}
	}

	if ( ExistingColumn.IsValid() )
	{
		PropertyTable->RemoveColumn( ExistingColumn.ToSharedRef() );
		const TSharedRef< FPropertyPath > ColumnPath = ExistingColumn->GetDataSource()->AsPropertyPath().ToSharedRef();
		for (int Index = PropertyPathsAddedAsColumns.Num() - 1; Index >= 0 ; Index--)
		{
			if ( FPropertyPath::AreEqual( ColumnPath, PropertyPathsAddedAsColumns[ Index ] ) )
			{
				PropertyPathsAddedAsColumns.RemoveAt( Index );
			}
		}
	}
	else
	{
		PropertyTable->AddColumn( NewPath );
		PropertyPathsAddedAsColumns.Add( NewPath );
	}
}


bool FPropertyEditorToolkit::TableHasCustomColumns() const
{
	return PropertyPathsAddedAsColumns.Num() > 0;
}

void FPropertyEditorToolkit::OnAssetsSaved(const TArray<UObject*>& SavedObjects)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyEditorModule.OnPropertyMatrixAssetsSaved().Broadcast(SavedObjects);
}

bool FPropertyEditorToolkit::IsExposedAsColumn( const TWeakPtr< IPropertyTreeRow >& Row ) const
{
	bool Result = false;

	if (Row.IsValid())
	{
		const TSharedPtr< FPropertyPath > RowPathPtr = Row.Pin()->GetPropertyPath();
		if ( RowPathPtr.IsValid() )
		{
			TSharedRef< FPropertyPath > TrimmedPath = RowPathPtr->TrimRoot( PropertyTable->GetRootPath()->GetNumProperties() );
			const TSet< TSharedRef< IPropertyTableRow > > SelectedRows = PropertyTable->GetSelectedRows();

			if (auto RowIter = SelectedRows.CreateConstIterator(); RowIter)
			{
				TrimmedPath = TrimmedPath->TrimRoot( (*RowIter)->GetPartialPath()->GetNumProperties() );
			}

			for (int Index = 0; Index < PropertyPathsAddedAsColumns.Num(); Index++)
			{
				if ( FPropertyPath::AreEqual( TrimmedPath, PropertyPathsAddedAsColumns[ Index ] ) )
				{
					Result = true;
					break;
				}
			}
		}
	}

	return Result;
}

void FPropertyEditorToolkit::TableColumnsChanged()
{
	PropertyPathsAddedAsColumns.Empty();

	for( auto ColumnIter = PropertyTable->GetColumns().CreateConstIterator(); ColumnIter; ++ColumnIter )
	{
		TSharedRef< IPropertyTableColumn > Column = *ColumnIter;
		TSharedPtr< FPropertyPath > ColumnPath = Column->GetDataSource()->AsPropertyPath();

		if ( ColumnPath.IsValid() && ColumnPath->GetNumProperties() > 0 )
		{
			PropertyPathsAddedAsColumns.Add( ColumnPath.ToSharedRef() );
		}
	}
}


void FPropertyEditorToolkit::GridSelectionChanged()
{
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	PropertyTable->GetSelectedTableObjects( SelectedObjects );

	if (SelectedObjects.Num() == 0)
	{
		// If none are selected, show all of them to match the initial open behavior
		SelectedObjects = PropertyTable->GetSelectedObjects();
	}

	TArray<UObject*> SelectedRawObjects;
	SelectedRawObjects.Reserve(SelectedObjects.Num());

	for (const TWeakObjectPtr<UObject>& Object : SelectedObjects)
	{
		SelectedRawObjects.Add(Object.Get());
	}

	PropertyTree->SetObjectArray(SelectedRawObjects);
	DetailsView->SetObjects(SelectedRawObjects, true);

	const TSet< TSharedRef< IPropertyTableRow > > SelectedRows = PropertyTable->GetSelectedRows();

	if ( SelectedRows.Num() == 1 )
	{
		if (auto RowIter = SelectedRows.CreateConstIterator(); RowIter)
		{
			PropertyTree->SetRootPath( PropertyTable->GetRootPath()->ExtendPath( (*RowIter)->GetPartialPath() ) );
		}
	}
	else if ( !FPropertyPath::AreEqual( PropertyTree->GetRootPath(), PropertyTable->GetRootPath() ) )
	{
		PropertyTree->SetRootPath( PropertyTable->GetRootPath() );
	}
}


void FPropertyEditorToolkit::GridRootPathChanged()
{
	GridSelectionChanged();
	PropertyTree->SetRootPath( PropertyTable->GetRootPath() );
}

FName FPropertyEditorToolkit::GetToolkitFName() const
{
	return ToolkitFName;
}

FText FPropertyEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Property Editor");
}

FText FPropertyEditorToolkit::GetToolkitName() const
{
	const auto& EditingObjs = GetEditingObjects();

	int32 NumEditingObjects = EditingObjs.Num();

	check( NumEditingObjects > 0 );

	if( NumEditingObjects == 1 )
	{
		const UObject* EditingObject = EditingObjs[ 0 ];

		FFormatNamedArguments Args;
		Args.Add( TEXT("ObjectName"), FText::FromString( EditingObject->GetName() ) );
		Args.Add( TEXT("ToolkitName"), GetBaseToolkitName() );
		return FText::Format( LOCTEXT("ToolkitName_SingleObject", "{ObjectName} - {ToolkitName}"), Args );
	}
	else
	{
		UClass* SharedBaseClass = NULL;
		for( int32 x = 0; x < NumEditingObjects; ++x )
		{
			UObject* Obj = EditingObjs[ x ];
			check( Obj );

			UClass* ObjClass = Cast<UClass>(Obj);
			if (ObjClass == NULL)
			{
				ObjClass = Obj->GetClass();
			}
			check( ObjClass );

			// Initialize with the class of the first object we encounter.
			if( SharedBaseClass == NULL )
			{
				SharedBaseClass = ObjClass;
			}

			// If we've encountered an object that's not a subclass of the current best baseclass,
			// climb up a step in the class hierarchy.
			while( !ObjClass->IsChildOf( SharedBaseClass ) )
			{
				SharedBaseClass = SharedBaseClass->GetSuperClass();
			}
		}

		FFormatNamedArguments Args;
		Args.Add( TEXT("NumberOfObjects"), EditingObjs.Num() );
		Args.Add( TEXT("ClassName"), FText::FromString( SharedBaseClass->GetName() ) );
		return FText::Format( LOCTEXT("ToolkitName_MultiObject", "{NumberOfObjects} {ClassName} Objects - Property Matrix Editor"), Args );
	}
}


FText FPropertyEditorToolkit::GetToolkitToolTipText() const
{
	const auto& EditingObjs = GetEditingObjects();

	int32 NumEditingObjects = EditingObjs.Num();

	check( NumEditingObjects > 0 );

	if( NumEditingObjects == 1 )
	{
		const UObject* EditingObject = EditingObjs[ 0 ];
		return FAssetEditorToolkit::GetToolTipTextForObject(EditingObject);
	}
	else
	{
		UClass* SharedBaseClass = NULL;
		for( int32 x = 0; x < NumEditingObjects; ++x )
		{
			UObject* Obj = EditingObjs[ x ];
			check( Obj );

			UClass* ObjClass = Cast<UClass>(Obj);
			if (ObjClass == NULL)
			{
				ObjClass = Obj->GetClass();
			}
			check( ObjClass );

			// Initialize with the class of the first object we encounter.
			if( SharedBaseClass == NULL )
			{
				SharedBaseClass = ObjClass;
			}

			// If we've encountered an object that's not a subclass of the current best baseclass,
			// climb up a step in the class hierarchy.
			while( !ObjClass->IsChildOf( SharedBaseClass ) )
			{
				SharedBaseClass = SharedBaseClass->GetSuperClass();
			}
		}

		FFormatNamedArguments Args;
		Args.Add( TEXT("NumberOfObjects"), NumEditingObjects );
		Args.Add( TEXT("ClassName"), FText::FromString( SharedBaseClass->GetName() ) );
		return FText::Format( LOCTEXT("ToolkitName_MultiObjectToolTip", "{NumberOfObjects} {ClassName} Objects - Property Matrix Editor"), Args );
	}
}


FReply FPropertyEditorToolkit::OnToggleColumnClicked( const TWeakPtr< IPropertyTreeRow > Row )
{
	if (Row.IsValid())
	{
		ToggleColumnForProperty( Row.Pin()->GetPropertyPath() );
	}

	return FReply::Handled();
}


const FSlateBrush* FPropertyEditorToolkit::GetToggleColumnButtonImageBrush( const TWeakPtr< IPropertyTreeRow > Row ) const
{
	if ( IsExposedAsColumn( Row ) )
	{
		return FAppStyle::GetBrush("Icons.Pinned");
	}

	return FAppStyle::GetBrush("Icons.Unpinned");
}

FString FPropertyEditorToolkit::GetWorldCentricTabPrefix() const
{
	check(0);
	return TEXT("");
}


FLinearColor FPropertyEditorToolkit::GetWorldCentricTabColorScale() const
{
	check(0);
	return FLinearColor();
}

#undef LOCTEXT_NAMESPACE
