// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Layout/Visibility.h"
#include "Animation/CurveSequence.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "PropertyPath.h"
#include "Toolkits/IToolkitHost.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "IPropertyTable.h"
#include "SPropertyTreeViewImpl.h"

class IPropertyTreeRow;

class FPropertyEditorToolkit : public FAssetEditorToolkit
{
public:

	FPropertyEditorToolkit();

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	virtual FName GetToolkitFName() const override;

	virtual FText GetBaseToolkitName() const override;

	virtual FText GetToolkitName() const override;

	virtual FText GetToolkitToolTipText() const override;

	virtual FString GetWorldCentricTabPrefix() const override;

	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	virtual bool IncludeAssetInRestoreOpenAssetsPrompt(UObject* Asset) const override { return false; }

	bool IsExposedAsColumn( const TWeakPtr< IPropertyTreeRow >& Row ) const;

	void ToggleColumnForProperty( const TSharedPtr< class FPropertyPath >& PropertyPath );

	bool TableHasCustomColumns() const;

	virtual bool IsPrimaryEditor() const override{ return false; };

	virtual void OnAssetsSaved(const TArray<UObject*>& SavedObjects) override;

public:

	static TSharedRef<FPropertyEditorToolkit> CreateEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit );

	static TSharedRef<FPropertyEditorToolkit> CreateEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, const TArray<UObject*>& ObjectsToEdit );


protected:

	/** "Find in Content Browser" is not visible in the property matrix because it only works on assets (and not on actors) */
	virtual bool IsFindInContentBrowserButtonVisible() const override { return false; }

private:
	static TSharedPtr<FPropertyEditorToolkit> FindExistingEditor( UObject* Object );

	void Initialize( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, const TArray<UObject*>& ObjectsToEdit );

	void CreatePropertyTree();

	void CreatePropertyTable();

	void CreateGridView();

	void CreateDetailsPanel();

	TSharedRef<SDockTab> SpawnTab_PropertyTree( const FSpawnTabArgs& Args );

	TSharedRef<SDockTab> SpawnTab_PropertyTable( const FSpawnTabArgs& Args ) ;

	TSharedRef<SDockTab> SpawnTab_DetailsPanel(const FSpawnTabArgs& Args);

	void GridSelectionChanged();

	void GridRootPathChanged();

	void ConstructTreeColumns( const TSharedRef< class SHeaderRow >& HeaderRow );

	TSharedRef< SWidget > ConstructTreeCell( const FName& ColumnName, const TSharedRef< class IPropertyTreeRow >& Row );

	FReply OnToggleColumnClicked( const TWeakPtr< class IPropertyTreeRow > Row );

	const FSlateBrush* GetToggleColumnButtonImageBrush( const TWeakPtr< class IPropertyTreeRow > Row ) const;

	EVisibility GetToggleColumnButtonVisibility( const TSharedRef< class IPropertyTreeRow > Row ) const;

	void TableColumnsChanged();

private:

	TSharedPtr< class SPropertyTreeViewImpl > PropertyTree;
	TSharedPtr< class IPropertyTable > PropertyTable;

	TSharedPtr< FPropertyPath > PathToRoot;

	/** Details panel */
	TSharedPtr<class IDetailsView> DetailsView;

	TArray< TSharedRef< FPropertyPath > > PropertyPathsAddedAsColumns;

	TArray< TWeakPtr<IPropertyTreeRow> > PinRows;

	static const FName ToolkitFName;

	static const FName ApplicationId;
	static const FName TreeTabId;
	static const FName GridTabId;
	static const FName DetailsTabId;
	static const FName TreePinAsColumnHeaderId;

};
