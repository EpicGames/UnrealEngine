// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyEditorUtils.h"

#include "PropertyCustomizationHelpers.h"
#include "PropertyPathHelpers.h"
#include "Algo/ForEach.h"
#include "UObject/PropertyText.h"

namespace PropertyEditorUtils
{
	
	void GetPropertyOptions(TArray<UObject*>& InOutContainers, FString& InOutPropertyPath,
		TArray<TSharedPtr<FString>>& InOutOptions)
	{
		TArray<FString> OptionsStrings;
		TArray<FText>* NullDisplayNames = nullptr;
		GetPropertyOptions(InOutContainers, InOutPropertyPath, OptionsStrings, NullDisplayNames);
		
		Algo::Transform(OptionsStrings, InOutOptions, [](const FString& InString) { return MakeShared<FString>(InString); });
	}

	struct FOptionsData
	{
		FString ValueString;
		FText DisplayName;

		friend uint32 GetTypeHash(const FOptionsData& Data)
		{
			// TODO: Is there a faster way to hash a FText?
			return HashCombine(GetTypeHash(Data.ValueString), GetTypeHash(Data.DisplayName.ToString()));
		}

		bool operator==(const FOptionsData& Other) const
		{
			return ValueString == Other.ValueString && DisplayName.IdenticalTo(Other.DisplayName, ETextIdenticalModeFlags::LexicalCompareInvariants);
		}

		bool operator!=(const FOptionsData& Other) const
		{
			return !(this->operator==(Other));
		}
	};

	void GetPropertyOptions(TArray<UObject*>& InOutContainers, FString& InOutPropertyPath,
		TArray<FString>& InOutOptions, TArray<FText>* OutDisplayNames)
	{
		// Check for external function references
		if (InOutPropertyPath.Contains(TEXT(".")))
		{
			InOutContainers.Empty();
			UFunction* GetOptionsFunction = FindObject<UFunction>(nullptr, *InOutPropertyPath, EFindObjectFlags::ExactClass);

			if (ensureMsgf(GetOptionsFunction && GetOptionsFunction->HasAnyFunctionFlags(EFunctionFlags::FUNC_Static), TEXT("Invalid GetOptions: %s"), *InOutPropertyPath))
			{
				UObject* GetOptionsCDO = GetOptionsFunction->GetOuterUClass()->GetDefaultObject();
				GetOptionsFunction->GetName(InOutPropertyPath);
				InOutContainers.Add(GetOptionsCDO);
			}
		}

		if (InOutContainers.Num() > 0)
		{
			TArray<FOptionsData> OptionIntersection;
			TSet<FOptionsData> OptionIntersectionSet;

			// Loop carried containers
			TArray<FOptionsData> LoopOptions;
			TArray<FString> StringOptions;
			TArray<FText> DisplayNameOptions;
			TArray<FName> NameOptions;
			TArray<FPropertyTextString> StringDisplayNameOptions;
			TArray<FPropertyTextFName> FNameDisplayNameOptions;
			
			// Need to find the intersection between all OptionsData.
			// If there is no OutDisplayNames or a bound function to GetOptions doesn't provide DisplayNames, then don't provide
			// display names either.
			bool bUseDisplayNames = true;
			
			for (UObject* Target : InOutContainers)
			{
				LoopOptions.Empty(LoopOptions.Num());
				StringOptions.Empty(StringOptions.Num());
				DisplayNameOptions.Empty(DisplayNameOptions.Num());
				NameOptions.Empty(NameOptions.Num());
				StringDisplayNameOptions.Empty(StringDisplayNameOptions.Num());
				FNameDisplayNameOptions.Empty(FNameDisplayNameOptions.Num());
				{
					FEditorScriptExecutionGuard ScriptExecutionGuard;

					// Test each signature of a function
					FCachedPropertyPath Path(InOutPropertyPath);

					// Handle function signature: "TArray<FString> GetOptions()"
					if (PropertyPathHelpers::GetPropertyValue(Target, Path, StringOptions))
					{
						Algo::Transform(StringOptions, LoopOptions, [](const FString& InName) { return FOptionsData{ .ValueString = InName, .DisplayName = FText() }; });;
						bUseDisplayNames = false;
					}
					// Handle function signature: "TArray<FText> GetOptions()"
					else if (PropertyPathHelpers::GetPropertyValue(Target, Path, NameOptions))
					{
						Algo::Transform(NameOptions, LoopOptions, [](const FName& InName) { return FOptionsData{ .ValueString = InName.ToString(), .DisplayName = FText() }; });;
						bUseDisplayNames = false;
					}
					else if (OutDisplayNames && bUseDisplayNames)
					{
						// Handle function signature: "TArray<FPropertyTextString> GetOptions()"
						if (PropertyPathHelpers::GetPropertyValue(Target, Path, StringDisplayNameOptions))
						{
							Algo::Transform(StringDisplayNameOptions, LoopOptions, [](const FPropertyTextString& Pair) { return FOptionsData{ .ValueString = Pair.ValueString, .DisplayName = Pair.DisplayName }; });;
						}
						// Handle function signature: "TArray<FPropertyTextFName> GetOptions()"
						else if (PropertyPathHelpers::GetPropertyValue(Target, Path, FNameDisplayNameOptions))
						{
							Algo::Transform(FNameDisplayNameOptions, LoopOptions, [](const FPropertyTextFName& Pair) { return FOptionsData{ .ValueString = Pair.ValueString.ToString(), .DisplayName = Pair.DisplayName }; });;
						}
					}
				}

				// If this is the first time there won't be any options.
				if (OptionIntersection.Num() == 0)
				{
					OptionIntersection = LoopOptions;
					OptionIntersectionSet = TSet<FOptionsData>(LoopOptions);
				}
				else
				{
					TSet<FOptionsData> LoopOptionsSet(LoopOptions);
					OptionIntersectionSet = LoopOptionsSet.Intersect(OptionIntersectionSet);
					OptionIntersection.RemoveAll([&OptionIntersectionSet](const FOptionsData& Option){ return !OptionIntersectionSet.Contains(Option); });
				}

				// If we're out of possible intersected options, we can stop.
				if (OptionIntersection.Num() == 0)
				{
					break;
				}
			}

			Algo::Transform(OptionIntersection, InOutOptions, [](const FOptionsData& Option) { return Option.ValueString; });
			if (OutDisplayNames && bUseDisplayNames)
			{
				TArray<FText>& OutDisplayNamesRef = *OutDisplayNames;
				Algo::Transform(OptionIntersection, OutDisplayNamesRef, [](const FOptionsData& Option) { return Option.DisplayName; });
			}
		}
	}

	void GetAllowedAndDisallowedClasses(const TArray<UObject*>& ObjectList, const FProperty& MetadataProperty, TArray<const UClass*>& AllowedClasses, TArray<const UClass*>& DisallowedClasses, bool bExactClass, const UClass* ObjectClass)
	{
		AllowedClasses = PropertyCustomizationHelpers::GetClassesFromMetadataString(MetadataProperty.GetOwnerProperty()->GetMetaData("AllowedClasses"));
		DisallowedClasses = PropertyCustomizationHelpers::GetClassesFromMetadataString(MetadataProperty.GetOwnerProperty()->GetMetaData("DisallowedClasses"));
		
		bool bMergeAllowedClasses = !AllowedClasses.IsEmpty();

		if (MetadataProperty.GetOwnerProperty()->HasMetaData("GetAllowedClasses"))
		{
			const FString GetAllowedClassesFunctionName = MetadataProperty.GetOwnerProperty()->GetMetaData("GetAllowedClasses");
			if (!GetAllowedClassesFunctionName.IsEmpty())
			{
				auto GetAllowedClasses = [&bMergeAllowedClasses, &AllowedClasses, &bExactClass, &DisallowedClasses, ObjectClass](UObject* InObject, const UFunction* InGetAllowedClassesFunction)-> bool
				{
					DECLARE_DELEGATE_RetVal(TArray<UClass*>, FGetAllowedClasses);
					if (!bMergeAllowedClasses)
					{
						AllowedClasses.Append(FGetAllowedClasses::CreateUFunction(InObject, InGetAllowedClassesFunction->GetFName()).Execute());
						if (AllowedClasses.IsEmpty())
						{
							// No allowed class means all classes are valid
							return true;
						}
						bMergeAllowedClasses = true;
					}
					else
					{
						TArray<UClass*> MergedClasses = FGetAllowedClasses::CreateUFunction(InObject, InGetAllowedClassesFunction->GetFName()).Execute();
						if (MergedClasses.IsEmpty())
						{
							// No allowed class means all classes are valid
							return true;
						}

						TArray<const UClass*> CurrentAllowedClassFilters = MoveTemp(AllowedClasses);
						ensure(AllowedClasses.IsEmpty());
						for (const UClass* MergedClass : MergedClasses)
						{
							// Keep classes that match both allow list
							for (const UClass* CurrentClass : CurrentAllowedClassFilters)
							{
								if (CurrentClass == MergedClass || (!bExactClass && CurrentClass->IsChildOf(MergedClass)))
								{
									AllowedClasses.Add(CurrentClass);
									break;
								}
								if (!bExactClass && MergedClass->IsChildOf(CurrentClass))
								{
									AllowedClasses.Add(MergedClass);
									break;
								}
							}
						}
						if (AllowedClasses.IsEmpty())
						{
							// An empty AllowedClasses array means that everything is allowed: in this case, forbid UObject
							DisallowedClasses.Add(ObjectClass);
							return false;
						}
					}
					return true;
				};

				// First look for a library function assuming fully qualified path, e.g. /Script/ModuleName.ClassName:FunctionName
				if(GetAllowedClassesFunctionName.Contains(TEXT(".")))
				{
					const UFunction* GetAllowedClassesFunction = FindObject<UFunction>(nullptr, *GetAllowedClassesFunctionName, EFindObjectFlags::ExactClass);
					if (ensureMsgf(GetAllowedClassesFunction && GetAllowedClassesFunction->HasAnyFunctionFlags(EFunctionFlags::FUNC_Static), TEXT("Invalid GetAllowedClasses: %s"), *GetAllowedClassesFunctionName))
					{
						UObject* GetAllowedClassesCDO = GetAllowedClassesFunction->GetOwnerClass()->GetDefaultObject();
						GetAllowedClasses(GetAllowedClassesCDO, GetAllowedClassesFunction);
					}
				}
				else
				{
					// Otherwise interrogate the object list
					for (UObject* Object : ObjectList)
					{
						const UFunction* GetAllowedClassesFunction = Object ? Object->FindFunction(*GetAllowedClassesFunctionName) : nullptr;
						if (GetAllowedClassesFunction)
						{
							if(!GetAllowedClasses(Object, GetAllowedClassesFunction))
							{
								return;
							}
						}
					}
				}
			}
		}

		if (MetadataProperty.GetOwnerProperty()->HasMetaData("GetDisallowedClasses"))
		{
			const FString GetDisallowedClassesFunctionName = MetadataProperty.GetOwnerProperty()->GetMetaData("GetDisallowedClasses");
			if (!GetDisallowedClassesFunctionName.IsEmpty())
			{
				for (UObject* Object : ObjectList)
				{
					const UFunction* GetDisallowedClassesFunction = Object ? Object->FindFunction(*GetDisallowedClassesFunctionName) : nullptr;
					if (GetDisallowedClassesFunction)
					{
						DECLARE_DELEGATE_RetVal(TArray<UClass*>, FGetDisallowedClasses);
						DisallowedClasses.Append(FGetDisallowedClasses::CreateUFunction(Object, GetDisallowedClassesFunction->GetFName()).Execute());
					}
				}
			}
		}
	}

	const FLazyName ShowObjectRootClassKey(TEXT("ShowObjectRootClass"));
}
