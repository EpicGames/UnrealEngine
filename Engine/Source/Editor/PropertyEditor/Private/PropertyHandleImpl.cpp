// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyHandleImpl.h"
#include "ClassViewerFilter.h"
#include "GameFramework/Actor.h"
#include "Editor.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "ObjectPropertyNode.h"
#include "Misc/MessageDialog.h"
#include "Misc/App.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/TextProperty.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "PropertyEditorHelpers.h"
#include "PropertyEditorUtils.h"
#include "StructurePropertyNode.h"
#include "ScopedTransaction.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Engine/Selection.h"
#include "ItemPropertyNode.h"
#include "Algo/Transform.h"

#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Types/SlateVector2.h"
#include "UObject/EnumProperty.h"
#include "UObject/FieldPathProperty.h"
#include "IDetailPropertyRow.h"
#include "IDetailsViewPrivate.h"
#include "ObjectEditorUtils.h"
#include "SResetToDefaultPropertyEditor.h"
#include "PropertyPathHelpers.h"
#include "PropertyTextUtilities.h"
#include "HAL/PlatformApplicationMisc.h"
#include "UObject/OverridableManager.h"
#include "UObject/PropertyOptional.h"
#include "SStandaloneCustomizedValueWidget.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "UObject/NonNullPropertyUtils.h"
#include "UObject/PropertyPathName.h"
#include "UObject/PropertyPathFunctions.h"
#include "UObject/PropertyStateTracking.h"

#define LOCTEXT_NAMESPACE "PropertyHandleImplementation"

static bool ShouldOwnInstance(const FObjectProperty* ObjectProperty, const FPropertyNode* ParentNode )
{
	return ObjectProperty && ParentNode && ObjectProperty->HasAnyPropertyFlags(CPF_InstancedReference) && !ParentNode->IsIgnoringInstancedReference();
}

bool IsTemplate(UObject* Obj)
{
	return Obj &&
		(Obj->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) ||
		(Obj->HasAnyFlags(RF_DefaultSubObject) && Obj->GetOuter()->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject)));
}

FPropertyValueImpl::FPropertyValueImpl( TSharedPtr<FPropertyNode> InPropertyNode, FNotifyHook* InNotifyHook, TSharedPtr<IPropertyUtilities> InPropertyUtilities )
	: PropertyNode( InPropertyNode )
	, PropertyUtilities( InPropertyUtilities )
	, NotifyHook( InNotifyHook )
	, bInteractiveChangeInProgress( false )
	, InvalidOperationError( nullptr )
{
}

void FPropertyValueImpl::EnumerateObjectsToModify(FPropertyNode* InPropertyNode, const EnumerateObjectsToModifyFuncRef& InObjectsToModifyCallback) const
{
	// EnumerateObjectsToModify catch the first ComplexNode and iterates on its number of instances to get the property address in memory.
	// A complex node can be an object node or a struct node (at this time of writing).
	// If the complex node is an object, we also extract the address of this object.
	// If the complex node is a struct, we still want to know if it is owned by an object, so we look further up to find the first object node (if it exists).
	// If it exists, we might have a complex struct owned by an object, so we try to extract the object address.
	FComplexPropertyNode* ComplexNode = InPropertyNode->FindComplexParent();
	if (ComplexNode)
	{
		const bool bIsStruct = FComplexPropertyNode::EPT_StandaloneStructure == ComplexNode->GetPropertyType();
		const int32 NumInstances = ComplexNode->GetInstancesNum();

		const FObjectPropertyNode* ObjectNode = bIsStruct ? InPropertyNode->FindObjectItemParent() : nullptr;
		const int32 NumObjectInstances = ObjectNode ? ObjectNode->GetInstancesNum() : 0;
		for (int32 Index = 0; Index < NumInstances; ++Index)
		{
			UObject* Object = nullptr;
			uint8* StructAddress = nullptr;
			if (bIsStruct)
			{
				StructAddress = ComplexNode->GetMemoryOfInstance(Index);

				// Make sure we have the same number of structs and objects, so there is a 1 for 1 mapping between the struct and its owning object.
				// If we don't, we don't have a clear way to resolve this, so we won't try to retrieve the object address (which was the previous behavior).
				if (ObjectNode && NumInstances == NumObjectInstances)
				{
					Object = ObjectNode->GetInstanceAsUObject(Index).Get();
				}
			}
			else
			{
				Object = ComplexNode->GetInstanceAsUObject(Index).Get();
				StructAddress = InPropertyNode->GetStartAddressFromObject(Object);
			}

			uint8* BaseAddress = InPropertyNode->GetValueBaseAddress(StructAddress, InPropertyNode->HasNodeFlags(EPropertyNodeFlags::IsSparseClassData), bIsStruct);

			if (!InObjectsToModifyCallback(FObjectBaseAddress(Object, StructAddress, BaseAddress, bIsStruct), Index, NumInstances))
			{
				break;
			}
		}
	}
}

void FPropertyValueImpl::GetObjectsToModify( TArray<FObjectBaseAddress>& ObjectsToModify, FPropertyNode* InPropertyNode ) const
{
	EnumerateObjectsToModify(InPropertyNode, [&](const FObjectBaseAddress& ObjectToModify, const int32 ObjectIndex, const int32 NumObjects) -> bool
	{
		if (ObjectIndex == 0)
		{
			ObjectsToModify.Reserve(ObjectsToModify.Num() + NumObjects);
		}
		ObjectsToModify.Add(ObjectToModify);
		return true;
	});
}

FPropertyAccess::Result FPropertyValueImpl::GetValueData( void*& OutAddress ) const
{
	FPropertyAccess::Result Res = FPropertyAccess::Fail;
	OutAddress = nullptr;
	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
	if (PropertyNodePin.IsValid())
	{
		uint8* ValueAddress = nullptr;
		FReadAddressList ReadAddresses;
		bool bAllValuesTheSame = PropertyNodePin->GetReadAddress( !!PropertyNodePin->HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ReadAddresses, false, true );

		if( (ReadAddresses.Num() > 0 && bAllValuesTheSame) || ReadAddresses.Num() == 1 ) 
		{
			ValueAddress = ReadAddresses.GetAddress(0);
			const FProperty* Property = PropertyNodePin->GetProperty();
			if (ValueAddress && Property)
			{
				const int32 Index = 0;
				OutAddress = ValueAddress + Index * Property->GetElementSize();
				Res = FPropertyAccess::Success;
			}
		}
		else if (ReadAddresses.Num() > 1)
		{
			Res = FPropertyAccess::MultipleValues;
		}
	}

	return Res;
}

FPropertyAccess::Result FPropertyValueImpl::GetValueEditStack(FPropertyNodeEditStack& OutStack) const
{
	FPropertyAccess::Result Res = FPropertyAccess::Fail;

	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
	if (PropertyNodePin.IsValid())
	{
		Res = PropertyNodePin->GetSingleEditStack(OutStack);
	}

	return Res;
}

FPropertyAccess::Result FPropertyValueImpl::ImportText( const FString& InValue, EPropertyValueSetFlags::Type Flags )
{
	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
	if( PropertyNodePin.IsValid() && !PropertyNodePin->IsEditConst() )
	{
		return ImportText( InValue, PropertyNodePin.Get(), Flags );
	}

	// The property node is not valid or cant be set.  If not valid it probably means this value was stored somewhere and selection changed causing the node to be destroyed
	return FPropertyAccess::Fail;
}

FString FPropertyValueImpl::GetPropertyValueArray() const
{
	FString String;
	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
	if( PropertyNodePin.IsValid() )
	{
		FReadAddressList ReadAddresses;

		bool bSingleValue = PropertyNodePin->GetReadAddress( !!PropertyNodePin->HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ReadAddresses, false );

		if( bSingleValue )
		{
			FProperty* NodeProperty = PropertyNodePin->GetProperty();
			if( NodeProperty != nullptr )
			{
				uint8* Addr = ReadAddresses.GetAddress(0);
				if( Addr )
				{
					if ( FArrayProperty* ArrayProperty = CastField<FArrayProperty>(NodeProperty) )
					{
						FScriptArrayHelper ArrayHelper(ArrayProperty, Addr);
						String = FString::Printf( TEXT("%%(%d)"), ArrayHelper.Num() );
					}
					else if ( CastField<FSetProperty>(NodeProperty) != nullptr )	
					{
						String = FString::Printf( TEXT("%%(%d)"), FScriptSetHelper::Num(Addr) );
					}
					else if (FMapProperty* MapProperty = CastField<FMapProperty>(NodeProperty))
					{
						FScriptMapHelper MapHelper(MapProperty, Addr);
						String = FString::Printf(TEXT("%%(%d)"), MapHelper.Num());
					}
					else
					{
						String = FString::Printf( TEXT("%%[%d]"), NodeProperty->ArrayDim );
					}
				}
			}
		}
		else
		{
			String = NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values").ToString();
		}
	}

	return String;
}

void LogAmbiguousArrayIndexWarning(const FPropertyNode* PropertyNode, const FString& CompetingName)
{
	FString LogMessage = TEXT("Found multiple properties in a property change chain with matching names but mismatched indices:\n");
	for (const FPropertyNode* IterationNode = PropertyNode; (IterationNode != nullptr); IterationNode = IterationNode->GetParentNode())
	{
		const FProperty* Property = IterationNode->GetProperty();
		if (Property->GetName() == CompetingName && IterationNode->GetArrayIndex() != INDEX_NONE)
		{
			LogMessage += FString::Format(TEXT(" - '{0}'\n"), { IterationNode->GetPropertyPath()});
		}
	}
	LogMessage += TEXT("This causes ambiguity in the property change notifications!");
	UE_LOGF(LogPropertyNode, Warning, "%ls", *LogMessage);
}


void FPropertyValueImpl::GenerateArrayIndexMapToObjectNode( TMap<FString,int32>& OutArrayIndexMap, FPropertyNode* PropertyNode )
{
	if( PropertyNode )
	{
		OutArrayIndexMap.Empty();
		for (FPropertyNode* IterationNode = PropertyNode; (IterationNode != nullptr); IterationNode = IterationNode->GetParentNode())
		{
			FProperty* Property = IterationNode->GetProperty();
			if (Property)
			{
				int32 NodeIndex = IterationNode->GetArrayIndex();
				if (int32* FoundIndex = OutArrayIndexMap.Find(Property->GetName()); FoundIndex)
				{
					if (*FoundIndex != INDEX_NONE && NodeIndex != INDEX_NONE && *FoundIndex != NodeIndex)
					{
						// TODO: due to the way array indices are stored, if this warning is printed, it's impossible to create a change notification that's resolvable :(
						// for example, given the following pseudo object:
						/*
							struct TestStruct
							{
								TArray<int32> MyArray;
							};

							class TestObject : UObject
							{
								TArray<TestStruct> MyArray;
							};
						*/
						// if TestObject.MyArray[0].MyArray[2] is edited, it will be ambiguous whether the key "MyArray" in the ArrayIndexMap is referring to the array in TestObject or the array in TestStruct.
						// at the moment this code sets ArrayIndexMap["MyArray"] to 2 for this example losing the index of TestObject::MyArray
						LogAmbiguousArrayIndexWarning(PropertyNode, Property->GetName());
					}

					if (*FoundIndex == INDEX_NONE)
					{
						*FoundIndex = NodeIndex;
					}
				}
				else
				{
					OutArrayIndexMap.Add(Property->GetName(), IterationNode->GetArrayIndex());
				}
			}
		}
	}
}

FPropertyAccess::Result FPropertyValueImpl::ImportText( const FString& InValue, FPropertyNode* InPropertyNode, EPropertyValueSetFlags::Type Flags )
{
	TArray<FObjectBaseAddress> ObjectsToModify;
	GetObjectsToModify(ObjectsToModify, InPropertyNode);

	TArray<FString> Values;
	for (const FObjectBaseAddress& BaseAddress : ObjectsToModify)
	{
		if (BaseAddress.BaseAddress)
		{
			Values.Add(InValue);
		}
	}

	FPropertyAccess::Result Result = FPropertyAccess::Fail;
	if (Values.Num() > 0)
	{
		Result = ImportText(ObjectsToModify, Values, InPropertyNode, Flags);
	}

	return Result;
}

FPropertyAccess::Result FPropertyValueImpl::ImportText( const TArray<FObjectBaseAddress>& InObjects, const TArray<FString>& InValues, FPropertyNode* InPropertyNode, EPropertyValueSetFlags::Type Flags )
{
	check(InPropertyNode);
	FProperty* NodeProperty = InPropertyNode->GetProperty();

	FPropertyAccess::Result Result = FPropertyAccess::Success;

	if( !NodeProperty )
	{
		// The property has been deleted out from under this
		Result = FPropertyAccess::Fail;
	}
	else if( NodeProperty->IsA<FObjectProperty>() || NodeProperty->IsA<FNameProperty>() )
	{
		// certain properties have requirements on the size of string values that can be imported.  Search for strings that are too large.
		for (const FString& Value : InValues)
		{
			if (Value.Len() >= NAME_SIZE)
			{
				Result = FPropertyAccess::Fail;
				break;
			}
		}
	}

	if( Result != FPropertyAccess::Fail )
	{
		UWorld* OldGWorld = nullptr;

		bool bIsGameWorld = false;
		// If the object we are modifying is in the PIE world, than make the PIE world the active
		// GWorld.  Assumes all objects managed by this property window belong to the same world.
		UObject* FirstObject = InObjects[0].Object;
		if (UPackage* ObjectPackage = (FirstObject ? FirstObject->GetOutermost() : nullptr))
		{
			const bool bIsPIEPackage = ObjectPackage->HasAnyPackageFlags(PKG_PlayInEditor);
			if (GUnrealEd && GUnrealEd->PlayWorld && bIsPIEPackage && !GIsPlayInEditorWorld)
			{
				OldGWorld = SetPlayInEditorWorld(GUnrealEd->PlayWorld);
				bIsGameWorld = true;
			}
		}
		///////////////

		// Send the values and assemble a list of pre/posteditchange values.
		bool bNotifiedPreChange = false;
		UObject *NotifiedObj = nullptr;
		TArray< TMap<FString,int32> > ArrayIndicesPerObject;

		const bool bTransactable = (Flags & EPropertyValueSetFlags::NotTransactable) == 0;
		const bool bFinished = ( Flags & EPropertyValueSetFlags::InteractiveChange) == 0;

		EPropertyChangeType::Type ChangeEventFlags = bFinished ? EPropertyChangeType::ValueSet : EPropertyChangeType::Interactive;
		if (Flags & EPropertyValueSetFlags::ResetToDefault)
		{
			ChangeEventFlags = ChangeEventFlags | EPropertyChangeType::ResetToDefault;
		}
		
		// List of top level objects sent to the PropertyChangedEvent
		TArray<const UObject*> TopLevelObjects;
		TopLevelObjects.Reserve(InObjects.Num());

		for (int32 ObjectIndex = 0 ; ObjectIndex < InObjects.Num() ; ++ObjectIndex)
		{	
			const FObjectBaseAddress& Cur = InObjects[ObjectIndex];
			if (Cur.BaseAddress == nullptr)
			{
				//Fully abort this procedure.  The data has changed out from under the object
				Result = FPropertyAccess::Fail;
				break;
			}

			UObject* CurObject = Cur.Object;

			if (CurObject)
			{
				TopLevelObjects.Add(CurObject);
			}
			
			const FString& NewValue = InValues[ObjectIndex];

			// Cache the value of the property before modifying it.
			FString PreviousValue;
			FPropertyTextUtilities::PropertyToTextHelper(PreviousValue, InPropertyNode, NodeProperty, Cur, PPF_ForDiff);

			// If this property is the inner-property of a container, cache the current value as well
			FString PreviousContainerValue;
			
			FPropertyNode* ParentNode = InPropertyNode->GetParentNode();
			FProperty* Property = ParentNode ? ParentNode->GetProperty() : nullptr;

			const bool bIsSparseClassData = InPropertyNode->HasNodeFlags(EPropertyNodeFlags::IsSparseClassData) != 0;
			const bool bIsStruct = Cur.bIsStruct;

			bool bIsInContainer = false;

			if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				bIsInContainer = (ArrayProperty->Inner == NodeProperty);
			}
			else if (FSetProperty* SetProperty = CastField<FSetProperty>(Property))
			{
				// If the element is part of a set, check for duplicate elements
				bIsInContainer = SetProperty->ElementProp == NodeProperty;

				if (bIsInContainer)
				{
					uint8* ValueBaseAddress = ParentNode->GetValueBaseAddress(Cur.StructAddress, bIsSparseClassData, bIsStruct);
					
					/**
					 * Checks if an element has already been added to the set
					 *
					 * @param	Helper			The set helper used to query the property.
					 * @param	InBaseAddress	The base address of the set
					 * @param	InElementValue	The element value to check for
					 *
					 * @return	True if the element is found in the set, false otherwise
					 */
					static auto HasElement = [](const FScriptSetHelper& Helper, void* InBaseAddress, const FString& InElementValue)
					{
						FProperty* ElementProp = Helper.GetElementProperty();

						void* TempElementStorage = ElementProp->AllocateAndInitializeValue();
						ON_SCOPE_EXIT
						{
							ElementProp->DestroyAndFreeValue(TempElementStorage);
						};

						for (FScriptSetHelper::FIterator It(Helper); It; ++It)
						{
							const uint8* Element = Helper.GetElementPtr(It);

							if (Element != InBaseAddress && ElementProp->ImportText_Direct(*InElementValue, TempElementStorage, nullptr, 0) && ElementProp->Identical(Element, TempElementStorage))
							{
								return true;
							}
						}

						return false;
					};

					FScriptSetHelper SetHelper(SetProperty, ValueBaseAddress);
					if (HasElement(SetHelper, Cur.BaseAddress, NewValue) &&
						(Flags & EPropertyValueSetFlags::InteractiveChange) == 0)
					{
						// Duplicate element in the set
						ShowInvalidOperationError(LOCTEXT("DuplicateSetElement", "Duplicate elements are not allowed in Set properties."));

						return FPropertyAccess::Fail;
					}
				}
			}
			else if (FMapProperty* MapProperty = CastField<FMapProperty>(Property))
			{
				bIsInContainer = MapProperty->KeyProp == NodeProperty;

				if (bIsInContainer)
				{
					uint8* ValueBaseAddress = ParentNode->GetValueBaseAddress(Cur.StructAddress, bIsSparseClassData, bIsStruct);

					/**
					 * Checks if a key in the map matches the specified key
					 *
					 * @param	Helper			The map helper used to query the property.
					 * @param	InBaseAddress	The base address of the map
					 * @param	InKeyValue		The key to find within the map
					 *
					 * @return	True if the key is found, false otherwise
					 */
					static auto HasKey = [](const FScriptMapHelper& Helper, void* InBaseAddress, const FString& InKeyValue)
					{
						FProperty* KeyProp = Helper.GetKeyProperty();

						void* TempKeyStorage = KeyProp->AllocateAndInitializeValue();
						ON_SCOPE_EXIT
						{
							KeyProp->DestroyAndFreeValue(TempKeyStorage);
						};

						for (FScriptMapHelper::FIterator It(Helper); It; ++It)
						{
							const uint8* PairPtr = Helper.GetPairPtr(It);
							const uint8* KeyPtr = KeyProp->ContainerPtrToValuePtr<const uint8>(PairPtr);

							if (KeyPtr != InBaseAddress && KeyProp->ImportText_Direct(*InKeyValue, TempKeyStorage, nullptr, 0) && KeyProp->Identical(KeyPtr, TempKeyStorage))
							{
								return true;
							}
						}

						return false;
					};

					FScriptMapHelper MapHelper(MapProperty, ValueBaseAddress);
					if (HasKey(MapHelper, Cur.BaseAddress, NewValue) && 
						(Flags & EPropertyValueSetFlags::InteractiveChange) == 0)
					{
						// Duplicate key in the map
						ShowInvalidOperationError(LOCTEXT("DuplicateMapKey", "Duplicate keys are not allowed in Map properties."));

						return FPropertyAccess::Fail;
					}
				}
				else
				{
					bIsInContainer = MapProperty->ValueProp == NodeProperty;
				}
			}

			if (bIsInContainer)
			{
				uint8* Addr = ParentNode->GetValueBaseAddress(Cur.StructAddress, bIsSparseClassData, bIsStruct);
				Property->ExportText_Direct(PreviousContainerValue, Addr, Addr, nullptr, 0);
			}

			//add on array index so we can tell which entry just changed
			ArrayIndicesPerObject.Add(TMap<FString,int32>());
			FPropertyValueImpl::GenerateArrayIndexMapToObjectNode(ArrayIndicesPerObject[ObjectIndex], InPropertyNode);

			// Check if we need to call PreEditChange on all objects.
			// Remove quotes from the original value because FName properties  
			// are wrapped in quotes before getting here. This causes the 
			// string comparison to fail even when the name is unchanged. 
			if (!bNotifiedPreChange && 
				(FCString::Strcmp(*NewValue.TrimQuotes(), *PreviousValue) != 0 || 
				(bFinished && bInteractiveChangeInProgress)))
			{
				bNotifiedPreChange = true;
				NotifiedObj = CurObject;

				if (!bInteractiveChangeInProgress)
				{
					// Begin a transaction only if we need to call PreChange
					if (GEditor && bTransactable)
					{
						// @todo: FProp
						//GEditor->BeginTransaction(TEXT("PropertyEditor"), FText::Format(NSLOCTEXT("PropertyEditor", "EditPropertyTransaction", "Edit {0}"), InPropertyNode->GetDisplayName()), NodeProperty);
						GEditor->BeginTransaction(TEXT("PropertyEditor"), FText::Format(NSLOCTEXT("PropertyEditor", "EditPropertyTransaction", "Edit {0}"), InPropertyNode->GetDisplayName()), nullptr);
					}
				}

				InPropertyNode->NotifyPreChange(NodeProperty, NotifyHook);

				if (PropertyUtilities.IsValid() && !bInteractiveChangeInProgress)
				{
					FPropertyChangedEvent StartedChangingPropertiesEvent(NodeProperty, ChangeEventFlags, MakeArrayView(TopLevelObjects));
					StartedChangingPropertiesEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);
					PropertyUtilities.Pin()->NotifyStartedChangingProperties(StartedChangingPropertiesEvent);
				}

				bInteractiveChangeInProgress = (Flags & EPropertyValueSetFlags::InteractiveChange) != 0;
			}

			const bool bWasSetOptional = !!(PropertyEditorHelpers::IsOptionalProperty(*InPropertyNode) && InPropertyNode->GetNumChildNodes());

			// Set the new value.
			EPropertyPortFlags PortFlags = (Flags & EPropertyValueSetFlags::InstanceObjects) != 0 ? PPF_InstanceSubobjects : PPF_None;
			FPropertyTextUtilities::TextToPropertyHelper(*NewValue, InPropertyNode, NodeProperty, Cur, PortFlags);

			if (Flags & EPropertyValueSetFlags::ResetToDefault)
            {
            	if (CurObject)
            	{
            		UObject* Archetype = CurObject->GetArchetype();
            		if (Archetype)
            		{
            			TSharedRef< FPropertyPath > PropertyPath = FPropertyNode::CreatePropertyPath(InPropertyNode->AsShared());
            			
            			const TArray<UE::FPropertyPathName> PathNames = PropertyPath->ToPropertyPathName();
			            
            			for (int32 i = 0; i < PathNames.Num(); i++)
            			{
            				const UE::FPropertyPathName& PathName = PathNames[i];
			            
            				const UE::FPropertyValueInContainer ArchetypeValue = UE::TryResolvePropertyPath(PathName, Archetype);
            				const UE::FPropertyValueInContainer CurrObjValue = UE::TryResolvePropertyPath(PathName, CurObject);
			            
							if (CurrObjValue && !CurrObjValue.IsDynamicContainerElement()) // initilization state tracking for container elements is not supported
							{
								if (ArchetypeValue && !ArchetypeValue.IsDynamicContainerElement()) // initilization state tracking for container elements is not supported
								{
									bool bWasSet = UE::FInitializedPropertyValueState(ArchetypeValue.Struct, ArchetypeValue.Container).IsSet(ArchetypeValue.Property, ArchetypeValue.ArrayIndex);
									UE::FInitializedPropertyValueState(CurrObjValue.Struct, CurrObjValue.Container).SetTo(bWasSet, CurrObjValue.Property, CurrObjValue.ArrayIndex);
								}
								else
								{
									UE::FInitializedPropertyValueState(CurrObjValue.Struct, CurrObjValue.Container).Clear(CurrObjValue.Property, CurrObjValue.ArrayIndex);
								}
							}
            			}
            		}
            	};
            }

			// If we are an FOptionalProperty, this check tells us whether we have changed between set/unset and 
			// thus require a re-draw of our node in the details panel UI which is accomplished by requesting a rebuild.
			if (FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(NodeProperty))
			{
				if (bWasSetOptional != OptionalProperty->IsSet(Cur.BaseAddress))
				{
					InPropertyNode->RequestRebuildChildren();
				}
			}

			// Cache the value of the property after having modified it.
			FString ValueAfterImport;
			FPropertyTextUtilities::PropertyToTextHelper(ValueAfterImport, InPropertyNode, NodeProperty, Cur, PPF_ForDiff);

			if (CurObject)
			{
				if ((CurObject->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) ||
					(CurObject->HasAnyFlags(RF_DefaultSubObject) && CurObject->GetOuter()->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))) &&
					!bIsGameWorld)
				{
					// propagate the changes to instances unless the object is using overridable serialization, 
					// the propagation is done via reinstantiation
					if (!FOverridableManager::Get().IsEnabled(CurObject))
					{
						InPropertyNode->PropagatePropertyChange(CurObject, *NewValue, PreviousContainerValue.IsEmpty() ? PreviousValue : PreviousContainerValue);
					}
				}
			}

			// If the values before and after setting the property differ, mark the object dirty.
			if (FCString::Strcmp(*PreviousValue, *ValueAfterImport) != 0)
			{
				if (CurObject && !InPropertyNode->ShouldSkipSerialization())
				{
					CurObject->MarkPackageDirty();
				}

				// For TMap and TSet, we need to rehash it in case a key was modified
				if (NodeProperty->GetOwner<FMapProperty>())
				{
					uint8* Addr = InPropertyNode->GetParentNode()->GetValueBaseAddress(Cur.StructAddress, bIsSparseClassData, bIsStruct);
					FScriptMapHelper MapHelper(NodeProperty->GetOwner<FMapProperty>(), Addr);
					MapHelper.Rehash();
				}
				else if (NodeProperty->GetOwner<FSetProperty>())
				{
					uint8* Addr = InPropertyNode->GetParentNode()->GetValueBaseAddress(Cur.StructAddress, bIsSparseClassData, bIsStruct);
					FScriptSetHelper SetHelper(NodeProperty->GetOwner<FSetProperty>(), Addr);
					SetHelper.Rehash();
				}
			}
		}
		
		FPropertyChangedEvent ChangeEvent(NodeProperty, ChangeEventFlags, MakeArrayView(TopLevelObjects));
		ChangeEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);

		// If PreEditChange was called, so should PostEditChange.
		if (bNotifiedPreChange)
		{
			// Call PostEditChange on all objects.
			InPropertyNode->NotifyPostChange( ChangeEvent, NotifyHook );

			if (bFinished)
			{
				bInteractiveChangeInProgress = false;

				if (bTransactable)
				{
					// End the transaction if we called PreChange
					GEditor->EndTransaction();
				}
			}
		}

		if (OldGWorld)
		{
			// restore the original (editor) GWorld
			RestoreEditorWorld( OldGWorld );
		}

		if (PropertyUtilities.IsValid() && !bInteractiveChangeInProgress)
		{
			InPropertyNode->FixPropertiesInEvent(ChangeEvent);
			PropertyUtilities.Pin()->NotifyFinishedChangingProperties(ChangeEvent);
		}
	}

	return Result;
}

void FPropertyValueImpl::EnumerateRawData( const IPropertyHandle::EnumerateRawDataFuncRef& InRawDataCallback )
{
	EnumerateObjectsToModify(PropertyNode.Pin().Get(), [&](const FObjectBaseAddress& ObjectToModify, const int32 ObjectIndex, const int32 NumObjects) -> bool
	{
		return InRawDataCallback(ObjectToModify.BaseAddress, ObjectIndex, NumObjects);
	});
}

void FPropertyValueImpl::EnumerateConstRawData( const IPropertyHandle::EnumerateConstRawDataFuncRef& InRawDataCallback ) const
{
	EnumerateObjectsToModify(PropertyNode.Pin().Get(), [&](const FObjectBaseAddress& ObjectToModify, const int32 ObjectIndex, const int32 NumObjects) -> bool
	{
		return InRawDataCallback(ObjectToModify.BaseAddress, ObjectIndex, NumObjects);
	});
}

void FPropertyValueImpl::AccessRawData( TArray<void*>& RawData )
{
	RawData.Empty();
	EnumerateObjectsToModify(PropertyNode.Pin().Get(), [&](const FObjectBaseAddress& ObjectToModify, const int32 ObjectIndex, const int32 NumObjects) -> bool
	{
		if (ObjectIndex == 0)
		{
			RawData.Reserve(NumObjects);
		}
		RawData.Add(ObjectToModify.BaseAddress);
		return true;
	});
}

void FPropertyValueImpl::AccessRawData( TArray<const void*>& RawData ) const
{
	RawData.Empty();
	EnumerateObjectsToModify(PropertyNode.Pin().Get(), [&](const FObjectBaseAddress& ObjectToModify, const int32 ObjectIndex, const int32 NumObjects) -> bool
	{
		if (ObjectIndex == 0)
		{
			RawData.Reserve(NumObjects);
		}
		RawData.Add(ObjectToModify.BaseAddress);
		return true;
	});
}

void FPropertyValueImpl::SetOnPropertyValueChanged( const FSimpleDelegate& InOnPropertyValueChanged )
{
	if( PropertyNode.IsValid() )
	{
		PropertyNode.Pin()->OnPropertyValueChanged().Add( InOnPropertyValueChanged );
	}
}

void FPropertyValueImpl::SetOnPropertyValueChangedWithData(const TDelegate<void(const FPropertyChangedEvent&)>& InOnPropertyValueChanged)
{
	if( PropertyNode.IsValid() )
	{
		PropertyNode.Pin()->OnPropertyValueChangedWithData().Add(InOnPropertyValueChanged);
	}
}

void FPropertyValueImpl::SetOnChildPropertyValueChanged( const FSimpleDelegate& InOnChildPropertyValueChanged )
{
	if( PropertyNode.IsValid() )
	{
		PropertyNode.Pin()->OnChildPropertyValueChanged().Add( InOnChildPropertyValueChanged );
	}
}

void FPropertyValueImpl::SetOnChildPropertyValueChangedWithData(const TDelegate<void(const FPropertyChangedEvent&)>& InOnChildPropertyValueChanged)
{
	if( PropertyNode.IsValid() )
	{
		PropertyNode.Pin()->OnChildPropertyValueChangedWithData().Add( InOnChildPropertyValueChanged );
	}
}

void FPropertyValueImpl::SetOnPropertyValuePreChange(const FSimpleDelegate& InOnPropertyValuePreChange)
{
	if (PropertyNode.IsValid())
	{
		PropertyNode.Pin()->OnPropertyValuePreChange().Add(InOnPropertyValuePreChange);
	}
}

void FPropertyValueImpl::SetOnChildPropertyValuePreChange(const FSimpleDelegate& InOnChildPropertyValuePreChange)
{
	if (PropertyNode.IsValid())
	{
		PropertyNode.Pin()->OnChildPropertyValuePreChange().Add(InOnChildPropertyValuePreChange);
	}
}

void FPropertyValueImpl::SetOnPropertyResetToDefault(const FSimpleDelegate& InOnPropertyResetToDefault)
{
	if (PropertyNode.IsValid())
	{
		PropertyNode.Pin()->OnPropertyResetToDefault().Add(InOnPropertyResetToDefault);
	}
}

void FPropertyValueImpl::SetOnCanItemChange(const FCanPropertyItemChange& InDelegate)
{
	if (PropertyNode.IsValid())
	{
		PropertyNode.Pin()->OnCanItemChangeEvent() = InDelegate;
	}
}

bool FPropertyValueImpl::CanItemChange(EPropertyChangeType::Type Type, int32 Index)
{
	if (const TSharedPtr<FPropertyNode> Node = PropertyNode.Pin())
	{
		const FCanPropertyItemChange& Event = Node->OnCanItemChangeEvent();
		if (Event.IsBound())
		{
			FCanPropertyItemChangeRetType Result = Event.Execute(Type, Index);
			if (Result.HasError())
			{
				const FPropertyChangeBlocked& ErrorEvent = Node->OnChangeBlockedEvent();
				ErrorEvent.Broadcast(Node->GetProperty(), Result.GetError());
				return false;
			}

			return Result.GetValue();
		}
	}

	return true;
}

FDelegateHandle FPropertyValueImpl::SetOnChangeBlocked(const FPropertyChangeBlocked::FDelegate& InDelegate)
{
	if (PropertyNode.IsValid())
	{
		return PropertyNode.Pin()->OnChangeBlockedEvent().Add(InDelegate);
	}

	return { };
}

void FPropertyValueImpl::UnregisterOnChangeBlocked(FDelegateHandle Handle)
{
	if (PropertyNode.IsValid())
	{
		PropertyNode.Pin()->OnChangeBlockedEvent().Remove(Handle);
	}
}

FDelegateHandle FPropertyValueImpl::SetOnRebuildChildren( const FSimpleDelegate& InOnRebuildChildren )
{
	if( PropertyNode.IsValid() )
	{
		return PropertyNode.Pin()->OnRebuildChildren().Add(InOnRebuildChildren);
	}
	return {};
} 

void FPropertyValueImpl::UnregisterOnRebuildChildren(FDelegateHandle Handle)
{
	if (PropertyNode.IsValid())
	{
		PropertyNode.Pin()->OnRebuildChildren().Remove(Handle);
	}
}

/**
 * Gets the max valid index for a array property of an object
 * @param InObjectNode - The parent of the variable being clamped
 * @param InArrayName - The array name we're hoping to clamp to the extents of
 * @return LastValidEntry in the array (if the array is found)
 */
static int32 GetArrayPropertyLastValidIndex( FObjectPropertyNode* InObjectNode, const FString& InArrayName )
{
	int32 ClampMax = MAX_int32;

	check(InObjectNode->GetNumObjects() == 1);
	UObject* ParentObject = InObjectNode->GetUObject(0);

	//find the associated property
	FProperty* FoundProperty = nullptr;
	for( TFieldIterator<FProperty> It(ParentObject->GetClass()); It; ++It )
	{
		FProperty* CurProp = *It;
		if (CurProp->GetName()==InArrayName)
		{
			FoundProperty = CurProp;
			break;
		}
	}

	if (FoundProperty && (FoundProperty->ArrayDim == 1))
	{
		FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>( FoundProperty );
		if (ArrayProperty)
		{
			uint8* PropertyAddressBase = ArrayProperty->ContainerPtrToValuePtr<uint8>(ParentObject);
			FScriptArrayHelper ArrayHelper(ArrayProperty, PropertyAddressBase);
			ClampMax = ArrayHelper.Num() - 1;
		}
		else
		{
			UE_LOGF(LogPropertyNode, Warning, "The property (%ls) passed for array clamping use is not an array.  Clamp will only ensure greater than zero.", *InArrayName);
		}
	}
	else
	{
		UE_LOGF(LogPropertyNode, Warning, "The property (%ls) passed for array clamping was not found.  Clamp will only ensure greater than zero.", *InArrayName);
	}

	return ClampMax;
}


FPropertyAccess::Result FPropertyValueImpl::GetValueAsString( FString& OutString, EPropertyPortFlags PortFlags ) const
{
	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();

	FPropertyAccess::Result Res = FPropertyAccess::Success;

	if( PropertyNodePin.IsValid() )
	{
		const bool bAllowAlternateDisplayValue = false;
		Res = PropertyNodePin->GetPropertyValueString( OutString, bAllowAlternateDisplayValue, PortFlags );
	}
	else
	{
		Res = FPropertyAccess::Fail;
	}

	return Res;
}

FPropertyAccess::Result FPropertyValueImpl::GetValueAsDisplayString( FString& OutString, EPropertyPortFlags PortFlags ) const
{
	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();

	FPropertyAccess::Result Res = FPropertyAccess::Success;

	if( PropertyNodePin.IsValid() )
	{
		const bool bAllowAlternateDisplayValue = true;
		Res = PropertyNodePin->GetPropertyValueString(OutString, bAllowAlternateDisplayValue, PortFlags);
	}
	else
	{
		Res = FPropertyAccess::Fail;
	}

	return Res;
}

FPropertyAccess::Result FPropertyValueImpl::GetValueAsText( FText& OutText ) const
{
	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();

	FPropertyAccess::Result Res = FPropertyAccess::Success;

	if( PropertyNodePin.IsValid() )
	{
		Res = PropertyNodePin->GetPropertyValueText( OutText, false/*bAllowAlternateDisplayValue*/ );
	}
	else
	{
		Res = FPropertyAccess::Fail;
	}

	return Res;
}

FPropertyAccess::Result FPropertyValueImpl::GetValueAsDisplayText( FText& OutText ) const
{
	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();

	FPropertyAccess::Result Res = FPropertyAccess::Success;

	if( PropertyNodePin.IsValid() )
	{
		Res = PropertyNodePin->GetPropertyValueText( OutText, true/*bAllowAlternateDisplayValue*/ );
	}
	else
	{
		Res = FPropertyAccess::Fail;
	}

	return Res;
}

FPropertyAccess::Result FPropertyValueImpl::SetValueAsString( const FString& InValue, EPropertyValueSetFlags::Type Flags )
{
	FPropertyAccess::Result Result = FPropertyAccess::Fail;

	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
	if( PropertyNodePin.IsValid() )
	{
		FProperty* NodeProperty = PropertyNodePin->GetProperty();

		FString Value = InValue;

		// Trim the name
		if( NodeProperty && NodeProperty->IsA( FNameProperty::StaticClass() ) )
		{
			Value.TrimStartAndEndInline();
		}

		// If more than one object is selected, an empty field indicates their values for this property differ.
		// Don't send it to the objects value in this case (if we did, they would all get set to None which isn't good).
		FComplexPropertyNode* ParentNode = PropertyNodePin->FindComplexParent();

		FString PreviousValue;
		GetValueAsString( PreviousValue );

		const bool bDidValueChange = Value.Len() && (FCString::Strcmp(*PreviousValue, *Value) != 0);
		const bool bComingOutOfInteractiveChange = bInteractiveChangeInProgress && ( ( Flags & EPropertyValueSetFlags::InteractiveChange ) != EPropertyValueSetFlags::InteractiveChange );

		if ( ParentNode && ( ParentNode->GetInstancesNum() == 1 || bComingOutOfInteractiveChange || bDidValueChange ) )
		{
			ImportText( Value, PropertyNodePin.Get(), Flags );
		}

		Result = FPropertyAccess::Success;
	}

	return Result;
}

bool FPropertyValueImpl::IsPropertyTypeOf(FFieldClass* ClassType ) const
{
	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
	if( PropertyNodePin.IsValid() )
	{
		if(FProperty* Property = PropertyNodePin->GetProperty())
		{
			return Property->IsA( ClassType );
		}
	}
	return false;
}

template< typename Type>
static Type ClampValueFromMetaData(Type InValue, FPropertyHandleBase& InPropertyHandle)
{
	Type RetVal = InValue;
	//enforce min
	const FString& MinString = InPropertyHandle.GetMetaData(TEXT("ClampMin"));
	if(MinString.Len())
	{
		checkSlow(MinString.IsNumeric());
		Type MinValue;
		TTypeFromString<Type>::FromString(MinValue, *MinString);
		RetVal = FMath::Max<Type>(MinValue, RetVal);
	}
	//Enforce max 
	const FString& MaxString = InPropertyHandle.GetMetaData(TEXT("ClampMax"));
	if(MaxString.Len())
	{
		checkSlow(MaxString.IsNumeric());
		Type MaxValue;
		TTypeFromString<Type>::FromString(MaxValue, *MaxString);
		RetVal = FMath::Min<Type>(MaxValue, RetVal);
	}

	return RetVal;
}

template <typename Type>
static Type ClampIntegerValueFromMetaData(Type InValue, FPropertyHandleBase& InPropertyHandle, FPropertyNode& InPropertyNode)
{
	Type RetVal = ClampValueFromMetaData<Type>(InValue, InPropertyHandle);

	//if there is "Multiple" meta data, the selected number is a multiple
	const FString& MultipleString = InPropertyHandle.GetMetaData(TEXT("Multiple"));
	if (MultipleString.Len())
	{
		check(MultipleString.IsNumeric());
		Type MultipleValue;
		TTypeFromString<Type>::FromString(MultipleValue, *MultipleString);
		if (MultipleValue != 0)
		{
			RetVal -= Type(RetVal) % MultipleValue;
		}
	}

	//enforce array bounds
	const FString& ArrayClampString = InPropertyHandle.GetMetaData(TEXT("ArrayClamp"));
	if (ArrayClampString.Len())
	{
		FObjectPropertyNode* ObjectPropertyNode = InPropertyNode.FindObjectItemParent();
		if (ObjectPropertyNode && ObjectPropertyNode->GetNumObjects() == 1)
		{
			Type LastValidIndex = static_cast<Type>(GetArrayPropertyLastValidIndex(ObjectPropertyNode, ArrayClampString));
			RetVal = FMath::Clamp<Type>(RetVal, 0, LastValidIndex);
		}
		else
		{
			UE_LOGF(LogPropertyNode, Warning, "Array Clamping isn't supported in multi-select (Param Name: %ls)", *InPropertyHandle.GetProperty()->GetName());
		}
	}

	return RetVal;
}


int32 FPropertyValueImpl::GetNumChildren() const
{
	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
	if( PropertyNodePin.IsValid() )
	{
		return PropertyNodePin->GetNumChildNodes();
	}

	return 0;
}

TSharedPtr<FPropertyNode> FPropertyValueImpl::GetPropertyNode() const
{
	return PropertyNode.Pin();
}

TSharedPtr<FPropertyNode> FPropertyValueImpl::GetChildNode( FName ChildName, bool bRecurse ) const
{
	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
	if( PropertyNodePin.IsValid() )
	{
		return PropertyNodePin->FindChildPropertyNode(ChildName, bRecurse);
	}

	return nullptr;
}


TSharedPtr<FPropertyNode> FPropertyValueImpl::GetChildNode( int32 ChildIndex ) const
{
	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
	if( PropertyNodePin.IsValid() && ChildIndex < PropertyNodePin->GetNumChildNodes() )
	{
		return PropertyNodePin->GetChildNode( ChildIndex );
	}

	return nullptr;
}

bool FPropertyValueImpl::GetChildNode(const int32 ChildArrayIndex, TSharedPtr<FPropertyNode>& OutChildNode) const
{
	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
	if (PropertyNodePin.IsValid())
	{
		return PropertyNodePin->GetChildNode(ChildArrayIndex, OutChildNode);
	}

	return false;
}


void FPropertyValueImpl::ResetToDefault()
{
	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
	if( PropertyNodePin.IsValid() && !PropertyNodePin->IsEditConst() && PropertyNodePin->GetDiffersFromDefault() )
	{
		EPropertyValueSetFlags::Type PropertyValueSetFlags = EPropertyValueSetFlags::ResetToDefault;
	
		// Ensure that we only instance the default value when required.
		const FProperty* Property = PropertyNodePin->GetProperty();
		if ((Property && Property->HasAnyPropertyFlags(CPF_InstancedReference))
			|| (PropertyNodePin->HasNodeFlags(EPropertyNodeFlags::EditInlineNew)))
		{
			PropertyValueSetFlags |= EPropertyValueSetFlags::InstanceObjects;
		}

		const bool bUseDisplayName = false;
		FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "PropertyWindowResetToDefault", "Reset to Default") );
		TSharedPtr<FPropertyNode>& KeyNode = PropertyNodePin->GetPropertyKeyNode();
		UE::CoreUObject::FScopedSetNullOnNonNullable ScopedSetNullOnNonNullable(UE::CoreUObject::EAllowSetNullOnNonNullableBehavior::Enabled);
		if(KeyNode.IsValid())
		{
			FPropertyValueImpl(KeyNode, NotifyHook, PropertyUtilities.Pin())
				.ImportText(KeyNode->GetDefaultValueAsString(bUseDisplayName), PropertyValueSetFlags);
		}
		ImportText(PropertyNodePin->GetDefaultValueAsString(bUseDisplayName), PropertyValueSetFlags);

		PropertyNodePin->BroadcastPropertyResetToDefault();
	}
}

bool FPropertyValueImpl::DiffersFromDefault() const
{
	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
	if( PropertyNodePin.IsValid()  )
	{
		return PropertyNodePin->GetDiffersFromDefault();
	}

	return false;
}

bool FPropertyValueImpl::IsEditConst() const
{
	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
	if( PropertyNodePin.IsValid()  )
	{
		return PropertyNodePin->IsEditConst();
	} 

	return false;
}

bool FPropertyValueImpl::IsExpanded() const
{
	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
	if( PropertyNodePin.IsValid()  )
	{
		return PropertyNodePin->HasNodeFlags(EPropertyNodeFlags::Expanded);
	} 

	return false;
}

void FPropertyValueImpl::SetExpanded(bool bExpanded)
{
	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
	if (PropertyNodePin.IsValid() && PropertyNodePin->HasNodeFlags(EPropertyNodeFlags::CanBeExpanded))
	{
		PropertyNodePin->SetNodeFlags(EPropertyNodeFlags::Expanded, bExpanded);
		// if the node is built, then we need to rebuild the child to expand it.
		PropertyNodePin->RequestRebuildChildren();
	}
}

FText FPropertyValueImpl::GetResetToDefaultLabel() const
{
	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
	if( PropertyNodePin.IsValid()  )
	{
		return PropertyNodePin->GetResetToDefaultLabel();
	} 

	return FText::GetEmpty();
}

int32 FPropertyValueImpl::AddChild()
{
	int32 ReturnLogicalIndex = INDEX_NONE;

	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
	if ( PropertyNodePin.IsValid() )
	{
		FProperty* NodeProperty = PropertyNodePin->GetProperty();

		FReadAddressList ReadAddresses;
		PropertyNodePin->GetReadAddress( !!PropertyNodePin->HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ReadAddresses, true, false, true );
		if ( ReadAddresses.Num() )
		{
			// determines whether we actually changed any values (if the user clicks the "empty" button when the array is already empty,
			// we don't want the objects to be marked dirty)
			bool bHasChanges = false;

			// If we added a new Map entry we need to rebuild children
			bool bAddedMapEntry = false;

			TArray< TMap<FString, int32> > PreAddArrayIndicesPerObject;
			TArray< TMap<FString, int32> > ArrayIndicesPerObject;
			TArray< TArray< UObject* > > AffectedInstancesPerObject;
			AffectedInstancesPerObject.SetNum(ReadAddresses.Num());

			// List of top level objects sent to the PropertyChangedEvent
			TArray<const UObject*> TopLevelObjects;
			TopLevelObjects.Reserve(ReadAddresses.Num());

			// Begin a property edit transaction.
			FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "AddChild", "Add Child") );
			FObjectPropertyNode* ObjectNode = PropertyNodePin->FindObjectItemParent();
			FArrayProperty* Array = CastField<FArrayProperty>(NodeProperty);
			FSetProperty* Set = CastField<FSetProperty>(NodeProperty);
			FMapProperty* Map = CastField<FMapProperty>(NodeProperty);

			check(Array || Set || Map);

			for ( int32 i = 0 ; i < ReadAddresses.Num() ; ++i )
			{
				void* Addr = ReadAddresses.GetAddress(i);
				if ( Addr )
				{
					bHasChanges = true;
					UObject* Obj = ObjectNode ? ObjectNode->GetUObject(i) : nullptr;
					
					if (!FApp::IsGame())
					{
						if (IsTemplate(Obj))
						{
							PropertyNodePin->GatherInstancesAffectedByContainerPropertyChange(Obj, Addr, EPropertyArrayChangeType::Add, AffectedInstancesPerObject[i]);
						}
					}

					if (Obj)
					{
						TopLevelObjects.Add(Obj);
					}
					
					//add on array index so we can tell which entry just changed
					PreAddArrayIndicesPerObject.Add(TMap<FString, int32>());
					FPropertyValueImpl::GenerateArrayIndexMapToObjectNode(PreAddArrayIndicesPerObject[i], PropertyNodePin.Get());
				}
			}

			if ( bHasChanges )
			{
				TSet< UObject* > AllAffectedInstances;
				for (const TArray<UObject*>& AffectedInstances : AffectedInstancesPerObject)
				{
					AllAffectedInstances.Append(AffectedInstances);
				}

				// send the PreEditChange notification to all selected objects
				PropertyNodePin->NotifyPreChange(NodeProperty, NotifyHook, AllAffectedInstances);
				
				if (PropertyUtilities.IsValid() && !bInteractiveChangeInProgress)
				{
					FPropertyChangedEvent StartedChangingPropertiesEvent(NodeProperty, EPropertyChangeType::ArrayAdd, MakeArrayView(TopLevelObjects));
					StartedChangingPropertiesEvent.SetArrayIndexPerObject(PreAddArrayIndicesPerObject);
					PropertyUtilities.Pin()->NotifyStartedChangingProperties(StartedChangingPropertiesEvent);
				}

				for (int32 i = 0 ; i < ReadAddresses.Num() ; ++i )
				{
					void* Addr = ReadAddresses.GetAddress(i);
					if ( Addr )
					{
						//add on array index so we can tell which entry just changed
						ArrayIndicesPerObject.Add(TMap<FString, int32>());
						FPropertyValueImpl::GenerateArrayIndexMapToObjectNode(ArrayIndicesPerObject[i], PropertyNodePin.Get());

						UObject* Obj = ObjectNode ? ObjectNode->GetUObject(i) : nullptr;
						if (Obj)
						{
							if (IsTemplate(Obj) && !FApp::IsGame())
							{
								PropertyNodePin->PropagateContainerPropertyChange(Obj, Addr, AffectedInstancesPerObject[i], EPropertyArrayChangeType::Add, -1);
							}

							TopLevelObjects.Add(Obj);
						}

						int32 Index = INDEX_NONE;

						// If the current object is an IDO, map the object back to the associated instance so that new items don't parent to the IDO. Note
						// however that we're still going to modify the container property's value on the IDO and not its associated instance in this case.
						UObject* NewItemOuter = Obj;
						if (const UObject* InstanceFromIDO = UE::FPropertyBagRepository::Get().FindInstanceForDataObject(NewItemOuter))
						{
							NewItemOuter = const_cast<UObject*>(InstanceFromIDO);
						}

						// Ensure that we propagate the appropriate flags to new instances below, and include the archetype flag if the outer is a template.
						EObjectFlags MaskedOuterFlags = RF_NoFlags;
						if (NewItemOuter)
						{
							MaskedOuterFlags = NewItemOuter->GetMaskedFlags(RF_PropagateToSubObjects);
							if (NewItemOuter->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
							{
								MaskedOuterFlags |= RF_ArchetypeObject;
							}
						}

						if (Array)
						{
							Array->PerformOperationWithSetter(Obj, Addr, [NewItemOuter, Array, &Index, &ReturnLogicalIndex, MaskedOuterFlags](void* DirectAddress)
							{								
								FScriptArrayHelper	ArrayHelper(Array, DirectAddress);
								Index = ArrayHelper.AddValue();
								ReturnLogicalIndex = Index;

								// check whether the inner type is flagged as a non-nullable. if so, create it.
								FObjectProperty* InnerObjectProperty = CastField<FObjectProperty>(Array->Inner);
								if (InnerObjectProperty && InnerObjectProperty->HasAnyPropertyFlags(CPF_NonNullable) && !InnerObjectProperty->PropertyClass->HasAnyClassFlags(CLASS_Abstract))
								{
									UObject* NewItem = NewObject<UObject>(NewItemOuter, InnerObjectProperty->PropertyClass, NAME_None, MaskedOuterFlags);
									InnerObjectProperty->SetObjectPropertyValue(ArrayHelper.GetRawPtr(Index), NewItem);
								}
							});
						}
						else if (Set)
						{
							Set->PerformOperationWithSetter(Obj, Addr, [NewItemOuter, Set, &Index, &ReturnLogicalIndex, MaskedOuterFlags](void* DirectAddress)
							{
								FScriptSetHelper	SetHelper(Set, DirectAddress);
								Index = SetHelper.AddDefaultValue_Invalid_NeedsRehash();
								ReturnLogicalIndex = SetHelper.FindLogicalIndex(Index);

								// check whether the element type is flagged as a non-nullable. if so, create it.
								FObjectProperty* ElementObjectProperty = CastField<FObjectProperty>(Set->ElementProp);
								if (ElementObjectProperty && ElementObjectProperty->HasAnyPropertyFlags(CPF_NonNullable) && !ElementObjectProperty->PropertyClass->HasAnyClassFlags(CLASS_Abstract))
								{
									UObject* NewItem = NewObject<UObject>(NewItemOuter, ElementObjectProperty->PropertyClass, NAME_None, MaskedOuterFlags);
									ElementObjectProperty->SetObjectPropertyValue(SetHelper.GetElementPtr(Index), NewItem);
								}

								SetHelper.Rehash();
							});
						}
						else if (Map)
						{
							Map->PerformOperationWithSetter(Obj, Addr, [NewItemOuter, Map, &Index, &bAddedMapEntry, &ReturnLogicalIndex, MaskedOuterFlags](void* DirectAddress)
							{
								FScriptMapHelper	MapHelper(Map, DirectAddress);
								Index = MapHelper.AddDefaultValue_Invalid_NeedsRehash();
								ReturnLogicalIndex = MapHelper.FindLogicalIndex(Index);

								// check whether the key or value type is flagged as a non-nullable. if so, create it.
								{
									FObjectProperty* KeyObjectProperty = CastField<FObjectProperty>(Map->KeyProp);
									if (KeyObjectProperty && KeyObjectProperty->HasAnyPropertyFlags(CPF_NonNullable) && !KeyObjectProperty->PropertyClass->HasAnyClassFlags(CLASS_Abstract))
									{
										UObject* NewItem = NewObject<UObject>(NewItemOuter, KeyObjectProperty->PropertyClass, NAME_None, MaskedOuterFlags);
										KeyObjectProperty->SetObjectPropertyValue(MapHelper.GetKeyPtr(Index), NewItem);
									}
								}

								{
									FObjectProperty* ValueObjectProperty = CastField<FObjectProperty>(Map->ValueProp);
									if (ValueObjectProperty && ValueObjectProperty->HasAnyPropertyFlags(CPF_NonNullable) && !ValueObjectProperty->PropertyClass->HasAnyClassFlags(CLASS_Abstract))
									{
										UObject* NewItem = NewObject<UObject>(NewItemOuter, ValueObjectProperty->PropertyClass, NAME_None, MaskedOuterFlags);
										ValueObjectProperty->SetObjectPropertyValue(MapHelper.GetValuePtr(Index), NewItem);
									}
								}

								MapHelper.Rehash();
								bAddedMapEntry = true;
							});
						}

						ArrayIndicesPerObject[i].Add(NodeProperty->GetName(), Index);
					}
				}

				FPropertyChangedEvent ChangeEvent(NodeProperty, EPropertyChangeType::ArrayAdd, MakeArrayView(TopLevelObjects));
				ChangeEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);
				ChangeEvent.SetInstancesChanged(MoveTemp(AllAffectedInstances));

				// send the PostEditChange notification; it will be propagated to all selected objects
				PropertyNodePin->NotifyPostChange(ChangeEvent, NotifyHook);

				if (PropertyUtilities.IsValid())
				{
					PropertyNodePin->FixPropertiesInEvent(ChangeEvent);
					PropertyUtilities.Pin()->NotifyFinishedChangingProperties(ChangeEvent);
				}

				PropertyNodePin->RebuildChildren();
			}
		}
	}

	return ReturnLogicalIndex;
}

void FPropertyValueImpl::ClearChildren()
{
	TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
	if ( PropertyNodePin.IsValid() )
	{
		FProperty* NodeProperty = PropertyNodePin->GetProperty();

		FReadAddressList ReadAddresses;
		PropertyNodePin->GetReadAddress( !!PropertyNodePin->HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ReadAddresses,
			true, //bComparePropertyContents
			false, //bObjectForceCompare
			true ); //bArrayPropertiesCanDifferInSize

		if ( ReadAddresses.Num() )
		{
			// determines whether we actually changed any values (if the user clicks the "emtpy" button when the array is already empty,
			// we don't want the objects to be marked dirty)
			bool bHasChanges = false;

			// List of top level objects sent to the PropertyChangedEvent
			TArray<const UObject*> TopLevelObjects;
			TopLevelObjects.Reserve(ReadAddresses.Num());

			TArray< TMap<FString, int32> > ArrayIndicesPerObject;
			TArray< TArray< UObject* > > AffectedInstancesPerObject;
			AffectedInstancesPerObject.SetNum(ReadAddresses.Num());

			// Begin a property edit transaction.
			FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "ClearChildren", "Clear Children") );
			FObjectPropertyNode* ObjectNode = PropertyNodePin->FindObjectItemParent();
			FArrayProperty* ArrayProperty = CastField<FArrayProperty>(NodeProperty);
			FSetProperty* SetProperty = CastField<FSetProperty>(NodeProperty);
			FMapProperty* MapProperty = CastField<FMapProperty>(NodeProperty);

			check(ArrayProperty || SetProperty || MapProperty);

			for ( int32 i = 0 ; i < ReadAddresses.Num() ; ++i )
			{
				void* Addr = ReadAddresses.GetAddress(i);
				if ( Addr )
				{
					bHasChanges = true;

					UObject* Obj = ObjectNode ? ObjectNode->GetUObject(i) : nullptr;
					if (!FApp::IsGame())
					{
						if (IsTemplate(Obj))
						{
							PropertyNodePin->GatherInstancesAffectedByContainerPropertyChange(Obj, Addr, EPropertyArrayChangeType::Clear, AffectedInstancesPerObject[i]);
						}
					}

					if (Obj)
					{
						TopLevelObjects.Add(Obj);
					}
					
					// Add on array index so we can tell which entry just changed
					ArrayIndicesPerObject.Add(TMap<FString, int32>());
					FPropertyValueImpl::GenerateArrayIndexMapToObjectNode(ArrayIndicesPerObject[i], PropertyNodePin.Get());
				}
			}

			if (bHasChanges)
			{
				TSet< UObject* > AllAffectedInstances;
				for (const TArray<UObject*>& AffectedInstances : AffectedInstancesPerObject)
				{
					AllAffectedInstances.Append(AffectedInstances);
				}
				
				// Send the PreEditChange notification to all selected objects
				PropertyNodePin->NotifyPreChange(NodeProperty, NotifyHook, AllAffectedInstances);

				if (PropertyUtilities.IsValid() && !bInteractiveChangeInProgress)
				{
					FPropertyChangedEvent StartedChangingPropertiesEvent(NodeProperty, EPropertyChangeType::ArrayClear, MakeArrayView(TopLevelObjects));
					StartedChangingPropertiesEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);
					PropertyUtilities.Pin()->NotifyStartedChangingProperties(StartedChangingPropertiesEvent);
				}

				for ( int32 i = 0 ; i < ReadAddresses.Num() ; ++i )
				{
					void* Addr = ReadAddresses.GetAddress(i);
					if ( Addr )
					{
						UObject* Obj = ObjectNode ? ObjectNode->GetUObject(i) : nullptr;
						if (Obj)
						{
							if (IsTemplate(Obj) && !FApp::IsGame())
							{
								PropertyNodePin->PropagateContainerPropertyChange(Obj, Addr, AffectedInstancesPerObject[i], EPropertyArrayChangeType::Clear, -1);
							}
						}

						if (ArrayProperty)
						{
							FScriptArrayHelper ArrayHelper(ArrayProperty, Addr);

							// If the inner property is an instanced property we must move the old objects to the 
							// transient package so code looking for objects of this type on the parent doesn't find them
							FObjectProperty* InnerObjectProperty = CastField<FObjectProperty>(ArrayProperty->Inner);
							if (ShouldOwnInstance(InnerObjectProperty, PropertyNodePin.Get()))
							{
								const int32 ArraySize = ArrayHelper.Num();
								for (int32 Index = 0; Index < ArraySize; ++Index)
								{
									if (UObject* InstancedObject = *reinterpret_cast<UObject**>(ArrayHelper.GetRawPtr(Index)))
									{
										InstancedObject->Modify();
										InstancedObject->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
									}
								}
							}

							ArrayHelper.EmptyValues();
						}
						else if (SetProperty)
						{
							FScriptSetHelper SetHelper(SetProperty, Addr);

							// If the element property is an instanced property we must move the old objects to the 
							// transient package so code looking for objects of this type on the parent doesn't find them
							FObjectProperty* ElementObjectProperty = CastField<FObjectProperty>(SetProperty->ElementProp);
							if (ShouldOwnInstance(ElementObjectProperty, PropertyNodePin.Get()))
							{
								for (FScriptSetHelper::FIterator It(SetHelper); It; ++It)
								{
									if (UObject* InstancedObject = *reinterpret_cast<UObject**>(SetHelper.GetElementPtr(It)))
									{
										InstancedObject->Modify();
										InstancedObject->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
									}
								}
							}

							SetHelper.EmptyElements();
						}
						else if (MapProperty)
						{
							FScriptMapHelper MapHelper(MapProperty, Addr);

							// If the map's value property is an instanced property we must move the old objects to the 
							// transient package so code looking for objects of this type on the parent doesn't find them
							FObjectProperty* ValueObjectProperty = CastField<FObjectProperty>(MapProperty->ValueProp);
							if (ShouldOwnInstance(ValueObjectProperty, PropertyNodePin.Get()))
							{
								for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
								{
									if (UObject* InstancedObject = *reinterpret_cast<UObject**>(MapHelper.GetValuePtr(It)))
									{
										InstancedObject->Modify();
										InstancedObject->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
									}
								}
							}

							MapHelper.EmptyValues();
						}
					}
				}

				FPropertyChangedEvent ChangeEvent(NodeProperty, EPropertyChangeType::ArrayClear, MakeArrayView(TopLevelObjects));
				ChangeEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);
				ChangeEvent.SetInstancesChanged(MoveTemp(AllAffectedInstances));

				// Send the PostEditChange notification; it will be propagated to all selected objects
				PropertyNodePin->NotifyPostChange( ChangeEvent, NotifyHook );

				if (PropertyUtilities.IsValid())
				{
					PropertyNodePin->FixPropertiesInEvent(ChangeEvent);
					PropertyUtilities.Pin()->NotifyFinishedChangingProperties(ChangeEvent);
				}
				
				PropertyNodePin->RebuildChildren();
			}
		}
	}
}

void FPropertyValueImpl::InsertChild( int32 Index )
{
	if( PropertyNode.IsValid() )
	{
		InsertChild( PropertyNode.Pin()->GetChildNode(Index));
	}
}

void FPropertyValueImpl::InsertChild( TSharedPtr<FPropertyNode> ChildNodeToInsertAfter )
{
	FPropertyNode* ChildNodePtr = ChildNodeToInsertAfter.Get();

	FPropertyNode* ParentNode = ChildNodePtr->GetParentNode();
	FObjectPropertyNode* ObjectNode = ChildNodePtr->FindObjectItemParent();

	FProperty* NodeProperty = ChildNodePtr->GetProperty();
	FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(NodeProperty->GetOwner<FField>()); // Insert is not supported for sets or maps


	FReadAddressList ReadAddresses;
	void* Addr = nullptr;
	ParentNode->GetReadAddress( !!ParentNode->HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ReadAddresses );
	if ( ReadAddresses.Num() )
	{
		Addr = ReadAddresses.GetAddress(0);
	}

	if( Addr )
	{
		TArray< UObject* > AffectedInstances;

		// Begin a property edit transaction.
		FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "InsertChild", "Insert Child") );

		// List of top level objects sent to the PropertyChangedEvent
		TArray<const UObject*> TopLevelObjects;

		if (!FApp::IsGame())
		{
			UObject* Obj = ObjectNode ? ObjectNode->GetUObject(0) : nullptr;
			if (IsTemplate(Obj))
			{
				ChildNodePtr->GatherInstancesAffectedByContainerPropertyChange(Obj, Addr, EPropertyArrayChangeType::Insert, AffectedInstances);
			}
			
			TopLevelObjects.Add(Obj);
		}

		TSet< UObject* > AllAffectedInstances;
		AllAffectedInstances.Append(AffectedInstances);
		
		//set up indices for the coming events
		TArray< TMap<FString,int32> > ArrayIndicesPerObject;
		for (int32 ObjectIndex = 0; ObjectIndex < ReadAddresses.Num(); ++ObjectIndex)
		{
			//add on array index so we can tell which entry just changed
			ArrayIndicesPerObject.Add(TMap<FString,int32>());
			FPropertyValueImpl::GenerateArrayIndexMapToObjectNode(ArrayIndicesPerObject[ObjectIndex], ChildNodePtr );
		}

		ChildNodePtr->NotifyPreChange( ParentNode->GetProperty(), NotifyHook, AllAffectedInstances );
		
		if (PropertyUtilities.IsValid() && !bInteractiveChangeInProgress)
		{
			FPropertyChangedEvent StartedChangingPropertiesEvent(NodeProperty, EPropertyChangeType::ArrayAdd, MakeArrayView(TopLevelObjects));
			StartedChangingPropertiesEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);
			PropertyUtilities.Pin()->NotifyStartedChangingProperties(StartedChangingPropertiesEvent);
		}
		
		FScriptArrayHelper	ArrayHelper(ArrayProperty,Addr);
		int32 Index = ChildNodePtr->GetArrayIndex();
		
		UObject* Obj = ObjectNode ? ObjectNode->GetUObject(0) : nullptr;
		if (Obj)
		{
			if (IsTemplate(Obj) && !FApp::IsGame())
			{
				ChildNodePtr->PropagateContainerPropertyChange(Obj, Addr, AffectedInstances, EPropertyArrayChangeType::Insert, Index);
			}
		}

		ArrayHelper.InsertValues(Index, 1 );

		FPropertyChangedEvent ChangeEvent(ParentNode->GetProperty(), EPropertyChangeType::ArrayAdd, MakeArrayView(TopLevelObjects));
		ChangeEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);
		ChangeEvent.SetInstancesChanged(MoveTemp(AllAffectedInstances));

		PropertyNode.Pin()->NotifyPostChange(ChangeEvent, NotifyHook);

		if (PropertyUtilities.IsValid())
		{
			ChildNodePtr->FixPropertiesInEvent(ChangeEvent);
			PropertyUtilities.Pin()->NotifyFinishedChangingProperties(ChangeEvent);
		}

		PropertyNode.Pin()->RebuildChildren();
	}
}


void FPropertyValueImpl::DeleteChild( int32 Index )
{
	TSharedPtr<FPropertyNode> ArrayParentPin = PropertyNode.Pin();
	if( ArrayParentPin.IsValid()  )
	{
		DeleteChild( ArrayParentPin->GetChildNode( Index ) );
	}
}

void FPropertyValueImpl::DeleteChild( TSharedPtr<FPropertyNode> ChildNodeToDelete )
{
	FPropertyNode* ChildNodePtr = ChildNodeToDelete.Get();

	FPropertyNode* ParentNode = ChildNodePtr->GetParentNode();
	FObjectPropertyNode* ObjectNode = ChildNodePtr->FindObjectItemParent();

	FProperty* NodeProperty = ChildNodePtr->GetProperty();
	FArrayProperty* ArrayProperty = NodeProperty->GetOwner<FArrayProperty>();
	FSetProperty* SetProperty = NodeProperty->GetOwner<FSetProperty>();
	FMapProperty* MapProperty = NodeProperty->GetOwner<FMapProperty>();

	TArray< TMap<FString, int32> > ArrayIndicesPerObject;

	check(ArrayProperty || SetProperty || MapProperty);

	FReadAddressList ReadAddresses;
	ParentNode->GetReadAddress( !!ParentNode->HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ReadAddresses ); 
	if ( ReadAddresses.Num() )
	{
		// determines whether we actually changed any values (if the user clicks the "empty" button when the array is already empty,
		// we don't want the objects to be marked dirty)
		bool bHasChanges = false;

		TArray< TArray< UObject* > > AffectedInstancesPerObject;
		AffectedInstancesPerObject.SetNum(ReadAddresses.Num());

		FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "DeleteChild", "Delete Child"));

		// List of top level objects sent to the PropertyChangedEvent
		TArray<const UObject*> TopLevelObjects;
		TopLevelObjects.Reserve(ReadAddresses.Num());

		for ( int32 i = 0 ; i < ReadAddresses.Num() ; ++i )
		{
			uint8* Address = ReadAddresses.GetAddress(i);

			if( Address ) 
			{
				bHasChanges = true;

				UObject* Obj = ObjectNode ? ObjectNode->GetUObject(i) : nullptr;
				
				if (!FApp::IsGame())
				{
					if (IsTemplate(Obj))
					{
						ChildNodePtr->GatherInstancesAffectedByContainerPropertyChange(Obj, Address, EPropertyArrayChangeType::Delete, AffectedInstancesPerObject[i]);
					}
				}

				if (Obj)
				{
					TopLevelObjects.Add(Obj);
				}
				
				int32 Index = ChildNodePtr->GetArrayIndex();
				
				// Add on array index so we can tell which entry just changed
				int32 ArrayIndicesPerObjectIndex = ArrayIndicesPerObject.Add(TMap<FString, int32>());
				FPropertyValueImpl::GenerateArrayIndexMapToObjectNode(ArrayIndicesPerObject[ArrayIndicesPerObjectIndex], ChildNodePtr);
				ArrayIndicesPerObject[ArrayIndicesPerObjectIndex].Add(NodeProperty->GetName(), Index);
			}
		}

		if (bHasChanges)
		{
			TSet< UObject* > AllAffectedInstances;
			for (const TArray<UObject*>& AffectedInstances : AffectedInstancesPerObject)
			{
				AllAffectedInstances.Append(AffectedInstances);
			}

			// Send the PreEditChange notification to all selected objects
			ChildNodePtr->NotifyPreChange(NodeProperty, NotifyHook, AllAffectedInstances);
			
			if (PropertyUtilities.IsValid() && !bInteractiveChangeInProgress)
			{
				FPropertyChangedEvent StartedChangingPropertiesEvent(ParentNode->GetProperty(), EPropertyChangeType::ArrayRemove, MakeArrayView(TopLevelObjects));
				StartedChangingPropertiesEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);
				PropertyUtilities.Pin()->NotifyStartedChangingProperties(StartedChangingPropertiesEvent);
			}

			// Perform the operation on the array for all selected objects
			for (int32 i = 0; i < ReadAddresses.Num(); ++i)
			{
				uint8* Address = ReadAddresses.GetAddress(i);

				if ( Address )
				{
					int32 Index = ChildNodePtr->GetArrayIndex();

					UObject* Obj = ObjectNode ? ObjectNode->GetUObject(i) : nullptr;
					if (Obj)
					{
						if (IsTemplate(Obj) && !FApp::IsGame())
						{
							ChildNodePtr->PropagateContainerPropertyChange(Obj, Address, AffectedInstancesPerObject[i], EPropertyArrayChangeType::Delete, Index);
						}
					}

					if (ArrayProperty)
					{
						FScriptArrayHelper ArrayHelper(ArrayProperty, Address);

						// If the inner property is an instanced property we must move the old object to the 
						// transient package so code looking for objects of this type on the parent doesn't find it
						FObjectProperty* InnerObjectProperty = CastField<FObjectProperty>(ArrayProperty->Inner);
						if (ShouldOwnInstance(InnerObjectProperty, ParentNode))
						{
							if (UObject* InstancedObject = *reinterpret_cast<UObject**>(ArrayHelper.GetRawPtr(ChildNodePtr->GetArrayIndex())))
							{
								InstancedObject->Modify();
								InstancedObject->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
							}
						}

						ArrayHelper.RemoveValues(ChildNodePtr->GetArrayIndex());
					}
					else if (SetProperty)
					{
						FScriptSetHelper SetHelper(SetProperty, Address);
						int32 InternalIndex = SetHelper.FindInternalIndex(ChildNodePtr->GetArrayIndex());
						
						// If the element property is an instanced property we must move the old object to the 
						// transient package so code looking for objects of this type on the parent doesn't find it
						FObjectProperty* ElementObjectProperty = CastField<FObjectProperty>(SetProperty->ElementProp);
						if (ShouldOwnInstance(ElementObjectProperty, ParentNode))
						{
							if (UObject* InstancedObject = *reinterpret_cast<UObject**>(SetHelper.GetElementPtr(InternalIndex)))
							{
								InstancedObject->Modify();
								InstancedObject->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
							}
						}

						SetHelper.RemoveAt(InternalIndex);
						SetHelper.Rehash();
					}
					else if (MapProperty)
					{
						FScriptMapHelper MapHelper(MapProperty, Address);
						int32 InternalIndex = MapHelper.FindInternalIndex(ChildNodePtr->GetArrayIndex());
						
						// If the map's value property is an instanced property we must move the old object to the 
						// transient package so code looking for objects of this type on the parent doesn't find it
						FObjectProperty* ValueObjectProperty = CastField<FObjectProperty>(MapProperty->ValueProp);
						if (ShouldOwnInstance(ValueObjectProperty, ParentNode))
						{
							if (UObject* InstancedObject = *reinterpret_cast<UObject**>(MapHelper.GetValuePtr(InternalIndex)))
							{
								InstancedObject->Modify();
								InstancedObject->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
							}
						}

						MapHelper.RemoveAt(InternalIndex);
						MapHelper.Rehash();
					}
				}
			}

			FPropertyChangedEvent ChangeEvent(ParentNode->GetProperty(), EPropertyChangeType::ArrayRemove, MakeArrayView(TopLevelObjects));
			ChangeEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);
			ChangeEvent.SetInstancesChanged(MoveTemp(AllAffectedInstances));

			PropertyNode.Pin()->NotifyPostChange(ChangeEvent, NotifyHook);

			if (PropertyUtilities.IsValid())
			{
				ChildNodePtr->FixPropertiesInEvent(ChangeEvent);
				PropertyUtilities.Pin()->NotifyFinishedChangingProperties(ChangeEvent);
			}

			PropertyNode.Pin()->RebuildChildren();
		}
	}
}

void FPropertyValueImpl::SwapChildren(int32 FirstIndex, int32 SecondIndex)
{
	TSharedPtr<FPropertyNode> ArrayParentPin = PropertyNode.Pin();
	if (ArrayParentPin.IsValid())
	{
		SwapChildren(ArrayParentPin->GetChildNode(FirstIndex), ArrayParentPin->GetChildNode(SecondIndex));
	}
}

void FPropertyValueImpl::SwapChildren( TSharedPtr<FPropertyNode> FirstChildNode, TSharedPtr<FPropertyNode> SecondChildNode)
{
	FPropertyNode* FirstChildNodePtr = FirstChildNode.Get();
	FPropertyNode* SecondChildNodePtr = SecondChildNode.Get();

	FPropertyNode* ParentNode = FirstChildNodePtr->GetParentNode();
	FObjectPropertyNode* ObjectNode = FirstChildNodePtr->FindObjectItemParent();

	FProperty* FirstNodeProperty = FirstChildNodePtr->GetProperty();
	FProperty* SecondNodeProperty = SecondChildNodePtr->GetProperty();
	FArrayProperty* ArrayProperty = FirstNodeProperty->GetOwner<FArrayProperty>();

	check(ArrayProperty);

	FReadAddressList ReadAddresses;
	ParentNode->GetReadAddress( !!ParentNode->HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ReadAddresses ); 
	if ( ReadAddresses.Num() )
	{
		FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "SwapChildren", "Swap Children") );

		FirstChildNodePtr->NotifyPreChange( FirstNodeProperty, NotifyHook );
		SecondChildNodePtr->NotifyPreChange( SecondNodeProperty, NotifyHook );
		
		// List of top level objects sent to the PropertyChangedEvent
		TArray<const UObject*> TopLevelObjects;
		TopLevelObjects.Reserve(ReadAddresses.Num());

		for ( int32 i = 0 ; i < ReadAddresses.Num() ; ++i )
		{
			if (UObject* Obj = ObjectNode ? ObjectNode->GetUObject(i) : nullptr)
			{
				TopLevelObjects.Add(Obj);
			}
		}

		if (PropertyUtilities.IsValid() && !bInteractiveChangeInProgress)
		{
			FPropertyChangedEvent StartedChangingPropertiesEvent(ParentNode->GetProperty(), EPropertyChangeType::ArrayMove, MakeArrayView(TopLevelObjects));
			PropertyUtilities.Pin()->NotifyStartedChangingProperties(StartedChangingPropertiesEvent);
		}

		// perform the operation on the array for all selected objects
		for ( int32 i = 0 ; i < ReadAddresses.Num() ; ++i )
		{
			uint8* Address = ReadAddresses.GetAddress(i);

			if( Address ) 
			{
				int32 FirstIndex = FirstChildNodePtr->GetArrayIndex();
				int32 SecondIndex = SecondChildNodePtr->GetArrayIndex();

				UObject* Obj = ObjectNode ? ObjectNode->GetUObject(i) : nullptr;
				if (Obj)
				{
					if ((Obj->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) || (Obj->HasAnyFlags(RF_DefaultSubObject) && Obj->GetOuter()->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))) 
						&& !FApp::IsGame())
					{
						FirstChildNodePtr->PropagateContainerPropertyChange(Obj, Address, EPropertyArrayChangeType::Swap, FirstIndex, SecondIndex);
					}
				}

				if (ArrayProperty)
				{
					FScriptArrayHelper ArrayHelper(ArrayProperty, Address);

					// If the inner property is an instanced property we must move the old object to the 
					// transient package so code looking for objects of this type on the parent doesn't find it
					FObjectProperty* InnerObjectProperty = CastField<FObjectProperty>(ArrayProperty->Inner);
					if(ShouldOwnInstance(InnerObjectProperty, ParentNode))
					{
						if (UObject* InstancedObject = *reinterpret_cast<UObject**>(ArrayHelper.GetRawPtr(FirstIndex)))
						{
							InstancedObject->Modify();
							InstancedObject->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
						}

						if (UObject* InstancedObject = *reinterpret_cast<UObject**>(ArrayHelper.GetRawPtr(SecondIndex)))
						{
							InstancedObject->Modify();
							InstancedObject->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
						}
					}

					ArrayHelper.SwapValues(FirstIndex, SecondIndex);
				}
			}
		}

		FPropertyChangedEvent ChangeEvent(ParentNode->GetProperty(), EPropertyChangeType::ArrayMove, MakeArrayView(TopLevelObjects));
		FirstChildNodePtr->NotifyPostChange(ChangeEvent, NotifyHook);
		SecondChildNodePtr->NotifyPostChange(ChangeEvent, NotifyHook);

		if (PropertyUtilities.IsValid())
		{
			FirstChildNodePtr->FixPropertiesInEvent(ChangeEvent);
			SecondChildNodePtr->FixPropertiesInEvent(ChangeEvent);
			PropertyUtilities.Pin()->NotifyFinishedChangingProperties(ChangeEvent);
		}

		PropertyNode.Pin()->RebuildChildren();
	}
}

void FPropertyValueImpl::MoveElementTo(int32 OriginalIndex, int32 NewIndex)
{
	if (OriginalIndex == NewIndex)
	{
		return;
	}

	// We don't do a true swap of the old index and new index here
	// Instead we insert a new item after the existing item at the desired NewIndex, swap the Original after the "NewIndex" node, and delete the item at OriginalIndex
	// To try to describe with array examples..

	// Case A NewIndex > OriginalIndex
	// Input Array: A B C D, where OriginalIndex = 1 and NewIndex = 2
	// We add a new node (?) at NewIndex + 1
	// A B C ? D
	// Now swap OriginalIndex (B) and NewIndex + 1 (?)
	// A ? C B D
	// Delete OriginalIndex (?)
	// A C B D

	// Or -
	// Input Array: A B C D, where OriginalIndex = 2 and NewIndex = 1
	// Insert at NewIndex (?), which increments OriginalIndex (C)
	// A ? B C D
	// Swap OriginalIndex + 1 (C) and NewIndex (?)
	// A C B ? D
	// Delete OriginalIndex + 1 (?)
	// A C B D

	// If we're in case A, increment the "NewIndex" for swap target
	if (NewIndex > OriginalIndex)
	{
		NewIndex += 1;
	}
	// Otherwise increment OriginalIndex
	else
	{
		OriginalIndex += 1;
	}

	// Insert into the middle of the array
	if (NewIndex < GetPropertyNode()->GetNumChildNodes())
	{
		TSharedPtr<FPropertyNode> InsertAfterChild = PropertyNode.Pin()->GetChildNode(NewIndex);
		FPropertyNode* ChildNodePtr = InsertAfterChild.Get();

		FPropertyNode* ParentNode = ChildNodePtr->GetParentNode();
		FObjectPropertyNode* ObjectNode = ChildNodePtr->FindObjectItemParent();

		FProperty* NodeProperty = ChildNodePtr->GetProperty();
		FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(NodeProperty->GetOwner<FArrayProperty>()); // Insert is not supported for sets or maps


		FReadAddressList ReadAddresses;
		ParentNode->GetReadAddress(!!ParentNode->HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ReadAddresses);
		for (int32 i = 0; i < ReadAddresses.Num(); ++i)
		{
			void* Addr = ReadAddresses.GetAddress(i);
			if (Addr)
			{
				FScriptArrayHelper	ArrayHelper(ArrayProperty, Addr);
				int32 Index = ChildNodePtr->GetArrayIndex();

				// List of top level objects sent to the PropertyChangedEvent
				TArray<const UObject*> TopLevelObjects;

				UObject* Obj = ObjectNode ? ObjectNode->GetUObject(0) : nullptr;
				if (Obj)
				{
					if ((Obj->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) ||
						(Obj->HasAnyFlags(RF_DefaultSubObject) && Obj->GetOuter()->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))))
					{
						ChildNodePtr->PropagateContainerPropertyChange(Obj, Addr, EPropertyArrayChangeType::Insert, Index);
					}

					TopLevelObjects.Add(Obj);
				}

				ArrayHelper.InsertValues(Index, 1);

				//set up indices for the coming events
				TArray< TMap<FString, int32> > ArrayIndicesPerObject;
				for (int32 ObjectIndex = 0; ObjectIndex < ReadAddresses.Num(); ++ObjectIndex)
				{
					//add on array index so we can tell which entry just changed
					ArrayIndicesPerObject.Add(TMap<FString, int32>());
					FPropertyValueImpl::GenerateArrayIndexMapToObjectNode(ArrayIndicesPerObject[ObjectIndex], ChildNodePtr);
				}

				FPropertyChangedEvent ChangeEvent(ParentNode->GetProperty(), EPropertyChangeType::ArrayAdd, MakeArrayView(TopLevelObjects));
				ChangeEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);

				if (PropertyUtilities.IsValid())
				{
					ChildNodePtr->FixPropertiesInEvent(ChangeEvent);
				}
			}
		}
	}
	else // or add to the end of the array
	{
		TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
		if (PropertyNodePin.IsValid())
		{
			FProperty* NodeProperty = PropertyNodePin->GetProperty();

			FReadAddressList ReadAddresses;
			PropertyNodePin->GetReadAddress(!!PropertyNodePin->HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ReadAddresses, true, false, true);
			if (ReadAddresses.Num())
			{
				// determines whether we actually changed any values (if the user clicks the "emtpy" button when the array is already empty,
				// we don't want the objects to be marked dirty)
				bool bNotifiedPreChange = false;

				TArray< TMap<FString, int32> > ArrayIndicesPerObject;

				// List of top level objects sent to the PropertyChangedEvent
				TArray<const UObject*> TopLevelObjects;
				TopLevelObjects.Reserve(ReadAddresses.Num());

				FObjectPropertyNode* ObjectNode = PropertyNodePin->FindObjectItemParent();
				FArrayProperty* Array = CastField<FArrayProperty>(NodeProperty);

				check(Array);

				for (int32 i = 0; i < ReadAddresses.Num(); ++i)
				{
					void* Addr = ReadAddresses.GetAddress(i);
					if (Addr)
					{
						//add on array index so we can tell which entry just changed
						ArrayIndicesPerObject.Add(TMap<FString, int32>());
						FPropertyValueImpl::GenerateArrayIndexMapToObjectNode(ArrayIndicesPerObject[i], PropertyNodePin.Get());

						UObject* Obj = ObjectNode ? ObjectNode->GetUObject(i) : nullptr;
						if (Obj)
						{
							if ((Obj->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) ||
								(Obj->HasAnyFlags(RF_DefaultSubObject) && Obj->GetOuter()->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))))
							{
								PropertyNodePin->PropagateContainerPropertyChange(Obj, Addr, EPropertyArrayChangeType::Add, -1);
							}

							TopLevelObjects.Add(Obj);
						}

						int32 Index = INDEX_NONE;

						FScriptArrayHelper	ArrayHelper(Array, Addr);
						Index = ArrayHelper.AddValue();

						ArrayIndicesPerObject[i].Add(NodeProperty->GetName(), Index);
					}
				}

				FPropertyChangedEvent ChangeEvent(NodeProperty, EPropertyChangeType::ArrayAdd, MakeArrayView(TopLevelObjects));
				ChangeEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);

				if (PropertyUtilities.IsValid())
				{
					PropertyNodePin->FixPropertiesInEvent(ChangeEvent);
				}
			}
		}
	}

	// Both Insert and Add are deferred so you need to rebuild the parent node's children
	GetPropertyNode()->RebuildChildren();

	//Swap
	{
		FPropertyNode* FirstChildNodePtr = GetPropertyNode()->GetChildNode(OriginalIndex).Get();
		FPropertyNode* SecondChildNodePtr = GetPropertyNode()->GetChildNode(NewIndex).Get();

		FPropertyNode* ParentNode = FirstChildNodePtr->GetParentNode();
		FObjectPropertyNode* ObjectNode = FirstChildNodePtr->FindObjectItemParent();

		FProperty* FirstNodeProperty = FirstChildNodePtr->GetProperty();
		FProperty* SecondNodeProperty = SecondChildNodePtr->GetProperty();
		FArrayProperty* ArrayProperty = FirstNodeProperty->GetOwner<FArrayProperty>();

		check(ArrayProperty);

		FReadAddressList ReadAddresses;
		ParentNode->GetReadAddress(!!ParentNode->HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ReadAddresses);
		if (ReadAddresses.Num())
		{
			// List of top level objects sent to the PropertyChangedEvent
			TArray<const UObject*> TopLevelObjects;
			TopLevelObjects.Reserve(ReadAddresses.Num());

			// perform the operation on the array for all selected objects
			for (int32 i = 0; i < ReadAddresses.Num(); ++i)
			{
				uint8* Address = ReadAddresses.GetAddress(i);

				if (Address)
				{
					int32 FirstIndex = FirstChildNodePtr->GetArrayIndex();
					int32 SecondIndex = SecondChildNodePtr->GetArrayIndex();

					UObject* Obj = ObjectNode ? ObjectNode->GetUObject(i) : nullptr;
					if (Obj)
					{
						if ((Obj->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) ||
							(Obj->HasAnyFlags(RF_DefaultSubObject) && Obj->GetOuter()->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))))
						{
							FirstChildNodePtr->PropagateContainerPropertyChange(Obj, Address, EPropertyArrayChangeType::Swap, FirstIndex, SecondIndex);
						}

						TopLevelObjects.Add(Obj);
					}

					FScriptArrayHelper ArrayHelper(ArrayProperty, Address);
					ArrayHelper.SwapValues(FirstIndex, SecondIndex);
				}
			}

			FPropertyChangedEvent ChangeEvent(ParentNode->GetProperty(), EPropertyChangeType::ArrayMove, MakeArrayView(TopLevelObjects));

			if (PropertyUtilities.IsValid())
			{
				FirstChildNodePtr->FixPropertiesInEvent(ChangeEvent);
				SecondChildNodePtr->FixPropertiesInEvent(ChangeEvent);
			}
		}
	}

	// Delete the original index
	{
		FPropertyNode* ChildNodePtr = GetPropertyNode()->GetChildNode(OriginalIndex).Get();

		FPropertyNode* ParentNode = ChildNodePtr->GetParentNode();
		FObjectPropertyNode* ObjectNode = ChildNodePtr->FindObjectItemParent();

		FProperty* NodeProperty = ChildNodePtr->GetProperty();
		FArrayProperty* ArrayProperty = NodeProperty->GetOwner<FArrayProperty>();

		FReadAddressList ReadAddresses;
		ParentNode->GetReadAddress(!!ParentNode->HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ReadAddresses);
		if (ReadAddresses.Num())
		{
			// List of top level objects sent to the PropertyChangedEvent
			TArray<const UObject*> TopLevelObjects;
			TopLevelObjects.Reserve(ReadAddresses.Num());

			// perform the operation on the array for all selected objects
			for (int32 i = 0; i < ReadAddresses.Num(); ++i)
			{
				uint8* Address = ReadAddresses.GetAddress(i);

				if (Address)
				{
					int32 Index = ChildNodePtr->GetArrayIndex();

					UObject* Obj = ObjectNode ? ObjectNode->GetUObject(i) : nullptr;
					if (Obj)
					{
						if ((Obj->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) ||
							(Obj->HasAnyFlags(RF_DefaultSubObject) && Obj->GetOuter()->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))) &&
							!FApp::IsGame())
						{
							ChildNodePtr->PropagateContainerPropertyChange(Obj, Address, EPropertyArrayChangeType::Delete, Index);
						}

						TopLevelObjects.Add(Obj);
					}

					FScriptArrayHelper ArrayHelper(ArrayProperty, Address);
					ArrayHelper.RemoveValues(ChildNodePtr->GetArrayIndex());
				}
			}

			FPropertyChangedEvent ChangeEvent(ParentNode->GetProperty(), EPropertyChangeType::ArrayRemove, MakeArrayView(TopLevelObjects));
			if (PropertyUtilities.IsValid())
			{
				ChildNodePtr->FixPropertiesInEvent(ChangeEvent);
			}
		}
	}

	GetPropertyNode()->RebuildChildren();
}

void FPropertyValueImpl::DuplicateChild( int32 Index )
{
	TSharedPtr<FPropertyNode> ArrayParentPin = PropertyNode.Pin();
	if( ArrayParentPin.IsValid() )
	{
		DuplicateChild( ArrayParentPin->GetChildNode( Index ) );
	}
}

void FPropertyValueImpl::DuplicateChild( TSharedPtr<FPropertyNode> ChildNodeToDuplicate )
{
	FPropertyNode* ChildNodePtr = ChildNodeToDuplicate.Get();

	FPropertyNode* ParentNode = ChildNodePtr->GetParentNode();
	FObjectPropertyNode* ObjectNode = ChildNodePtr->FindObjectItemParent();

	FProperty* NodeProperty = ChildNodePtr->GetProperty();
	FArrayProperty* ArrayProperty = NodeProperty->GetOwner<FArrayProperty>(); // duplication is only supported for arrays

	FReadAddressList ReadAddresses;
	void* Addr = nullptr;
	ParentNode->GetReadAddress( !!ParentNode->HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ReadAddresses );
	if ( ReadAddresses.Num() )
	{
		Addr = ReadAddresses.GetAddress(0);
	}

	if( Addr )
	{
		TArray< UObject* > AffectedInstances;

		// List of top level objects sent to the PropertyChangedEvent
		TArray<const UObject*> TopLevelObjects;

		FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "DuplicateChild", "Duplicate Child") );

		TArray< TMap<FString, int32> > ArrayIndicesPerObject;
		TSet< UObject* > AllAffectedInstances;
		AllAffectedInstances.Append(AffectedInstances);
		
		if (!FApp::IsGame())
		{
			UObject* Obj = ObjectNode ? ObjectNode->GetUObject(0) : nullptr;
			if (IsTemplate(Obj))
			{
				ChildNodePtr->GatherInstancesAffectedByContainerPropertyChange(Obj, Addr, EPropertyArrayChangeType::Duplicate, AffectedInstances);
			}

			if (Obj)
			{
				TopLevelObjects.Add(Obj);
			}

			ArrayIndicesPerObject.Add(TMap<FString, int32>());
			FPropertyValueImpl::GenerateArrayIndexMapToObjectNode(ArrayIndicesPerObject[0], ChildNodePtr);
		}
		
		ChildNodePtr->NotifyPreChange( ParentNode->GetProperty(), NotifyHook, AllAffectedInstances );

		if (PropertyUtilities.IsValid() && !bInteractiveChangeInProgress)
		{
			FPropertyChangedEvent StartedChangingPropertiesEvent(ParentNode->GetProperty(), EPropertyChangeType::Duplicate, MakeArrayView(TopLevelObjects));
			StartedChangingPropertiesEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);
			PropertyUtilities.Pin()->NotifyStartedChangingProperties(StartedChangingPropertiesEvent);
		}

		int32 Index = ChildNodePtr->GetArrayIndex();
		UObject* Obj = ObjectNode ? ObjectNode->GetUObject(0) : nullptr;

		if (Obj)
		{
			if (IsTemplate(Obj) && !FApp::IsGame())
			{
				ChildNodePtr->PropagateContainerPropertyChange(Obj, Addr, AffectedInstances, EPropertyArrayChangeType::Duplicate, Index);
			}
		}
		
		FScriptArrayHelper ArrayHelper(ArrayProperty, Addr);
		FPropertyNode::DuplicateArrayEntry(Obj, NodeProperty, ArrayHelper, Index);
		
		FPropertyChangedEvent ChangeEvent(ParentNode->GetProperty(), EPropertyChangeType::Duplicate, MakeArrayView(TopLevelObjects));
		ChangeEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);
		ChangeEvent.SetInstancesChanged(MoveTemp(AllAffectedInstances));

		TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
		PropertyNodePin->NotifyPostChange(ChangeEvent, NotifyHook);

		if (PropertyUtilities.IsValid())
		{
			ChildNodePtr->FixPropertiesInEvent(ChangeEvent);
			PropertyUtilities.Pin()->NotifyFinishedChangingProperties(ChangeEvent);
		}

		PropertyNodePin->RebuildChildren();
	}
}

bool FPropertyValueImpl::HasValidPropertyNode() const
{
	return PropertyNode.IsValid() && !PropertyNode.Pin()->IsDestroyed();
}

FText FPropertyValueImpl::GetDisplayName() const
{
	return PropertyNode.IsValid() ? PropertyNode.Pin()->GetDisplayName() : FText::GetEmpty();
}

void FPropertyValueImpl::ShowInvalidOperationError(const FText& ErrorText)
{
	if (!InvalidOperationError.IsValid())
	{
		FNotificationInfo InvalidOperation(ErrorText);
		InvalidOperation.ExpireDuration = 3.0f;
		InvalidOperationError = FSlateNotificationManager::Get().AddNotification(InvalidOperation);
	}
}

#define IMPLEMENT_PROPERTY_ACCESSOR( ValueType ) \
	FPropertyAccess::Result FPropertyHandleBase::SetValue( ValueType const& InValue, EPropertyValueSetFlags::Type Flags ) \
	{ \
		return FPropertyAccess::Fail; \
	} \
	FPropertyAccess::Result FPropertyHandleBase::GetValue( ValueType& OutValue ) const \
	{ \
		return FPropertyAccess::Fail; \
	}

IMPLEMENT_PROPERTY_ACCESSOR( bool )
IMPLEMENT_PROPERTY_ACCESSOR( int8 )
IMPLEMENT_PROPERTY_ACCESSOR( int16 )
IMPLEMENT_PROPERTY_ACCESSOR( int32 )
IMPLEMENT_PROPERTY_ACCESSOR( int64 )
IMPLEMENT_PROPERTY_ACCESSOR( uint8 )
IMPLEMENT_PROPERTY_ACCESSOR( uint16 )
IMPLEMENT_PROPERTY_ACCESSOR( uint32 )
IMPLEMENT_PROPERTY_ACCESSOR( uint64 )
IMPLEMENT_PROPERTY_ACCESSOR( float )
IMPLEMENT_PROPERTY_ACCESSOR( double )
IMPLEMENT_PROPERTY_ACCESSOR( FString )
IMPLEMENT_PROPERTY_ACCESSOR( FText )
IMPLEMENT_PROPERTY_ACCESSOR( FName )
IMPLEMENT_PROPERTY_ACCESSOR( FVector )
IMPLEMENT_PROPERTY_ACCESSOR( FVector2D )
IMPLEMENT_PROPERTY_ACCESSOR( FVector4 )
IMPLEMENT_PROPERTY_ACCESSOR( FQuat )
IMPLEMENT_PROPERTY_ACCESSOR( FRotator )
IMPLEMENT_PROPERTY_ACCESSOR( UObject* )
IMPLEMENT_PROPERTY_ACCESSOR( const UObject* )
IMPLEMENT_PROPERTY_ACCESSOR( FAssetData )
IMPLEMENT_PROPERTY_ACCESSOR( FProperty* )
IMPLEMENT_PROPERTY_ACCESSOR( const FProperty* )

FPropertyAccess::Result FPropertyHandleBase::SetValue( const TCHAR* InValue, EPropertyValueSetFlags::Type Flags )
{
	return FPropertyAccess::Fail;
}

FPropertyHandleBase::FPropertyHandleBase( TSharedPtr<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities )
	: Implementation( MakeShareable( new FPropertyValueImpl( PropertyNode, NotifyHook, PropertyUtilities ) ) )
{

}

FPropertyAccess::Result FPropertyHandleBase::GetValueData(void*& OutAddress) const
{
	return Implementation->GetValueData(OutAddress);
}

bool FPropertyHandleBase::IsValidHandle() const
{
	return Implementation->HasValidPropertyNode();
}

bool FPropertyHandleBase::IsSamePropertyNode(TSharedPtr<IPropertyHandle> OtherHandle) const
{
	if (OtherHandle.IsValid() && OtherHandle->IsValidHandle())
	{
		return GetPropertyNode() == StaticCastSharedPtr<FPropertyHandleBase>(OtherHandle)->GetPropertyNode();
	}

	return !IsValidHandle();
}

FText FPropertyHandleBase::GetPropertyDisplayName() const
{
	return Implementation->GetDisplayName();
}

void FPropertyHandleBase::SetPropertyDisplayName(FText InDisplayName)
{
	if (Implementation->GetPropertyNode().IsValid())
	{
		Implementation->GetPropertyNode()->SetDisplayNameOverride(InDisplayName);
	}
}
	
void FPropertyHandleBase::ResetToDefault()
{
	Implementation->ResetToDefault();
}

bool FPropertyHandleBase::DiffersFromDefault() const
{
	return Implementation->DiffersFromDefault();
}

FText FPropertyHandleBase::GetResetToDefaultLabel() const
{
	return Implementation->GetResetToDefaultLabel();
}

void FPropertyHandleBase::MarkHiddenByCustomization()
{
	if(TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode(); PropertyNode.IsValid() )
	{
		PropertyNode->SetNodeFlags( EPropertyNodeFlags::IsCustomized, true );
	}
}

void FPropertyHandleBase::MarkResetToDefaultCustomized(bool bCustomized/* = true*/)
{
	if (TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode(); PropertyNode.IsValid())
	{
		PropertyNode->SetNodeFlags(EPropertyNodeFlags::HasCustomResetToDefault, bCustomized);
	}
}

void FPropertyHandleBase::ClearResetToDefaultCustomized()
{
	if (TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode(); PropertyNode.IsValid())
	{
		PropertyNode->SetNodeFlags(EPropertyNodeFlags::HasCustomResetToDefault, false);
	}
}

FStringView FPropertyHandleBase::GetPropertyPath() const
{
	if (Implementation->GetPropertyNode().IsValid())
	{
		return Implementation->GetPropertyNode()->GetPropertyPath();
	}

	return FStringView();
}

TSharedPtr<FPropertyPath> FPropertyHandleBase::CreateFPropertyPath() const
{
	if (Implementation->GetPropertyNode().IsValid())
 	{
 		return FPropertyNode::CreatePropertyPath(Implementation->GetPropertyNode().ToSharedRef());
 	}
	return TSharedPtr<FPropertyPath>();
}

int32 FPropertyHandleBase::GetArrayIndex() const
{
	if (Implementation->GetPropertyNode().IsValid())
	{
		return Implementation->GetPropertyNode()->GetArrayIndex();
	}
	return INDEX_NONE;
}

void FPropertyHandleBase::RequestRebuildChildren()
{
	if (Implementation->GetPropertyNode().IsValid())
	{
		Implementation->GetPropertyNode()->RequestRebuildChildren();
	}
}

bool FPropertyHandleBase::IsFavorite() const
{
	if (Implementation->GetPropertyNode().IsValid())
	{
		return Implementation->GetPropertyNode()->IsFavorite();
	}
	return false;
}

bool FPropertyHandleBase::IsCategoryHandle() const
{
	return Implementation->GetPropertyNode().IsValid() && Implementation->GetPropertyNode()->AsCategoryNode();
}

bool FPropertyHandleBase::IsCustomized() const
{
	return Implementation->GetPropertyNode()->HasNodeFlags( EPropertyNodeFlags::IsCustomized ) != 0;
}

bool FPropertyHandleBase::IsResetToDefaultCustomized() const
{
	return Implementation->GetPropertyNode()->HasNodeFlags(EPropertyNodeFlags::HasCustomResetToDefault) != 0;
}

FString FPropertyHandleBase::GeneratePathToProperty() const
{
	FString OutPath;
	if( Implementation.IsValid() && Implementation->GetPropertyNode().IsValid() )
	{
		const bool bArrayIndex = true;
		const bool bIgnoreCategories = true;
		FPropertyNode* StopParent = Implementation->GetPropertyNode()->FindObjectItemParent();
		Implementation->GetPropertyNode()->GetQualifiedName( OutPath, bArrayIndex, StopParent, bIgnoreCategories );
	}

	return OutPath;
}

TSharedRef<SWidget> FPropertyHandleBase::CreatePropertyNameWidget(const FText& NameOverride, const FText& ToolTipOverride, bool bDisplayResetToDefault, bool bDisplayText, bool bDisplayThumbnail) const
{
	return CreatePropertyNameWidget(NameOverride, ToolTipOverride);
}

TSharedRef<SWidget> FPropertyHandleBase::CreatePropertyNameWidget(const FText& NameOverride, const FText& ToolTipOverride) const
{
	if( Implementation.IsValid() && Implementation->GetPropertyNode().IsValid() )
	{
		struct FPropertyNodeDisplayNameOverrideHelper
		{
			FPropertyNodeDisplayNameOverrideHelper(TSharedPtr<FPropertyValueImpl> InImplementation, const FText& InNameOverride, const FText& InToolTipOverride)
				: Implementation(InImplementation)
				, bResetDisplayName(false)
				, bResetToolTipText(false)
			{
				if (!InNameOverride.IsEmpty())
				{
					bResetDisplayName = true;
					Implementation->GetPropertyNode()->SetDisplayNameOverride(InNameOverride);
				}
				
				if (!InToolTipOverride.IsEmpty())
				{
					bResetToolTipText = true;
					Implementation->GetPropertyNode()->SetToolTipOverride(InToolTipOverride);
				}
			}

			~FPropertyNodeDisplayNameOverrideHelper()
			{
				if (bResetDisplayName)
				{
					Implementation->GetPropertyNode()->SetDisplayNameOverride(FText::GetEmpty());
				}
				
				if (bResetToolTipText)
				{
					Implementation->GetPropertyNode()->SetToolTipOverride(FText::GetEmpty());
				}
			}

		private:
			TSharedPtr<FPropertyValueImpl> Implementation;
			bool bResetDisplayName;
			bool bResetToolTipText;
		};

		FPropertyNodeDisplayNameOverrideHelper TempPropertyNameOverride(Implementation, NameOverride, ToolTipOverride);

		TSharedPtr<FPropertyEditor> PropertyEditor = FPropertyEditor::Create( Implementation->GetPropertyNode().ToSharedRef(), Implementation->GetPropertyUtilities().ToSharedRef() );

		return SNew( SPropertyNameWidget, PropertyEditor );
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> FPropertyHandleBase::CreatePropertyValueWidget( bool bDisplayDefaultPropertyButtons ) const
{
	if( Implementation.IsValid() && Implementation->GetPropertyNode().IsValid() )
	{
		TSharedPtr<FPropertyEditor> PropertyEditor = FPropertyEditor::Create( Implementation->GetPropertyNode().ToSharedRef(), Implementation->GetPropertyUtilities().ToSharedRef() );

		return SNew( SPropertyValueWidget, PropertyEditor, Implementation->GetPropertyUtilities() )
				.ShowPropertyButtons( bDisplayDefaultPropertyButtons );
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> FPropertyHandleBase::CreatePropertyValueWidgetWithCustomization( const IDetailsView* DetailsView )
{
	FCustomPropertyTypeLayoutMap CustomPropertyTypeLayoutMap = [DetailsView]()
	{
		if ( DetailsView )
		{
			return static_cast<const IDetailsViewPrivate*>( DetailsView )->GetCustomPropertyTypeLayoutMap();
		}
		else
		{
			return FCustomPropertyTypeLayoutMap();
		}
	}();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FPropertyTypeLayoutCallback LayoutCallback = PropertyEditorModule.GetPropertyTypeCustomization( GetProperty(), *this, CustomPropertyTypeLayoutMap );
	if (LayoutCallback.IsValid())
	{
		TSharedRef<IPropertyTypeCustomization> PropertyTypeCustomization = LayoutCallback.GetCustomizationInstance();

		return SNew( SStandaloneCustomizedValueWidget, PropertyTypeCustomization, AsShared() );
	}
	else
	{
		// Opting to not generate the default buttons since the code path that goes through a customization would likely not have them
		const bool bDisplayDefaultPropertyButtons = false;
		return CreatePropertyValueWidget( bDisplayDefaultPropertyButtons );
	}
}

class SDefaultPropertyButtonWidgets : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SDefaultPropertyButtonWidgets)	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FPropertyEditor> InPropertyEditor)
	{
		PropertyEditor = InPropertyEditor;
		TSharedRef<SHorizontalBox> ButtonBox = SNew(SHorizontalBox);

		TArray<TSharedRef<SWidget>> RequiredButtons;
		PropertyEditorHelpers::MakeRequiredPropertyButtons(PropertyEditor.ToSharedRef(), RequiredButtons);
		for (TSharedRef<SWidget> RequiredButton : RequiredButtons)
		{
			ButtonBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f)
			[
				RequiredButton
			];
		}

		ChildSlot
		[
			ButtonBox
		];
	}

private:
	TSharedPtr<FPropertyEditor> PropertyEditor;
};

TSharedRef<SWidget> FPropertyHandleBase::CreateDefaultPropertyButtonWidgets() const
{
	TArray<TSharedRef<SWidget>> DefaultButtons;
	if (Implementation.IsValid() && Implementation->GetPropertyNode().IsValid())
	{
		TSharedRef<FPropertyEditor> PropertyEditor = FPropertyEditor::Create(Implementation->GetPropertyNode().ToSharedRef(), Implementation->GetPropertyUtilities().ToSharedRef());
		return SNew(SDefaultPropertyButtonWidgets, PropertyEditor);
	}
	return SNullWidget::NullWidget;
}

void FPropertyHandleBase::CreateDefaultPropertyCopyPasteActions(FUIAction& OutCopyAction, FUIAction& OutPasteAction) const
{
	if(Implementation.IsValid())
	{
		OutCopyAction.ExecuteAction.BindStatic(&FPropertyHandleBase::CopyValueToClipboard, TWeakPtr<FPropertyValueImpl>(Implementation.ToSharedRef()));
		OutPasteAction.ExecuteAction.BindStatic(&FPropertyHandleBase::PasteValueFromClipboard, TWeakPtr<FPropertyValueImpl>(Implementation.ToSharedRef()));
	}
}

bool FPropertyHandleBase::IsEditConst() const
{
	return Implementation->IsEditConst();
}

bool FPropertyHandleBase::IsEditable() const
{
	return !IsEditConst();
}

bool FPropertyHandleBase::IsExpanded() const
{
	return Implementation->IsExpanded();
}

void FPropertyHandleBase::SetExpanded(bool bExpanded)
{
	Implementation->SetExpanded(bExpanded);
}

FPropertyAccess::Result FPropertyHandleBase::GetValueAsFormattedString( FString& OutValue, EPropertyPortFlags PortFlags ) const
{
	return Implementation->GetValueAsString(OutValue, PortFlags);
}

FPropertyAccess::Result FPropertyHandleBase::GetValueAsDisplayString( FString& OutValue, EPropertyPortFlags PortFlags ) const
{
	return Implementation->GetValueAsDisplayString(OutValue, PortFlags);
}

FPropertyAccess::Result FPropertyHandleBase::GetValueAsFormattedText( FText& OutValue ) const
{
	return Implementation->GetValueAsText(OutValue);
}

FPropertyAccess::Result FPropertyHandleBase::GetValueAsDisplayText( FText& OutValue ) const
{
	return Implementation->GetValueAsDisplayText(OutValue);
}

FPropertyAccess::Result FPropertyHandleBase::SetValueFromFormattedString( const FString& InValue, EPropertyValueSetFlags::Type Flags )
{
	return Implementation->SetValueAsString( InValue, Flags );
}

TSharedPtr<IPropertyHandle> FPropertyHandleBase::GetChildHandle( FName ChildName, bool bRecurse ) const
{
	// Container children cannot be accessed in this manner
	if( ! ( Implementation->IsPropertyTypeOf(FArrayProperty::StaticClass() ) || Implementation->IsPropertyTypeOf(FSetProperty::StaticClass()) || Implementation->IsPropertyTypeOf(FMapProperty::StaticClass()) ) )
	{
		TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetChildNode( ChildName, bRecurse );

		if( PropertyNode.IsValid() )
		{
			return PropertyEditorHelpers::GetPropertyHandle( PropertyNode.ToSharedRef(), Implementation->GetNotifyHook(), Implementation->GetPropertyUtilities() );
		}
	}
	
	return nullptr;
}

TSharedPtr<IPropertyHandle> FPropertyHandleBase::GetChildHandle( uint32 ChildIndex ) const
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetChildNode( ChildIndex );

	if( PropertyNode.IsValid() )
	{
		return PropertyEditorHelpers::GetPropertyHandle( PropertyNode.ToSharedRef(), Implementation->GetNotifyHook(), Implementation->GetPropertyUtilities() );
	}
	return nullptr;
}

TSharedPtr<IPropertyHandle> FPropertyHandleBase::GetParentHandle() const
{
	TSharedPtr<FPropertyNode> ParentNode = Implementation->GetPropertyNode()->GetParentNodeSharedPtr();
	if( ParentNode.IsValid() )
	{
		return PropertyEditorHelpers::GetPropertyHandle( ParentNode.ToSharedRef(), Implementation->GetNotifyHook(), Implementation->GetPropertyUtilities() );
	}

	return nullptr;
}

TSharedPtr<IPropertyHandle> FPropertyHandleBase::GetKeyHandle() const
{
	TSharedPtr<FPropertyNode> KeyNode = Implementation->GetPropertyNode()->GetPropertyKeyNode();
	if (KeyNode.IsValid())
	{
		return PropertyEditorHelpers::GetPropertyHandle(KeyNode.ToSharedRef(), Implementation->GetNotifyHook(), Implementation->GetPropertyUtilities());
	}

	return nullptr;
}

FPropertyAccess::Result FPropertyHandleBase::GetNumChildren( uint32& OutNumChildren ) const
{
	OutNumChildren = Implementation->GetNumChildren();
	return FPropertyAccess::Success;
}

uint32 FPropertyHandleBase::GetNumOuterObjects() const
{
	uint32 NumObjects = 0;
	if (Implementation->GetPropertyNode().IsValid())
	{
		FObjectPropertyNode* ObjectNode = Implementation->GetPropertyNode()->FindObjectItemParent();


		if (ObjectNode)
		{
			NumObjects = ObjectNode->GetNumObjects();
		}
	}

	return NumObjects;
}

void FPropertyHandleBase::GetOuterObjects( TArray<UObject*>& OuterObjects ) const
{
	if (Implementation->GetPropertyNode().IsValid())
	{
		FObjectPropertyNode* ObjectNode = Implementation->GetPropertyNode()->FindObjectItemParent();
		if (ObjectNode)
		{
			for (int32 ObjectIndex = 0; ObjectIndex < ObjectNode->GetNumObjects(); ++ObjectIndex)
			{
				OuterObjects.Add(ObjectNode->GetUObject(ObjectIndex));
			}
		}
	}
}

void FPropertyHandleBase::GetOuterStructs( TArray<TSharedPtr<FStructOnScope>>& OutStructs ) const
{
	if (Implementation->GetPropertyNode().IsValid())
	{
		if (const FStructurePropertyNode* StructNode = Implementation->GetPropertyNode()->FindStructureItemParent())
		{
			StructNode->GetAllStructureData(OutStructs);
		}
	}
}

const UClass* FPropertyHandleBase::GetOuterBaseClass() const
{
	if (Implementation->GetPropertyNode().IsValid())
	{
		FObjectPropertyNode* ObjectNode = Implementation->GetPropertyNode()->FindObjectItemParent();
		if (ObjectNode)
		{
			return ObjectNode->GetObjectBaseClass();
		}
	}

	return nullptr;
}

void FPropertyHandleBase::ReplaceOuterObjects(const TArray<UObject*>& OuterObjects)
{
	if (Implementation->GetPropertyNode().IsValid())
	{
		FObjectPropertyNode* ObjectNode = Implementation->GetPropertyNode()->FindObjectItemParent();
		if (ObjectNode)
		{
			ObjectNode->RemoveAllObjects();
			ObjectNode->ClearCachedReadAddresses(true);
			ObjectNode->AddObjects(OuterObjects);
		}
	}
}

void FPropertyHandleBase::GetOuterPackages(TArray<UPackage*>& OuterPackages) const
{
	if(Implementation->GetPropertyNode().IsValid())
	{
		FComplexPropertyNode* ComplexNode = Implementation->GetPropertyNode()->FindComplexParent();
		if (ComplexNode)
		{
			switch (ComplexNode->GetPropertyType())
			{
			case FComplexPropertyNode::EPT_Object:
			{
				FObjectPropertyNode* ObjectNode = static_cast<FObjectPropertyNode*>(ComplexNode);
				for (int32 ObjectIndex = 0; ObjectIndex < ObjectNode->GetNumObjects(); ++ObjectIndex)
				{
					OuterPackages.Add(ObjectNode->GetUPackage(ObjectIndex));
				}
			}
			break;

			case FComplexPropertyNode::EPT_StandaloneStructure:
			{
				FStructurePropertyNode* StructNode = static_cast<FStructurePropertyNode*>(ComplexNode);
				StructNode->GetOwnerPackages(OuterPackages);
			}
			break;

			default:
				break;
			}
		}
	}
}

void FPropertyHandleBase::EnumerateRawData( const EnumerateRawDataFuncRef& InRawDataCallback )
{
	Implementation->EnumerateRawData( InRawDataCallback );
}

void FPropertyHandleBase::EnumerateConstRawData( const EnumerateConstRawDataFuncRef& InRawDataCallback ) const
{
	Implementation->EnumerateConstRawData( InRawDataCallback );
}

void FPropertyHandleBase::AccessRawData( TArray<void*>& RawData )
{
	Implementation->AccessRawData( RawData );
}

void FPropertyHandleBase::AccessRawData( TArray<const void*>& RawData ) const
{
	Implementation->AccessRawData(RawData);
}

void FPropertyHandleBase::SetOnPropertyValueChanged( const FSimpleDelegate& InOnPropertyValueChanged )
{
	Implementation->SetOnPropertyValueChanged(InOnPropertyValueChanged);
}

void FPropertyHandleBase::SetOnPropertyValueChangedWithData(const TDelegate<void(const FPropertyChangedEvent&)>& InOnPropertyValueChanged)
{
	Implementation->SetOnPropertyValueChangedWithData(InOnPropertyValueChanged);
}

void FPropertyHandleBase::SetOnChildPropertyValueChanged( const FSimpleDelegate& InOnChildPropertyValueChanged )
{
	Implementation->SetOnChildPropertyValueChanged( InOnChildPropertyValueChanged );
}

void FPropertyHandleBase::SetOnChildPropertyValueChangedWithData(const TDelegate<void(const FPropertyChangedEvent&)>& InOnChildPropertyValueChanged)
{
	Implementation->SetOnChildPropertyValueChangedWithData(InOnChildPropertyValueChanged);
}

void FPropertyHandleBase::SetOnPropertyValuePreChange(const FSimpleDelegate& InOnPropertyValuePreChange)
{
	Implementation->SetOnPropertyValuePreChange(InOnPropertyValuePreChange);
}

void FPropertyHandleBase::SetOnChildPropertyValuePreChange(const FSimpleDelegate& InOnChildPropertyValuePreChange)
{
	Implementation->SetOnChildPropertyValuePreChange(InOnChildPropertyValuePreChange);
}

void FPropertyHandleBase::SetOnPropertyResetToDefault(const FSimpleDelegate& InOnPropertyResetToDefault)
{
	Implementation->SetOnPropertyResetToDefault(InOnPropertyResetToDefault);
}

TSharedPtr<FPropertyNode> FPropertyHandleBase::GetPropertyNode() const
{
	return Implementation->GetPropertyNode();
}

void FPropertyHandleBase::OnCustomResetToDefault(const FResetToDefaultOverride& CustomResetToDefault)
{
	if (CustomResetToDefault.HasResetToDefaultHandler())
	{
		FScopedTransaction Transaction(LOCTEXT("PropertyCustomResetToDefault", "Custom Reset to Default"));
		if (Implementation->GetPropertyUtilities().IsValid() && Implementation->GetPropertyUtilities()->GetNotifyHook() != nullptr)
		{
			Implementation->GetPropertyNode()->NotifyPreChange(Implementation->GetPropertyNode()->GetProperty(), Implementation->GetPropertyUtilities()->GetNotifyHook());
			FPropertyChangedEvent StartedChangingPropertiesEvent(Implementation->GetPropertyNode()->GetProperty());
			Implementation->GetPropertyUtilities()->NotifyStartedChangingProperties(StartedChangingPropertiesEvent);
		}

		FResetToDefaultHandler Delegate = CustomResetToDefault.GetPropertyResetToDefaultDelegate();
		Delegate.ExecuteIfBound(SharedThis(this));

		// Call PostEditchange on all the objects
		FPropertyChangedEvent ChangeEvent(Implementation->GetPropertyNode()->GetProperty());
		if (Implementation->GetPropertyUtilities().IsValid() && Implementation->GetPropertyUtilities()->GetNotifyHook() != nullptr)
		{
			Implementation->GetPropertyNode()->NotifyPostChange(ChangeEvent, Implementation->GetPropertyUtilities()->GetNotifyHook());
		}
	}
}

int32 FPropertyHandleBase::GetIndexInArray() const
{
	if( Implementation->GetPropertyNode().IsValid() )
	{
		return Implementation->GetPropertyNode()->GetArrayIndex();
	}

	return INDEX_NONE;
}

const FFieldClass* FPropertyHandleBase::GetPropertyClass() const
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if( PropertyNode.IsValid() && PropertyNode->GetProperty() )
	{
		return PropertyNode->GetProperty()->GetClass();
	}

	return nullptr;
}

FProperty* FPropertyHandleBase::GetProperty() const
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if( PropertyNode.IsValid() )
	{
		return PropertyNode->GetProperty();
	}

	return nullptr;
}

FProperty* FPropertyHandleBase::GetMetaDataProperty() const
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if( PropertyNode.IsValid() )
	{
		return PropertyNode->GetMetaDataProperty();
	}

	return nullptr;
}

bool FPropertyHandleBase::HasMetaData(const FName& Key) const
{
	if (const FString* InstanceValue = GetInstanceMetaData(Key))
	{
		return true;
	}

	const FProperty* MetaDataProperty = GetMetaDataProperty();
	return MetaDataProperty ? MetaDataProperty->HasMetaData(Key) : false;
}

const FString& FPropertyHandleBase::GetMetaData(const FName& Key) const
{
	// if not found, return a static empty string
	static const FString EmptyString = TEXT("");

	if (const FString* InstanceValue = GetInstanceMetaData(Key))
	{
		return *InstanceValue;
	}

	const FProperty* MetaDataProperty = GetMetaDataProperty();
	return MetaDataProperty ? MetaDataProperty->GetMetaData(Key) : EmptyString;
}

bool FPropertyHandleBase::GetBoolMetaData(const FName& Key) const
{
	if (!HasMetaData(Key))
	{
		return false;
	}

	const FString& StringValue = GetMetaData(Key);
	return StringValue.IsEmpty() || StringValue == TEXT("true");
}

int32 FPropertyHandleBase::GetIntMetaData(const FName& Key) const
{
	const FString& StringValue = GetMetaData(Key);
	return FCString::Atoi(*StringValue);
}

float FPropertyHandleBase::GetFloatMetaData(const FName& Key) const
{
	const FString& StringValue = GetMetaData(Key);
	return FCString::Atof(*StringValue);
}

double FPropertyHandleBase::GetDoubleMetaData(const FName& Key) const
{
	const FString& StringValue = GetMetaData(Key);
	return FCString::Atod(*StringValue);
}

UClass* FPropertyHandleBase::GetClassMetaData(const FName& Key) const
{
	const FString& ClassName = GetMetaData(Key);
	UClass* FoundClass = UClass::TryFindTypeSlow<UClass>(ClassName);
	return FoundClass;
}

void FPropertyHandleBase::SetInstanceMetaData(const FName& Key, const FString& Value)
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if (PropertyNode.IsValid())
	{
		PropertyNode->SetInstanceMetaData(Key, Value);
	}
}

const FString* FPropertyHandleBase::GetInstanceMetaData(const FName& Key) const
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if (PropertyNode.IsValid())
	{
		return PropertyNode->GetInstanceMetaData(Key);
	}

	return nullptr;
}

const TMap<FName, FString>* FPropertyHandleBase::GetInstanceMetaDataMap() const
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if (PropertyNode.IsValid())
	{
		return PropertyNode->GetInstanceMetaDataMap();
	}

	return nullptr;
}

FText FPropertyHandleBase::GetToolTipText() const
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if( PropertyNode.IsValid() )
	{
		return PropertyNode->GetToolTipText();
	}

	return FText::GetEmpty();
}

void FPropertyHandleBase::SetToolTipText( const FText& ToolTip )
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if (PropertyNode.IsValid())
	{
		PropertyNode->SetToolTipOverride( ToolTip );
	}
}

uint8* FPropertyHandleBase::GetValueBaseAddress(uint8* Base) const
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if (PropertyNode.IsValid())
	{
		return PropertyNode->GetValueBaseAddress(Base, PropertyNode->HasNodeFlags(EPropertyNodeFlags::IsSparseClassData) != 0);
	}

	return nullptr;
}

int32 FPropertyHandleBase::GetNumPerObjectValues() const
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if (PropertyNode.IsValid() && PropertyNode->GetProperty())
	{
		FComplexPropertyNode* ComplexNode = PropertyNode->FindComplexParent();
		if (ComplexNode)
		{
			return ComplexNode->GetInstancesNum();
		}
	}
	return 0;
}

FPropertyAccess::Result FPropertyHandleBase::SetPerObjectValues( const TArray<FString>& InPerObjectValues, EPropertyValueSetFlags::Type Flags )
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if( PropertyNode.IsValid() && PropertyNode->GetProperty() )
	{
		FComplexPropertyNode* ComplexNode = PropertyNode->FindComplexParent();

		if (ComplexNode && InPerObjectValues.Num() == ComplexNode->GetInstancesNum())
		{
			TArray<FObjectBaseAddress> ObjectsToModify;
			Implementation->GetObjectsToModify( ObjectsToModify, PropertyNode.Get() );

			if(ObjectsToModify.Num() > 0)
			{
				Implementation->ImportText( ObjectsToModify, InPerObjectValues, PropertyNode.Get(), Flags );
			}

			return FPropertyAccess::Success;
		}
	}

	return FPropertyAccess::Fail;
}

FPropertyAccess::Result FPropertyHandleBase::GetPerObjectValues( TArray<FString>& OutPerObjectValues ) const
{
	FPropertyAccess::Result Result = FPropertyAccess::Fail;
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if( PropertyNode.IsValid() && PropertyNode->GetProperty() )
	{
		// Get a list of addresses for objects handled by the property window.
		FReadAddressList ReadAddresses;
		PropertyNode->GetReadAddress( !!PropertyNode->HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ReadAddresses, false );

		FProperty* NodeProperty = PropertyNode->GetProperty();

		if( ReadAddresses.Num() > 0 )
		{
			// Copy each object's value into the value list
			OutPerObjectValues.SetNum( ReadAddresses.Num(), EAllowShrinking::No );
			for ( int32 AddrIndex = 0 ; AddrIndex < ReadAddresses.Num() ; ++AddrIndex )
			{
				uint8* Address = ReadAddresses.GetAddress(AddrIndex);
				if( Address )
				{
					NodeProperty->ExportText_Direct(OutPerObjectValues[AddrIndex], Address, Address, nullptr, 0 );
				}
				else
				{
					OutPerObjectValues[AddrIndex].Reset();
				}
			}

			Result = FPropertyAccess::Success;
		}
	}
	
	return Result;
}

FPropertyAccess::Result FPropertyHandleBase::SetPerObjectValue( const int32 ObjectIndex, const FString& ObjectValue, EPropertyValueSetFlags::Type Flags )
{
	FPropertyAccess::Result Result = FPropertyAccess::Fail;

	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if (PropertyNode.IsValid() && PropertyNode->GetProperty())
	{
		Implementation->EnumerateObjectsToModify(PropertyNode.Get(), [&](const FObjectBaseAddress& ObjectToModify, const int32 ObjIndex, const int32 NumObjects) -> bool
		{
			if (ObjIndex == ObjectIndex)
			{
				TArray<FObjectBaseAddress> ObjectsToModify;
				ObjectsToModify.Add(ObjectToModify);

				TArray<FString> PerObjectValues;
				PerObjectValues.Add(ObjectValue);

				Implementation->ImportText(ObjectsToModify, PerObjectValues, PropertyNode.Get(), Flags);

				Result = FPropertyAccess::Success;
				return false; // End enumeration
			}
			return true;
		});
	}

	return Result;
}

FPropertyAccess::Result FPropertyHandleBase::GetPerObjectValue( const int32 ObjectIndex, FString& OutObjectValue ) const
{
	FPropertyAccess::Result Result = FPropertyAccess::Fail;

	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if (PropertyNode.IsValid() && PropertyNode->GetProperty())
	{
		// Get a list of addresses for objects handled by the property window.
		FReadAddressList ReadAddresses;
		PropertyNode->GetReadAddress(!!PropertyNode->HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ReadAddresses, false);

		FProperty* NodeProperty = PropertyNode->GetProperty();

		if (ReadAddresses.IsValidIndex(ObjectIndex))
		{
			uint8* Address = ReadAddresses.GetAddress(ObjectIndex);
			if (Address)
			{
				NodeProperty->ExportText_Direct(OutObjectValue, Address, Address, nullptr, 0);
			}
			else
			{
				OutObjectValue.Reset();
			}

			Result = FPropertyAccess::Success;
		}
	}

	return Result;
}

bool FPropertyHandleBase::GeneratePossibleValues(TArray< TSharedPtr<FString> >& OutOptionStrings, TArray< FText >& OutToolTips, TArray<bool>& OutRestrictedItems)
{
	TArray<FText> DisplayNames;
	TArray<FString> ValueStrings;
	const bool bUsesDisplayNames = GeneratePossibleValues(ValueStrings, OutToolTips, OutRestrictedItems, &DisplayNames);

	if (bUsesDisplayNames)
	{
		Algo::Transform(DisplayNames, OutOptionStrings, [](const FText& Str) { return MakeShared<FString>(Str.ToString()); });
	}
	else
	{
		Algo::Transform(ValueStrings, OutOptionStrings, [](const FString& Str) { return MakeShared<FString>(Str); });
	}
	
	return bUsesDisplayNames;
}

bool FPropertyHandleBase::GeneratePossibleValues(TArray<FString>& OutOptionStrings, TArray< FText >& OutToolTips, TArray<bool>& OutRestrictedItems, TArray<FText>* OutDisplayNames)
{
	FProperty* Property = GetProperty();
	if (Property == nullptr)
	{
		return false;
	}

	bool bUsesAlternateDisplayValues = false;

	UEnum* Enum = nullptr;
	if( const FByteProperty* ByteProperty = CastField<const FByteProperty>( Property ) )
	{
		Enum = ByteProperty->Enum;
	}
	else if( const FEnumProperty* EnumProperty = CastField<const FEnumProperty>( Property ) )
	{
		Enum = EnumProperty->GetEnum();
	}
	else if ( Property->IsA(FStrProperty::StaticClass()) && Property->HasMetaData( TEXT("Enum") ) )
	{
		const FString& EnumName = Property->GetMetaData(TEXT("Enum"));
		Enum = UClass::TryFindTypeSlow<UEnum>(EnumName, EFindFirstObjectOptions::ExactClass);
		check( Enum );
	}

	if( Enum )
	{
		TArray<UObject*> OuterObjects;
		GetOuterObjects(OuterObjects);		
		
		const TArray<FName> ValidEnumValues = PropertyEditorHelpers::GetValidEnumsFromPropertyOverride(Property, Enum);
		const TArray<FName> InvalidEnumValues = PropertyEditorHelpers::GetInvalidEnumsFromPropertyOverride(Property, Enum);
		const TArray<FName> RestrictedEnumValues = PropertyEditorHelpers::GetRestrictedEnumsFromPropertyOverride(OuterObjects, Property, Enum);
		
		const TMap<FName, FText> EnumValueDisplayNameOverrides = PropertyEditorHelpers::GetEnumValueDisplayNamesFromPropertyOverride(Property, Enum);

		//NumEnums() - 1, because the last item in an enum is the _MAX item
		for( int32 EnumIndex = 0; EnumIndex < Enum->NumEnums() - 1; ++EnumIndex )
		{
			// Ignore hidden enums
			bool bShouldBeHidden = Enum->HasMetaData(TEXT("Hidden"), EnumIndex ) || Enum->HasMetaData(TEXT("Spacer"), EnumIndex );
			if (!bShouldBeHidden)
			{
				if(ValidEnumValues.Num() > 0)
				{
					bShouldBeHidden = !ValidEnumValues.Contains(Enum->GetNameByIndex(EnumIndex));
				}
				// If both are specified, InvalidEnumValues takes precedence
				else if(InvalidEnumValues.Num() > 0)
				{
					bShouldBeHidden = InvalidEnumValues.Contains(Enum->GetNameByIndex(EnumIndex));
				}
			}

			if (!bShouldBeHidden)
			{
				bShouldBeHidden = IsHidden(Enum->GetNameStringByIndex(EnumIndex));
			}

			if( !bShouldBeHidden )
			{
				// See if we specified an alternate name for this value using metadata
				FString EnumName = Enum->GetNameStringByIndex(EnumIndex);
				FString EnumDisplayName = EnumValueDisplayNameOverrides.FindRef(Enum->GetNameByIndex(EnumIndex)).ToString();
				if (EnumDisplayName.IsEmpty())
				{
					EnumDisplayName = Enum->GetDisplayNameTextByIndex(EnumIndex).ToString();
				}

				FText RestrictionTooltip;
				const bool bIsRestricted = GenerateRestrictionToolTip(EnumName, RestrictionTooltip) || RestrictedEnumValues.Contains(Enum->GetNameByIndex(EnumIndex));
				OutRestrictedItems.Add(bIsRestricted);

				const bool bHasDisplayName = EnumDisplayName.Len() > 0;

				if (OutDisplayNames)
				{
					if (bHasDisplayName)
					{
						OutDisplayNames->Add(FText::FromString(EnumDisplayName));
						bUsesAlternateDisplayValues = true;
					}
					else
					{
						// Added to ensure matching index of DisplayNames to ValueStrings
						OutDisplayNames->Add(FText::FromString(EnumName));
					}
				}
				
				OutOptionStrings.Add(EnumName);

				FText EnumValueToolTip = bIsRestricted ? RestrictionTooltip : Enum->GetToolTipTextByIndex(EnumIndex);
				OutToolTips.Add(MoveTemp(EnumValueToolTip));
			}
			else
			{
				OutToolTips.Add(FText());
			}
		}
	}

	FName MetaDataKey = PropertyEditorHelpers::GetPropertyOptionsMetaDataKey(Property);
	if (!MetaDataKey.IsNone())
	{
		FString GetOptionsFunctionName = Property->GetOwnerProperty()->GetMetaData(MetaDataKey);
		if (!GetOptionsFunctionName.IsEmpty())
		{
			TArray<UObject*> OuterObjects;
			GetOuterObjects(OuterObjects);

			PropertyEditorUtils::GetPropertyOptions(OuterObjects, GetOptionsFunctionName, OutOptionStrings, OutDisplayNames);

			if (OutDisplayNames && !OutDisplayNames->IsEmpty())
			{
				bUsesAlternateDisplayValues = true;
			}
		}
	}
	else if( Property->IsA(FClassProperty::StaticClass()) || Property->IsA(FSoftClassProperty::StaticClass()) )		
	{
		UClass* MetaClass = Property->IsA(FClassProperty::StaticClass()) 
			? CastFieldChecked<FClassProperty>(Property)->MetaClass
			: CastFieldChecked<FSoftClassProperty>(Property)->MetaClass;

		FString NoneStr( TEXT("None") );
		OutOptionStrings.Add( NoneStr );
		if (OutDisplayNames)
		{
			OutDisplayNames->Add(FText::FromString(NoneStr));
		}

		const bool bAllowAbstract = Property->GetOwnerProperty()->HasMetaData(TEXT("AllowAbstract"));
		const bool bBlueprintBaseOnly = Property->GetOwnerProperty()->HasMetaData(TEXT("BlueprintBaseOnly"));
		const bool bAllowOnlyPlaceable = Property->GetOwnerProperty()->HasMetaData(TEXT("OnlyPlaceable"));
		UClass* InterfaceThatMustBeImplemented = Property->GetOwnerProperty()->GetClassMetaData(TEXT("MustImplement"));

		if (!bAllowOnlyPlaceable || MetaClass->IsChildOf<AActor>())
		{
			for (TObjectIterator<UClass> It; It; ++It)
			{
				if (It->IsChildOf(MetaClass)
					&& PropertyEditorHelpers::IsEditInlineClassAllowed(*It, bAllowAbstract)
					&& (!bBlueprintBaseOnly || FKismetEditorUtilities::CanCreateBlueprintOfClass(*It))
					&& (!InterfaceThatMustBeImplemented || It->ImplementsInterface(InterfaceThatMustBeImplemented))
					&& (!bAllowOnlyPlaceable || !It->HasAnyClassFlags(CLASS_Abstract | CLASS_NotPlaceable)))
				{
					OutOptionStrings.Add(It->GetName());
					if (OutDisplayNames)
					{
						OutDisplayNames->Add(FText::FromString(It->GetName()));
					}
				}
			}
		}
	}

	return bUsesAlternateDisplayValues;
}

void FPropertyHandleBase::NotifyPreChange()
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if( PropertyNode.IsValid() )
	{
		PropertyNode->NotifyPreChange( PropertyNode->GetProperty(), Implementation->GetNotifyHook() );
	}
}

static TArray<TMap<FString, int32>> BuildArrayIndices(const TSharedPtr<FPropertyNode>& PropertyNode)
{
	TArray<TMap<FString, int32>> ArrayIndices;
	FReadAddressList ReadAddresses;
	PropertyNode->GetReadAddress(PropertyNode->HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ReadAddresses, false, true);

	const int32 ObjectNum = ReadAddresses.Num();
	if (ArrayIndices.IsEmpty())
	{
		ArrayIndices.SetNum(ObjectNum);
	}
	
	for (const FPropertyNode* ItemNode = PropertyNode.Get(); ItemNode; ItemNode = ItemNode->GetParentNode())
	{
		if (int32 Index = ItemNode->GetArrayIndex(); Index != INDEX_NONE)
		{
			for (int ObjectIndex = 0; ObjectIndex < ObjectNum; ++ObjectIndex)
			{
				ArrayIndices[ObjectIndex].Add(ItemNode->GetProperty()->GetName(), Index);
			}
		}
	} 

	return ArrayIndices;
}

void FPropertyHandleBase::NotifyPostChange( EPropertyChangeType::Type ChangeType )
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if( PropertyNode.IsValid() )
	{
		TArray<const UObject*> ObjectsBeingChanged;
		FObjectPropertyNode* ObjectNode = PropertyNode->FindObjectItemParent();
		if(ObjectNode)
		{
			ObjectsBeingChanged.Reserve(ObjectNode->GetNumObjects());
			for(int32 ObjectIndex = 0; ObjectIndex < ObjectNode->GetNumObjects(); ++ObjectIndex)
			{
				ObjectsBeingChanged.Add(ObjectNode->GetUObject(ObjectIndex));
			}
		}

		FPropertyChangedEvent PropertyChangedEvent( PropertyNode->GetProperty(), ChangeType, MakeArrayView(ObjectsBeingChanged) );
		TArray<TMap<FString, int32>> ArrayIndices = BuildArrayIndices(PropertyNode);
		PropertyChangedEvent.SetArrayIndexPerObject(ArrayIndices);
		PropertyNode->NotifyPostChange( PropertyChangedEvent, Implementation->GetNotifyHook());
	}
}

void FPropertyHandleBase::NotifyFinishedChangingProperties()
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if( PropertyNode.IsValid() )
	{
		FPropertyChangedEvent ChangeEvent(PropertyNode->GetProperty(), EPropertyChangeType::ValueSet);
		PropertyNode->FixPropertiesInEvent(ChangeEvent);
		Implementation->GetPropertyUtilities()->NotifyFinishedChangingProperties(ChangeEvent);
	}
}

FPropertyAccess::Result FPropertyHandleBase::SetObjectValueFromSelection()
{
	// Only implemented by Object handles
	return FPropertyAccess::Result::Fail;
}

void FPropertyHandleBase::AddRestriction( TSharedRef<const FPropertyRestriction> Restriction )
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if( PropertyNode.IsValid() )
	{
		PropertyNode->AddRestriction(Restriction);
	}
}

bool FPropertyHandleBase::IsRestricted(const FString& Value) const
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if( PropertyNode.IsValid() )
	{
		return PropertyNode->IsRestricted(Value);
	}
	return false;
}

bool FPropertyHandleBase::IsRestricted(const FString& Value, TArray<FText>& OutReasons) const
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if( PropertyNode.IsValid() )
	{
		return PropertyNode->IsRestricted(Value, OutReasons);
	}
	return false;
}

bool FPropertyHandleBase::IsHidden(const FString& Value) const
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if( PropertyNode.IsValid() )
	{
		return PropertyNode->IsHidden(Value);
	}
	return false;
}

bool FPropertyHandleBase::IsHidden(const FString& Value, TArray<FText>& OutReasons) const
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if( PropertyNode.IsValid() )
	{
		return PropertyNode->IsHidden(Value, &OutReasons);
	}
	return false;
}

bool FPropertyHandleBase::IsDisabled(const FString& Value) const
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if( PropertyNode.IsValid() )
	{
		return PropertyNode->IsDisabled(Value);
	}
	return false;
}

bool FPropertyHandleBase::IsDisabled(const FString& Value, TArray<FText>& OutReasons) const
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if( PropertyNode.IsValid() )
	{
		return PropertyNode->IsDisabled(Value, &OutReasons);
	}
	return false;
}

bool FPropertyHandleBase::GenerateRestrictionToolTip(const FString& Value, FText& OutTooltip)const
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if( PropertyNode.IsValid() )
	{
		return PropertyNode->GenerateRestrictionToolTip(Value, OutTooltip);
	}
	return false;
}

void FPropertyHandleBase::SetIgnoreValidation(bool bInIgnore)
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if( PropertyNode.IsValid() )
	{
		PropertyNode->SetNodeFlags( EPropertyNodeFlags::SkipChildValidation, bInIgnore); 
	}
}

TArray<TSharedPtr<IPropertyHandle>> FPropertyHandleBase::AddChildStructure( TSharedRef<FStructOnScope> InStruct )
{
	TSharedRef<FStructOnScopeStructureDataProvider> StructProvider = MakeShared<FStructOnScopeStructureDataProvider>(InStruct.ToSharedPtr());
	return AddChildStructure(StructProvider);
}

TArray<TSharedPtr<IPropertyHandle>> FPropertyHandleBase::AddChildStructure(TSharedRef<IStructureDataProvider> InStructProvider)
{
	TArray<TSharedPtr<IPropertyHandle>> PropertyHandles;

	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	if( !PropertyNode.IsValid() )
	{
		return PropertyHandles;
	}

	TSharedPtr<FStructurePropertyNode> StructPropertyNode( new FStructurePropertyNode );
	StructPropertyNode->SetStructure(InStructProvider);

	FPropertyNodeInitParams RootInitParams;
	RootInitParams.ParentNode = PropertyNode;
	RootInitParams.Property = nullptr;
	RootInitParams.ArrayOffset = 0;
	RootInitParams.ArrayIndex = INDEX_NONE;
	RootInitParams.bAllowChildren = true;
	RootInitParams.bForceHiddenPropertyVisibility = FPropertySettings::Get().ShowHiddenProperties();
	RootInitParams.bCreateCategoryNodes = false;

	StructPropertyNode->InitNode(RootInitParams);

	// Loop over the structs child nodes and add property handles for each of them.
	for (int32 ChildNodeIndex = 0; ChildNodeIndex < StructPropertyNode->GetNumChildNodes(); ChildNodeIndex++)
	{
		PropertyHandles.Add(PropertyEditorHelpers::GetPropertyHandle(
			StructPropertyNode->GetChildNode(ChildNodeIndex).ToSharedRef(),
			Implementation->GetNotifyHook(),
			Implementation->GetPropertyUtilities()
		));
	}
	
	PropertyNode->AddChildNode(StructPropertyNode);

	return PropertyHandles;
}

void FPropertyHandleBase::RemoveChildren()
{
	if (const TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode())
	{
		PropertyNode->RemoveAllChildNodes();
	}
}

bool FPropertyHandleBase::CanResetToDefault() const
{
	FProperty* Property = GetProperty();
	if (Property == nullptr)
	{
		return false;
	}

	// Should not be able to reset fixed size arrays
	const bool bFixedSized = (Property->PropertyFlags & CPF_EditFixedSize) != 0;
	const bool bCanResetToDefault = (Property->PropertyFlags & CPF_Config) == 0 && !IsEditConst();

	return bCanResetToDefault && !bFixedSized && DiffersFromDefault();
}

void FPropertyHandleBase::ExecuteCustomResetToDefault(const FResetToDefaultOverride& InOnCustomResetToDefault)
{
	// This action must be deferred until next tick so that we avoid accessing invalid data before we have a chance to tick
	TSharedPtr<IPropertyUtilities> PropertyUtilities = Implementation->GetPropertyUtilities();
	if (PropertyUtilities.IsValid())
	{
		TSharedPtr<FPropertyHandleBase> ThisShared = SharedThis(this);
		PropertyUtilities->EnqueueDeferredAction(FSimpleDelegate::CreateLambda([ThisShared, InOnCustomResetToDefault]()
			{
				if (ThisShared.IsValid())
				{
					ThisShared->OnCustomResetToDefault(InOnCustomResetToDefault);
				}
			}));
	}
}

FName FPropertyHandleBase::GetDefaultCategoryName() const
{
	FProperty* Property = GetProperty();
	if (Property)
	{
		return FObjectEditorUtils::GetCategoryFName(Property);
	}

	return NAME_None;
}

FText FPropertyHandleBase::GetDefaultCategoryText() const
{
	FProperty* Property = GetProperty();

	if (Property)
	{
		return FObjectEditorUtils::GetCategoryText(Property);
	}

	return FText::GetEmpty();
}

void FPropertyHandleBase::CopyValueToClipboard(TWeakPtr<FPropertyValueImpl> ImplementationWeak)
{
	TSharedPtr<FPropertyValueImpl> Implementation = ImplementationWeak.Pin();
	if (Implementation.IsValid())
	{
		FString Value;
		if (Implementation->GetValueAsString(Value, PPF_Copy) == FPropertyAccess::Success)
		{
			FPlatformApplicationMisc::ClipboardCopy(*Value);
		}
	}
}

void FPropertyHandleBase::PasteValueFromClipboard(TWeakPtr<FPropertyValueImpl> ImplementationWeak)
{
	TSharedPtr<FPropertyValueImpl> Implementation = ImplementationWeak.Pin();
	if (Implementation.IsValid())
	{
		FString Value;
		FPlatformApplicationMisc::ClipboardPaste(Value);
		if (Value.IsEmpty() == false)
		{
			Implementation->SetValueAsString(Value, EPropertyValueSetFlags::DefaultFlags);
		}
	}
}

/** Implements common property value functions */
#define IMPLEMENT_PROPERTY_VALUE( ClassName ) \
	ClassName::ClassName( TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities ) \
	: FPropertyHandleBase( PropertyNode, NotifyHook, PropertyUtilities )  \
	{}

IMPLEMENT_PROPERTY_VALUE( FPropertyHandleInt )
IMPLEMENT_PROPERTY_VALUE( FPropertyHandleFloat )
IMPLEMENT_PROPERTY_VALUE( FPropertyHandleDouble )
IMPLEMENT_PROPERTY_VALUE( FPropertyHandleBool )
IMPLEMENT_PROPERTY_VALUE( FPropertyHandleByte )
IMPLEMENT_PROPERTY_VALUE( FPropertyHandleString )
IMPLEMENT_PROPERTY_VALUE( FPropertyHandleObject )
IMPLEMENT_PROPERTY_VALUE( FPropertyHandleArray )
IMPLEMENT_PROPERTY_VALUE(FPropertyHandleOptional)
IMPLEMENT_PROPERTY_VALUE( FPropertyHandleText )
IMPLEMENT_PROPERTY_VALUE( FPropertyHandleSet )
IMPLEMENT_PROPERTY_VALUE( FPropertyHandleMap )
IMPLEMENT_PROPERTY_VALUE( FPropertyHandleFieldPath )
IMPLEMENT_PROPERTY_VALUE( FPropertyHandleStruct )

// int32 
bool FPropertyHandleInt::Supports( TSharedRef<FPropertyNode> PropertyNode )
{
	FProperty* Property = PropertyNode->GetProperty();

	if ( Property == nullptr )
	{
		return false;
	}

	bool bIsSupported = 
			Property->IsA<FInt8Property>()
		||	Property->IsA<FInt16Property>()
		||  Property->IsA<FIntProperty>()
		||	Property->IsA<FInt64Property>()
		||  Property->IsA<FUInt16Property>()
		||  Property->IsA<FUInt32Property>()
		||  Property->IsA<FUInt64Property>();

	if (!bIsSupported)
	{
		if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			// We only support an enum if it's underlying type is NOT uint8, which is instead handled by FPropertyHandleByte
			const FProperty* UnderlyingProperty = EnumProperty->GetUnderlyingProperty();
			bIsSupported = UnderlyingProperty && !UnderlyingProperty->IsA<FByteProperty>();
		}
	}

	// The value is an integer (or enum that isn't uint8-based)
	return bIsSupported;
}

template <typename PropertyClass, typename ValueType>
ValueType GetIntegerValue(void* PropValue, FPropertyValueImpl& Implementation)
{
	if (Implementation.IsPropertyTypeOf(PropertyClass::StaticClass()))
	{
		return Implementation.GetPropertyValue<PropertyClass>(PropValue);
	}

	const FNumericProperty* NumericProperty = nullptr;
	if (Implementation.IsPropertyTypeOf(FEnumProperty::StaticClass()))
	{
		const FEnumProperty* EnumProperty = CastFieldChecked<FEnumProperty>(Implementation.GetPropertyNode()->GetProperty());
		NumericProperty = EnumProperty->GetUnderlyingProperty();
	}
	else if (Implementation.IsPropertyTypeOf(FNumericProperty::StaticClass()))
	{
		NumericProperty = CastFieldChecked<const FNumericProperty>(Implementation.GetPropertyNode()->GetProperty());
	}

	if (NumericProperty)
	{
		const bool bIsSignedInteger = NumericProperty->CanHoldValue(-1);
		if (bIsSignedInteger)
		{
			return static_cast<ValueType>(NumericProperty->GetSignedIntPropertyValue(PropValue));
		}
		return static_cast<ValueType>(NumericProperty->GetUnsignedIntPropertyValue(PropValue));
	}

	return 0;
}

FPropertyAccess::Result FPropertyHandleInt::GetValue(int8& OutValue) const
{
	void* PropValue = nullptr;
	FPropertyAccess::Result Res = Implementation->GetValueData(PropValue);

	if (Res == FPropertyAccess::Success)
	{
		OutValue = GetIntegerValue<FInt8Property, int8>(PropValue, *Implementation);
	}

	return Res;
}

FPropertyAccess::Result FPropertyHandleInt::GetValue(int16& OutValue) const
{
	void* PropValue = nullptr;
	FPropertyAccess::Result Res = Implementation->GetValueData(PropValue);

	if (Res == FPropertyAccess::Success)
	{
		OutValue = GetIntegerValue<FInt16Property, int16>(PropValue, *Implementation);
	}

	return Res;
}

FPropertyAccess::Result FPropertyHandleInt::GetValue(int32& OutValue) const
{
	void* PropValue = nullptr;
	FPropertyAccess::Result Res = Implementation->GetValueData(PropValue);

	if (Res == FPropertyAccess::Success)
	{
		OutValue = GetIntegerValue<FIntProperty, int32>(PropValue, *Implementation);
	}

	return Res;
}
FPropertyAccess::Result FPropertyHandleInt::GetValue( int64& OutValue ) const
{
	void* PropValue = nullptr;
	FPropertyAccess::Result Res = Implementation->GetValueData( PropValue );

	if( Res == FPropertyAccess::Success )
	{
		OutValue = GetIntegerValue<FInt64Property,int64>(PropValue, *Implementation);
	}

	return Res;
}

FPropertyAccess::Result FPropertyHandleInt::GetValue(uint8& OutValue) const
{
	void* PropValue = nullptr;
	FPropertyAccess::Result Res = Implementation->GetValueData(PropValue);

	if (Res == FPropertyAccess::Success)
	{
		OutValue = GetIntegerValue<FByteProperty, uint8>(PropValue, *Implementation);
	}

	return Res;
}

FPropertyAccess::Result FPropertyHandleInt::GetValue(uint16& OutValue) const
{
	void* PropValue = nullptr;
	FPropertyAccess::Result Res = Implementation->GetValueData(PropValue);

	if (Res == FPropertyAccess::Success)
	{
		OutValue = GetIntegerValue<FUInt16Property, uint16>(PropValue, *Implementation);
	}

	return Res;
}

FPropertyAccess::Result FPropertyHandleInt::GetValue(uint32& OutValue) const
{
	void* PropValue = nullptr;
	FPropertyAccess::Result Res = Implementation->GetValueData(PropValue);

	if (Res == FPropertyAccess::Success)
	{
		OutValue = GetIntegerValue<FUInt32Property, uint32>(PropValue, *Implementation);
	}

	return Res;
}

FPropertyAccess::Result FPropertyHandleInt::GetValue(uint64& OutValue) const
{
	void* PropValue = nullptr;
	FPropertyAccess::Result Res = Implementation->GetValueData(PropValue);

	if (Res == FPropertyAccess::Success)
	{
		OutValue = GetIntegerValue<FUInt64Property, uint64>(PropValue, *Implementation);
	}

	return Res;
}

FPropertyAccess::Result FPropertyHandleInt::SetValue(const int8& NewValue, EPropertyValueSetFlags::Type Flags)
{
	FPropertyAccess::Result Res;
	// Clamp the value from any meta data ranges stored on the property value
	int8 FinalValue = ClampIntegerValueFromMetaData<int8>( NewValue, *this, *Implementation->GetPropertyNode());

	FString ValueStr;
	if (const FEnumProperty* EnumProp = CastField<FEnumProperty>(GetProperty()))
	{
		ValueStr = EnumProp->GetEnum()->GetNameStringByValue(FinalValue);
	}
	else
	{
		ValueStr = LexToString(FinalValue);
	}
	Res = Implementation->ImportText(ValueStr, Flags);

	return Res;
}


FPropertyAccess::Result FPropertyHandleInt::SetValue(const int16& NewValue, EPropertyValueSetFlags::Type Flags)
{
	FPropertyAccess::Result Res;
	// Clamp the value from any meta data ranges stored on the property value
	int16 FinalValue = ClampIntegerValueFromMetaData<int16>(NewValue, *this, *Implementation->GetPropertyNode());

	FString ValueStr;
	if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(GetProperty()))
	{
		ValueStr = EnumProperty->GetEnum()->GetNameStringByValue(FinalValue);
	}
	else
	{
		ValueStr = LexToString(FinalValue);
	}
	Res = Implementation->ImportText(ValueStr, Flags);

	return Res;
}


FPropertyAccess::Result FPropertyHandleInt::SetValue( const int32& NewValue, EPropertyValueSetFlags::Type Flags )
{
	FPropertyAccess::Result Res;
	// Clamp the value from any meta data ranges stored on the property value
	int32 FinalValue = ClampIntegerValueFromMetaData<int32>( NewValue, *this, *Implementation->GetPropertyNode());

	FString ValueStr;
	if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(GetProperty()))
	{
		ValueStr = EnumProperty->GetEnum()->GetNameStringByValue(FinalValue);
	}
	else
	{
		ValueStr = LexToString(FinalValue);
	}
	Res = Implementation->ImportText( ValueStr, Flags );

	return Res;
}

FPropertyAccess::Result FPropertyHandleInt::SetValue(const int64& NewValue, EPropertyValueSetFlags::Type Flags)
{
	FPropertyAccess::Result Res;

	// Clamp the value from any meta data ranges stored on the property value
	int64 FinalValue = ClampIntegerValueFromMetaData<int64>(NewValue, *this, *Implementation->GetPropertyNode());

	FString ValueStr;
	if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(GetProperty()))
	{
		ValueStr = EnumProperty->GetEnum()->GetNameStringByValue(FinalValue);
	}
	else
	{
		ValueStr = LexToString(FinalValue);
	}
	Res = Implementation->ImportText(ValueStr, Flags);
	return Res;
}

FPropertyAccess::Result FPropertyHandleInt::SetValue(const uint8& NewValue, EPropertyValueSetFlags::Type Flags)
{
	FPropertyAccess::Result Res;
	// Clamp the value from any meta data ranges stored on the property value
	uint8 FinalValue = ClampIntegerValueFromMetaData<uint8>(NewValue, *this, *Implementation->GetPropertyNode());

	FString ValueStr;
	if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(GetProperty()))
	{
		ValueStr = EnumProperty->GetEnum()->GetNameStringByValue(FinalValue);
	}
	else
	{
		ValueStr = LexToString(FinalValue);
	}
	Res = Implementation->ImportText(ValueStr, Flags);

	return Res;
}

FPropertyAccess::Result FPropertyHandleInt::SetValue(const uint16& NewValue, EPropertyValueSetFlags::Type Flags)
{
	FPropertyAccess::Result Res;
	// Clamp the value from any meta data ranges stored on the property value
	uint16 FinalValue = ClampIntegerValueFromMetaData<uint16>(NewValue, *this, *Implementation->GetPropertyNode());

	FString ValueStr;
	if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(GetProperty()))
	{
		ValueStr = EnumProperty->GetEnum()->GetNameStringByValue(FinalValue);
	}
	else
	{
		ValueStr = LexToString(FinalValue);
	}
	Res = Implementation->ImportText(ValueStr, Flags);

	return Res;
}


FPropertyAccess::Result FPropertyHandleInt::SetValue(const uint32& NewValue, EPropertyValueSetFlags::Type Flags)
{
	FPropertyAccess::Result Res;
	// Clamp the value from any meta data ranges stored on the property value
	uint32 FinalValue = ClampIntegerValueFromMetaData<uint32>(NewValue, *this, *Implementation->GetPropertyNode());

	FString ValueStr;
	if (const FEnumProperty* EnumProp = CastField<FEnumProperty>(GetProperty()))
	{
		ValueStr = EnumProp->GetEnum()->GetNameStringByValue(FinalValue);
	}
	else
	{
		ValueStr = LexToString(FinalValue);
	}
	Res = Implementation->ImportText(ValueStr, Flags);

	return Res;
}

FPropertyAccess::Result FPropertyHandleInt::SetValue(const uint64& NewValue, EPropertyValueSetFlags::Type Flags)
{
	FPropertyAccess::Result Res;
	// Clamp the value from any meta data ranges stored on the property value
	uint64 FinalValue = ClampIntegerValueFromMetaData<uint64>(NewValue, *this, *Implementation->GetPropertyNode());

	FString ValueStr;
	if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(GetProperty()))
	{
		ValueStr = EnumProperty->GetEnum()->GetNameStringByValue(FinalValue);
	}
	else
	{
		ValueStr = LexToString(FinalValue);
	}
	Res = Implementation->ImportText(ValueStr, Flags);
	return Res;
}

// float
bool FPropertyHandleFloat::Supports( TSharedRef<FPropertyNode> PropertyNode )
{
	FProperty* Property = PropertyNode->GetProperty();

	if ( Property == nullptr )
	{
		return false;
	}

	return Property->IsA(FFloatProperty::StaticClass());
}

FPropertyAccess::Result FPropertyHandleFloat::GetValue( float& OutValue ) const
{
	void* PropValue = nullptr;
	FPropertyAccess::Result Res = Implementation->GetValueData( PropValue );

	if( Res == FPropertyAccess::Success )
	{
		OutValue = Implementation->GetPropertyValue<FFloatProperty>(PropValue);
	}

	return Res;
}

FPropertyAccess::Result FPropertyHandleFloat::SetValue( const float& NewValue, EPropertyValueSetFlags::Type Flags )
{
	FPropertyAccess::Result Res;
	// Clamp the value from any meta data ranges stored on the property value
	float FinalValue = ClampValueFromMetaData<float>( NewValue, *this);

	const FString ValueStr = FString::Printf( TEXT("%f"), FinalValue );
	Res = Implementation->ImportText( ValueStr, Flags );

	return Res;
}

// double
bool FPropertyHandleDouble::Supports( TSharedRef<FPropertyNode> PropertyNode )
{
	FProperty* Property = PropertyNode->GetProperty();

	if (Property == nullptr)
	{
		return false;
	}

	return Property->IsA(FDoubleProperty::StaticClass());
}

FPropertyAccess::Result FPropertyHandleDouble::GetValue( double& OutValue ) const
{
	void* PropValue = nullptr;
	FPropertyAccess::Result Res = Implementation->GetValueData( PropValue );

	if (Res == FPropertyAccess::Success)
	{
		OutValue = Implementation->GetPropertyValue<FDoubleProperty>(PropValue);
	}

	return Res;
}

FPropertyAccess::Result FPropertyHandleDouble::SetValue( const double& NewValue, EPropertyValueSetFlags::Type Flags )
{
	FPropertyAccess::Result Res;
	// Clamp the value from any meta data ranges stored on the property value
	double FinalValue = ClampValueFromMetaData<double>( NewValue, *this);
	
	FDoubleProperty* NumericProperty = CastFieldChecked<FDoubleProperty>(Implementation->GetPropertyNode()->GetProperty());
	const FString ValueStr = NumericProperty->GetNumericPropertyValueToString(&FinalValue);

	Res = Implementation->ImportText( ValueStr, Flags );

	return Res;
}

// bool
bool FPropertyHandleBool::Supports( TSharedRef<FPropertyNode> PropertyNode )
{
	FProperty* Property = PropertyNode->GetProperty();

	if ( Property == nullptr )
	{
		return false;
	}

	return Property->IsA(FBoolProperty::StaticClass());
}

FPropertyAccess::Result FPropertyHandleBool::GetValue( bool& OutValue ) const
{
	void* PropValue = nullptr;
	FPropertyAccess::Result Res = Implementation->GetValueData( PropValue );

	if( Res == FPropertyAccess::Success )
	{
		OutValue = Implementation->GetPropertyValue<FBoolProperty>(PropValue);
	}

	return Res;
}

FPropertyAccess::Result FPropertyHandleBool::SetValue( const bool& NewValue, EPropertyValueSetFlags::Type Flags )
{
	FPropertyAccess::Result Res = FPropertyAccess::Fail;

	//These are not localized values because ImportText does not accept localized values!
	FString ValueStr; 
	if( NewValue == false )
	{
		ValueStr = TEXT("False");
	}
	else
	{
		ValueStr = TEXT("True");
	}

	Res = Implementation->ImportText( ValueStr, Flags );

	return Res;
}

bool FPropertyHandleByte::Supports( TSharedRef<FPropertyNode> PropertyNode )
{
	FProperty* Property = PropertyNode->GetProperty();

	if ( Property == nullptr )
	{
		return false;
	}

	return Property->IsA<FByteProperty>() || Property->IsA<FEnumProperty>();
}

FPropertyAccess::Result FPropertyHandleByte::GetValue( uint8& OutValue ) const
{
	void* PropValue = nullptr;
	FPropertyAccess::Result Res = Implementation->GetValueData( PropValue );

	if( Res == FPropertyAccess::Success )
	{
		TSharedPtr<FPropertyNode> PropertyNodePin = Implementation->GetPropertyNode();

		FProperty* Property = PropertyNodePin->GetProperty();
		if( Property->IsA<FByteProperty>() )
		{
			OutValue = Implementation->GetPropertyValue<FByteProperty>(PropValue);
		}
		else
		{
			check(PropertyNodePin.IsValid());
			OutValue = static_cast<uint8>(CastFieldChecked<FEnumProperty>(Property)->GetUnderlyingProperty()->GetUnsignedIntPropertyValue(PropValue));
		}
	}

	return Res;
}

FPropertyAccess::Result FPropertyHandleByte::SetValue( const uint8& NewValue, EPropertyValueSetFlags::Type Flags )
{
	FPropertyAccess::Result Res;
	FString ValueStr;

	FProperty* Property = GetProperty();

	UEnum* Enum = nullptr;
	if (FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
	{
		Enum = ByteProperty->Enum;
	}
	else if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		Enum = EnumProperty->GetEnum();
	}

	if (Enum)
	{
		// Handle Enums using enum names to make sure they're compatible with FByteProperty::ExportText.
		ValueStr = Enum->GetNameStringByValue(NewValue);
	}
	else
	{
		// Ordinary byte, convert value to string.
		ValueStr = FString::Printf( TEXT("%i"), NewValue );
	}
	Res = Implementation->ImportText( ValueStr, Flags );

	return Res;
}


// String
bool FPropertyHandleString::Supports( TSharedRef<FPropertyNode> PropertyNode )
{
	FProperty* Property = PropertyNode->GetProperty();

	if ( Property == nullptr )
	{
		return false;
	}

	// Supported if the property is a name, string or object/interface that can be set via string
	return	( Property->IsA(FNameProperty::StaticClass()) && Property->GetFName() != NAME_InitialState )
		||	Property->IsA( FStrProperty::StaticClass() )
		||	( Property->IsA( FObjectPropertyBase::StaticClass() ) && !Property->HasAnyPropertyFlags(CPF_InstancedReference) )
		||	Property->IsA(FInterfaceProperty::StaticClass());
}

FPropertyAccess::Result FPropertyHandleString::GetValue( FString& OutValue ) const
{
	return Implementation->GetValueAsString( OutValue );
}

FPropertyAccess::Result FPropertyHandleString::SetValue( const FString& NewValue, EPropertyValueSetFlags::Type Flags )
{
	return Implementation->SetValueAsString( NewValue, Flags );
}

FPropertyAccess::Result FPropertyHandleString::SetValue( const TCHAR* NewValue, EPropertyValueSetFlags::Type Flags )
{
	return Implementation->SetValueAsString( NewValue, Flags );
}

FPropertyAccess::Result FPropertyHandleString::GetValue( FName& OutValue ) const
{
	void* PropValue = nullptr;
	FPropertyAccess::Result Res = Implementation->GetValueData( PropValue );

	if( Res == FPropertyAccess::Success )
	{
		OutValue = Implementation->GetPropertyValue<FNameProperty>(PropValue);
	}

	return Res;
}

FPropertyAccess::Result FPropertyHandleString::SetValue( const FName& NewValue, EPropertyValueSetFlags::Type Flags )
{
	return Implementation->SetValueAsString( NewValue.ToString(), Flags );
}

// Object

bool FPropertyHandleObject::Supports( TSharedRef<FPropertyNode> PropertyNode )
{
	FProperty* Property = PropertyNode->GetProperty();

	if ( Property == nullptr )
	{
		return false;
	}

	return Property->IsA(FObjectPropertyBase::StaticClass()) || Property->IsA(FInterfaceProperty::StaticClass()) 
		|| PropertyEditorHelpers::IsSoftClassPath(Property) || PropertyEditorHelpers::IsSoftObjectPath(Property);
}

FPropertyAccess::Result FPropertyHandleObject::GetValue( UObject*& OutValue ) const
{
	return FPropertyHandleObject::GetValue((const UObject*&)OutValue);
}

FPropertyAccess::Result FPropertyHandleObject::GetValue( const UObject*& OutValue ) const
{
	void* PropValue = nullptr;
	FPropertyAccess::Result Res = Implementation->GetValueData( PropValue );

	if( Res == FPropertyAccess::Success )
	{
		FProperty* Property = GetProperty();

		if (Property->IsA(FObjectPropertyBase::StaticClass()))
		{
			OutValue = Implementation->GetObjectPropertyValue(PropValue);
		}
		else if (Property->IsA(FInterfaceProperty::StaticClass()))
		{
			FInterfaceProperty* InterfaceProp = CastField<FInterfaceProperty>(Property);
			const FScriptInterface& ScriptInterface = InterfaceProp->GetPropertyValue(PropValue);
			OutValue = ScriptInterface.GetObject();
		}
		else
		{
			// This is a struct path, get the path string and search for the object
			FString ObjectPathString;
			Res = Implementation->GetValueAsString(ObjectPathString);
			FSoftObjectPath ObjectPath(ObjectPathString);
			OutValue = ObjectPath.ResolveObject();
		}
	}

	return Res;
}

FPropertyAccess::Result FPropertyHandleObject::SetValue( UObject* const& NewValue, EPropertyValueSetFlags::Type Flags )
{
	return FPropertyHandleObject::SetValue((const UObject*)NewValue);
}

FPropertyAccess::Result FPropertyHandleObject::SetValue( const UObject* const& NewValue, EPropertyValueSetFlags::Type Flags )
{
	const TSharedPtr<FPropertyNode>& PropertyNode = Implementation->GetPropertyNode();

	if (!PropertyNode->HasNodeFlags(EPropertyNodeFlags::EditInlineNew))
	{
		FString ObjectPathName = NewValue ? NewValue->GetPathName() : TEXT("None");
		return SetValueFromFormattedString(ObjectPathName, Flags);
	}

	return FPropertyAccess::Fail;
}

FPropertyAccess::Result FPropertyHandleObject::GetValue(FAssetData& OutValue) const
{
	UObject* ObjectValue = nullptr;
	FPropertyAccess::Result	Result = GetValue(ObjectValue);
	
	if ( Result == FPropertyAccess::Success )
	{
		OutValue = FAssetData(ObjectValue);
	}

	return Result;
}

FPropertyAccess::Result FPropertyHandleObject::SetValue(const FAssetData& NewValue, EPropertyValueSetFlags::Type Flags)
{
	const TSharedPtr<FPropertyNode>& PropertyNode = Implementation->GetPropertyNode();

	if (!PropertyNode->HasNodeFlags(EPropertyNodeFlags::EditInlineNew))
	{
		const bool bSkipResolve = PropertyNode->GetProperty()->IsA(FSoftObjectProperty::StaticClass());
		if (!bSkipResolve)
		{
			// Make sure the asset is loaded if we are not a soft reference
			NewValue.GetAsset();
		}

		FString ObjectPathName = NewValue.IsValid() ? FString::Printf(TEXT("%s'%s'"), *NewValue.AssetClassPath.ToString(), *NewValue.GetObjectPathString()) : TEXT("None");
		return SetValueFromFormattedString(ObjectPathName, Flags, bSkipResolve);
	}

	return FPropertyAccess::Fail;
}

FPropertyAccess::Result FPropertyHandleObject::SetValueFromFormattedString(const FString& InValue, EPropertyValueSetFlags::Type Flags)
{
	const bool bSkipResolve = false;
	return SetValueFromFormattedString(InValue, Flags, bSkipResolve);
}

FPropertyAccess::Result FPropertyHandleObject::SetValueFromFormattedString(const FString& InValue, EPropertyValueSetFlags::Type Flags, bool bSkipResolve)
{
	// We need to do all of the type validation up front, to correctly support soft objects
	TSharedPtr<FPropertyNode> PropertyNodePin = Implementation->GetPropertyNode();
	FProperty* NodeProperty = PropertyNodePin.IsValid() ? PropertyNodePin->GetProperty() : nullptr;
	const TCHAR* ObjectBuffer = *InValue;
	TObjectPtr<UObject> QualifiedObject = nullptr;

	// Only allow finding any object with the same name if package path not provided
	const bool bAllowAnyPackage = !InValue.Contains(TEXT("/"));

	if (!NodeProperty)
	{
		return FPropertyAccess::Fail;
	} // Skip the resolving, just set the value
	else if (bSkipResolve)
	{
		return FPropertyHandleBase::SetValueFromFormattedString(InValue, Flags);
	} // This will attempt to load the object if it is not in memory. We purposefully pass in null as owner to avoid issues with cross level references
	else if (FObjectPropertyBase::ParseObjectPropertyValue(NodeProperty, nullptr, UObject::StaticClass(), 0, ObjectBuffer, QualifiedObject, nullptr, bAllowAnyPackage))
	{
		if (QualifiedObject)
		{
			FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(NodeProperty);
			FInterfaceProperty* InterfaceProperty = CastField<FInterfaceProperty>(NodeProperty);
			FClassProperty* ClassProperty = CastField<FClassProperty>(NodeProperty);
			FSoftClassProperty* SoftClassProperty = CastField<FSoftClassProperty>(NodeProperty);

			// Figure out what classes are required
			UClass* InterfaceThatMustBeImplemented = nullptr;
			UClass* RequiredClass = nullptr;
			if (ClassProperty)
			{
				RequiredClass = ClassProperty->MetaClass;
			}
			else if (SoftClassProperty)
			{
				RequiredClass = SoftClassProperty->MetaClass;
			}
			else if (ObjectProperty)
			{
				RequiredClass = ObjectProperty->PropertyClass;
				InterfaceThatMustBeImplemented = ObjectProperty->GetOwnerProperty()->GetClassMetaData(TEXT("MustImplement"));
			}
			else if (InterfaceProperty)
			{
				InterfaceThatMustBeImplemented = InterfaceProperty->InterfaceClass;
			}
			else if (PropertyEditorHelpers::IsSoftClassPath(NodeProperty))
			{
				RequiredClass = NodeProperty->GetOwnerProperty()->GetClassMetaData(TEXT("MetaClass"));
				InterfaceThatMustBeImplemented = NodeProperty->GetOwnerProperty()->GetClassMetaData(TEXT("MustImplement"));
			}
			else if (PropertyEditorHelpers::IsSoftObjectPath(NodeProperty))
			{
				// No metaclass, allowedclasses is the only filter
			}
			else
			{
				return FPropertyAccess::Fail;
			}

			// Figure out what class to check against
			UClass* QualifiedClass = Cast<UClass>(QualifiedObject);
			if (!QualifiedClass)
			{
				QualifiedClass = QualifiedObject->GetClass();
			}
			
			// Only instance new objects if required/directed
			if ((Flags & EPropertyValueSetFlags::InstanceObjects) != 0
				&& !NodeProperty->HasAnyPropertyFlags(CPF_InstancedReference)
				&& !PropertyNodePin->HasNodeFlags(EPropertyNodeFlags::EditInlineNew))
			{
				Flags &= ~EPropertyValueSetFlags::InstanceObjects;
			}

			const FString& AllowedClassesString = NodeProperty->GetMetaData("AllowedClasses");
			TArray<FString> AllowedClassNames;
			AllowedClassesString.ParseIntoArrayWS(AllowedClassNames, TEXT(","), true);
			bool bSupportedObject = false;

			// Check AllowedClasses metadata
			if (AllowedClassNames.Num() > 0)
			{
				for (const FString& ClassName : AllowedClassNames)
				{
					const UClass* AllowedClass = nullptr;
					if (!FPackageName::IsShortPackageName(ClassName))
					{
						AllowedClass = FindObject<UClass>(nullptr, *ClassName);
					}
					else
					{ 
						AllowedClass = FindFirstObject<UClass>(*ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("FPropertyHandleObject::SetValueFromFormattedString"));
					}
					
					const bool bIsInterface = AllowedClass && AllowedClass->HasAnyClassFlags(CLASS_Interface);
				
					// Check if the object is an allowed class type this property supports
					if ((AllowedClass && QualifiedClass->IsChildOf(AllowedClass)) || (bIsInterface && QualifiedObject->GetClass()->ImplementsInterface(AllowedClass)))
					{
						bSupportedObject = true;
						break;
					}
				}
			}
			else
			{
				bSupportedObject = true;
			}

			if (bSupportedObject)
			{
				const FString& GetAllowedClassesFunctionName = NodeProperty->GetMetaData("GetAllowedClasses");
				if (!GetAllowedClassesFunctionName.IsEmpty() && NodeProperty->GetOwnerUObject())
				{
					auto GetAllowedClasses = [&QualifiedObject, &QualifiedClass, &bSupportedObject](UObject* InObject, const UFunction* InGetAllowedClassesFunction) -> bool
					{
						DECLARE_DELEGATE_RetVal(TArray<UClass*>, FGetAllowedClasses);
						TArray<UClass*> AllowedClasses = FGetAllowedClasses::CreateUFunction(InObject, InGetAllowedClassesFunction->GetFName()).Execute();
						if (AllowedClasses.Num() > 0)
						{
							bSupportedObject = false;
							for (const UClass* AllowedClass : AllowedClasses)
							{
								const bool bIsInterface = AllowedClass && AllowedClass->HasAnyClassFlags(CLASS_Interface);
			
								// Check if the object is an allowed class type this property supports
								if ((AllowedClass && QualifiedClass->IsChildOf(AllowedClass)) || (bIsInterface && QualifiedObject->GetClass()->ImplementsInterface(AllowedClass)))
								{
									bSupportedObject = true;
									break;
								}
							}
							if (!bSupportedObject)
							{
								return false;
							}
						}

						return true;
					};

					// First look for a library function assuming fully qualified path, e.g. /Script/ModuleName.ClassName:FunctionName
					const UFunction* LibraryFunction = FindObject<UFunction>(nullptr, *GetAllowedClassesFunctionName);
					if(LibraryFunction)
					{
						if(UObject* CDO = LibraryFunction->GetOwnerClass()->GetDefaultObject())
						{
							GetAllowedClasses(CDO, LibraryFunction);
						}
					}
					else
					{
						// Otherwise interrogate the object list
						TArray<UObject*> OuterObjects;
						GetOuterObjects(OuterObjects);
						for (UObject* Object : OuterObjects)
						{
							const UFunction* GetAllowedClassesFunction = Object->FindFunction(*GetAllowedClassesFunctionName);
							if (GetAllowedClassesFunction)
							{
								if(!GetAllowedClasses(Object, GetAllowedClassesFunction))
								{
									break;
								}
							}
						}
					}
				}
			}

			if (bSupportedObject)
			{
				const FString& DisallowedClassesString = NodeProperty->GetMetaData("DisallowedClasses");
				TArray<FString> DisallowedClassNames;
				DisallowedClassesString.ParseIntoArrayWS(DisallowedClassNames, TEXT(","), true);

				for (const FString& DisallowedClassName : DisallowedClassNames)
				{
					const UClass* DisallowedClass = UClass::TryFindTypeSlow<UClass>(DisallowedClassName);
					const bool bIsInterface = DisallowedClass && DisallowedClass->HasAnyClassFlags(CLASS_Interface);

					if ((DisallowedClass && QualifiedClass->IsChildOf(DisallowedClass)) || (bIsInterface && QualifiedObject->GetClass()->ImplementsInterface(DisallowedClass)))
					{
						bSupportedObject = false;
						break;
					}
				}
			}

			if (bSupportedObject)
			{
				const FString& GetDisallowedClassesFunctionName = NodeProperty->GetMetaData("GetDisallowedClasses");
				if (!GetDisallowedClassesFunctionName.IsEmpty() && NodeProperty->GetOwnerUObject())
				{
					TArray<UObject*> OuterObjects;
					GetOuterObjects(OuterObjects);
					for (UObject* Object : OuterObjects)
					{
						const UFunction* GetDisallowedClassesFunction = Object->FindFunction(*GetDisallowedClassesFunctionName);
						if (GetDisallowedClassesFunction)
						{
							DECLARE_DELEGATE_RetVal(TArray<UClass*>, FGetAllowedClasses);
							TArray<UClass*> DisallowedClasses = FGetAllowedClasses::CreateUFunction(Object, GetDisallowedClassesFunction->GetFName()).Execute();
							if (DisallowedClasses.Num() > 0)
							{
								for (const UClass* DisallowedClass : DisallowedClasses)
								{
									const bool bIsInterface = DisallowedClass && DisallowedClass->HasAnyClassFlags(CLASS_Interface);
					
									// Check if the object is an allowed class type this property supports
									if ((DisallowedClass && QualifiedClass->IsChildOf(DisallowedClass)) || (bIsInterface && QualifiedObject->GetClass()->ImplementsInterface(DisallowedClass)))
									{
										bSupportedObject = false;
										break;
									}
								}
								if (!bSupportedObject)
								{
									break;
								}
							}
						}
					}
				}
			}

			// Check required class
			if (RequiredClass && !QualifiedClass->IsChildOf(RequiredClass))
			{
				bSupportedObject = false;
			}

			// Check required interface
			if (InterfaceThatMustBeImplemented && !QualifiedClass->ImplementsInterface(InterfaceThatMustBeImplemented))
			{
				bSupportedObject = false;
			}

			// Check node restrictions
			TArray<FText> RestrictReasons;
			if (PropertyNodePin->IsRestricted(QualifiedObject->GetPathName(), RestrictReasons))
			{
				bSupportedObject = false;
			}

			// Check level actor
			bool const bMustBeLevelActor = ObjectProperty ? ObjectProperty->GetOwnerProperty()->GetBoolMetaData(TEXT("MustBeLevelActor")) : false;
			if (bMustBeLevelActor)
			{
				if (AActor* Actor = Cast<AActor>(QualifiedObject))
				{
					if (!Actor->GetLevel())
					{
						// Not in a level
						bSupportedObject = false;
					}
				}
				else
				{
					// Not an actor
					bSupportedObject = false;
				}
			}

			if (!bSupportedObject)
			{
				return FPropertyAccess::Fail;
			}
		}

		// Parsing passed and QualifiedObject !nullptr and bSupportedObject is true and so we should set the value or
		// Parsing passed but QualifiedObject is nullptr and we want to set it to null explicitly
		return FPropertyHandleBase::SetValueFromFormattedString(InValue, Flags);
	}

		// Failed parsing, it's either invalid format or a nonexistent object
		return FPropertyAccess::Fail;
}

FPropertyAccess::Result FPropertyHandleObject::SetObjectValueFromSelection()
{
	FPropertyAccess::Result Res = FPropertyAccess::Fail;
	TSharedPtr<FPropertyNode> PropertyNodePin = Implementation->GetPropertyNode();
	if (PropertyNodePin.IsValid())
	{
		FProperty* NodeProperty = PropertyNodePin->GetProperty();

		FObjectPropertyBase* ObjProp = CastField<FObjectPropertyBase>(NodeProperty);
		FInterfaceProperty* IntProp = CastField<FInterfaceProperty>(NodeProperty);
		FClassProperty* ClassProp = CastField<FClassProperty>(NodeProperty);
		FSoftClassProperty* SoftClassProperty = CastField<FSoftClassProperty>(NodeProperty);
		UClass* const InterfaceThatMustBeImplemented = NodeProperty ? NodeProperty->GetOwnerProperty()->GetClassMetaData(TEXT("MustImplement")) : nullptr;

		if (ClassProp || SoftClassProperty || PropertyEditorHelpers::IsSoftClassPath(NodeProperty))
		{
			FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

			UClass* RequiredClass = nullptr;
			if (ClassProp)
			{
				RequiredClass = ClassProp->MetaClass;
			}
			else if (SoftClassProperty)
			{
				RequiredClass = SoftClassProperty->MetaClass;
			}
			else
			{
				RequiredClass = NodeProperty->GetOwnerProperty()->GetClassMetaData(TEXT("MetaClass"));
			}

			const UClass* const SelectedClass = GEditor->GetFirstSelectedClass(RequiredClass);
			if (SelectedClass)
			{
				// MetaData class filter
				if (NodeProperty->GetOwnerProperty()->HasMetaData("GetClassFilter"))
				{
					const FString& GetClassFilterFunctionName = NodeProperty->GetOwnerProperty()->GetMetaData("GetClassFilter");
					if (!GetClassFilterFunctionName.IsEmpty())
					{
						TArray<UObject*> ObjectList;
						GetOuterObjects(ObjectList);
						if (const UClass* OuterBaseClass = GetOuterBaseClass())
						{
							UObject* OuterBaseCDO = OuterBaseClass->GetDefaultObject<UObject>();
							const UFunction* GetClassFilterFunction = OuterBaseCDO ? OuterBaseCDO->FindFunction(*GetClassFilterFunctionName) : nullptr;
							if (ensure(GetClassFilterFunction && GetClassFilterFunction->HasAllFunctionFlags(FUNC_Static)))
							{
								if (FOnShouldFilterClass::CreateUFunction(OuterBaseCDO, GetClassFilterFunction->GetFName()).Execute(SelectedClass))
								{
									// The class fails the metadata filters
									return FPropertyAccess::Fail;
								}
							}
						}
					}
				}
				
				if (!InterfaceThatMustBeImplemented || SelectedClass->ImplementsInterface(InterfaceThatMustBeImplemented))
				{
					FString const ClassPathName = SelectedClass->GetPathName();

					TArray<FText> RestrictReasons;
					if (PropertyNodePin->IsRestricted(ClassPathName, RestrictReasons))
					{
						check(RestrictReasons.Num() > 0);
						FMessageDialog::Open(EAppMsgType::Ok, RestrictReasons[0]);
					}
					else
					{

						Res = SetValueFromFormattedString(ClassPathName, EPropertyValueSetFlags::DefaultFlags);
					}
				}
			}
		}
		else
		{
			FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

			UClass* ObjPropClass = UObject::StaticClass();
			if (ObjProp)
			{
				ObjPropClass = ObjProp->PropertyClass;
			}
			else if (IntProp)
			{
				ObjPropClass = IntProp->InterfaceClass;
			}

			bool const bMustBeLevelActor = ObjProp ? ObjProp->GetOwnerProperty()->GetBoolMetaData(TEXT("MustBeLevelActor")) : false;

			// Find best appropriate selected object
			UObject* SelectedObject = nullptr;

			if (bMustBeLevelActor)
			{
				// @todo: ensure at compile time that MustBeLevelActor flag is only set on actor properties

				// looking only for level actors here
				USelection* const SelectedSet = GEditor->GetSelectedActors();
				SelectedObject = SelectedSet->GetTop(ObjPropClass, InterfaceThatMustBeImplemented);
			}
			else
			{
				// normal behavior, where actor classes will look for level actors and 
				USelection* const SelectedSet = GEditor->GetSelectedSet(ObjPropClass);
				SelectedObject = SelectedSet->GetTop(ObjPropClass, InterfaceThatMustBeImplemented);
			}

			if (SelectedObject)
			{
				FString const ObjPathName = SelectedObject->GetPathName();

				TArray<FText> RestrictReasons;
				if (PropertyNodePin->IsRestricted(ObjPathName, RestrictReasons))
				{
					check(RestrictReasons.Num() > 0);
					FMessageDialog::Open(EAppMsgType::Ok, RestrictReasons[0]);
				}
				else if (SetValueFromFormattedString(SelectedObject->GetPathName(), EPropertyValueSetFlags::DefaultFlags) != FPropertyAccess::Success)
				{
					// Warn that some object assignments failed.
					FMessageDialog::Open(EAppMsgType::Ok, FText::Format(
						NSLOCTEXT("UnrealEd", "ObjectAssignmentsFailed", "Failed to assign {0} to the {1} property, see log for details."),
						FText::FromString(SelectedObject->GetPathName()), PropertyNodePin->GetDisplayName()));
				}
				else
				{
					Res = FPropertyAccess::Success;
				}
			}
		}
	}

	return Res;
}


// Temporary mixed float/double property handle to support the various default template types having differing component types. LWC_TODO: Remove once all types support double.
FPropertyHandleMixed::FPropertyHandleMixed(TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities)
	: FPropertyHandleBase(PropertyNode, NotifyHook, PropertyUtilities) {}

bool FPropertyHandleMixed::Supports(TSharedRef<FPropertyNode> PropertyNode)
{
	FProperty* Property = PropertyNode->GetProperty();

	if (Property == nullptr)
	{
		return false;
	}

	return Property->IsA(FFloatProperty::StaticClass()) || Property->IsA(FDoubleProperty::StaticClass());
}

FPropertyAccess::Result FPropertyHandleMixed::GetValue(double& OutValue) const
{
	void* PropValue = nullptr;
	FPropertyAccess::Result Res = Implementation->GetValueData(PropValue);

	if (Res == FPropertyAccess::Success)
	{
		if(Implementation->IsPropertyTypeOf(FFloatProperty::StaticClass()))
		{
			OutValue = Implementation->GetPropertyValue<FFloatProperty>(PropValue);
		}
		else
		{
			OutValue = Implementation->GetPropertyValue<FDoubleProperty>(PropValue);
		}
	}

	return Res;
}

FPropertyAccess::Result FPropertyHandleMixed::SetValue(const double& NewValue, EPropertyValueSetFlags::Type Flags)
{
	FPropertyAccess::Result Res;
	// Clamp the value from any meta data ranges stored on the property value
	double FinalValue = ClampValueFromMetaData<double>(NewValue, *this);

	const FString ValueStr = FString::Printf(TEXT("%f"), FinalValue);
	Res = Implementation->ImportText(ValueStr, Flags);

	return Res;
}

FPropertyAccess::Result FPropertyHandleMixed::GetValue(float& OutValue) const
{
	double AsDouble;
	FPropertyAccess::Result Res = GetValue(AsDouble);
	OutValue = static_cast<float>(AsDouble);
	return Res;
}

FPropertyAccess::Result FPropertyHandleMixed::SetValue(const float& NewValue, EPropertyValueSetFlags::Type Flags)
{
	return SetValue((double)NewValue);
}

// Struct
bool FPropertyHandleStruct::Supports( TSharedRef<FPropertyNode> PropertyNode )
{
	if (FComplexPropertyNode* ComplexNode = PropertyNode->AsComplexNode())
	{
		return (ComplexNode->AsStructureNode() != nullptr);
	}

	return false;
}

TSharedPtr<IPropertyHandleStruct> FPropertyHandleStruct::AsStruct()
{
	return SharedThis(this);
}

TSharedPtr<FStructOnScope> FPropertyHandleStruct::GetStructData() const
{
	TSharedPtr<FStructurePropertyNode> StructPropertyNode = StaticCastSharedPtr<FStructurePropertyNode>(Implementation->GetPropertyNode());

	if (StructPropertyNode.IsValid())
	{
		return StructPropertyNode->GetStructData();
	}

	return nullptr;
}

// Vector
bool FPropertyHandleVector::Supports( TSharedRef<FPropertyNode> PropertyNode )
{
	FProperty* Property = PropertyNode->GetProperty();

	if ( Property == nullptr )
	{
		return false;
	}

	FStructProperty* StructProp = CastField<FStructProperty>(Property);

	bool bSupported = false;
	if( StructProp && StructProp->Struct )
	{
		FName StructName = StructProp->Struct->GetFName();

		bSupported = StructName == NAME_Vector ||
			StructName == NAME_Vector2D ||
			StructName == NAME_Vector4 ||
			StructName == NAME_Quat ||
			StructName == FDeprecateSlateVector2D::StaticStruct()->GetFName();
	}

	return bSupported;
}

FPropertyHandleVector::FPropertyHandleVector( TSharedRef<class FPropertyNode> PropertyNode, class FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities )
	: FPropertyHandleStruct( PropertyNode, NotifyHook, PropertyUtilities ) 
{
	const bool bRecurse = false;
	if( TSharedPtr<FPropertyNode> XComponentPropertyNode = Implementation->GetChildNode("X", bRecurse) )
	{
		VectorComponents.Add( MakeShareable( new FPropertyHandleMixed( XComponentPropertyNode.ToSharedRef(), NotifyHook, PropertyUtilities ) ) );
	}

	if( TSharedPtr<FPropertyNode> YComponentPropertyNode = Implementation->GetChildNode("Y", bRecurse) )
		{
		VectorComponents.Add( MakeShareable( new FPropertyHandleMixed( YComponentPropertyNode.ToSharedRef(), NotifyHook, PropertyUtilities ) ) );
		}
		
	if( TSharedPtr<FPropertyNode> ZComponentPropertyNode = Implementation->GetChildNode("Z", bRecurse) )
		{
		VectorComponents.Add( MakeShareable( new FPropertyHandleMixed( ZComponentPropertyNode.ToSharedRef(), NotifyHook, PropertyUtilities ) ) );
		}
		
	if( TSharedPtr<FPropertyNode> WComponentPropertyNode = Implementation->GetChildNode("W", bRecurse) )
		{
		VectorComponents.Add( MakeShareable( new FPropertyHandleMixed( WComponentPropertyNode.ToSharedRef(), NotifyHook, PropertyUtilities ) ) );
	}
}

FPropertyAccess::Result FPropertyHandleVector::GetValue( FVector2D& OutValue ) const
{
	if( VectorComponents.Num() == 2 )
	{
		// To get the value from the vector we read each child.  If reading a child fails, the value for that component is not set
		FPropertyAccess::Result ResX = VectorComponents[0]->GetValue( OutValue.X );
		FPropertyAccess::Result ResY = VectorComponents[1]->GetValue( OutValue.Y );

		if( ResX == FPropertyAccess::Fail || ResY == FPropertyAccess::Fail )
		{
			// If reading any value failed the entire value fails
			return FPropertyAccess::Fail;
		}
		else if( ResX == FPropertyAccess::MultipleValues || ResY == FPropertyAccess::MultipleValues )
		{
			// At least one component had multiple values
			return FPropertyAccess::MultipleValues;
		}
		else
		{
			return FPropertyAccess::Success;
		}
	}

	// Not a 2 component vector
	return FPropertyAccess::Fail;
}

FPropertyAccess::Result FPropertyHandleVector::SetValue( const FVector2D& NewValue, EPropertyValueSetFlags::Type Flags )
{
	// To set the value from the vector we set each child. 
	FPropertyAccess::Result ResX = VectorComponents[0]->SetValue( NewValue.X, Flags );
	FPropertyAccess::Result ResY = VectorComponents[1]->SetValue( NewValue.Y, Flags );

	if( ResX == FPropertyAccess::Fail || ResY == FPropertyAccess::Fail )
	{
		return FPropertyAccess::Fail;
	}
	else
	{
		return FPropertyAccess::Success;
	}
}

FPropertyAccess::Result FPropertyHandleVector::GetValue( FVector& OutValue ) const
{
	if( VectorComponents.Num() == 3 )
	{
		// To get the value from the vector we read each child.  If reading a child fails, the value for that component is not set
		FPropertyAccess::Result ResX = VectorComponents[0]->GetValue( OutValue.X );
		FPropertyAccess::Result ResY = VectorComponents[1]->GetValue( OutValue.Y );
		FPropertyAccess::Result ResZ = VectorComponents[2]->GetValue( OutValue.Z );

		if( ResX == FPropertyAccess::Fail || ResY == FPropertyAccess::Fail || ResZ == FPropertyAccess::Fail )
		{
			// If reading any value failed the entire value fails
			return FPropertyAccess::Fail;
		}
		else if( ResX == FPropertyAccess::MultipleValues || ResY == FPropertyAccess::MultipleValues || ResZ == FPropertyAccess::MultipleValues )
		{
			// At least one component had multiple values
			return FPropertyAccess::MultipleValues;
		}
		else
		{
			return FPropertyAccess::Success;
		}
	}

	// Not a 3 component vector
	return FPropertyAccess::Fail;
}

FPropertyAccess::Result FPropertyHandleVector::SetValue( const FVector& NewValue, EPropertyValueSetFlags::Type Flags )
{
	if( VectorComponents.Num() == 3)
	{
		// To set the value from the vector we set each child. 
		FPropertyAccess::Result ResX = VectorComponents[0]->SetValue( NewValue.X, Flags );
		FPropertyAccess::Result ResY = VectorComponents[1]->SetValue( NewValue.Y, Flags );
		FPropertyAccess::Result ResZ = VectorComponents[2]->SetValue( NewValue.Z, Flags );

		if( ResX == FPropertyAccess::Fail || ResY == FPropertyAccess::Fail || ResZ == FPropertyAccess::Fail )
		{
			return FPropertyAccess::Fail;
		}
		else
		{
			return FPropertyAccess::Success;
		}
	}

	return FPropertyAccess::Fail;
}

FPropertyAccess::Result FPropertyHandleVector::GetValue( FVector4& OutValue ) const
{
	if( VectorComponents.Num() == 4 )
	{
		// To get the value from the vector we read each child.  If reading a child fails, the value for that component is not set
		FPropertyAccess::Result ResX = VectorComponents[0]->GetValue( OutValue.X );
		FPropertyAccess::Result ResY = VectorComponents[1]->GetValue( OutValue.Y );
		FPropertyAccess::Result ResZ = VectorComponents[2]->GetValue( OutValue.Z );
		FPropertyAccess::Result ResW = VectorComponents[3]->GetValue( OutValue.W );

		if( ResX == FPropertyAccess::Fail || ResY == FPropertyAccess::Fail || ResZ == FPropertyAccess::Fail || ResW == FPropertyAccess::Fail )
		{
			// If reading any value failed the entire value fails
			return FPropertyAccess::Fail;
		}
		else if( ResX == FPropertyAccess::MultipleValues || ResY == FPropertyAccess::MultipleValues || ResZ == FPropertyAccess::MultipleValues || ResW == FPropertyAccess::MultipleValues )
		{
			// At least one component had multiple values
			return FPropertyAccess::MultipleValues;
		}
		else
		{
			return FPropertyAccess::Success;
		}
	}

	// Not a 4 component vector
	return FPropertyAccess::Fail;
}


FPropertyAccess::Result FPropertyHandleVector::SetValue( const FVector4& NewValue, EPropertyValueSetFlags::Type Flags )
{
	// To set the value from the vector we set each child. 
	FPropertyAccess::Result ResX = VectorComponents[0]->SetValue( NewValue.X, Flags );
	FPropertyAccess::Result ResY = VectorComponents[1]->SetValue( NewValue.Y, Flags );
	FPropertyAccess::Result ResZ = VectorComponents[2]->SetValue( NewValue.Z, Flags );
	FPropertyAccess::Result ResW = VectorComponents[3]->SetValue( NewValue.W, Flags );

	if( ResX == FPropertyAccess::Fail || ResY == FPropertyAccess::Fail || ResZ == FPropertyAccess::Fail || ResW == FPropertyAccess::Fail )
	{
		return FPropertyAccess::Fail;
	}
	else
	{
		return FPropertyAccess::Success;
	}
}

FPropertyAccess::Result FPropertyHandleVector::GetValue( FQuat& OutValue ) const
{
	FVector4 VectorProxy;
	FPropertyAccess::Result Res = GetValue(VectorProxy);
	if (Res == FPropertyAccess::Success)
	{
		OutValue.X = VectorProxy.X;
		OutValue.Y = VectorProxy.Y;
		OutValue.Z = VectorProxy.Z;
		OutValue.W = VectorProxy.W;
	}

	return Res;
}

FPropertyAccess::Result FPropertyHandleVector::SetValue( const FQuat& NewValue, EPropertyValueSetFlags::Type Flags )
{
	FVector4 VectorProxy;
	VectorProxy.X = NewValue.X;
	VectorProxy.Y = NewValue.Y;
	VectorProxy.Z = NewValue.Z;
	VectorProxy.W = NewValue.W;

	return SetValue(VectorProxy);
}

FPropertyAccess::Result FPropertyHandleVector::SetX( double InValue, EPropertyValueSetFlags::Type Flags )
{
	FPropertyAccess::Result Res = VectorComponents[0]->SetValue( InValue, Flags );

	return Res;
}

FPropertyAccess::Result FPropertyHandleVector::SetY(double InValue, EPropertyValueSetFlags::Type Flags )
{
	FPropertyAccess::Result Res = VectorComponents[1]->SetValue( InValue, Flags );

	return Res;
}

FPropertyAccess::Result FPropertyHandleVector::SetZ(double InValue, EPropertyValueSetFlags::Type Flags )
{
	if( VectorComponents.Num() > 2 )
	{
		FPropertyAccess::Result Res = VectorComponents[2]->SetValue( InValue, Flags );

		return Res;
	}
	
	return FPropertyAccess::Fail;
}

FPropertyAccess::Result FPropertyHandleVector::SetW(double InValue, EPropertyValueSetFlags::Type Flags )
{
	if( VectorComponents.Num() == 4 )
	{
		FPropertyAccess::Result Res = VectorComponents[3]->SetValue( InValue, Flags );
		
		return Res;
	}

	return FPropertyAccess::Fail;
}

// Rotator

bool FPropertyHandleRotator::Supports( TSharedRef<FPropertyNode> PropertyNode )
{
	FProperty* Property = PropertyNode->GetProperty();

	if ( Property == nullptr )
	{
		return false;
	}

	FStructProperty* StructProp = CastField<FStructProperty>(Property);
	return StructProp && StructProp->Struct->GetFName() == NAME_Rotator;
}

FPropertyHandleRotator::FPropertyHandleRotator( TSharedRef<class FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities )
	: FPropertyHandleStruct( PropertyNode, NotifyHook, PropertyUtilities ) 
{
	if (Implementation->GetNumChildren() > 0)
	{
		const bool bRecurse = false;

		// A rotator is a struct property that has 3 children.  We get/set the values from the children
		if (TSharedPtr<FPropertyNode> RollNode = Implementation->GetChildNode("Roll", bRecurse))
		{
			RollValue = MakeShareable(new FPropertyHandleMixed(RollNode.ToSharedRef(), NotifyHook, PropertyUtilities));
		}

		if (TSharedPtr<FPropertyNode> PitchNode = Implementation->GetChildNode("Pitch", bRecurse))
		{
			PitchValue = MakeShareable(new FPropertyHandleMixed(PitchNode.ToSharedRef(), NotifyHook, PropertyUtilities));
		}

		if (TSharedPtr<FPropertyNode> YawNode = Implementation->GetChildNode("Yaw", bRecurse))
		{
			YawValue = MakeShareable(new FPropertyHandleMixed(YawNode.ToSharedRef(), NotifyHook, PropertyUtilities));
		}
	}
}


FPropertyAccess::Result FPropertyHandleRotator::GetValue( FRotator& OutValue ) const
{
	if (!RollValue || !PitchValue || !YawValue)
	{
		return FPropertyAccess::Fail;
	}

	// To get the value from the rotator we read each child.  If reading a child fails, the value for that component is not set
	FPropertyAccess::Result ResR = RollValue->GetValue( OutValue.Roll );
	FPropertyAccess::Result ResP = PitchValue->GetValue( OutValue.Pitch );
	FPropertyAccess::Result ResY = YawValue->GetValue( OutValue.Yaw );

	if( ResR == FPropertyAccess::MultipleValues || ResP == FPropertyAccess::MultipleValues || ResY == FPropertyAccess::MultipleValues )
	{
		return FPropertyAccess::MultipleValues;
	}
	else if( ResR == FPropertyAccess::Fail || ResP == FPropertyAccess::Fail || ResY == FPropertyAccess::Fail )
	{
		return FPropertyAccess::Fail;
	}
	else
	{
		return FPropertyAccess::Success;
	}
}

FPropertyAccess::Result FPropertyHandleRotator::SetValue( const FRotator& NewValue, EPropertyValueSetFlags::Type Flags )
{
	if (!RollValue || !PitchValue || !YawValue)
	{
		return FPropertyAccess::Fail;
	}

	// To set the value from the rotator we set each child. 
	FPropertyAccess::Result ResR = RollValue->SetValue( NewValue.Roll, Flags );
	FPropertyAccess::Result ResP = PitchValue->SetValue( NewValue.Pitch, Flags );
	FPropertyAccess::Result ResY = YawValue->SetValue( NewValue.Yaw, Flags );

	if( ResR == FPropertyAccess::Fail || ResP == FPropertyAccess::Fail || ResY == FPropertyAccess::Fail )
	{
		return FPropertyAccess::Fail;
	}
	else
	{
		return FPropertyAccess::Success;
	}
}

FPropertyAccess::Result FPropertyHandleRotator::GetValue(FQuat& OutValue) const
{
	if (!RollValue || !PitchValue || !YawValue)
	{
		return FPropertyAccess::Fail;
	}

	FRotator RotatorProxy;
	FPropertyAccess::Result Res = GetValue(RotatorProxy);
	if (Res == FPropertyAccess::Success)
	{
		OutValue = RotatorProxy.Quaternion();
	}

	return Res;
}

FPropertyAccess::Result FPropertyHandleRotator::SetValue(const FQuat& InValue, EPropertyValueSetFlags::Type Flags)
{
	if (!RollValue || !PitchValue || !YawValue)
	{
		return FPropertyAccess::Fail;
	}

	const FRotator RotatorProxy(InValue);
	return SetValue(RotatorProxy, Flags);
}

FPropertyAccess::Result FPropertyHandleRotator::SetRoll( double InRoll, EPropertyValueSetFlags::Type Flags )
{
	if (!RollValue)
	{
		return FPropertyAccess::Fail;
	}

	FPropertyAccess::Result Res = RollValue->SetValue( InRoll, Flags );
	return Res;
}

FPropertyAccess::Result FPropertyHandleRotator::SetPitch( double InPitch, EPropertyValueSetFlags::Type Flags )
{
	if (!PitchValue)
	{
		return FPropertyAccess::Fail;
	}

	FPropertyAccess::Result Res = PitchValue->SetValue( InPitch, Flags );
	return Res;
}

FPropertyAccess::Result FPropertyHandleRotator::SetYaw( double InYaw, EPropertyValueSetFlags::Type Flags )
{
	if (!YawValue)
	{
		return FPropertyAccess::Fail;
	}

	FPropertyAccess::Result Res = YawValue->SetValue( InYaw, Flags );
	return Res;
}

FPropertyHandleColor::FPropertyHandleColor(TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities)
	: FPropertyHandleStruct(PropertyNode, NotifyHook, PropertyUtilities)
{
}

bool FPropertyHandleColor::Supports(TSharedRef<FPropertyNode> PropertyNode)
{
	FProperty* Property = PropertyNode->GetProperty();

	if ( FStructProperty* StructProperty = CastField<FStructProperty>(Property) )
	{
		return StructProperty->Struct == FSlateColor::StaticStruct() 
			|| StructProperty->Struct == TBaseStructure<FLinearColor>::Get();
	}

	return false;
}

FPropertyAccess::Result FPropertyHandleColor::SetValueFromFormattedString(const FString& InValue, EPropertyValueSetFlags::Type Flags)
{
	FProperty* Property = Implementation->GetPropertyNode()->GetProperty();

	if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		FString OutValue;
		FLinearColor LinearColor;

		if (LinearColor.InitFromString(InValue))
		{
			if (StructProperty->Struct == FSlateColor::StaticStruct())
			{
				if (InValue.Contains("ColorUseRule"))
				{
					return Implementation->SetValueAsString(InValue, Flags);
				}
				else
				{
					LinearColor.InitFromString(InValue);
					FSlateColor SlateColor(LinearColor);
					Implementation->GetPropertyNode()->GetProperty()->ExportText_Direct(OutValue, &SlateColor, &SlateColor, nullptr, 0);
					return Implementation->SetValueAsString(OutValue, Flags);
				}
			}
			else if (StructProperty->Struct == TBaseStructure<FLinearColor>::Get())
			{
				Implementation->GetPropertyNode()->GetProperty()->ExportText_Direct(OutValue, &LinearColor, &LinearColor, nullptr, 0);
				return Implementation->SetValueAsString(OutValue, Flags);
			}
		}
	}

	return FPropertyAccess::Result::Fail;
}


bool FPropertyHandleArray::Supports( TSharedRef<FPropertyNode> PropertyNode )
{
	FProperty* Property = PropertyNode->GetProperty();
	int32 ArrayIndex = PropertyNode->GetArrayIndex();

	// Static array or dynamic array
	return ( ( Property && Property->ArrayDim != 1 && ArrayIndex == -1 ) || CastField<const FArrayProperty>(Property) );
}


FPropertyHandleItemAddResult FPropertyHandleArray::AddItem()
{
	FPropertyAccess::Result Result = FPropertyAccess::Fail;
	int32 Index = INDEX_NONE;
	if( IsEditable() )
	{
		if (Implementation->CanItemChange(EPropertyChangeType::ArrayAdd, Index))
		{
			Index = Implementation->AddChild();
			Result = FPropertyAccess::Success;
			return FPropertyHandleItemAddResult(Index, Result);
		}
	}
	return FPropertyHandleItemAddResult();
}

FPropertyAccess::Result FPropertyHandleArray::EmptyArray()
{
	FPropertyAccess::Result Result = FPropertyAccess::Fail;
	if( IsEditable() )
	{
		if (Implementation->CanItemChange(EPropertyChangeType::ArrayClear, INDEX_NONE))
		{
			Implementation->ClearChildren();
			Result = FPropertyAccess::Success;
		}
	}

	return Result;
}

FPropertyAccess::Result FPropertyHandleArray::Insert( int32 Index )
{
	FPropertyAccess::Result Result = FPropertyAccess::Fail;
	if( IsEditable() && Index < Implementation->GetNumChildren()  )
	{
		if (Implementation->CanItemChange(EPropertyChangeType::ArrayAdd, Index))
		{
			Implementation->InsertChild( Index );
			Result = FPropertyAccess::Success;
		}
	}

	return Result;
}

FPropertyAccess::Result FPropertyHandleArray::DuplicateItem( int32 Index )
{
	FPropertyAccess::Result Result = FPropertyAccess::Fail;
	if( IsEditable() && Index < Implementation->GetNumChildren() )
	{
		if (Implementation->CanItemChange(EPropertyChangeType::Duplicate, Index))
		{
			Implementation->DuplicateChild( Index );
			Result = FPropertyAccess::Success;
		}
	}

	return Result;
}

FPropertyAccess::Result FPropertyHandleArray::DeleteItem( int32 Index )
{
	FPropertyAccess::Result Result = FPropertyAccess::Fail;
	if( IsEditable() && Index < Implementation->GetNumChildren() )
	{
		if (Implementation->CanItemChange(EPropertyChangeType::ArrayRemove, Index))
		{
			Implementation->DeleteChild( Index );
			Result = FPropertyAccess::Success;
		}
	}

	return Result;
}

FPropertyAccess::Result FPropertyHandleArray::SwapItems(int32 FirstIndex, int32 SecondIndex)
{
	FPropertyAccess::Result Result = FPropertyAccess::Fail;
	if (IsEditable() && FirstIndex >= 0 && SecondIndex >= 0 && FirstIndex < Implementation->GetNumChildren() && SecondIndex < Implementation->GetNumChildren())
	{
		if (Implementation->CanItemChange(EPropertyChangeType::ArrayMove, FirstIndex) &&
			Implementation->CanItemChange(EPropertyChangeType::ArrayMove, SecondIndex))
		{
			Implementation->SwapChildren(FirstIndex, SecondIndex);
			Result = FPropertyAccess::Success;
		}
	}

	return Result;
}

FPropertyAccess::Result FPropertyHandleArray::GetNumElements( uint32 &OutNumItems ) const
{
	OutNumItems = Implementation->GetNumChildren();
	return FPropertyAccess::Success;
}

FDelegateHandle FPropertyHandleArray::SetOnNumElementsChanged( const FSimpleDelegate& OnChildrenChanged )
{
	return Implementation->SetOnRebuildChildren( OnChildrenChanged );
}

void FPropertyHandleArray::UnregisterOnNumElementsChanged(FDelegateHandle Handle)
{
	Implementation->UnregisterOnRebuildChildren(Handle);
}

void FPropertyHandleArray::SetOnCanItemChange(const FCanPropertyItemChange& InDelegate)
{
	Implementation->SetOnCanItemChange(InDelegate);
}

FDelegateHandle FPropertyHandleArray::SetOnChangeBlocked(const FPropertyChangeBlocked::FDelegate& InDelegate)
{
	return Implementation->SetOnChangeBlocked(InDelegate);
}

void FPropertyHandleArray::UnregisterOnChangeBlocked(FDelegateHandle Handle)
{
	return Implementation->UnregisterOnChangeBlocked(Handle);
}

TSharedPtr<IPropertyHandleArray> FPropertyHandleArray::AsArray()
{
	return SharedThis(this);
}

TSharedRef<IPropertyHandle> FPropertyHandleArray::GetElement( int32 Index ) const
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetChildNode( Index );
	return PropertyEditorHelpers::GetPropertyHandle( PropertyNode.ToSharedRef(), Implementation->GetNotifyHook(), Implementation->GetPropertyUtilities() ).ToSharedRef();
}

FPropertyAccess::Result FPropertyHandleArray::MoveElementTo(int32 OriginalIndex, int32 NewIndex)
{
	FPropertyAccess::Result Result = FPropertyAccess::Fail;
	if (IsEditable() && OriginalIndex >= 0 && NewIndex >= 0)
	{
		Implementation->MoveElementTo(OriginalIndex, NewIndex);
		Result = FPropertyAccess::Success;
	}

	return Result;
}

FPropertyAccess::Result FPropertyHandleArray::SetValueFromFormattedString(const FString& InValue, EPropertyValueSetFlags::Type Flags)
{
	FPropertyAccess::Result Result = FPropertyAccess::Success;

	// for array of objects, copy each object individually in case some objects are forbidden
	if (InValue.StartsWith("(\"") && InValue.EndsWith("\")"))
	{
		if (EmptyArray() != FPropertyAccess::Success)
		{
			return FPropertyAccess::Fail;
		}

		TArray<FString> Values;

		// Remove firsts and lasts parenthesis, and split the string 
		InValue.LeftChop(1).RightChop(1).ParseIntoArray(Values, TEXT(","), true);

		for (const FString& Value : Values)
		{
			if (AddItem() != FPropertyAccess::Success)
			{
				return FPropertyAccess::Fail;
			}
			uint32 NumElements = 0;
			if (GetNumElements(NumElements) != FPropertyAccess::Success || NumElements == 0)
			{
				return FPropertyAccess::Fail;
			}
			
			const TSharedRef<IPropertyHandle> Property = GetElement( NumElements - 1 );
			ensure(!Property->AsStruct());

			// Trim ""
			FString FormattedString = Value.TrimQuotes();
			
			if (Property->IsValidHandle() && Property->SetValueFromFormattedString(FormattedString, Flags) == FPropertyAccess::Fail)
			{
				Result = FPropertyAccess::Fail;
			}
		}
	}
	else
	{
		Result = FPropertyHandleBase::SetValueFromFormattedString(InValue, Flags);
	}
	return Result;
}

bool FPropertyHandleArray::IsEditable() const
{
	// Property is editable if its a non-const dynamic array
	return Implementation->HasValidPropertyNode() && !Implementation->IsEditConst() && Implementation->IsPropertyTypeOf(FArrayProperty::StaticClass());
}

bool FPropertyHandleOptional::Supports(TSharedRef<FPropertyNode> PropertyNode)
{
	FProperty* Property = PropertyNode->GetProperty();

	return !!CastField<FOptionalProperty>(Property);
}

/** IPropertyHandleOptional interface */
FPropertyAccess::Result FPropertyHandleOptional::GetOptionalValue(FProperty*& OutValue)
{
	const TSharedPtr<FPropertyNode>& PropertyNode = Implementation->GetPropertyNode();
	FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(PropertyNode->GetProperty());
	
	uint8* ValueAddress = nullptr;
	FPropertyAccess::Result Result = PropertyNode->GetSingleReadAddress(ValueAddress);

	if (Result != FPropertyAccess::Success)
	{
		return Result;
	}

	if (OptionalProperty->IsSet(ValueAddress))
	{
		OutValue = OptionalProperty->GetValueProperty();
	}

	return Result;
}

FPropertyAccess::Result FPropertyHandleOptional::SetOptionalValue(FProperty* NewProperty, const UClass* NewObjectClass)
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetPropertyNode();
	FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(PropertyNode->GetProperty());
	
	FReadAddressList ReadAddresses;
	if (!PropertyNode->GetReadAddress(ReadAddresses))
	{
		return FPropertyAccess::Fail;
	}
	
	TArray< TMap<FString, int32> > ArrayIndicesPerObject;
	ArrayIndicesPerObject.Reserve(ReadAddresses.Num());
	TArray<TArray<UObject*>> AffectedInstancesPerObject;
	AffectedInstancesPerObject.SetNum(ReadAddresses.Num());

	// List of top level objects sent to the PropertyChangedEvent
	TArray<const UObject*> TopLevelObjects;
	TopLevelObjects.Reserve(ReadAddresses.Num());

	// Begin a property edit transaction.
	FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "SetOptional", "Set Optional"));
	FObjectPropertyNode* ObjectNode = PropertyNode->FindObjectItemParent();

	for ( int32 i = 0 ; i < ReadAddresses.Num() ; ++i )
	{
		void* Addr = ReadAddresses.GetAddress(i);
		if ( Addr )
		{
			if (!FApp::IsGame())
			{
				UObject* Obj = ObjectNode ? ObjectNode->GetUObject(i) : nullptr;
				if (IsTemplate(Obj))
				{
					PropertyNode->GatherInstancesAffectedByContainerPropertyChange(Obj, Addr, EPropertyArrayChangeType::Add, AffectedInstancesPerObject[i]);
				}

				if (Obj)
				{
					TopLevelObjects.Add(Obj);
				}
			}
		}

		// Add on array index so we can tell which entry just changed
		ArrayIndicesPerObject.Add(TMap<FString, int32>());
		FPropertyValueImpl::GenerateArrayIndexMapToObjectNode(ArrayIndicesPerObject[i], PropertyNode.Get());
	}

	TSet< UObject* > AllAffectedInstances;
	for (const TArray<UObject*>& AffectedInstances : AffectedInstancesPerObject)
	{
		AllAffectedInstances.Append(AffectedInstances);
	}

	// send the PreEditChange notification to all selected objects
	FNotifyHook* NotifyHook = Implementation->GetNotifyHook();
	PropertyNode->NotifyPreChange(OptionalProperty, NotifyHook, AllAffectedInstances);
	
	if (TSharedPtr<IPropertyUtilities> PropertyUtilities = Implementation->GetPropertyUtilities())
	{
		FPropertyChangedEvent StartedChangingPropertiesEvent(OptionalProperty, EPropertyChangeType::ValueSet, MakeArrayView(TopLevelObjects));
		StartedChangingPropertiesEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);
		StartedChangingPropertiesEvent.SetInstancesChanged(MoveTemp(AllAffectedInstances));
		PropertyUtilities->NotifyStartedChangingProperties(StartedChangingPropertiesEvent);
	}
	
	for (int i = 0; i < ReadAddresses.Num(); i++)
	{
		void* Addr = ReadAddresses.GetAddress(i);
		void* ValuePtr = OptionalProperty->MarkSetAndGetInitializedValuePointerToReplace(Addr);

		if (NewProperty)
		{
			OptionalProperty->SetValueProperty(NewProperty);
		}

		// TODO: This should be re-worked to leverage using 'import_text' instead of directly setting new values as we are duplicating logic somewhat here (non-trivial task)
		FProperty* ValueProperty = OptionalProperty->GetValueProperty();

		static const FName DefaultSomeMetaDataName = "DefaultSome";
		const FString* DefaultSomeString = OptionalProperty->FindMetaData(DefaultSomeMetaDataName);
		if (ValueProperty && DefaultSomeString)
		{
			ValueProperty->ImportText_Direct(**DefaultSomeString, ValuePtr, nullptr, 0);
		}

		if (ValueProperty && NewObjectClass)
		{
			if (ValueProperty->IsA(FClassProperty::StaticClass()) || ValueProperty->IsA(FSoftClassProperty::StaticClass()))
			{
				if (FObjectPropertyBase* ClassProperty = CastField<FObjectPropertyBase>(ValueProperty))
				{
					void* ClassPropertyValuePtr = ClassProperty->ContainerPtrToValuePtr<void>(ValuePtr);
					ClassProperty->SetObjectPropertyValue(ClassPropertyValuePtr, const_cast<UClass*>(NewObjectClass));
				}
			}
			else if (NewObjectClass->HasAnyClassFlags(EClassFlags::CLASS_Abstract))
			{
				Implementation->ShowInvalidOperationError(LOCTEXT("SetOptionalAbstractError", "Error: Cannot construct an object from an 'abstract' class."));
			}
			else if (NewObjectClass->HasAnyClassFlags(EClassFlags::CLASS_Interface))
			{
				Implementation->ShowInvalidOperationError(LOCTEXT("SetOptionalInterfaceError", "Error: Cannot construct an object from an 'interface' class."));
			}
			else
			{
				UObject* Outer = ObjectNode ? ObjectNode->GetUObject(i) : nullptr;
				if(!Outer)
				{
					Implementation->ShowInvalidOperationError(LOCTEXT("SetOptionalOuterError", "Could not create value for optional object as could not determine outer object."));
				}
				TopLevelObjects.Add(Outer);

				UObject* UseOuter = (NewObjectClass->IsChildOf(UClass::StaticClass()) ? Cast<UClass>(Outer)->GetDefaultObject() : Outer);
				EObjectFlags MaskedOuterFlags = UseOuter ? UseOuter->GetMaskedFlags(RF_PropagateToSubObjects) : RF_NoFlags;
				if (UseOuter && UseOuter->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
				{
					MaskedOuterFlags |= RF_ArchetypeObject;
				}

				// TODO: Construct object using `NewObjectClass`
				// Note: Using 'NAME_None' here for now since it looks like we don't support using the default value as a template for a new instance override.
				// This means the new name may not match the template name if we start utilizing defaults here, which could result in archetype lookup failures.
				UObject* Obj = NewObject<UObject>(UseOuter, NewObjectClass, NAME_None, MaskedOuterFlags);
				if (Obj)
				{
					if (FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(ValueProperty))
					{
						void* ObjectPropertyValuePtr = ObjectProperty->ContainerPtrToValuePtr<void>(Addr);
						ObjectProperty->SetObjectPropertyValue(ObjectPropertyValuePtr, Obj);
					}
				}
				else
				{
					Implementation->ShowInvalidOperationError(LOCTEXT("SetOptionalUnknownType", "Could not create value for optional object (support for this type may not yet be implemented)"));
				}
			}
		}
	}

	FPropertyChangedEvent ChangeEvent(OptionalProperty, EPropertyChangeType::ValueSet, MakeArrayView(TopLevelObjects));
	ChangeEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);
	ChangeEvent.SetInstancesChanged(MoveTemp(AllAffectedInstances));

	// send the PostEditChange notification; it will be propagated to all selected objects
	PropertyNode->NotifyPostChange(ChangeEvent, NotifyHook);

	if (TSharedPtr<IPropertyUtilities> PropertyUtilities = Implementation->GetPropertyUtilities())
	{
		PropertyNode->FixPropertiesInEvent(ChangeEvent);
		PropertyUtilities->NotifyFinishedChangingProperties(ChangeEvent);
	}

	// `Request` rebuild forces a refresh of the details panel (the UI doesn't update if we just call `RebuildChildren` here)
	PropertyNode->RequestRebuildChildren();

	return FPropertyAccess::Success;
}

FPropertyAccess::Result FPropertyHandleOptional::ClearOptionalValue()
{
	if (!IsEditable())
	{
		return FPropertyAccess::Fail;
	}

	const TSharedPtr<FPropertyNode>& PropertyNode = Implementation->GetPropertyNode();
	FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(PropertyNode->GetProperty());

	if (!OptionalProperty)
	{
		return FPropertyAccess::Fail;
	}

	FReadAddressList ReadAddresses;
	if (!PropertyNode->GetReadAddress(ReadAddresses))
	{
		return FPropertyAccess::Fail;
	}

	TArray< TMap<FString, int32> > ArrayIndicesPerObject;
	ArrayIndicesPerObject.Reserve(ReadAddresses.Num());
	TArray<TArray<UObject*>> AffectedInstancesPerObject;
	AffectedInstancesPerObject.SetNum(ReadAddresses.Num());

	// List of top level objects sent to the PropertyChangedEvent
	TArray<const UObject*> TopLevelObjects;
	TopLevelObjects.Reserve(ReadAddresses.Num());

	// Begin a property edit transaction.
	FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "ClearOptional", "Clear Optional"));
	FObjectPropertyNode* ObjectNode = PropertyNode->FindObjectItemParent();

	for ( int32 i = 0 ; i < ReadAddresses.Num() ; ++i )
	{
		void* Addr = ReadAddresses.GetAddress(i);
		if ( Addr )
		{
			if (!FApp::IsGame())
			{
				UObject* Obj = ObjectNode ? ObjectNode->GetUObject(i) : nullptr;
				if (IsTemplate(Obj))
				{
					PropertyNode->GatherInstancesAffectedByContainerPropertyChange(Obj, Addr, EPropertyArrayChangeType::Clear, AffectedInstancesPerObject[i]);
				}
			}
		}
		// Add on array index so we can tell which entry just changed
		ArrayIndicesPerObject.Add(TMap<FString, int32>());
		FPropertyValueImpl::GenerateArrayIndexMapToObjectNode(ArrayIndicesPerObject[i], PropertyNode.Get());
	}

	TSet< UObject* > AllAffectedInstances;
	for (const TArray<UObject*>& AffectedInstances : AffectedInstancesPerObject)
	{
		AllAffectedInstances.Append(AffectedInstances);
	}

	// send the PreEditChange notification to all selected objects
	FNotifyHook* NotifyHook = Implementation->GetNotifyHook();
	PropertyNode->NotifyPreChange(OptionalProperty, NotifyHook, AllAffectedInstances);
	
	if (TSharedPtr<IPropertyUtilities> PropertyUtilities = Implementation->GetPropertyUtilities())
	{
		FPropertyChangedEvent StartedChangingPropertiesEvent(OptionalProperty, EPropertyChangeType::ValueSet, MakeArrayView(TopLevelObjects));
		StartedChangingPropertiesEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);
		PropertyUtilities->NotifyStartedChangingProperties(StartedChangingPropertiesEvent);
	}
	
	for (int i = 0; i < ReadAddresses.Num(); i++)
	{
		UObject* Obj = ObjectNode ? ObjectNode->GetUObject(i) : nullptr;
		TopLevelObjects.Add(Obj);

		void* Addr = ReadAddresses.GetAddress(i);
		OptionalProperty->MarkUnset(Addr);
	}

	FPropertyChangedEvent ChangeEvent(OptionalProperty, EPropertyChangeType::ValueSet, MakeArrayView(TopLevelObjects));
	ChangeEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);
	ChangeEvent.SetInstancesChanged(MoveTemp(AllAffectedInstances));

	// send the PostEditChange notification; it will be propagated to all selected objects
	PropertyNode->NotifyPostChange(ChangeEvent, NotifyHook);

	if (TSharedPtr<IPropertyUtilities> PropertyUtilities = Implementation->GetPropertyUtilities())
	{
		PropertyNode->FixPropertiesInEvent(ChangeEvent);
		PropertyUtilities->NotifyFinishedChangingProperties(ChangeEvent);
	}

	// `Request` rebuild forces a refresh of the details panel (the UI doesn't update if we just call `RebuildChildren` here)
	PropertyNode->RequestRebuildChildren();

	return FPropertyAccess::Success;
}

TSharedPtr<IPropertyHandleOptional> FPropertyHandleOptional::AsOptional()
{
	return SharedThis(this);
}

bool FPropertyHandleOptional::IsEditable() const
{
	// Property is editable if its a non-const dynamic array
	return Implementation->HasValidPropertyNode() && !Implementation->IsEditConst() && Implementation->IsPropertyTypeOf(FOptionalProperty::StaticClass());
}

// Localized Text
bool FPropertyHandleText::Supports(TSharedRef<FPropertyNode> PropertyNode)
{
	FProperty* Property = PropertyNode->GetProperty();

	if ( Property == nullptr )
	{
		return false;
	}

	// Supported if the property is a text property only
	return Property->IsA(FTextProperty::StaticClass());
}

FPropertyAccess::Result FPropertyHandleText::GetValue(FText& OutValue) const
{
	return Implementation->GetValueAsText(OutValue);
}

FPropertyAccess::Result FPropertyHandleText::SetValue(const FText& NewValue, EPropertyValueSetFlags::Type Flags)
{
	FString StringValue;
	FTextStringHelper::WriteToBuffer(StringValue, NewValue);
	return ImportFormattedTextString(StringValue, Flags);
}

FPropertyAccess::Result FPropertyHandleText::SetValue(const FString& NewValue, EPropertyValueSetFlags::Type Flags)
{
	return SetValue(FText::FromString(NewValue), Flags);
}

FPropertyAccess::Result FPropertyHandleText::SetValue(const TCHAR* NewValue, EPropertyValueSetFlags::Type Flags)
{
	return SetValue(FText::FromString(NewValue), Flags);
}

FPropertyAccess::Result FPropertyHandleText::SetValueFromFormattedString(const FString& InValue, EPropertyValueSetFlags::Type Flags)
{
	return ImportFormattedTextString(InValue, Flags);
}

FPropertyAccess::Result FPropertyHandleText::ImportFormattedTextString(const FString& NewValue, EPropertyValueSetFlags::Type Flags)
{
	TSharedPtr<FPropertyNode> PropertyNode = GetPropertyNode();

	TArray<FObjectBaseAddress> ObjectsToModify;
	Implementation->GetObjectsToModify(ObjectsToModify, PropertyNode.Get());

	TArray<FString> NewTextValuesToImport;
	{
		TArray<UPackage*> OuterPackages;
		GetOuterPackages(OuterPackages);

		FTextProperty* TextProperty = CastField<FTextProperty>(GetProperty());
		FStringView PropertyPath = GetPropertyPath();

		const FText NewTextValue = FTextStringHelper::CreateFromBuffer(*NewValue);
		for (int32 ObjectIndex = 0; ObjectIndex < ObjectsToModify.Num(); ++ObjectIndex)
		{
			const FObjectBaseAddress& ObjectToModify = ObjectsToModify[ObjectIndex];
			if (ObjectToModify.BaseAddress)
			{
				FText ResolvedTextValue = NewTextValue;

				// If the FText value was initialized from a string, then try and generate a stable and deterministic key for it instead
				if (TextProperty && NewTextValue.IsInitializedFromString())
				{
					UPackage* OuterPackage = OuterPackages.IsValidIndex(ObjectIndex) ? OuterPackages[ObjectIndex] : nullptr;
					UObject* Object = ObjectToModify.Object;

					auto DeterministicTextKeyGenerator = [PropertyPath, OuterPackage, Object, TextProperty]()
					{
						auto GetNameKeyHash = [](const FName Name)
						{
							FNameBuilder Builder(Name);
							return TextKeyUtil::HashString(Builder.ToString(), Builder.Len());
						};

						auto GetObjectKeyHash = [](const UObject* Obj)
						{
							FNameBuilder Builder;
							if (Obj)
							{
								Obj->GetPathName(nullptr, Builder);
							}
							return TextKeyUtil::HashString(Builder.ToString(), Builder.Len());
						};

						// Build a unique and deterministic key from a combination of the outer package, object and text property info.
						// We can use both the Object or Outerpackage because in some cases (like for structs), we might not have an object but, if we do, we need to use that to create a unique key.
						FGuid KeyGuid(GetObjectKeyHash(Object != nullptr ? Object : OuterPackage), TextKeyUtil::HashString(PropertyPath), GetNameKeyHash(TextProperty->GetOwnerStruct()->GetFName()), GetNameKeyHash(TextProperty->GetFName()));
#if USE_STABLE_LOCALIZATION_KEYS
						if (OuterPackage)
						{
							if (const FString PackageNamespace = TextNamespaceUtil::EnsurePackageNamespace(OuterPackage);
								!PackageNamespace.IsEmpty())
							{
								// Mix the package namespace hash into the outer hash
								KeyGuid.A = TextKeyUtil::HashString(PackageNamespace, KeyGuid.A);
							}
						}
#endif // USE_STABLE_LOCALIZATION_KEYS
						return KeyGuid.ToString();
					};

					const FText* CurrentTextValue = reinterpret_cast<FText*>(ObjectToModify.BaseAddress);
					ResolvedTextValue = *CurrentTextValue;
					TextNamespaceUtil::EditTextProperty_Direct(OuterPackage, &ResolvedTextValue, TextProperty, TextNamespaceUtil::ETextEditAction::SourceString, NewTextValue.ToString(), DeterministicTextKeyGenerator);
				}

				FString& NewTextValueToImport = NewTextValuesToImport.AddDefaulted_GetRef();
				FTextStringHelper::WriteToBuffer(NewTextValueToImport, ResolvedTextValue);
			}
		}
	}

	FPropertyAccess::Result Result = FPropertyAccess::Fail;
	if (NewTextValuesToImport.Num() > 0)
	{
		Result = Implementation->ImportText(ObjectsToModify, NewTextValuesToImport, PropertyNode.Get(), Flags);
	}

	return Result;
}

// Sets

bool FPropertyHandleSet::Supports(TSharedRef<FPropertyNode> PropertyNode)
{
	FProperty* Property = PropertyNode->GetProperty();

	return CastField<const FSetProperty>(Property) != nullptr;
}

bool FPropertyHandleSet::HasDefaultElement()
{
	TSharedPtr<FPropertyNode> PropNode = Implementation->GetPropertyNode();
	if (PropNode.IsValid())
	{
		TArray<FObjectBaseAddress> Addresses;
		Implementation->GetObjectsToModify(Addresses, PropNode.Get());

		if (Addresses.Num() > 0)
		{
			const bool bIsSparseClassData = PropNode->HasNodeFlags(EPropertyNodeFlags::IsSparseClassData) != 0;
			const bool bIsStruct = Addresses[0].bIsStruct;
			FSetProperty* SetProperty = CastFieldChecked<FSetProperty>(PropNode->GetProperty());
			FScriptSetHelper SetHelper(SetProperty, PropNode->GetValueBaseAddress(Addresses[0].StructAddress, bIsSparseClassData, bIsStruct));

			FDefaultConstructedPropertyElement DefaultElement(SetHelper.ElementProp);
			return SetHelper.FindElementIndex(DefaultElement.GetObjAddress()) != INDEX_NONE;
		}
	}

	return false;
}

FPropertyHandleItemAddResult FPropertyHandleSet::AddItem()
{
	if (IsEditable())
	{
		/**
		 * Checks if an element has already been added to the set
		 *
		 * @param	Helper			The set helper used to query the property.
		 * @param	InBaseAddress	The base address of the set
		 * @param	InElementValue	The element value to check for
		 *
		 * @return	True if the element is found in the set, false otherwise
		 */
		static auto HasElement = [](const FScriptSetHelper& Helper, void* InBaseAddress, const FString& InElementValue)
			{
				FProperty* ElementProp = Helper.GetElementProperty();

				void* TempElementStorage = ElementProp->AllocateAndInitializeValue();
				ON_SCOPE_EXIT
				{
					ElementProp->DestroyAndFreeValue(TempElementStorage);
				};

				for (int32 Index = 0, ItemsLeft = Helper.Num(); ItemsLeft > 0; ++Index)
				{
					if (Helper.IsValidIndex(Index))
					{
						--ItemsLeft;

						const uint8* Element = Helper.GetElementPtr(Index);

						if (Element != InBaseAddress && ElementProp->ImportText_Direct(*InElementValue, TempElementStorage, nullptr, 0) && ElementProp->Identical(Element, TempElementStorage))
						{
							return true;
						}
					}
				}

				return false;
			};

		TSharedPtr<FPropertyNode> PropNode = Implementation->GetPropertyNode();
		if (PropNode.IsValid())
		{
			TArray<FObjectBaseAddress> Addresses;
			Implementation->GetObjectsToModify(Addresses, PropNode.Get());

			if (Addresses.Num() > 0)
			{
				const bool bIsSparseClassData = PropNode->HasNodeFlags(EPropertyNodeFlags::IsSparseClassData) != 0;
				const bool bIsStruct = Addresses[0].bIsStruct;
				uint8* ValueBaseAddress = PropNode->GetValueBaseAddress(Addresses[0].StructAddress, bIsSparseClassData, bIsStruct);

				FSetProperty* SetProperty = CastFieldChecked<FSetProperty>(PropNode->GetProperty());
				FScriptSetHelper SetHelper(SetProperty, ValueBaseAddress);

				if (const FEnumProperty* EnumProperty = CastField<const FEnumProperty>(SetProperty->GetElementProperty()))
				{
					if (const UEnum* Enum = EnumProperty->GetEnum())
					{
						const TArray<FName> ValidPropertyEnums = PropertyEditorHelpers::GetValidEnumsFromPropertyOverride(EnumProperty, Enum);
						const TArray<FName> InvalidPropertyEnums = PropertyEditorHelpers::GetInvalidEnumsFromPropertyOverride(EnumProperty, Enum);

						// NumEnums() - 1, because the last item in an enum is the _MAX item
						for (int32 EnumIndex = 0; EnumIndex < Enum->NumEnums() - 1; ++EnumIndex)
						{
							static FString HiddenName(TEXT("Hidden"));
							static FString SpacerName(TEXT("Spacer"));

							// Ignore hidden enums
							bool bShouldBeHidden = Enum->HasMetaData(*HiddenName, EnumIndex) || Enum->HasMetaData(*SpacerName, EnumIndex);

							if (!bShouldBeHidden)
							{
								if (ValidPropertyEnums.Num() > 0)
								{
									bShouldBeHidden = ValidPropertyEnums.Find(Enum->GetNameByIndex(EnumIndex)) == INDEX_NONE;
								}

								// If both are specified, the metadata "InvalidEnumValues" takes precedence
								if (InvalidPropertyEnums.Num() > 0)
								{
									bShouldBeHidden = InvalidPropertyEnums.Find(Enum->GetNameByIndex(EnumIndex)) != INDEX_NONE;
								}
							}

							if (bShouldBeHidden)
							{
								continue;
							}

							// Skip duplicate elements.
							if (HasElement(SetHelper, ValueBaseAddress, Enum->GetNameStringByIndex(EnumIndex)))
							{
								continue;
							}

							// If we don't have this element then add an entry and set it to this element value.
							const int32 ChildNodeIndex = Implementation->AddChild();
							Implementation->GetPropertyNode()->RebuildChildren();
							if (ChildNodeIndex >= 0)
							{
								TSharedPtr<FPropertyNode> ChildNode = Implementation->GetChildNode(ChildNodeIndex);
								if (ChildNode.IsValid())
								{
									// Set the new entry to have the element value.
									Implementation->ImportText(Enum->GetNameStringByIndex(EnumIndex), ChildNode.Get(), EPropertyValueSetFlags::DefaultFlags);
									return FPropertyHandleItemAddResult(ChildNodeIndex, FPropertyAccess::Success);
								}
							}
						}
					}

					Implementation->ShowInvalidOperationError(LOCTEXT("EnumSet_Add_AllValuesAdded", "Cannot add a new element to the set as all the enum values are already added"));
					return FPropertyHandleItemAddResult();
				}

				if (!HasDefaultElement())
				{
					const int32 Index = Implementation->AddChild();
					Implementation->GetPropertyNode()->RebuildChildren();
					return FPropertyHandleItemAddResult(Index, FPropertyAccess::Success);
				}
			}
		}
	}

	Implementation->ShowInvalidOperationError(LOCTEXT("DuplicateSetElement_Add", "Cannot add a new element to the set while an element with the same value exists"));
	return FPropertyHandleItemAddResult();
}

FPropertyAccess::Result FPropertyHandleSet::Empty()
{
	FPropertyAccess::Result Result = FPropertyAccess::Fail;
	if (IsEditable())
	{
		Implementation->ClearChildren();
		Result = FPropertyAccess::Success;
	}

	return Result;
}

FPropertyAccess::Result FPropertyHandleSet::DeleteItem(int32 Index)
{
	FPropertyAccess::Result Result = FPropertyAccess::Fail;
	TSharedPtr<FPropertyNode> ItemNode;

	if (IsEditable() && Implementation->GetChildNode(Index, ItemNode))
	{
		Implementation->DeleteChild(ItemNode);
		Result = FPropertyAccess::Success;
	}

	return Result;
}

FPropertyAccess::Result FPropertyHandleSet::GetNumElements(uint32& OutNumChildren)
{
	OutNumChildren = Implementation->GetNumChildren();
	return FPropertyAccess::Success;
}

TSharedRef<IPropertyHandle> FPropertyHandleSet::GetElement(int32 Index) const
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetChildNode(Index);
	return PropertyEditorHelpers::GetPropertyHandle(PropertyNode.ToSharedRef(), Implementation->GetNotifyHook(), Implementation->GetPropertyUtilities()).ToSharedRef();
}

FDelegateHandle FPropertyHandleSet::SetOnNumElementsChanged( const FSimpleDelegate& OnChildrenChanged )
{
	return Implementation->SetOnRebuildChildren(OnChildrenChanged);
}

void FPropertyHandleSet::UnregisterOnNumElementsChanged(FDelegateHandle Handle)
{
	Implementation->UnregisterOnRebuildChildren(Handle);
}

TSharedPtr<IPropertyHandleSet> FPropertyHandleSet::AsSet()
{
	return SharedThis(this);
}

bool FPropertyHandleSet::IsEditable() const
{
	// Property is editable if its a non-const dynamic array
	return Implementation->HasValidPropertyNode() && !Implementation->IsEditConst() && Implementation->IsPropertyTypeOf(FSetProperty::StaticClass());
}

FPropertyAccess::Result FPropertyHandleSet::SetValueFromFormattedString(const FString& InValue, EPropertyValueSetFlags::Type Flags)
{
	FPropertyAccess::Result Result = FPropertyAccess::Success;

	// for sets of objects, copy each object individually in case some objects are forbidden
	if (InValue.StartsWith("(\"") && InValue.EndsWith("\")"))
	{
		if (Empty() != FPropertyAccess::Success)
		{
			return FPropertyAccess::Fail;
		}
		
		TArray<FString> Values;

		// Remove firsts and lasts parenthesis  
		InValue.LeftChop(1).RightChop(1).ParseIntoArray(Values, TEXT(","), true);
		for (const FString& Value : Values)
		{
			AddItem();
			uint32 NumElements = 0;
			if (GetNumElements(NumElements) != FPropertyAccess::Success || NumElements == 0)
			{
				return FPropertyAccess::Fail;
			}
			const TSharedRef<IPropertyHandle> Property = GetElement( NumElements - 1 );
			ensure(!Property->AsStruct());
			
			// Trim ""
			FString FormattedString = Value.TrimQuotes();
			
			if ( Property->IsValidHandle() && Property->SetValueFromFormattedString(FormattedString, Flags) == FPropertyAccess::Fail)
			{
				Result = FPropertyAccess::Fail;
			}
		}
	}
	else
	{
		Result = FPropertyHandleBase::SetValueFromFormattedString(InValue, Flags);
	}
	return Result;
}

// Maps

bool FPropertyHandleMap::Supports(TSharedRef<FPropertyNode> PropertyNode)
{
	FProperty* Property = PropertyNode->GetProperty();

	return CastField<const FMapProperty>(Property) != nullptr;
}

bool FPropertyHandleMap::HasDefaultKey()
{
	TSharedPtr<FPropertyNode> PropNode = Implementation->GetPropertyNode();
	if (PropNode.IsValid())
	{
		TArray<FObjectBaseAddress> Addresses;
		Implementation->GetObjectsToModify(Addresses, PropNode.Get());

		if (Addresses.Num() > 0)
		{
			const bool bIsSparseClassData = PropNode->HasNodeFlags(EPropertyNodeFlags::IsSparseClassData) != 0;
			const bool bIsStruct = Addresses[0].bIsStruct;
			FMapProperty* MapProperty = CastFieldChecked<FMapProperty>(PropNode->GetProperty());
			FScriptMapHelper MapHelper(MapProperty, PropNode->GetValueBaseAddress(Addresses[0].StructAddress, bIsSparseClassData, bIsStruct));

			FDefaultConstructedPropertyElement DefaultKey(MapHelper.KeyProp);
			return MapHelper.FindMapIndexWithKey(DefaultKey.GetObjAddress()) != INDEX_NONE;
		}
	}

	return false;
}

FPropertyHandleItemAddResult FPropertyHandleMap::AddItem()
{
	if (IsEditable())
	{
		/**
		 * Checks if a key in the map matches the specified key
		 *
		 * @param	Helper			The map helper used to query the property.
		 * @param	InBaseAddress	The base address of the map
		 * @param	InKeyValue		The key to find within the map
		 *
		 * @return	True if the key is found, false otherwise
		 */
		static auto HasKey = [](const FScriptMapHelper& Helper, void* InBaseAddress, const FString& InKeyValue)
			{
				FProperty* KeyProp = Helper.GetKeyProperty();

				void* TempKeyStorage = KeyProp->AllocateAndInitializeValue();
				ON_SCOPE_EXIT
				{
					KeyProp->DestroyAndFreeValue(TempKeyStorage);
				};

				for (int32 Index = 0, ItemsLeft = Helper.Num(); ItemsLeft > 0; ++Index)
				{
					if (Helper.IsValidIndex(Index))
					{
						--ItemsLeft;

						const uint8* PairPtr = Helper.GetPairPtr(Index);
						const uint8* KeyPtr = KeyProp->ContainerPtrToValuePtr<const uint8>(PairPtr);

						if (KeyPtr != InBaseAddress && KeyProp->ImportText_Direct(*InKeyValue, TempKeyStorage, nullptr, 0) && KeyProp->Identical(KeyPtr, TempKeyStorage))
						{
							return true;
						}
					}
				}

				return false;
			};

		TSharedPtr<FPropertyNode> PropNode = Implementation->GetPropertyNode();
		if (PropNode.IsValid())
		{
			TArray<FObjectBaseAddress> Addresses;
			Implementation->GetObjectsToModify(Addresses, PropNode.Get());

			if (Addresses.Num() > 0)
			{
				const bool bIsSparseClassData = PropNode->HasNodeFlags(EPropertyNodeFlags::IsSparseClassData) != 0;
				const bool bIsStruct = Addresses[0].bIsStruct;
				uint8* ValueBaseAddress = PropNode->GetValueBaseAddress(Addresses[0].StructAddress, bIsSparseClassData, bIsStruct);
			
				FMapProperty* MapProperty = CastFieldChecked<FMapProperty>(PropNode->GetProperty());
				FScriptMapHelper MapHelper(MapProperty, ValueBaseAddress);

				static const FName GetNextKeyMetaDataName = "GetNextKey";
				const FString* GetNextKeyMetaData = MapProperty->FindMetaData(GetNextKeyMetaDataName);
				if (GetNextKeyMetaData && !bIsStruct)
				{
					if (UFunction* Function = Addresses[0].Object->FindFunction(FName(*GetNextKeyMetaData)))
					{
						const FProperty* KeyProperty = MapProperty->GetKeyProperty();
						void* Value = FMemory_Alloca_Aligned(KeyProperty->GetElementSize(), MapProperty->GetMinAlignment());
						KeyProperty->InitializeValue(Value);

						Addresses[0].Object->ProcessEvent(Function, Value);

						FString StringValue;
						KeyProperty->ExportText_Direct(
							StringValue, Value, nullptr, nullptr, PPF_None
						);

						if (!HasKey(MapHelper, ValueBaseAddress, StringValue))
						{
							const int32 ChildNodeIndex = AddItemWithKey(StringValue);
							if (ChildNodeIndex != INDEX_NONE)
							{
								return FPropertyHandleItemAddResult(ChildNodeIndex, FPropertyAccess::Success);
							}
						}
					}
				}
				else if (const FEnumProperty* EnumProperty = CastField<const FEnumProperty>(MapProperty->GetKeyProperty()))
				{
					if (const UEnum* Enum = EnumProperty->GetEnum())
					{
						const TArray<FName> ValidPropertyEnums = PropertyEditorHelpers::GetValidEnumsFromPropertyOverride(EnumProperty, Enum);
						const TArray<FName> InvalidPropertyEnums = PropertyEditorHelpers::GetInvalidEnumsFromPropertyOverride(EnumProperty, Enum);

						// NumEnums() - 1, because the last item in an enum is the _MAX item
						for (int32 EnumIndex = 0; EnumIndex < Enum->NumEnums() - 1; ++EnumIndex)
						{
							static FString HiddenName(TEXT("Hidden"));
							static FString SpacerName(TEXT("Spacer"));

							// Ignore hidden enums
							bool bShouldBeHidden = Enum->HasMetaData(*HiddenName, EnumIndex) || Enum->HasMetaData(*SpacerName, EnumIndex);

							if (!bShouldBeHidden)
							{
								if (ValidPropertyEnums.Num() > 0)
								{
									bShouldBeHidden = ValidPropertyEnums.Find(Enum->GetNameByIndex(EnumIndex)) == INDEX_NONE;
								}

								// If both are specified, the metadata "InvalidEnumValues" takes precedence
								if (InvalidPropertyEnums.Num() > 0)
								{
									bShouldBeHidden = InvalidPropertyEnums.Find(Enum->GetNameByIndex(EnumIndex)) != INDEX_NONE;
								}
							}

							if (bShouldBeHidden)
							{
								continue;
							}

							// Skip duplicate keys.
							if (HasKey(MapHelper, ValueBaseAddress, Enum->GetNameStringByIndex(EnumIndex)))
							{
								continue;
							}

							const int32 ChildNodeIndex = AddItemWithKey(Enum->GetNameStringByIndex(EnumIndex));
							if (ChildNodeIndex != INDEX_NONE)
							{
								return FPropertyHandleItemAddResult(ChildNodeIndex, FPropertyAccess::Success);
							}
						}

						Implementation->ShowInvalidOperationError(LOCTEXT("EnumMap_Add_AllValuesAdded", "Cannot add a new element to the map as all the enum values are already added"));
						return FPropertyHandleItemAddResult();
					}
				}
				else
				{
					if (!HasDefaultKey())
					{
						const int32 Index = Implementation->AddChild();
						return FPropertyHandleItemAddResult(Index, FPropertyAccess::Success);
					}
				}
			}
		}
	}

	Implementation->ShowInvalidOperationError(LOCTEXT("DuplicateMapKey_Add", "Cannot add a new key to the map while a key with the same value exists"));
	return FPropertyHandleItemAddResult();
}

FPropertyAccess::Result FPropertyHandleMap::Empty()
{
	FPropertyAccess::Result Result = FPropertyAccess::Fail;
	if (IsEditable())
	{
		Implementation->ClearChildren();
		Result = FPropertyAccess::Success;
	}

	return Result;
}

FPropertyAccess::Result FPropertyHandleMap::DeleteItem(int32 Index)
{
	FPropertyAccess::Result Result = FPropertyAccess::Fail;
	TSharedPtr<FPropertyNode> ItemNode;

	if (IsEditable() && Implementation->GetChildNode(Index, ItemNode))
	{
		Implementation->DeleteChild(ItemNode);
		Result = FPropertyAccess::Success;
	}

	return Result;
}

FPropertyAccess::Result FPropertyHandleMap::GetNumElements(uint32& OutNumChildren)
{
	OutNumChildren = Implementation->GetNumChildren();
	return FPropertyAccess::Success;
}

TSharedRef<IPropertyHandle> FPropertyHandleMap::GetElement(int32 Index) const
{
	TSharedPtr<FPropertyNode> PropertyNode = Implementation->GetChildNode(Index);
	return PropertyEditorHelpers::GetPropertyHandle(PropertyNode.ToSharedRef(), Implementation->GetNotifyHook(), Implementation->GetPropertyUtilities()).ToSharedRef();
}

FDelegateHandle FPropertyHandleMap::SetOnNumElementsChanged( const FSimpleDelegate& OnChildrenChanged )
{
	return Implementation->SetOnRebuildChildren(OnChildrenChanged);
}

void FPropertyHandleMap::UnregisterOnNumElementsChanged(FDelegateHandle Handle)
{
	Implementation->UnregisterOnRebuildChildren(Handle);
}

TSharedPtr<IPropertyHandleMap> FPropertyHandleMap::AsMap()
{
	return SharedThis(this);
}

bool FPropertyHandleMap::IsEditable() const
{
	// Property is editable if its a non-const dynamic array
	return Implementation->HasValidPropertyNode() && !Implementation->IsEditConst() && Implementation->IsPropertyTypeOf(FMapProperty::StaticClass());
}

int32 FPropertyHandleMap::AddItemWithKey(const FString& Value)
{
	// If we don't have this key then add an entry and set it to this key value.
	const int32 ChildNodeIndex = Implementation->AddChild();
	if (ChildNodeIndex >= 0)
	{
		TSharedPtr<FPropertyNode> ChildNode = Implementation->GetChildNode(ChildNodeIndex);
		if (ChildNode.IsValid())
		{
			// We want to change the key, so get the key property.
			TSharedPtr<FPropertyNode> ChildKeyNode = ChildNode->GetPropertyKeyNode();
			if (ChildKeyNode.IsValid())
			{
				// Set the new entry to have the key value.
				Implementation->ImportText(Value, ChildKeyNode.Get(), EPropertyValueSetFlags::DefaultFlags);
				return ChildNodeIndex;
			}
		}
	}

	return INDEX_NONE;
}

// FieldPath

bool FPropertyHandleFieldPath::Supports(TSharedRef<FPropertyNode> PropertyNode)
{
	FProperty* Property = PropertyNode->GetProperty();

	if (Property == nullptr)
	{
		return false;
	}

	// The value is a field path
	return Property->IsA<FFieldPathProperty>();
}

FPropertyAccess::Result FPropertyHandleFieldPath::GetValue(FProperty*& OutValue) const
{
	return FPropertyHandleFieldPath::GetValue((const FProperty*&)OutValue);
}

FPropertyAccess::Result FPropertyHandleFieldPath::GetValue(const FProperty*& OutValue) const
{
	void* PropValue = nullptr;
	FPropertyAccess::Result Res = Implementation->GetValueData(PropValue);

	if (Res == FPropertyAccess::Success)
	{
		FProperty* Property = GetProperty();
		check(Property->IsA(FFieldPathProperty::StaticClass()));
		const TFieldPath<FProperty>* FieldPathValue = (const TFieldPath<FProperty>*)(PropValue);
		OutValue = FieldPathValue->Get();
	}

	return Res;
}

FPropertyAccess::Result FPropertyHandleFieldPath::SetValue(FProperty* const& NewValue, EPropertyValueSetFlags::Type Flags)
{
	return FPropertyHandleFieldPath::SetValue((const FProperty*)NewValue);
}

FPropertyAccess::Result FPropertyHandleFieldPath::SetValue(const FProperty* const& NewValue, EPropertyValueSetFlags::Type Flags)
{
	const TSharedPtr<FPropertyNode>& PropertyNode = Implementation->GetPropertyNode();

	if (!PropertyNode->HasNodeFlags(EPropertyNodeFlags::EditInlineNew))
	{
		FString ObjectPathName = NewValue ? NewValue->GetPathName() : TEXT("None");
		return SetValueFromFormattedString(ObjectPathName, Flags);
	}

	return FPropertyAccess::Fail;
}

#undef LOCTEXT_NAMESPACE
