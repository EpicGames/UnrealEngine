// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/UnrealType.h"
#include "Widgets/SWidget.h"
#include "PropertyHandle.h"
#include "AssetRegistry/AssetData.h"
#include "PropertyNode.h"

class FNotifyHook;
class FPropertyRestriction;
	
class FObjectBaseAddress
{
public:
	FObjectBaseAddress() = default;
	FObjectBaseAddress(UObject* InObject, uint8* InStructAddress, uint8* InBaseAddress, bool bInIsStruct)
		: Object(InObject)
		, StructAddress(InStructAddress)
		, BaseAddress(InBaseAddress)
		, bIsStruct(bInIsStruct)
	{}

	/** Object associated with the value, or owner object when editing a struct. */
	UObject* Object = nullptr;
	/** Pointer to the object/struct instance that actually owns the value (may be pointing to sidecar data). */
	uint8* StructAddress = nullptr;
	/** Pointer to the base address of the value within StructAddress. */
	uint8* BaseAddress = nullptr;
	/** Editing a struct */
	bool bIsStruct = false;
};

/**
 * Encapsulates a property node (and property) and provides functionality to read and write to that node     
 */
class FPropertyValueImpl
{
public:
	/**
	 * Constructor
	 * 
	 * @param InPropertyNode	The property node containing information about the property to be edited
	 * @param NotifyHook		Optional notify hook for pre/post edit change messages sent outside of objects being modified
	 */
	FPropertyValueImpl( TSharedPtr<FPropertyNode> InPropertyNode, FNotifyHook* InNotifyHook, TSharedPtr<IPropertyUtilities> InPropertyUtilities );

	/**
	 * Recurse up to the next object node, adding all array indices into a map according to their property name
	 * @param ArrayIndexMap - for the current object, what properties use which array offsets
	 * @param InNode - node to start adding array offsets for.  This function will move upward until it gets to an object node
	 */
	static void GenerateArrayIndexMapToObjectNode( TMap<FString,int32>& OutArrayIndexMap, FPropertyNode* PropertyNode );

	/**
	 * Gets the value as a string formatted for multiple values in an array                 
	 */
	FString GetPropertyValueArray() const;

	/**
	 * Enumerate the objects that need to be modified from the passed in property node
	 *
	 * @param InPropertyNode			The property node to get objects from
	 * @param InObjectsToModifyCallback	The function to call for each object
	 */
	typedef TFunctionRef<bool(const FObjectBaseAddress& /*ObjectToModify*/, const int32 /*ObjectIndex*/, const int32 /*NumObjects*/)> EnumerateObjectsToModifyFuncRef; /** Return true to continue enumeration */
	void EnumerateObjectsToModify( FPropertyNode* InPropertyNode, const EnumerateObjectsToModifyFuncRef& InObjectsToModifyCallback ) const;

	/**
	 * Gets the objects that need to be modified from the passed in property node
	 * 
	 * @param ObjectsToModify	The addresses of the objects to modify
	 * @param InPropertyNode	The property node to get objects from
	 */
	void GetObjectsToModify( TArray<FObjectBaseAddress>& ObjectsToModify, FPropertyNode* InPropertyNode ) const;

	/**
	 * Gets the union of values with the appropriate type for the property set
	 *
	 * @param OutAddress	The location of the property value
	 * @return The result of the query 
	 */
	FPropertyAccess::Result GetValueData( void*& OutAddress ) const;

	/**
	 * Gets edit stack for this property
	 *
	 * @param OutStack	Edit stack
	 * @return The result of the query
	 */
	FPropertyAccess::Result GetValueEditStack(FPropertyNodeEditStack& OutStack) const;

	/**
	 * Given an address and a property type, get the actual value out
	 *
	 * @param Address	The location of the property value
	 * @return The property value
	 */
	template<class TProperty>
	typename TProperty::TCppType GetPropertyValue(void const* Address) const
	{
		TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
		check(PropertyNodePin.IsValid());
		return CastFieldChecked<TProperty>(PropertyNodePin->GetProperty())->GetPropertyValue(Address);
	}

	/**
	 * Given an address, get the actual UObject* value out
	 *
	 * @param Address	The location of the property value
	 * @return The object property value
	 */
	UObject* GetObjectPropertyValue(void const* Address) const
	{
		TSharedPtr<FPropertyNode> PropertyNodePin = PropertyNode.Pin();
		check(PropertyNodePin.IsValid());
		return CastFieldChecked<FObjectPropertyBase>(PropertyNodePin->GetProperty())->GetObjectPropertyValue(Address);
	}


	/**
	 * The core functionality for setting values on a property
	 *
	 * @param	The value formatted as a string
	 * @return	The result of attempting to set the value
	 */
	FPropertyAccess::Result ImportText( const FString& InValue, EPropertyValueSetFlags::Type Flags );
	FPropertyAccess::Result ImportText( const FString& InValue, FPropertyNode* PropertyNode, EPropertyValueSetFlags::Type Flags );
	FPropertyAccess::Result ImportText( const TArray<FObjectBaseAddress>& InObjects, const TArray<FString>& InValues, FPropertyNode* PropertyNode, EPropertyValueSetFlags::Type Flags );

	/**
	 * Enumerate the raw data of this property.  (Each pointer can be cast to the property data type)
	 *
	 * @param InRawDataCallback		The function to call for each data
	 */ 
	void EnumerateRawData( const IPropertyHandle::EnumerateRawDataFuncRef& InRawDataCallback );
	void EnumerateConstRawData( const IPropertyHandle::EnumerateConstRawDataFuncRef& InRawDataCallback ) const;

	/**
	 * Accesses the raw data of this property. 
	 *
	 * @param RawData	An array of raw data. 
	 */ 
	void AccessRawData( TArray<void*>& RawData );
	void AccessRawData( TArray<const void*>& RawData ) const;

	/**
	 * Sets a delegate to call when the property value changes
	 */
	void SetOnPropertyValueChanged( const FSimpleDelegate& InOnPropertyValueChanged );
	void SetOnPropertyValueChangedWithData(const TDelegate<void(const FPropertyChangedEvent&)>& InOnPropertyValueChanged);
	/**
	 * Sets a delegate to call when the property value of a child changes
	 */
	void SetOnChildPropertyValueChanged( const FSimpleDelegate& InOnChildPropertyValueChanged );
	void SetOnChildPropertyValueChangedWithData(const TDelegate<void(const FPropertyChangedEvent&)>& InOnChildPropertyValueChanged);
	/**
	 * Sets a delegate to call when the property value is about to change
	 */
	void SetOnPropertyValuePreChange(const FSimpleDelegate& InOnPropertyValuePreChange);

	/**
	 * Sets a delegate to call when the property value of a child is about to change
	 */
	void SetOnChildPropertyValuePreChange(const FSimpleDelegate& InOnChildPropertyValuePreChange);

	/**
	 * Sets a delegate to call when a property is reset to default
	 */
	void SetOnPropertyResetToDefault(const FSimpleDelegate& InOnPropertyResetToDefault);

	/**
	 * Sets a delegate to call before an item in a container is being mutated
	 */
	void SetOnCanItemChange(const FCanPropertyItemChange& InDelegate);
	bool CanItemChange(EPropertyChangeType::Type Type, int32 Index);

	/**
	 * Sets a delegate to call when a property change has been disallowed
	 */
	FDelegateHandle SetOnChangeBlocked(const FPropertyChangeBlocked::FDelegate& InDelegate);

	/**
	 * Unregisters a delegate that's bound to OnChangeBlocked
	 */
	void UnregisterOnChangeBlocked(FDelegateHandle Handle);

	/**
	 * Sets a delegate to call when children of the property node must be rebuilt
	 */
	FDelegateHandle SetOnRebuildChildren( const FSimpleDelegate& InOnRebuildChildren );

	/**
	 * Unregisters a delegate called when children of the property node must be rebuilt
	 */
	void UnregisterOnRebuildChildren(FDelegateHandle Handle);

	/**
	 * Get the value of a property as a formatted string.
	 * Each FProperty has a specific string format that it sets
	 *
	 * @param OutValue	The formatted string value to set
	 * @param PortFlags	Determines how the property's value is accessed. Defaults to PPF_PropertyWindow
	 * @return The result of attempting to get the value
	 */
	FPropertyAccess::Result GetValueAsString( FString& OutString, EPropertyPortFlags PortFlags = PPF_PropertyWindow ) const;

	/**
	 * Get the value of a property as a formatted string, possibly using an alternate form more suitable for display in the UI
	 *
	 * @param OutValue	The formatted string value to set
	 * @param PortFlags	Determines how the property's value is accessed. 
	 * @return The result of attempting to get the value
	 */
	FPropertyAccess::Result GetValueAsDisplayString( FString& OutString, EPropertyPortFlags PortFlags ) const;

	/**
	 * Get the value of a property as FText.
	 *
	 * @param OutValue The formatted text value to set
	 * @return The result of attempting to get the value
	 */
	FPropertyAccess::Result GetValueAsText( FText& OutText ) const;

	/**
	 * Get the value of a property as FText, possibly using an alternate form more suitable for display in the UI
	 *
	 * @param OutValue The formatted text value to set
	 * @return The result of attempting to get the value
	 */
	FPropertyAccess::Result GetValueAsDisplayText( FText& OutText ) const;

	/**
	 * Sets the value of a property formatted from a string.
	 * Each FProperty has a specific string format it requires
	 *
	 * @param InValue The formatted string value to set
	 * @return The result of attempting to set the value
	 */
	FPropertyAccess::Result SetValueAsString( const FString& InValue, EPropertyValueSetFlags::Type Flags );

	/**
	 * Returns whether or not a property is of a specific subclass of FProperty
	 *
	 * @param ClassType	The class type to check
	 * @param true if the property is a ClassType
	 */
	bool IsPropertyTypeOf( FFieldClass* ClassType ) const ;
	
	/**
	 * @return The property node used by this value
	 */
	TSharedPtr<FPropertyNode> GetPropertyNode() const;

	/**
	 * @return The number of children the property node has
	 */
	int32 GetNumChildren() const;

	/**
	 * Gets a child node of the property node
	 * 
	 * @param ChildName	The name of the child to get
	 * @return The child property node or nullptr if it doesnt exist
	 */
	TSharedPtr<FPropertyNode> GetChildNode( FName ChildName, bool bRecurse = true ) const;

	/**
	 * Gets a child node of the property node
	 * 
	 * @param ChildIndex	The child index where the child is stored
	 * @return The child property node or nullptr if it doesnt exist
	 */
	TSharedPtr<FPropertyNode> GetChildNode( int32 ChildIndex ) const ;

	/**
	 * Gets a child of the property node that has the given array index
	 *
	 * @param ChildArrayIndex	The array index we're searching for. This is not necessarily the same value as the index where the child is stored
	 * @param OutChildNode		The child property node whose array index matches ChildArrayIndex, or nullptr if it could not be found
	 * @return True if a matching child property node was found, false otherwise
	 */
	bool GetChildNode(const int32 ChildArrayIndex, TSharedPtr<FPropertyNode>& OutChildNode) const;

	/**
	 * Rests the value to its default value
	 */
	void ResetToDefault() ;

	/**
	 * @return true if the property value differs from its default value
	 */
	bool DiffersFromDefault() const ;

	/**
	 * @return true if the property  is edit const and cannot be changed
	 */
	bool IsEditConst() const;
	
	/**
	 * @return Whether or not the property is expanded
	 */
	bool IsExpanded() const;

	/**
	* Set the expanded flag
	*/
	void SetExpanded(bool bExpanded);

	/**
	 * @return The label to use for displaying reset to default values
	 */
	FText GetResetToDefaultLabel() const;

	/**
	 * Adds a child to the property node (container properties only)
	 * @return The logical index of the added child.
	 */
	int32 AddChild();

	/**
	 * Removes all children from the property node (container properties only)
	 */
	void ClearChildren();

	/** 
	 * Inserts a child at Index (arrays only)
	 */ 
	void InsertChild( int32 Index );

	/**
	 * Inserts a child at the index provided by the child node (arrays only)
	 */
	void InsertChild( TSharedPtr<FPropertyNode> ChildNodeToInsertAfter );

	/** 
	 * Duplicates the child at Index (containers only)
	 */ 
	void DuplicateChild( int32 Index );

	/**
	 * Duplicates the provided child (containers only)                                                             
	 */
	void DuplicateChild( TSharedPtr<FPropertyNode> ChildNodeToDuplicate );

	/** 
	 * Deletes the child at Index (containers only)
	 */ 
	void DeleteChild( int32 Index );

	/** 
	 * Deletes the provided child (containers only)
	 */ 
	void DeleteChild( TSharedPtr<FPropertyNode> ChildNodeToDelete );

	/** 
	 * Swaps the children at FirstIndex and SecondIndex
	 */ 
	void SwapChildren( int32 FirstIndex, int32 SecondIndex );

	/** 
	 * Swaps the children provided children (containers only)
	 */ 
	void SwapChildren( TSharedPtr<FPropertyNode> FirstChildNode, TSharedPtr<FPropertyNode> SecondChildNode );

	/**
	* Moves the element at OriginalIndex to NewIndex
	*/
	void MoveElementTo(int32 OriginalIndex, int32 NewIndex);

	/**
	 * @return true if the property node is valid
	 */
	bool HasValidPropertyNode() const;

	/**
	 * @return The display name of the property
	 */
	FText GetDisplayName() const;

	/** @return The notify hook being used */
	FNotifyHook* GetNotifyHook() const { return NotifyHook; }

	TSharedPtr<IPropertyUtilities> GetPropertyUtilities() const { return PropertyUtilities.Pin(); }

	void ShowInvalidOperationError(const FText& ErrorText);

protected:
	/** Property node used to access FProperty and address of object to change */
	TWeakPtr<FPropertyNode> PropertyNode;
	TWeakPtr<IPropertyUtilities> PropertyUtilities;
	/** Notify hook to call when properties change */
	FNotifyHook* NotifyHook;
	/** Set true if a change was made with bFinished=false */
	bool bInteractiveChangeInProgress;

private:
	TWeakPtr<class SNotificationItem> InvalidOperationError;
};

#define DECLARE_PROPERTY_ACCESSOR( ValueType ) \
	virtual FPropertyAccess::Result SetValue( ValueType const& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override; \
	virtual FPropertyAccess::Result GetValue( ValueType& OutValue ) const override; 

class FDetailCategoryImpl;

/**
 * The base implementation of a property handle
 */
class FPropertyHandleBase : public IPropertyHandle
{
public:
	FPropertyHandleBase( TSharedPtr<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities );

	/** IPropertyHandle interface */
	DECLARE_PROPERTY_ACCESSOR( bool )
	DECLARE_PROPERTY_ACCESSOR( int8 )
	DECLARE_PROPERTY_ACCESSOR( int16)
	DECLARE_PROPERTY_ACCESSOR( int32 )
	DECLARE_PROPERTY_ACCESSOR( int64 )
	DECLARE_PROPERTY_ACCESSOR( uint8 )
	DECLARE_PROPERTY_ACCESSOR( uint16 )
	DECLARE_PROPERTY_ACCESSOR( uint32 )
	DECLARE_PROPERTY_ACCESSOR( uint64 )
	DECLARE_PROPERTY_ACCESSOR( float )
	DECLARE_PROPERTY_ACCESSOR( double )
	DECLARE_PROPERTY_ACCESSOR( FString )
	DECLARE_PROPERTY_ACCESSOR( FText )
	DECLARE_PROPERTY_ACCESSOR( FName )
	DECLARE_PROPERTY_ACCESSOR( FVector )
	DECLARE_PROPERTY_ACCESSOR( FVector2D )
	DECLARE_PROPERTY_ACCESSOR( FVector4 )
	DECLARE_PROPERTY_ACCESSOR( FQuat )
	DECLARE_PROPERTY_ACCESSOR( FRotator )
	DECLARE_PROPERTY_ACCESSOR( UObject* )
	DECLARE_PROPERTY_ACCESSOR( const UObject* )
	DECLARE_PROPERTY_ACCESSOR( FAssetData )
	DECLARE_PROPERTY_ACCESSOR( FProperty* )
	DECLARE_PROPERTY_ACCESSOR( const FProperty* )
	virtual FPropertyAccess::Result GetValueData(void*& OutAddress) const override;

	virtual FPropertyAccess::Result SetValue( const TCHAR* InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override;

	/** IPropertyHandle interface */
	virtual bool IsValidHandle() const override;
	virtual bool IsSamePropertyNode(TSharedPtr<IPropertyHandle> OtherHandle) const override;
	virtual FText GetPropertyDisplayName() const override;
	virtual void SetPropertyDisplayName(FText InDisplayName) override;
	virtual void ResetToDefault() override;
	virtual bool DiffersFromDefault() const override;
	virtual FText GetResetToDefaultLabel() const override;
	virtual void MarkHiddenByCustomization() override;
	virtual void MarkResetToDefaultCustomized(bool bCustomized = true) override;
	virtual void ClearResetToDefaultCustomized() override;
	virtual bool IsCustomized() const override;
	virtual bool IsResetToDefaultCustomized() const override;
	virtual FString GeneratePathToProperty() const override;
	virtual TSharedRef<SWidget> CreatePropertyNameWidget( const FText& NameOverride, const FText& ToolTipOverride, bool bDisplayResetToDefault, bool bDisplayText, bool bDisplayThumbnail ) const override;
	virtual TSharedRef<SWidget> CreatePropertyNameWidget(const FText& NameOverride, const FText& ToolTipOverride) const override;
	virtual TSharedRef<SWidget> CreatePropertyValueWidget( bool bDisplayDefaultPropertyButtons = true ) const override;
	virtual TSharedRef<SWidget> CreatePropertyValueWidgetWithCustomization( const IDetailsView* DetailsView ) override;
	virtual TSharedRef<SWidget> CreateDefaultPropertyButtonWidgets() const override;
	virtual void CreateDefaultPropertyCopyPasteActions(FUIAction& OutCopyAction, FUIAction& OutPasteAction) const override;
	virtual bool IsEditConst() const override;
	virtual bool IsEditable() const override;
	virtual bool IsExpanded() const override;
	virtual void SetExpanded(bool bExpanded) override;
	virtual void SetOnPropertyValueChanged( const FSimpleDelegate& InOnPropertyValueChanged ) override;
	virtual void SetOnPropertyValueChangedWithData(const TDelegate<void(const FPropertyChangedEvent&)>& InOnPropertyValueChanged) override;
	virtual void SetOnChildPropertyValueChanged( const FSimpleDelegate& InOnPropertyValueChanged ) override;
	virtual void SetOnChildPropertyValueChangedWithData( const TDelegate<void(const FPropertyChangedEvent&)>& InOnChildPropertyValueChanged ) override;
	virtual void SetOnPropertyValuePreChange(const FSimpleDelegate& InOnPropertyValuePreChange) override;
	virtual void SetOnChildPropertyValuePreChange(const FSimpleDelegate& InOnPropertyValuePreChange) override;
	virtual void SetOnPropertyResetToDefault(const FSimpleDelegate& InOnPropertyResetToDefault) override;

	virtual int32 GetIndexInArray() const override;
	virtual FPropertyAccess::Result GetValueAsFormattedString( FString& OutValue, EPropertyPortFlags PortFlags = PPF_PropertyWindow ) const override;
	virtual FPropertyAccess::Result GetValueAsDisplayString( FString& OutValue, EPropertyPortFlags PortFlags = PPF_PropertyWindow) const override;
	virtual FPropertyAccess::Result GetValueAsFormattedText( FText& OutValue ) const override;
	virtual FPropertyAccess::Result GetValueAsDisplayText( FText& OutValue ) const override;
	virtual FPropertyAccess::Result SetValueFromFormattedString( const FString& InValue,  EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override;
	virtual TSharedPtr<IPropertyHandle> GetChildHandle( uint32 ChildIndex ) const override;
	virtual TSharedPtr<IPropertyHandle> GetChildHandle( FName ChildName, bool bRecurse = true ) const override;
	virtual TSharedPtr<IPropertyHandle> GetParentHandle() const override;
	virtual TSharedPtr<IPropertyHandle> GetKeyHandle() const override;
	virtual void EnumerateRawData( const EnumerateRawDataFuncRef& InRawDataCallback ) override;
	virtual void EnumerateConstRawData( const EnumerateConstRawDataFuncRef& InRawDataCallback ) const override;
	virtual void AccessRawData( TArray<void*>& RawData ) override;
	virtual void AccessRawData( TArray<const void*>& RawData ) const override;
	virtual uint32 GetNumOuterObjects() const override;
	virtual void GetOuterObjects(TArray<UObject*>& OuterObjects) const override;
	virtual void GetOuterStructs( TArray<TSharedPtr<FStructOnScope>>& OutStructs ) const override;
	virtual const UClass* GetOuterBaseClass() const override;
	virtual void ReplaceOuterObjects(const TArray<UObject*>& OuterObjects) override;
	virtual void GetOuterPackages(TArray<UPackage*>& OuterPackages) const override;
	virtual FPropertyAccess::Result GetNumChildren( uint32& OutNumChildren ) const override;
	virtual TSharedPtr<IPropertyHandleArray> AsArray() override { return nullptr; }
	virtual TSharedPtr<IPropertyHandleSet> AsSet() override { return nullptr; }
	virtual TSharedPtr<IPropertyHandleMap> AsMap() override { return nullptr; }
	virtual TSharedPtr<IPropertyHandleOptional> AsOptional() override { return nullptr; }
	virtual TSharedPtr<IPropertyHandleStruct> AsStruct() override { return nullptr; }
	virtual const FFieldClass* GetPropertyClass() const override;
	virtual FProperty* GetProperty() const override;
	virtual FProperty* GetMetaDataProperty() const override;
	virtual bool HasMetaData(const FName& Key) const override;
	virtual const FString& GetMetaData(const FName& Key) const override;
	virtual bool GetBoolMetaData(const FName& Key) const override;
	virtual int32 GetIntMetaData(const FName& Key) const override;
	virtual float GetFloatMetaData(const FName& Key) const override;
	virtual double GetDoubleMetaData(const FName& Key) const override;
	virtual UClass* GetClassMetaData(const FName& Key) const override;
	virtual void SetInstanceMetaData(const FName& Key, const FString& Value) override;
	virtual const FString* GetInstanceMetaData(const FName& Key) const override;
	virtual const TMap<FName, FString>* GetInstanceMetaDataMap() const override;	
	virtual FText GetToolTipText() const override;
	virtual void SetToolTipText(const FText& ToolTip) override;
	virtual bool HasDocumentation() override { return false; }
	virtual FString GetDocumentationLink() override { return FString(); }
	virtual FString GetDocumentationExcerptName() override { return FString(); }
	virtual uint8* GetValueBaseAddress( uint8* Base ) const override;
	virtual int32 GetNumPerObjectValues() const override;
	virtual FPropertyAccess::Result SetPerObjectValues( const TArray<FString>& InPerObjectValues,  EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override;
	virtual FPropertyAccess::Result GetPerObjectValues( TArray<FString>& OutPerObjectValues ) const override;
	virtual FPropertyAccess::Result SetPerObjectValue( const int32 ObjectIndex, const FString& ObjectValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override;
	virtual FPropertyAccess::Result GetPerObjectValue( const int32 ObjectIndex, FString& OutObjectValue ) const override;
	virtual bool GeneratePossibleValues(TArray< TSharedPtr<FString> >& OutOptionStrings, TArray< FText >& OutToolTips, TArray<bool>& OutRestrictedItems) override;
	virtual bool GeneratePossibleValues(TArray<FString>& OutOptionStrings, TArray< FText >& OutToolTips, TArray<bool>& OutRestrictedItems, TArray<FText>* OutDisplayNames) override;
	virtual FPropertyAccess::Result SetObjectValueFromSelection() override;
	virtual void NotifyPreChange() override;
	virtual void NotifyPostChange(EPropertyChangeType::Type ChangeType) override;
	virtual void NotifyFinishedChangingProperties() override;
	virtual void AddRestriction(TSharedRef<const FPropertyRestriction> Restriction) override;
	virtual bool IsHidden(const FString& Value) const override;
	virtual bool IsHidden(const FString& Value, TArray<FText>& OutReasons) const override;
	virtual bool IsDisabled(const FString& Value) const override;
	virtual bool IsDisabled(const FString& Value, TArray<FText>& OutReasons) const override;
	virtual bool IsRestricted(const FString& Value) const override;
	virtual bool IsRestricted(const FString& Value, TArray<FText>& OutReasons) const override;
	virtual bool GenerateRestrictionToolTip(const FString& Value, FText& OutTooltip) const override;
	virtual void SetIgnoreValidation(bool bInIgnore) override;
	virtual TArray<TSharedPtr<IPropertyHandle>> AddChildStructure(TSharedRef<FStructOnScope> ChildStructure) override;
	virtual TArray<TSharedPtr<IPropertyHandle>> AddChildStructure(TSharedRef<IStructureDataProvider> ChildStructProvider) override;
	virtual void RemoveChildren() override;
	virtual bool CanResetToDefault() const override;
	virtual void ExecuteCustomResetToDefault(const FResetToDefaultOverride& InOnCustomResetToDefault) override;
	virtual FName GetDefaultCategoryName() const override;
	virtual FText GetDefaultCategoryText() const override;
	virtual FStringView GetPropertyPath() const override;
	virtual TSharedPtr<FPropertyPath> CreateFPropertyPath() const override;
	virtual int32 GetArrayIndex() const override;
	virtual void RequestRebuildChildren() override;
	virtual bool IsFavorite() const override;
	virtual bool IsCategoryHandle() const override;

	PROPERTYEDITOR_API TSharedPtr<FPropertyNode> GetPropertyNode() const;
	void OnCustomResetToDefault(const FResetToDefaultOverride& OnCustomResetToDefault);

private:
	static void CopyValueToClipboard(TWeakPtr<FPropertyValueImpl> ImplementationWeak);
	static void PasteValueFromClipboard(TWeakPtr<FPropertyValueImpl> ImplementationWeak);

protected:
	TSharedPtr<FPropertyValueImpl> Implementation;
};

class FPropertyHandleInt : public FPropertyHandleBase
{	
public:
	FPropertyHandleInt( TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities );
	static bool Supports( TSharedRef<FPropertyNode> PropertyNode );

	virtual FPropertyAccess::Result GetValue(int8& OutValue ) const override;
	virtual FPropertyAccess::Result SetValue(const int8& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override;

	virtual FPropertyAccess::Result GetValue(int16& OutValue) const override;
	virtual FPropertyAccess::Result SetValue(const int16& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) override;

	virtual FPropertyAccess::Result GetValue(int32& OutValue) const override;
	virtual FPropertyAccess::Result SetValue(const int32& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) override;

	virtual FPropertyAccess::Result GetValue(int64& OutValue) const override;
	virtual FPropertyAccess::Result SetValue(const int64& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) override;

	virtual FPropertyAccess::Result GetValue(uint8& OutValue) const override;
	virtual FPropertyAccess::Result SetValue(const uint8& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) override;

	virtual FPropertyAccess::Result GetValue(uint16& OutValue) const override;
	virtual FPropertyAccess::Result SetValue(const uint16& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) override;

	virtual FPropertyAccess::Result GetValue(uint32& OutValue) const override;
	virtual FPropertyAccess::Result SetValue(const uint32& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) override;

	virtual FPropertyAccess::Result GetValue(uint64& OutValue) const override;
	virtual FPropertyAccess::Result SetValue(const uint64& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) override;
};

class FPropertyHandleFloat : public FPropertyHandleBase
{
public:
	FPropertyHandleFloat( TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities );
	static bool Supports( TSharedRef<FPropertyNode> PropertyNode );
	virtual FPropertyAccess::Result GetValue( float& OutValue ) const override;
	virtual FPropertyAccess::Result SetValue( const float& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override;
};

class FPropertyHandleDouble : public FPropertyHandleBase
{
public:
	FPropertyHandleDouble( TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities );
	static bool Supports( TSharedRef<FPropertyNode> PropertyNode );
	virtual FPropertyAccess::Result GetValue( double& OutValue ) const override;
	virtual FPropertyAccess::Result SetValue( const double& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override;
};

class FPropertyHandleBool : public FPropertyHandleBase
{
public:
	FPropertyHandleBool( TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities );
	static bool Supports( TSharedRef<FPropertyNode> PropertyNode );
	virtual FPropertyAccess::Result GetValue( bool& OutValue ) const override;
	virtual FPropertyAccess::Result SetValue( const bool& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override;
};

class FPropertyHandleByte : public FPropertyHandleBase
{	
public:
	FPropertyHandleByte( TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities );
	static bool Supports( TSharedRef<FPropertyNode> PropertyNode );
	virtual FPropertyAccess::Result GetValue( uint8& OutValue ) const override;
	virtual FPropertyAccess::Result SetValue( const uint8& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override;
};

class FPropertyHandleString : public FPropertyHandleBase
{
public:
	FPropertyHandleString( TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook,TSharedPtr<IPropertyUtilities> PropertyUtilities );
	static bool Supports( TSharedRef<FPropertyNode> PropertyNode );
	virtual FPropertyAccess::Result GetValue( FString& OutValue ) const override;
	virtual FPropertyAccess::Result SetValue( const FString& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override;
	virtual FPropertyAccess::Result SetValue( const TCHAR* InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override;

	virtual FPropertyAccess::Result GetValue( FName& OutValue ) const override;
	virtual FPropertyAccess::Result SetValue( const FName& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override;
};

class FPropertyHandleObject : public FPropertyHandleBase
{
public:
	FPropertyHandleObject( TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities );
	static bool Supports( TSharedRef<FPropertyNode> PropertyNode );
	virtual FPropertyAccess::Result GetValue( UObject*& OutValue ) const override;
	virtual FPropertyAccess::Result SetValue( UObject* const& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override;
	virtual FPropertyAccess::Result GetValue( const UObject*& OutValue ) const override;
	virtual FPropertyAccess::Result SetValue( const UObject* const& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override;
	virtual FPropertyAccess::Result GetValue( FAssetData& OutValue ) const override;
	virtual FPropertyAccess::Result SetValue( const FAssetData& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override;
	virtual FPropertyAccess::Result SetValueFromFormattedString(const FString& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) override;
	FPropertyAccess::Result SetValueFromFormattedString(const FString& InValue, EPropertyValueSetFlags::Type Flags, bool bSkipResolve);
	virtual FPropertyAccess::Result SetObjectValueFromSelection() override;
};

// LWC_TODO: Replace with FPropertyHandleDouble once all types support it
class FPropertyHandleMixed : public FPropertyHandleBase
{
public:
	FPropertyHandleMixed(TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities);
	static bool Supports(TSharedRef<FPropertyNode> PropertyNode);
	virtual FPropertyAccess::Result GetValue(float& OutValue) const override;
	virtual FPropertyAccess::Result GetValue(double& OutValue) const override;
	virtual FPropertyAccess::Result SetValue(const float& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) override;
	virtual FPropertyAccess::Result SetValue(const double& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) override;
};

class FPropertyHandleStruct : public FPropertyHandleBase, public IPropertyHandleStruct
{
public:
	FPropertyHandleStruct(TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities);
	static bool Supports(TSharedRef<FPropertyNode> PropertyNode);
	virtual TSharedPtr<IPropertyHandleStruct> AsStruct() override;
	virtual TSharedPtr<FStructOnScope> GetStructData() const override;
};

class FPropertyHandleVector : public FPropertyHandleStruct
{
public:
	FPropertyHandleVector( TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities );
	static bool Supports( TSharedRef<FPropertyNode> PropertyNode );
	virtual FPropertyAccess::Result GetValue( FVector& OutValue ) const override;
	virtual FPropertyAccess::Result SetValue( const FVector& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override;

	virtual FPropertyAccess::Result GetValue( FVector2D& OutValue ) const override;
	virtual FPropertyAccess::Result SetValue( const FVector2D& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override;

	virtual FPropertyAccess::Result GetValue( FVector4& OutValue ) const override;
	virtual FPropertyAccess::Result SetValue( const FVector4& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override;

	virtual FPropertyAccess::Result GetValue( FQuat& OutValue ) const override;
	virtual FPropertyAccess::Result SetValue( const FQuat& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override;

	virtual FPropertyAccess::Result SetX(double InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags);
	virtual FPropertyAccess::Result SetY(double InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags);
	virtual FPropertyAccess::Result SetZ(double InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags);
	virtual FPropertyAccess::Result SetW(double InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags);
private:
	TArray< TSharedPtr<FPropertyHandleMixed> > VectorComponents;	
};

class FPropertyHandleRotator : public FPropertyHandleStruct
{
public:
	FPropertyHandleRotator( TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities  );
	static bool Supports( TSharedRef<FPropertyNode> PropertyNode );
	virtual FPropertyAccess::Result GetValue( FRotator& OutValue ) const override;
	virtual FPropertyAccess::Result SetValue( const FRotator& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) override;
	virtual FPropertyAccess::Result GetValue(FQuat& OutValue) const override;
	virtual FPropertyAccess::Result SetValue(const FQuat& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) override;

	virtual FPropertyAccess::Result SetRoll( double InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags );
	virtual FPropertyAccess::Result SetPitch( double InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags );
	virtual FPropertyAccess::Result SetYaw( double InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags );
private:
	TSharedPtr<FPropertyHandleMixed> RollValue;
	TSharedPtr<FPropertyHandleMixed> PitchValue;
	TSharedPtr<FPropertyHandleMixed> YawValue;
};

class FPropertyHandleColor : public FPropertyHandleStruct
{
public:
	FPropertyHandleColor( TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities );
	static bool Supports( TSharedRef<FPropertyNode> PropertyNode );

	virtual FPropertyAccess::Result SetValueFromFormattedString(const FString& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) override;
};

class FPropertyHandleArray : public FPropertyHandleBase, public IPropertyHandleArray
{
public:
	FPropertyHandleArray( TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities );
	static bool Supports( TSharedRef<FPropertyNode> PropertyNode );
	/** IPropertyHandleArray interface */
	virtual FPropertyHandleItemAddResult AddItem() override;
	virtual FPropertyAccess::Result EmptyArray() override;
	virtual FPropertyAccess::Result Insert( int32 Index ) override;
	virtual FPropertyAccess::Result DuplicateItem( int32 Index ) override;
	virtual FPropertyAccess::Result DeleteItem( int32 Index ) override;
	virtual FPropertyAccess::Result SwapItems(int32 FirstIndex, int32 SecondIndex) override;
	virtual FPropertyAccess::Result GetNumElements( uint32& OutNumItems ) const override;
	virtual FDelegateHandle SetOnNumElementsChanged( const FSimpleDelegate& InOnNumElementsChanged ) override;
	virtual void UnregisterOnNumElementsChanged(FDelegateHandle Handle) override;
	virtual void SetOnCanItemChange(const FCanPropertyItemChange& InDelegate) override;
	virtual FDelegateHandle SetOnChangeBlocked(const FPropertyChangeBlocked::FDelegate& InDelegate) override;
	virtual void UnregisterOnChangeBlocked(FDelegateHandle Handle) override;
	virtual TSharedPtr<IPropertyHandleArray> AsArray() override;
	virtual TSharedRef<IPropertyHandle> GetElement( int32 Index ) const override;
	virtual FPropertyAccess::Result MoveElementTo(int32 OriginalIndex, int32 NewIndex) override;
	virtual FPropertyAccess::Result SetValueFromFormattedString(const FString& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) override;
	virtual bool IsEditable() const override;
};

class FPropertyHandleOptional : public FPropertyHandleBase, public IPropertyHandleOptional
{
public:
	FPropertyHandleOptional(TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities);
	static bool Supports(TSharedRef<FPropertyNode> PropertyNode);
	/** IPropertyHandleOptional interface */
	virtual FPropertyAccess::Result GetOptionalValue(FProperty*& OutValue) override;
	virtual FPropertyAccess::Result SetOptionalValue(FProperty* NewProperty, const UClass* NewObjectClass) override;
	virtual FPropertyAccess::Result ClearOptionalValue() override;
	virtual TSharedPtr<IPropertyHandleOptional> AsOptional() override;
	virtual bool IsEditable() const override;
};

class FPropertyHandleText : public FPropertyHandleBase
{
public:
	FPropertyHandleText(TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook,TSharedPtr<IPropertyUtilities> PropertyUtilities);
	static bool Supports(TSharedRef<FPropertyNode> PropertyNode);
	virtual FPropertyAccess::Result GetValue(FText& OutValue) const override;
	virtual FPropertyAccess::Result SetValue(const FText& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) override;

	virtual FPropertyAccess::Result SetValue(const FString& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) override;
	virtual FPropertyAccess::Result SetValue(const TCHAR* InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) override;
	virtual FPropertyAccess::Result SetValueFromFormattedString(const FString& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) override;

private:
	FPropertyAccess::Result ImportFormattedTextString(const FString& InValue, EPropertyValueSetFlags::Type Flags);
};

class FPropertyHandleSet : public FPropertyHandleBase, public IPropertyHandleSet
{
public:
	FPropertyHandleSet(TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities);
	static bool Supports(TSharedRef<FPropertyNode> PropertyNode);
	virtual TSharedPtr<IPropertyHandleSet> AsSet() override;

	/** IPropertyHandleSet Interface */
	virtual bool HasDefaultElement() override;
	virtual FPropertyHandleItemAddResult AddItem() override;
	virtual FPropertyAccess::Result Empty() override;
	virtual FPropertyAccess::Result DeleteItem(int32 Index) override;
	virtual FPropertyAccess::Result GetNumElements(uint32& OutNumElements) override;
	virtual TSharedRef<IPropertyHandle> GetElement(int32 Index) const override;
	virtual FPropertyAccess::Result SetValueFromFormattedString(const FString& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) override;

	virtual FDelegateHandle SetOnNumElementsChanged( const FSimpleDelegate& InOnNumElementsChanged ) override;
	virtual void UnregisterOnNumElementsChanged(FDelegateHandle Handle) override;
	virtual bool HasDocumentation() override { return true; }
	virtual FString GetDocumentationLink() override { return FString("Engine/UI/LevelEditor/Details/Properties/Set/"); }
	virtual FString GetDocumentationExcerptName() override { return FString("Sets"); }
	virtual bool IsEditable() const override;
};

class FPropertyHandleMap : public FPropertyHandleBase, public IPropertyHandleMap
{
public:
	FPropertyHandleMap(TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities);
	static bool Supports(TSharedRef<FPropertyNode> PropertyNode);
	virtual TSharedPtr<IPropertyHandleMap> AsMap() override;

	/** IPropertyHandleMap Interface */
	virtual bool HasDefaultKey() override;
	virtual FPropertyHandleItemAddResult AddItem() override;
	virtual FPropertyAccess::Result Empty() override;
	virtual FPropertyAccess::Result DeleteItem(int32 Index) override;
	virtual FPropertyAccess::Result GetNumElements(uint32& OutNumElements) override;
	virtual TSharedRef<IPropertyHandle> GetElement(int32 Index) const override;
	virtual FDelegateHandle SetOnNumElementsChanged( const FSimpleDelegate& InOnNumElementsChanged ) override;
	virtual void UnregisterOnNumElementsChanged(FDelegateHandle Handle) override;
	virtual bool HasDocumentation() override { return true; }
	virtual FString GetDocumentationLink() override { return FString("Engine/UI/LevelEditor/Details/Properties/Map/"); }
	virtual FString GetDocumentationExcerptName() override { return FString("Maps"); }
	virtual bool IsEditable() const override;

private:
	int32 AddItemWithKey(const FString& Value);
};

class FPropertyHandleFieldPath : public FPropertyHandleBase
{
public:
	FPropertyHandleFieldPath(TSharedRef<FPropertyNode> PropertyNode, FNotifyHook* NotifyHook, TSharedPtr<IPropertyUtilities> PropertyUtilities);
	static bool Supports(TSharedRef<FPropertyNode> PropertyNode);
	virtual FPropertyAccess::Result GetValue(FProperty*& OutValue) const override;
	virtual FPropertyAccess::Result SetValue(FProperty* const& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) override;
	virtual FPropertyAccess::Result GetValue(const FProperty*& OutValue) const override;
	virtual FPropertyAccess::Result SetValue(const FProperty* const& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) override;
};