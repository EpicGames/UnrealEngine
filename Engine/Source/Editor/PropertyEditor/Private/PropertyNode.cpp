// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyNode.h"
#include "Components/ActorComponent.h"
#include "Containers/Deque.h"
#include "EditConditionContext.h"
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorSupportDelegates.h"
#include "StructUtils/UserDefinedStruct.h"
#include "InstancedReferenceSubobjectHelper.h"
#include "ItemPropertyNode.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeExit.h"
#include "ObjectPropertyNode.h"
#include "PropertyHandleImpl.h"
#include "PropertyRestriction.h"
#include "PropertyTextUtilities.h"
#include "PropertyEditorEditConstPolicy.h"
#include "PropertyEditorArchetypePolicy.h"
#include "PropertyPermissionList.h"
#include "StringPrefixTree.h"
#include "StructurePropertyNode.h"

#include "UObject/MetaData.h"
#include "UObject/TextProperty.h"
#include "UObject/EnumProperty.h"
#include "UObject/PropertyBagRepository.h"
#include "UObject/UnrealType.h"

#include "UObject/PropertyOptional.h"
#include "UObject/UObjectArchetypeHelper.h"

#define LOCTEXT_NAMESPACE "PropertyNode"

namespace UE::PropertyEditor::Private
{
	static bool bShowInlineEditConditionToggleWhenNotSpecifiedAndNotEditable = true;
	static FAutoConsoleVariableRef CVarShowInlineEditConditionToggleWhenNotSpecifiedAndNotEditable(
		TEXT("PropertyEditor.ShowInlineEditConditionToggleWhenNotSpecifiedAndNotEditable"),
		bShowInlineEditConditionToggleWhenNotSpecifiedAndNotEditable,
		TEXT("Enables legacy behavior to show the InlineEditConditionToggle when the edit condition property does not have this specifier and is not editable.")
	);
}

namespace PropertyEditorPolicy
{
	struct FPropertyNodePolicyImpl : public FObjectArchetypeHelper::IObjectArchetypePolicy
	{
		FPropertyNodePolicyImpl() {}
		virtual ~FPropertyNodePolicyImpl() {}
		
		virtual UObject* GetArchetype(const UObject* Object) const override
		{
			for (const IArchetypePolicy* ArchetypePolicy : ArchetypePolicies)
			{
				if (UObject* Archetype = ArchetypePolicy->GetArchetypeForObject(Object))
				{
					return Archetype;
				}
			}

			return nullptr;
		}

		bool CanEditProperty(const FEditPropertyChain& PropertyChain, const UObject* Object) const
		{
			for (const IEditConstPolicy* EditConstPolicy : EditConstPolicies)
			{
				if (!EditConstPolicy->CanEditProperty(PropertyChain, Object))
				{
					return false;
				}
			}
			
			return true;
		}

		bool CanEditProperty(const FProperty* Property, const UObject* Object) const
		{
			for (const IEditConstPolicy* EditConstPolicy : EditConstPolicies)
			{
				if (!EditConstPolicy->CanEditProperty(Property, Object))
				{
					return false;
				}
			}

			return true;
		}

		void RegisterArchetypePolicy(IArchetypePolicy* ArchetypePolicy)
		{
			check(!ArchetypePolicies.Contains(ArchetypePolicy));
			ArchetypePolicies.Add(ArchetypePolicy);
			PolicyEpoch++;
		}

		void UnregisterArchetypePolicy(IArchetypePolicy* ArchetypePolicy)
		{
			check(ArchetypePolicies.Contains(ArchetypePolicy));
			ArchetypePolicies.Remove(ArchetypePolicy);
			PolicyEpoch++;
		}

		void RegisterEditConstPolicy(IEditConstPolicy* EditConstPolicy)
		{
			check(!EditConstPolicies.Contains(EditConstPolicy));
			EditConstPolicies.Add(EditConstPolicy);
			PolicyEpoch++;
		}

		void UnregisterEditConstPolicy(IEditConstPolicy* EditConstPolicy)
		{
			check(EditConstPolicies.Contains(EditConstPolicy));
			EditConstPolicies.Remove(EditConstPolicy);
			PolicyEpoch++;
		}

		int32 GetPolicyEpoch() const
		{
			return PolicyEpoch;
		}
				
		TArray<IArchetypePolicy*> ArchetypePolicies;
		TArray<IEditConstPolicy*> EditConstPolicies;
		int32 PolicyEpoch = 0;
	};

	FPropertyNodePolicyImpl& Get()
	{
		static FPropertyNodePolicyImpl Policy;
		return Policy;
	}
}
void FPropertyNode::RegisterArchetypePolicy(PropertyEditorPolicy::IArchetypePolicy* ArchetypePolicy)
{
	PropertyEditorPolicy::Get().RegisterArchetypePolicy(ArchetypePolicy);
}

void FPropertyNode::UnregisterArchetypePolicy(PropertyEditorPolicy::IArchetypePolicy* ArchetypePolicy)
{
	PropertyEditorPolicy::Get().UnregisterArchetypePolicy(ArchetypePolicy);
}

void FPropertyNode::RegisterEditConstPolicy(PropertyEditorPolicy::IEditConstPolicy* EditConstPolicy)
{
	PropertyEditorPolicy::Get().RegisterEditConstPolicy(EditConstPolicy);
}

void FPropertyNode::UnregisterEditConstPolicy(PropertyEditorPolicy::IEditConstPolicy* EditConstPolicy)
{
	PropertyEditorPolicy::Get().UnregisterEditConstPolicy(EditConstPolicy);
}

UObject* FPropertyNode::GetArchetype(const UObject* Object)
{
	return Object ? FObjectArchetypeHelper::GetArchetype(Object, &PropertyEditorPolicy::Get()) : nullptr;
}

bool FPropertyNode::IsPropertyEditConst(const FEditPropertyChain& PropertyChain, UObject* Object)
{
	return !PropertyEditorPolicy::Get().CanEditProperty(PropertyChain, Object);
}

bool FPropertyNode::IsPropertyEditConst(const FProperty* Property, UObject* Object)
{
	return !PropertyEditorPolicy::Get().CanEditProperty(Property, Object);
}

FEditConditionParser FPropertyNode::EditConditionParser;

FPropertySettings& FPropertySettings::Get()
{
	static FPropertySettings Settings;

	return Settings;
}

FPropertySettings::FPropertySettings()
	: bShowFriendlyPropertyNames( true )
	, bExpandDistributions( false )
	, bShowHiddenProperties(false)
{
	GConfig->GetBool(TEXT("PropertySettings"), TEXT("ShowHiddenProperties"), bShowHiddenProperties, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("PropertySettings"), TEXT("ShowFriendlyPropertyNames"), bShowFriendlyPropertyNames, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("PropertySettings"), TEXT("ExpandDistributions"), bExpandDistributions, GEditorPerProjectIni);
}

DEFINE_LOG_CATEGORY(LogPropertyNode);

static FObjectPropertyNode* NotifyFindObjectItemParent(FPropertyNode* InNode)
{
	FObjectPropertyNode* Result = NULL;
	check(InNode);
	FPropertyNode* ParentNode = InNode->GetParentNode(); 
	if (ParentNode)
	{
		Result = ParentNode->FindObjectItemParent();
	}
	return Result;
}

FPropertyNode::FPropertyNode()
	: Property(nullptr)
	, ArrayOffset(0)
	, ArrayIndex(-1)
	, MaxChildDepthAllowed(FPropertyNodeConstants::NoDepthRestrictions)
	, PropertyNodeFlags (EPropertyNodeFlags::NoFlags)
	, bRebuildChildrenRequested( false )
	, bChildrenRebuilt(false)
	, bIgnoreInstancedReference(false)
	, PropertyPath(TEXT(""))
	, bIsEditConst(false)
	, bUpdateEditConstState(true)
	, UpdateEditConstStateEpoch(0)
	, bDiffersFromDefault(false)
	, bUpdateDiffersFromDefault(true)
	, UpdateDiffersFromDefaultEpoch(0)
{
}

FPropertyNode::~FPropertyNode()
{
	DestroyTree();
}

void FPropertyNode::InitNode(const FPropertyNodeInitParams& InitParams)
{
	//Dismantle the previous tree
	DestroyTree(/*bInDestroySelf*/false);

	//tree hierarchy
	check(InitParams.ParentNode.Get() != this);
	ParentNodeWeakPtr = InitParams.ParentNode;
	
	//Property Data
	Property = InitParams.Property;
	ArrayOffset = InitParams.ArrayOffset;
	ArrayIndex = InitParams.ArrayIndex;

	bool bIsSparse = InitParams.IsSparseProperty == FPropertyNodeInitParams::EIsSparseDataProperty::True;

	TSharedPtr<FPropertyNode> ParentNode = ParentNodeWeakPtr.Pin();
	if (ParentNode.IsValid() && InitParams.IsSparseProperty == FPropertyNodeInitParams::EIsSparseDataProperty::Inherit)
	{
		//default to parents max child depth
		MaxChildDepthAllowed = ParentNode->MaxChildDepthAllowed;
		//if limitless or has hit the full limit
		if (MaxChildDepthAllowed > 0)
		{
			--MaxChildDepthAllowed;
		}

		// if the parent node's property is sparse data, our property must be too
		bIsSparse = bIsSparse || ParentNode->HasNodeFlags(EPropertyNodeFlags::IsSparseClassData);
	}

	// Property is advanced if it is marked advanced or the entire class is advanced and the property not marked as simple
	static const FName Name_AdvancedClassDisplay("AdvancedClassDisplay");
	bool bAdvanced = Property.IsValid() ? ( Property->HasAnyPropertyFlags(CPF_AdvancedDisplay) || ( !Property->HasAnyPropertyFlags( CPF_SimpleDisplay ) && Property->GetOwnerClass() && Property->GetOwnerClass()->GetBoolMetaData(Name_AdvancedClassDisplay) ) ) : false;

	PropertyNodeFlags = EPropertyNodeFlags::NoFlags;
	SetNodeFlags(EPropertyNodeFlags::IsSparseClassData, bIsSparse);

	static const FName Name_ShouldShowInViewport("ShouldShowInViewport");
	bool bShouldShowInViewport = Property.IsValid() ? Property->GetBoolMetaData(Name_ShouldShowInViewport) : false;
	SetNodeFlags(EPropertyNodeFlags::ShouldShowInViewport, bShouldShowInViewport);

	//default to copying from the parent
	if (ParentNode)
	{
		if (ParentNode->HasNodeFlags(EPropertyNodeFlags::ShowCategories))
		{
			SetNodeFlags(EPropertyNodeFlags::ShowCategories, true);
		}
		else
		{
			SetNodeFlags(EPropertyNodeFlags::ShowCategories, false);
		}

		// We are advanced if our parent is advanced or our property is marked as advanced
		SetNodeFlags(EPropertyNodeFlags::IsAdvanced, ParentNode->HasNodeFlags(EPropertyNodeFlags::IsAdvanced) || bAdvanced );

		SetNodeFlags(EPropertyNodeFlags::ResolveInstanceDataObjects, ParentNode->HasNodeFlags(EPropertyNodeFlags::ResolveInstanceDataObjects));
	}
	else
	{
		SetNodeFlags(EPropertyNodeFlags::ShowCategories, InitParams.bCreateCategoryNodes );
		SetNodeFlags(EPropertyNodeFlags::ResolveInstanceDataObjects, InitParams.bResolveInstanceDataObjects );
	}

	SetNodeFlags(EPropertyNodeFlags::ShouldShowHiddenProperties, InitParams.bForceHiddenPropertyVisibility);
	SetNodeFlags(EPropertyNodeFlags::ShouldShowDisableEditOnInstance, InitParams.bCreateDisableEditOnInstanceNodes);

	//Custom code run prior to setting property flags
	//needs to happen after the above SetNodeFlags calls so that ObjectPropertyNode can properly respond to CollapseCategories
	InitBeforeNodeFlags();

	bool bIsEditInlineNew = false;
	bool bShowInnerObjectProperties = false;
	const FProperty* MyProperty = Property.Get();
	if (MyProperty == nullptr)
	{
		// Disable all flags if no property is bound.
		SetNodeFlags(EPropertyNodeFlags::SingleSelectOnly | EPropertyNodeFlags::EditInlineNew | EPropertyNodeFlags::ShowInnerObjectProperties, false);
	}
	else
	{
		const bool GotReadAddresses = GetReadAddressUncached( *this, false, nullptr, false );
		const bool bSingleSelectOnly = GetReadAddressUncached( *this, true, nullptr);
		SetNodeFlags(EPropertyNodeFlags::SingleSelectOnly, bSingleSelectOnly);

		const FProperty* OwnerProperty = MyProperty->GetOwnerProperty();

		const bool bIsObjectOrInterface = CastField<FObjectPropertyBase>(MyProperty) || CastField<FInterfaceProperty>(MyProperty);
		bool bIsInsideContainer = CastField<FArrayProperty>(OwnerProperty) || CastField<FSetProperty>(OwnerProperty) || CastField<FMapProperty>(OwnerProperty) || CastField<FOptionalProperty>(OwnerProperty);

		// Don't consider the container's inline status if the key is a class property that is not inline
		if (const FMapProperty* MapProperty = CastField<FMapProperty>(OwnerProperty))
		{
			const FObjectPropertyBase* KeyObjectProperty = CastField<FObjectPropertyBase>(MapProperty->GetKeyProperty());

			if (KeyObjectProperty && KeyObjectProperty->PropertyClass && !KeyObjectProperty->PropertyClass->HasAnyClassFlags(EClassFlags::CLASS_EditInlineNew))
			{
				bIsInsideContainer = false;
			}
		}

		// true if the property can be expanded into the property window; that is, instead of seeing
		// a pointer to the object, you see the object's properties.
		static const FName Name_EditInline("EditInline");
		static const FName Name_ShowInnerProperties("ShowInnerProperties");
		static const FName Name_NoEditInline("NoEditInline");

		// we are EditInlineNew if this property has the flag, or if inside a container that has the flag.
		bIsEditInlineNew = GotReadAddresses && bIsObjectOrInterface && !MyProperty->HasMetaData(Name_NoEditInline) && 
			(MyProperty->HasMetaData(Name_EditInline) || (bIsInsideContainer && OwnerProperty->HasMetaData(Name_EditInline)));
		bShowInnerObjectProperties = bIsObjectOrInterface && MyProperty->HasMetaData(Name_ShowInnerProperties);

		if (bIsEditInlineNew)
		{
			SetNodeFlags(EPropertyNodeFlags::EditInlineNew, true);
		}
		else if (bShowInnerObjectProperties)
		{
			SetNodeFlags(EPropertyNodeFlags::ShowInnerObjectProperties, true);
		}

		//Get the property max child depth
		static const FName Name_MaxPropertyDepth("MaxPropertyDepth");
		if (Property->HasMetaData(Name_MaxPropertyDepth))
		{
			int32 NewMaxChildDepthAllowed = Property->GetIntMetaData(Name_MaxPropertyDepth);
			//Ensure new depth is valid.  Otherwise just let the parent specified value stand
			if (NewMaxChildDepthAllowed > 0)
			{
				//if there is already a limit on the depth allowed, take the minimum of the allowable depths
				if (MaxChildDepthAllowed >= 0)
				{
					MaxChildDepthAllowed = FMath::Min(MaxChildDepthAllowed, NewMaxChildDepthAllowed);
				}
				else
				{
					//no current limit, go ahead and take the new limit
					MaxChildDepthAllowed = NewMaxChildDepthAllowed;
				}
			}
		}

		const FString& EditConditionString = MyProperty->GetMetaData(TEXT("EditCondition"));

		// see if the property supports some kind of edit condition and this isn't the "parent" property of a static array
		const bool bIsStaticArrayParent = MyProperty->ArrayDim > 1 && GetArrayIndex() != -1;
		if (!EditConditionString.IsEmpty() && !bIsStaticArrayParent)
		{
			EditConditionExpression = EditConditionParser.Parse(EditConditionString);
			if (EditConditionExpression.IsValid())
			{
				EditConditionContext = MakeShareable(new FEditConditionContext(*this));
			}
		}
		
		bool bRequiresValidation = bIsEditInlineNew || bShowInnerObjectProperties;
	
		// We require validation if we are in a container.
		bRequiresValidation |= MyProperty->IsA<FArrayProperty>() || MyProperty->IsA<FSetProperty>() || MyProperty->IsA<FMapProperty>() || MyProperty->IsA<FOptionalProperty>();

		// We require validation if our parent also needs validation (if an array parent was resized all the addresses of children are invalid)
		bRequiresValidation |= (GetParentNode() && GetParentNode()->HasNodeFlags(EPropertyNodeFlags::RequiresValidation));

		// We require validation is we are on a structure node (the value of the structure may change externally, which invalidates the addresses).
		const FComplexPropertyNode* ComplexParent = GetParentNode() ? GetParentNode()->AsComplexNode() : nullptr;
		bRequiresValidation |= ComplexParent && ComplexParent->GetPropertyType() == FComplexPropertyNode::EPT_StandaloneStructure;

		bRequiresValidation |= HasNodeFlags(EPropertyNodeFlags::SupportsDynamicInstancing);
		
		SetNodeFlags( EPropertyNodeFlags::RequiresValidation, bRequiresValidation );

		UEnum* Enum = nullptr;
		if (const FByteProperty* ByteProperty = CastField<FByteProperty>(MyProperty))
		{
			if (ByteProperty->IsEnum())
			{
				Enum = ByteProperty->Enum;
			}
		}
		else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(MyProperty))
		{
			Enum = EnumProperty->GetEnum();
		}

		if (Enum && FEnumValuePermissionList::Get().HasFiltering(Enum))
		{
			TSharedRef<FPropertyRestriction> Restriction = MakeShared<FPropertyRestriction>(FText::FromString(TEXT("Not available")));

			for (int32 Index = 0; Index < Enum->NumEnums(); ++Index)
			{
				const FString NameString = Enum->GetNameStringByIndex(Index);

				if (!FEnumValuePermissionList::Get().DoesEnumValuePassFilter(Enum, FName(NameString)))
				{
					Restriction->AddHiddenValue(NameString);
					Restriction->AddDisabledValue(NameString);
				}
			}

			AddRestriction(MoveTemp(Restriction));
		}
	}

	InitExpansionFlags();

	if (InitParams.bAllowChildren)
	{
		RebuildChildren();
	}

	PropertyPath = FPropertyNode::CreatePropertyPath(this->AsShared())->ToString();
}

namespace FPropertyNodeUtils
{
	void GetExpandedItems(const TSharedPtr<FPropertyNode>& InPropertyNode, FStringPrefixTree& OutExpandedItems)
	{
		if (InPropertyNode->HasNodeFlags(EPropertyNodeFlags::Expanded))
		{
			constexpr bool bWithArrayIndex = true;
			FString Path;
			Path.Empty(128);
			InPropertyNode->GetQualifiedName(Path, bWithArrayIndex);

			OutExpandedItems.Insert(Path);
		}

		for (int32 ChildIndex = 0; ChildIndex < InPropertyNode->GetNumChildNodes(); ++ChildIndex)
		{
			GetExpandedItems(InPropertyNode->GetChildNode(ChildIndex), OutExpandedItems);
		}
	}

	void SetExpandedItems(const TSharedPtr<FPropertyNode>& InPropertyNode, const FStringPrefixTree& InExpandedItems)
	{
		constexpr bool bWithArrayIndex = true;
		FString Path;
		Path.Empty(128);
		InPropertyNode->GetQualifiedName(Path, bWithArrayIndex);

		if (InExpandedItems.Contains(Path))
		{
			InPropertyNode->SetNodeFlags(EPropertyNodeFlags::Expanded, true);
		}

		if (InExpandedItems.AnyStartsWith(Path))
		{
			for (int32 NodeIndex = 0; NodeIndex < InPropertyNode->GetNumChildNodes(); ++NodeIndex)
			{
				SetExpandedItems(InPropertyNode->GetChildNode(NodeIndex), InExpandedItems);
			}
		}
	}
}

/**
 * Used for rebuilding a sub portion of the tree
 */
void FPropertyNode::RebuildChildren()
{
	CachedReadAddresses.Reset();
	
	FStringPrefixTree ExpandedPropertyItemSet;
	const TSharedRef<FPropertyNode> ThisAsSharedRef = AsShared();
	FPropertyNodeUtils::GetExpandedItems(ThisAsSharedRef, ExpandedPropertyItemSet);

	constexpr bool bDestroySelf = false;
	DestroyTree(bDestroySelf);

	if (MaxChildDepthAllowed != 0)
	{
		//the case where we don't want init child nodes is when an Item has children that we don't want to display
		//the other option would be to make each node "Read only" under that item.
		//The example is a material assigned to a static mesh.
		if (HasNodeFlags(EPropertyNodeFlags::CanBeExpanded) && (ChildNodes.Num() == 0))
		{
			InitChildNodes();
			if (ExpandedPropertyItemSet.Size() > 0)
			{
				FPropertyNodeUtils::SetExpandedItems(ThisAsSharedRef, ExpandedPropertyItemSet);
			}
		}
	}

	// Children have been rebuilt, clear any pending rebuild requests
	bRebuildChildrenRequested = false;
	bChildrenRebuilt = true;

	// Notify any listener that children have been rebuilt
	OnRebuildChildrenEvent.Broadcast();
}

void FPropertyNode::AddChildNode(TSharedPtr<FPropertyNode> InNode)
{
	ChildNodes.Add(InNode); 
}

void FPropertyNode::RemoveAllChildNodes()
{
	DestroyTree(false);
}

void FPropertyNode::ClearCachedReadAddresses( bool bRecursive )
{
	CachedReadAddresses.Reset();

	if( bRecursive )
	{
		for( int32 ChildIndex = 0; ChildIndex < ChildNodes.Num(); ++ChildIndex )
		{
			ChildNodes[ChildIndex]->ClearCachedReadAddresses( bRecursive );
		}
	}
}

// Follows the chain of items upwards until it finds the object window that houses this item.
FComplexPropertyNode* FPropertyNode::FindComplexParent()
{
	FPropertyNode* Cur = this;
	FComplexPropertyNode* Found = NULL;

	while( true )
	{
		Found = Cur->AsComplexNode();
		if( Found )
		{
			break;
		}
		Cur = Cur->GetParentNode();
		if( !Cur )
		{
			// There is a break in the parent chain
			break;
		}
	}

	return Found;
}


// Follows the chain of items upwards until it finds the object window that houses this item.
const FComplexPropertyNode* FPropertyNode::FindComplexParent() const
{
	const FPropertyNode* Cur = this;
	const FComplexPropertyNode* Found = NULL;

	while( true )
	{
		Found = Cur->AsComplexNode();
		if( Found )
		{
			break;
		}
		Cur = Cur->GetParentNode();
		if( !Cur )
		{
			// There is a break in the parent chain
			break;
		}

	}

	return Found;
}

class FObjectPropertyNode* FPropertyNode::FindObjectItemParent()
{
	FComplexPropertyNode* ComplexParent = FindComplexParent();
	if (!ComplexParent)
	{
		return nullptr;
	}

	if (FObjectPropertyNode* ObjectNode = ComplexParent->AsObjectNode())
	{
		return ObjectNode;
	}
	else if (FPropertyNode* ParentNodePtr = ComplexParent->GetParentNode())
	{
		return ParentNodePtr->FindObjectItemParent();
	}
	return nullptr;
}

const class FObjectPropertyNode* FPropertyNode::FindObjectItemParent() const
{
	const FComplexPropertyNode* ComplexParent = FindComplexParent();
	if (!ComplexParent)
	{
		return nullptr;
	}

	if (const FObjectPropertyNode* ObjectNode = ComplexParent->AsObjectNode())
	{
		return ObjectNode;
	}
	else if (const FPropertyNode* ParentNodePtr = ComplexParent->GetParentNode())
	{
		return ParentNodePtr->FindObjectItemParent();
	}
	return nullptr;
}

FStructurePropertyNode* FPropertyNode::FindStructureItemParent()
{
	FComplexPropertyNode* ComplexParent = FindComplexParent();
	if (!ComplexParent)
	{
		return nullptr;
	}

	if (FStructurePropertyNode* StructureNode = ComplexParent->AsStructureNode())
	{
		return StructureNode;
	}

	return nullptr;
}

const FStructurePropertyNode* FPropertyNode::FindStructureItemParent() const
{
	const FComplexPropertyNode* ComplexParent = FindComplexParent();
	if (!ComplexParent)
	{
		return nullptr;
	}

	if (const FStructurePropertyNode* StructureNode = ComplexParent->AsStructureNode())
	{
		return StructureNode;
	}

	return nullptr;
}

/**
 * Follows the top-most object window that contains this property window item.
 */
FObjectPropertyNode* FPropertyNode::FindRootObjectItemParent()
{
	// not every type of change to property values triggers a proper refresh of the hierarchy, so find the topmost container window and trigger a refresh manually.
	FObjectPropertyNode* TopmostObjectItem=NULL;

	FObjectPropertyNode* NextObjectItem = FindObjectItemParent();
	while ( NextObjectItem != NULL )
	{
		TopmostObjectItem = NextObjectItem;
		FPropertyNode* NextObjectParent = NextObjectItem->GetParentNode();
		if ( NextObjectParent != NULL )
		{
			NextObjectItem = NextObjectParent->FindObjectItemParent();
		}
		else
		{
			break;
		}
	}

	return TopmostObjectItem;
}

bool FPropertyNode::DoesChildPropertyRequireValidation(FProperty* InChildProp)
{
	return InChildProp != nullptr && (CastField<FObjectProperty>(InChildProp) || CastField<FStructProperty>(InChildProp) || CastField<FArrayProperty>(InChildProp) 
			|| CastField<FSetProperty>(InChildProp) || CastField<FMapProperty>(InChildProp) || CastField<FOptionalProperty>(InChildProp));
}

void FPropertyNode::MarkChildrenAsRebuilt()
{
	bChildrenRebuilt = false;

	for (const TSharedPtr<FPropertyNode>& ChildNode : ChildNodes)
	{
		ChildNode->MarkChildrenAsRebuilt();
	}
}

/** 
 * Used to see if any data has been destroyed from under the property tree.
 *
 * Note: The details panel is completely reliant on this to return a result which refreshes the UI for array/set/map actions...
 *	they should be refreshing their UI when performing their actions (add, clear, etc) but debugging has shown this function
 *	returning to `SDetailsViewBase::Tick` and setting `bUpdateFilteredDetails` to true is what actually causes the UI to refresh.
 */
EPropertyDataValidationResult FPropertyNode::EnsureDataIsValid()
{
	using FPropertyNodeInlineArray = TArray<TSharedPtr<const FPropertyNode>, TInlineAllocator<2>>;
	auto GetChildStructureNodes = [Self = this](FPropertyNodeInlineArray& OutChildStructurePropertyNodes)
	{
		for (const TSharedPtr<FPropertyNode>& ChildNode : Self->ChildNodes)
		{
			if (const FComplexPropertyNode* ComplexNode = ChildNode->AsComplexNode())
			{
				if (ComplexNode->AsStructureNode())
				{
					OutChildStructurePropertyNodes.Emplace(ChildNode);
				}
			}
		}
	};

	bool bValidateChildren = !HasNodeFlags(EPropertyNodeFlags::SkipChildValidation);
	bool bValidateChildrenKeyNodes = false;		// by default, we don't check this, since it's just for Map properties

	// If we have rebuilt children since last EnsureDataIsValid call let the caller know
	if (bChildrenRebuilt)
	{
		MarkChildrenAsRebuilt();
		return EPropertyDataValidationResult::ChildrenRebuilt;
	}

	// The root must always be validated
	if (GetParentNode() == nullptr || HasNodeFlags(EPropertyNodeFlags::RequiresValidation))
	{
		CachedReadAddresses.Reset();

		// Figure out if an array mismatch can be ignored
		bool bIgnoreAllMismatch = false;
		// Make sure that force depth-limited trees don't cause a refresh
		bIgnoreAllMismatch |= (MaxChildDepthAllowed==0);

		// Check my property
		if (Property.IsValid())
		{
			FProperty* MyProperty = Property.Get();
			UStruct* OwnerStruct = MyProperty->GetOwnerStruct();

			if (!OwnerStruct || OwnerStruct->IsStructTrashed())
			{
				// Verify that the property is not part of an invalid trash class, treat it as an invalid object if it is which will cause a refresh
				return EPropertyDataValidationResult::ObjectInvalid;
			}

			// Cached Child Structure nodes that are manually attached to this property node
			FPropertyNodeInlineArray CachedChildStructureNodes;
			GetChildStructureNodes(CachedChildStructureNodes);

			// Verify that the number of container children is correct
			FArrayProperty* ArrayProperty = CastField<FArrayProperty>(MyProperty);
			FSetProperty* SetProperty = CastField<FSetProperty>(MyProperty);
			FMapProperty* MapProperty = CastField<FMapProperty>(MyProperty);
			FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(MyProperty);

			// Default to unknown array length
			int32 NumContainerChildren = -1;
			// Assume all arrays have the same length
			bool bContainersHaveEqualNum = true;
			// Assume all arrays match the number of property window children
			bool bContainersMatchChildNum = true;

			bool bContainerHasNewItem = false;

			FProperty* ContainerElementProperty = MyProperty;

			if (ArrayProperty)
			{
				ContainerElementProperty = ArrayProperty->Inner;
			}
			else if (SetProperty)
			{
				ContainerElementProperty = SetProperty->ElementProp;
			}
			else if (MapProperty)
			{
				// Need to attempt to validate both the key and value properties...
				bValidateChildrenKeyNodes = DoesChildPropertyRequireValidation(MapProperty->KeyProp);

				ContainerElementProperty = MapProperty->ValueProp;
			}
			else if (OptionalProperty)
			{
				ContainerElementProperty = OptionalProperty->GetValueProperty();
			}

			bValidateChildren = DoesChildPropertyRequireValidation(ContainerElementProperty);
			for (TSharedPtr<const FPropertyNode>& ChildStructureNode : CachedChildStructureNodes)
			{
				bValidateChildren |= DoesChildPropertyRequireValidation(ChildStructureNode->Property.Get());
			}

			// Verify that the number of object children are the same too
			FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(MyProperty);
			// Check to see, if this an object property, whether the contents are NULL or not.
			// This is the check to see if an object property was changed from NULL to non-NULL, or vice versa, from non-property window code.
			bool bObjectPropertyNull = true;

			// Edit inline properties can change underneath the window
			bool bIgnoreChangingChildren = !(
				HasNodeFlags(EPropertyNodeFlags::EditInlineNew) ||
				HasNodeFlags(EPropertyNodeFlags::ShowInnerObjectProperties) ||
			 	HasNodeFlags(EPropertyNodeFlags::DynamicInstance)
			);

			// Ignore this node if the consistency check should happen for theW children
			bool bIgnoreStaticArray = (Property->ArrayDim > 1) && (ArrayIndex == -1);

			// If this node can't possibly have children (or causes a circular reference loop) then ignore this as a object property
			if (bIgnoreChangingChildren || bIgnoreStaticArray || HasNodeFlags(EPropertyNodeFlags::NoChildrenDueToCircularReference))
			{
				// This will bypass object property consistency checks
				ObjectProperty = nullptr;
			}

			FReadAddressList ReadAddresses;
			const bool bSuccess = GetReadAddress( ReadAddresses );
			// Make sure we got the addresses correctly
			if (!bSuccess)
			{
				UE_LOGF( LogPropertyNode, Verbose, "Object is invalid %ls", *Property->GetName() );
				return EPropertyDataValidationResult::ObjectInvalid;
			}

			// If an object property with ShowInnerProperties changed object values out from under the property
			bool bShowInnerObjectPropertiesObjectChanged = false;

			// Check for null, if we find one, there is a problem.
			for (int32 Scan = 0; Scan < ReadAddresses.Num(); ++Scan)
			{
				uint8* Addr = ReadAddresses.GetAddress(Scan);
				//make sure the data still exists
				if (Addr==NULL)
				{
					UE_LOGF( LogPropertyNode, Verbose, "Object is invalid %ls", *Property->GetName() );
					return EPropertyDataValidationResult::ObjectInvalid;
				}

				if( ArrayProperty && !bIgnoreAllMismatch)
				{
					// Ensure that array structures have the proper number of children
					FScriptArrayHelper ArrayHelper(ArrayProperty, Addr);
					int32 ArrayNum = ArrayHelper.Num();
					// If first child
					if (NumContainerChildren == -1)
					{
						NumContainerChildren = ArrayNum;
					}
					bContainerHasNewItem = GetNumChildNodes() < ArrayNum;
					// Make sure multiple arrays match
					bContainersHaveEqualNum = bContainersHaveEqualNum && (NumContainerChildren == ArrayNum);
					// Make sure the array matches the number of property node children
					bContainersMatchChildNum = bContainersMatchChildNum && (GetNumChildNodes() == ArrayNum);
				}

				if (SetProperty && !bIgnoreAllMismatch)
				{
					// Like arrays, ensure that set structures have the proper number of children
					int32 SetNum = FScriptSetHelper::Num(Addr);

					if (NumContainerChildren == -1)
					{
						NumContainerChildren = SetNum;
					}

					bContainerHasNewItem = GetNumChildNodes() < SetNum;
					bContainersHaveEqualNum = bContainersHaveEqualNum && (NumContainerChildren == SetNum);
					bContainersMatchChildNum = bContainersMatchChildNum && (GetNumChildNodes() == SetNum);
				}

				if (MapProperty && !bIgnoreAllMismatch)
				{
					FScriptMapHelper MapHelper(MapProperty, Addr);
					int32 MapNum = MapHelper.Num();

					if (NumContainerChildren == -1)
					{
						NumContainerChildren = MapNum;
					}

					bContainerHasNewItem = GetNumChildNodes() < MapNum;
					bContainersHaveEqualNum = bContainersHaveEqualNum && (NumContainerChildren == MapNum);
					bContainersMatchChildNum = bContainersMatchChildNum && (GetNumChildNodes() == MapNum);
				}

				if (ObjectProperty && !bIgnoreAllMismatch)
				{
					UObject* Obj = ObjectProperty->GetObjectPropertyValue(Addr);
					if (HasNodeFlags(EPropertyNodeFlags::ResolveInstanceDataObjects))
					{
						UE::FPropertyBagRepository& Repository = UE::FPropertyBagRepository::Get();
						if (UObject* Found = Repository.FindInstanceDataObject(Obj))
						{
							Obj = Found;
						}
					}
					if (IsValid(Obj))
					{
						if (!bShowInnerObjectPropertiesObjectChanged &&
							HasNodeFlags(EPropertyNodeFlags::DynamicInstance|EPropertyNodeFlags::ShowInnerObjectProperties|EPropertyNodeFlags::EditInlineNew) && 
							ChildNodes.Num() == 1)
						{
							bool bChildObjectFound = false;
							FObjectPropertyNode* ChildObjectNode = ChildNodes[0]->AsObjectNode();
							for (int32 ObjectIndex = 0; ObjectIndex < ChildObjectNode->GetNumObjects(); ++ObjectIndex)
							{
								if (Obj == ChildObjectNode->GetUObject(ObjectIndex))
								{
									bChildObjectFound = true;
									break;
								}
							}
							bShowInnerObjectPropertiesObjectChanged = !bChildObjectFound;
						}
					}

					if (Obj != nullptr)
					{
						bObjectPropertyNull = false;
						break;
					}
				}

				if (OptionalProperty && !bIgnoreAllMismatch)
				{
					uint32 bIsSet = OptionalProperty->IsSet(Addr);
					// re-use `bArrayHasNewItem` as its logic is what we end up wanting exactly
					bContainerHasNewItem = bIsSet != GetNumChildNodes();
				}
			}

			// If all arrays match each other but they do NOT match the property structure, cause a rebuild
			// Don't trigger a rebuild if we have any structure child nodes, because we can't retrieve that information from the actual memory
			// todo: Find a way to support a mix of container elements and manually added structure as child nodes
			if (CachedChildStructureNodes.IsEmpty() && bContainersHaveEqualNum && !bContainersMatchChildNum)
			{
				RebuildChildren();

				if( bContainerHasNewItem && ChildNodes.Num() )
				{
					TSharedPtr<FPropertyNode> LastChildNode = ChildNodes.Last();
					// Don't expand huge children
					if( LastChildNode->GetNumChildNodes() > 0 && LastChildNode->GetNumChildNodes() < 10 )
					{
						// Expand the last item for convenience since generally the user will want to edit the new value they added.
						LastChildNode->SetNodeFlags(EPropertyNodeFlags::Expanded, true);
					}
				}

				return EPropertyDataValidationResult::ArraySizeChanged;
			}

			// Don't trigger a rebuild if we have any structure child nodes, because we can't retrieve that information from the actual memory
			if (CachedChildStructureNodes.IsEmpty() && bShowInnerObjectPropertiesObjectChanged)
			{
				RebuildChildren();
				return EPropertyDataValidationResult::EditInlineNewValueChanged;
			}

			const bool bHasChildren = (GetNumChildNodes() > 0);
			// If the object property is not null and has no children, its children need to be rebuilt
			// If the object property is null and this node has children, the node needs to be rebuilt
			if (!HasNodeFlags(EPropertyNodeFlags::ShowInnerObjectProperties) && 
				ObjectProperty != nullptr && 
				((!bObjectPropertyNull && !bHasChildren) || (bObjectPropertyNull && bHasChildren)))
			{
				RebuildChildren();
				return EPropertyDataValidationResult::PropertiesChanged;
			}
		}
	}

	if( bRebuildChildrenRequested )
	{
		RebuildChildren();
		// If this property is editinline and not edit const then its editinline new and we can optimize some of the refreshing in some cases.  Otherwise we need to refresh all properties in the view
		return HasNodeFlags(EPropertyNodeFlags::ShowInnerObjectProperties) || (HasNodeFlags(EPropertyNodeFlags::EditInlineNew) && !IsEditConst()) ? EPropertyDataValidationResult::EditInlineNewValueChanged : EPropertyDataValidationResult::PropertiesChanged;
	}
	
	EPropertyDataValidationResult FinalResult = EPropertyDataValidationResult::DataValid;

	// Validate children and/or their key nodes.
	if (bValidateChildren || bValidateChildrenKeyNodes)
	{
		for (int32 Scan = 0; Scan < ChildNodes.Num(); ++Scan)
		{
			TSharedPtr<FPropertyNode>& ChildNode = ChildNodes[Scan];
			check(ChildNode.IsValid());

			if (bValidateChildren)
			{
				EPropertyDataValidationResult ChildDataResult = ChildNode->EnsureDataIsValid();
				if (FinalResult == EPropertyDataValidationResult::DataValid && ChildDataResult != EPropertyDataValidationResult::DataValid)
				{
					FinalResult = ChildDataResult;
				}
			}

			// If the child property has a key node that needs validation, validate it here
			TSharedPtr<FPropertyNode>& ChildKeyNode = ChildNode->GetPropertyKeyNode();
			if (bValidateChildrenKeyNodes && ChildKeyNode.IsValid())
			{
				EPropertyDataValidationResult ChildDataResult = ChildKeyNode->EnsureDataIsValid();
				if (FinalResult == EPropertyDataValidationResult::DataValid && ChildDataResult != EPropertyDataValidationResult::DataValid)
				{
					FinalResult = ChildDataResult;
				}
			}
		}
	}

	return FinalResult;
}

FPropertyNodeEditStack::FPropertyNodeEditStack(const FPropertyNode* InNode, const UObject* InObj)
{
	Initialize(InNode, InObj);
}

FPropertyAccess::Result FPropertyNodeEditStack::Initialize(const FPropertyNode* InNode, const UObject* InObj)
{
	Cleanup();
	FPropertyAccess::Result Result = InitializeInternal(InNode, InObj);
	if (Result != FPropertyAccess::Success)
	{
		Cleanup();
	}
	return Result;
}

FPropertyAccess::Result FPropertyNodeEditStack::InitializeInternal(const FPropertyNode* InNode, const UObject* InObj)
{
	FPropertyAccess::Result Result = FPropertyAccess::Success;
	const FPropertyNode* Parent = InNode->GetParentNode();
	const FProperty* Property = InNode->GetProperty();
	if (Parent && Parent->GetProperty())
	{
		// Recursively initialize the stack
		Result = InitializeInternal(Parent, InObj);
		if (Result != FPropertyAccess::Success)
		{
			return Result;
		}

		// Get the direct memory pointer for the current property
		const FProperty* ParentProperty = Parent->GetProperty();
		if (Property == ParentProperty) // Static array items
		{
			// Static array property node creates subnodes that point to individual array items
			MemoryStack.Add(FMemoryFrame(Property, MemoryStack.Last().Memory + InNode->GetArrayIndex() * Property->GetElementSize()));
		}
		else if (const FStructProperty* StructProp = CastField<FStructProperty>(ParentProperty)) // structs
		{
			if (Property->HasSetterOrGetter())
			{
				// If a property has a setter or getter we allocate temp memory to hold its value so that we can
				// change the value using direct memory pointer access. After we're done editing we will copy the memory back to the property in CommitChanges
				FMemoryFrame PropertyFrame(Property, (uint8*)Property->AllocateAndInitializeValue());
				int32 StackIndex = MemoryStack.Add(PropertyFrame);
				Property->GetValue_InContainer(MemoryStack[StackIndex - 1].Memory, PropertyFrame.Memory);
			}
			else
			{
				MemoryStack.Add(FMemoryFrame(Property, Property->ContainerPtrToValuePtr<uint8>(MemoryStack.Last().Memory)));
			}
		}
		else if (Property->GetOwner<FProperty>() == ParentProperty) // TArrays, TMaps and TSets
		{
			uint8* ItemAddress = (uint8*)ParentProperty->GetValueAddressAtIndex_Direct(Property, (void*)MemoryStack.Last().Memory, InNode->GetArrayIndex());
			if (ItemAddress)
			{
				MemoryStack.Add(FMemoryFrame(Property, ItemAddress));
			}
			else
			{
				return FPropertyAccess::Fail;
			}
		}
		else
		{
			checkf(false, TEXT("Unsupported property chain: Current: %s, Parent: %s"), *Property->GetFullName(), *ParentProperty->GetFullName());
		}
	}
	else
	{
		const UObject* Object = InObj;
		if (!Object)
		{
			UObject* NodeObject = nullptr;
			Result = InNode->GetSingleObject(NodeObject);
			if (Result != FPropertyAccess::Success)
			{
				return Result;
			}
			Object = NodeObject;
		}

		// Determine the root container address (Struct address, UObject instance or sparse class data) for this property stack
		uint8* Container = nullptr;
		if (InNode->HasNodeFlags(EPropertyNodeFlags::IsSparseClassData))
		{
			checkf(Object != nullptr, TEXT("No object pointer for property %s"), *GetNameSafe(Property));
			Container = (uint8*)Object->GetClass()->GetOrCreateSparseClassData();
		}
		else if (Object)
		{
			Container = (uint8*)Object;
		}
		else
		{
			Result = InNode->GetSingleReadAddress(Container);
			if (Result != FPropertyAccess::Success)
			{
				return Result;
			}
		}
		if (!Container)
		{
			// This may happen when the node points at stale object
			return FPropertyAccess::Fail;
		}
		MemoryStack.Add(FMemoryFrame(nullptr, Container));
		
		// Get the direct memory pointer for the root property
		if (Property->HasSetterOrGetter())
		{
			FMemoryFrame PropertyFrame(Property, (uint8*)Property->AllocateAndInitializeValue());
			int32 StackIndex = MemoryStack.Add(PropertyFrame);
			Property->GetValue_InContainer(MemoryStack[StackIndex - 1].Memory, PropertyFrame.Memory);
		}
		else if ((uint8*)Object == Container)
		{
			MemoryStack.Add(FMemoryFrame(Property, (uint8*)Property->ContainerPtrToValuePtr<uint8>(Container)));
		}
		else
		{
			// This node represents a struct in which case the Container represents direct memory for the root property.
			// todo: RobM: ideally we want Container to be the struct memory and not a property address
			MemoryStack.Add(FMemoryFrame(Property, Container));
		}
	}
	return Result;
}

void FPropertyNodeEditStack::CommitChanges()
{
	for (int32 Index = MemoryStack.Num() - 1; Index > 0; --Index)
	{
		if (MemoryStack[Index].Property->HasSetterOrGetter() &&
			MemoryStack[Index].Property != MemoryStack[Index - 1].Property) // If this property is identical to the one below then it represents an item from an array
		{
			// Set the actual property value with the temp allocated memory
			MemoryStack[Index].Property->SetValue_InContainer(MemoryStack[Index - 1].Memory, MemoryStack[Index].Memory);
		}
	}
}

void FPropertyNodeEditStack::Cleanup()
{
	for (int32 Index = MemoryStack.Num() - 1; Index > 0; --Index)
	{
		if (MemoryStack[Index].Property->HasSetterOrGetter() &&
			MemoryStack[Index].Property != MemoryStack[Index - 1].Property) // If this property is identical to the one below then it represents an item from an array
		{
			MemoryStack[Index].Property->DestroyAndFreeValue(MemoryStack[Index].Memory);
			MemoryStack[Index].Memory = nullptr;
		}
	}
	MemoryStack.Empty();
}

FPropertyNodeEditStack::~FPropertyNodeEditStack()
{
	Cleanup();
}

FPropertyAccess::Result FPropertyNode::GetPropertyValueString(FString& OutString, const bool bAllowAlternateDisplayValue, EPropertyPortFlags PortFlags) const
{
	uint8* ValueAddress = nullptr;
	FPropertyAccess::Result Result = GetSingleReadAddress(ValueAddress);

	if (ValueAddress != nullptr)
	{
		const FProperty* PropertyPtr = GetProperty();

		// Check for bogus data
		if (PropertyPtr != nullptr && GetParentNode() != nullptr)
		{
			FPropertyTextUtilities::PropertyToTextHelper(OutString, this, PropertyPtr, ValueAddress, nullptr, PortFlags);

			UEnum* Enum = nullptr;
			int64 EnumValue = 0;
			if (const FByteProperty* ByteProperty = CastField<FByteProperty>(PropertyPtr))
			{
				if (ByteProperty->Enum != nullptr)
				{
					Enum = ByteProperty->Enum;
					EnumValue = ByteProperty->GetPropertyValue(ValueAddress);
				}
			}
			else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(PropertyPtr))
			{
				Enum = EnumProperty->GetEnum();
				EnumValue = EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(ValueAddress);
			}

			if (Enum != nullptr)
			{
				if (Enum->IsValidEnumValue(EnumValue))
				{
					// See if we specified an alternate name for this value using metadata
					OutString = Enum->GetDisplayNameTextByValue(EnumValue).ToString();
					if (!bAllowAlternateDisplayValue || OutString.Len() == 0)
					{
						OutString = Enum->GetNameStringByValue(EnumValue);
					}
				}
				else
				{
					Result = FPropertyAccess::Fail;
				}
			}
		}
		else
		{
			Result = FPropertyAccess::Fail;
		}
	}

	return Result;
}

FPropertyAccess::Result FPropertyNode::GetPropertyValueText(FText& OutText, const bool bAllowAlternateDisplayValue) const
{
	uint8* ValueAddress = nullptr;
	FPropertyAccess::Result Result = GetSingleReadAddress(ValueAddress);

	if (ValueAddress != nullptr)
	{
		const FProperty* PropertyPtr = GetProperty();
		if (PropertyPtr)
		{
			if (PropertyPtr->IsA(FTextProperty::StaticClass()))
			{
				OutText = CastField<FTextProperty>(PropertyPtr)->GetPropertyValue(ValueAddress);
			}
			else
			{
				FString ExportedTextString;
				FPropertyTextUtilities::PropertyToTextHelper(ExportedTextString, this, PropertyPtr, ValueAddress, nullptr, PPF_PropertyWindow);

				UEnum* Enum = nullptr;
				int64 EnumValue = 0;
				if (const FByteProperty* ByteProperty = CastField<FByteProperty>(PropertyPtr))
				{
					Enum = ByteProperty->Enum;
					EnumValue = ByteProperty->GetPropertyValue(ValueAddress);
				}
				else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(PropertyPtr))
				{
					Enum = EnumProperty->GetEnum();
					EnumValue = EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(ValueAddress);
				}

				if (Enum)
				{
					if (Enum->IsValidEnumValue(EnumValue))
					{
						// Text form is always display name
						OutText = Enum->GetDisplayNameTextByValue(EnumValue);
					}
					else
					{
						Result = FPropertyAccess::Fail;
					}
				}
				else
				{
					OutText = FText::FromString(ExportedTextString);
				}
			}
		}
		else
		{
			Result = FPropertyAccess::Fail;
		}
	}

	return Result;
}

/**
 * Sets the flags used by the window and the root node
 * @param InFlags - flags to turn on or off
 * @param InOnOff - whether to toggle the bits on or off
 */
void FPropertyNode::SetNodeFlags(const EPropertyNodeFlags::Type InFlags, const bool InOnOff)
{
	if (InOnOff)
	{
		PropertyNodeFlags |= InFlags;
	}
	else
	{
		PropertyNodeFlags &= (~InFlags);
	}
}

bool FPropertyNode::GetChildNode(const int32 ChildArrayIndex, TSharedPtr<FPropertyNode>& OutChildNode)
{
	OutChildNode = nullptr;

	for (auto Child = ChildNodes.CreateIterator(); Child; ++Child)
	{
		if (Child->IsValid() && (*Child)->ArrayIndex == ChildArrayIndex)
		{
			OutChildNode = *Child;
			return true;
		}
	}

	return false;
}

bool FPropertyNode::GetChildNode(const int32 ChildArrayIndex, TSharedPtr<FPropertyNode>& OutChildNode) const
{
	OutChildNode = nullptr;

	for (auto Child = ChildNodes.CreateConstIterator(); Child; ++Child)
	{
		if (Child->IsValid() && (*Child)->ArrayIndex == ChildArrayIndex)
		{
			OutChildNode = *Child;
			return true;
		}
	}

	return false;
}

TSharedPtr<FPropertyNode> FPropertyNode::FindChildPropertyNode( const FName InPropertyName, bool bRecurse )
{
	// search children breadth-first, so that identically-named properties are first picked up in top-level classes, eg:
	// class UFoo 
	// { 
	//    struct FBar
	//    { 
	//       int ID; 
	//    } Bar;
	//    int ID;
	// };
	// depth-first search would find FBar::ID before UFoo::ID when searching for "ID", which is rarely what was intended

	TDeque<TSharedPtr<FPropertyNode>> NodesToSearch;

	auto PushAll = [&NodesToSearch](const TArray<TSharedPtr<FPropertyNode>>& Nodes)
	{
		NodesToSearch.Reserve(NodesToSearch.Num() + Nodes.Num());
		for (const TSharedPtr<FPropertyNode>& Node : Nodes)
		{
			NodesToSearch.PushLast(Node);
		}
	};

	PushAll(ChildNodes);
	while (!NodesToSearch.IsEmpty())
	{
		TSharedPtr<FPropertyNode> Node = NodesToSearch.First();
		NodesToSearch.PopFirst();

		if (Node->GetProperty() && Node->GetProperty()->GetFName() == InPropertyName)
		{
			return Node;
		}

		if (bRecurse)
		{
			PushAll(Node->ChildNodes);
		}
	}

	return nullptr;
}

/**
 * Returns whether this window's property is read only or has the CPF_EditConst flag.
 */
bool FPropertyNode::IsPropertyConst() const
{
	if (HasNodeFlags(EPropertyNodeFlags::IsReadOnly))
	{
		return true;
	}

	if (Property != nullptr)
	{
		return Property->HasAllPropertyFlags(CPF_EditConst);
	}

	return false;
}

/** @return whether this window's property is constant (can't be edited by the user) */
bool FPropertyNode::IsEditConst(const bool bIncludeEditCondition) const
{
	if (bUpdateEditConstState || UpdateEditConstStateEpoch != PropertyEditorPolicy::Get().GetPolicyEpoch())
	{
		UpdateEditConstStateEpoch = PropertyEditorPolicy::Get().GetPolicyEpoch();

		// Ask the objects whether this property can be changed
		const FObjectPropertyNode* ObjectPropertyNode = FindObjectItemParent();

		bIsEditConst = IsPropertyConst();
		if (!bIsEditConst && Property.IsValid() && ObjectPropertyNode)
		{
			TSharedRef<FEditPropertyChain> PropertyChain = BuildPropertyChain(Property.Get());
			
			// travel up the chain to see if this property's owner struct is EditConst - if it is, so is this property
			TSharedPtr<FPropertyNode> CurParent = ParentNodeWeakPtr.Pin();
			while (CurParent != nullptr)
			{
				FStructProperty* StructProperty = CastField<FStructProperty>(CurParent->GetProperty());
				if (StructProperty == nullptr)
				{
					const bool bIsContainerProperty = CastField<FArrayProperty>(CurParent->GetProperty()) || CastField<FSetProperty>(CurParent->GetProperty()) || CastField<FMapProperty>(CurParent->GetProperty()) || CastField<FOptionalProperty>(CurParent->GetProperty());
					
					if (!bIsContainerProperty)
					{
						break;
					}
				}

				if (CurParent->IsEditConst(bIncludeEditCondition))
				{
					// An owning struct is edit const, so the child property is too
					bIsEditConst = true;
				}
				else if (StructProperty)
				{
					// See if the struct has a problem with this property being editable
					UScriptStruct* ScriptStruct = StructProperty->Struct;
					if (ScriptStruct && ScriptStruct->StructFlags & STRUCT_CanEditChange)
					{
						UScriptStruct::ICppStructOps* TheCppStructOps = ScriptStruct->GetCppStructOps();
						check(TheCppStructOps);

						const int32 NumInstances = ObjectPropertyNode->GetInstancesNum();

						TArray<const void*> StructAddresses;
						StructAddresses.Reset(NumInstances);

						for (int32 Index = 0; Index < NumInstances; ++Index)
						{
							StructAddresses.Add(CurParent->GetValueAddressFromObject(ObjectPropertyNode->GetUObject(Index)));
						}

						for (const void* StructAddr : StructAddresses)
						{
							if (!TheCppStructOps->CanEditChange(*PropertyChain, StructAddr))
							{
								bIsEditConst = true;
								break;
							}
						}
					}
				}

				if (bIsEditConst)
				{
					break;
				}
				
				CurParent = CurParent->ParentNodeWeakPtr.Pin();
			}

			if (!bIsEditConst)
			{
				for (TPropObjectConstIterator CurObjectIt(ObjectPropertyNode->ObjectConstIterator()); CurObjectIt; ++CurObjectIt)
				{
					const TWeakObjectPtr<UObject> CurObject = *CurObjectIt;
					if (CurObject.IsValid())
					{
						if (!PropertyEditorPolicy::Get().CanEditProperty(*PropertyChain, CurObject.Get()))
						{
							bIsEditConst = true;
							break;
						}
						
						if (!CurObject->CanEditChange(*PropertyChain))
						{
							// At least one of the objects didn't like the idea of this property being changed.
							bIsEditConst = true;
							break;
						}
					}
				}
			}
		}

		// this ignores EditCondition check below
		bIsEditConstWithoutCondition = bIsEditConst;

		// check edit condition
		if (!bIsEditConst && HasEditCondition())
		{
			bIsEditConst = !IsEditConditionMet();
		}

		bUpdateEditConstState = false;
	}

	return bIncludeEditCondition ? bIsEditConst : bIsEditConstWithoutCondition;
}

bool FPropertyNode::ShouldSkipSerialization() const
{
	return Property != nullptr && Property->HasAnyPropertyFlags(CPF_SkipSerialization);
}

bool FPropertyNode::HasEditCondition() const 
{ 
	return EditConditionExpression.IsValid();
}

bool FPropertyNode::IsEditConditionMet() const 
{ 
	if (HasEditCondition())
	{
		TValueOrError<bool, FText> Result = EditConditionParser.Evaluate(*EditConditionExpression.Get(), *EditConditionContext.Get());
		if (Result.IsValid())
		{
			return Result.GetValue();
		}
	}

	return true;
}

bool FPropertyNode::SupportsEditConditionToggle() const
{
	if (!Property.IsValid())
	{
		return false;
	}

	FProperty* MyProperty = Property.Get();

	static const FName Name_HideEditConditionToggle("HideEditConditionToggle");
	if (EditConditionExpression.IsValid() && !Property->HasMetaData(Name_HideEditConditionToggle))
	{
		const FBoolProperty* ConditionalProperty = EditConditionContext->GetSingleBoolProperty(EditConditionExpression);
		if (ConditionalProperty != nullptr)
		{
			static const FName Name_InlineEditConditionToggle("InlineEditConditionToggle");
			const bool bIsInlineEditCondition = ConditionalProperty->HasMetaData(Name_InlineEditConditionToggle);
			const bool bIsEditable = ConditionalProperty->HasAllPropertyFlags(CPF_Edit);
			
			// Support for legacy behavior ( case 2. in the comment below ) if enabled
			if (UE::PropertyEditor::Private::bShowInlineEditConditionToggleWhenNotSpecifiedAndNotEditable)
			{
				// There are 2 valid states for inline edit conditions:
				// 1. The property is marked as editable and has InlineEditConditionToggle set. 
				// 2. The property is not marked as editable and does not have InlineEditConditionToggle set.
				// In both cases, the original property will be hidden and only show up as a toggle.
				if (bIsInlineEditCondition == bIsEditable)
				{
					return true;
				}
			}
			
			if (bIsInlineEditCondition)
			{
				if (!bIsEditable)
				{
					UE_LOGF(LogPropertyNode, Warning, "Property being used as inline edit condition is not editable, but has InlineEditConditionToggle flag. Field \"%ls\" in class \"%ls\".", *ConditionalProperty->GetNameCPP(), *Property->GetOwnerStruct()->GetName());
				}
				return true;
			}
		}
	}

	return false;
}

void FPropertyNode::ToggleEditConditionState()
{
	const FBoolProperty* EditConditionProperty = EditConditionContext->GetSingleBoolProperty(EditConditionExpression);
	check(EditConditionProperty != nullptr);

	FPropertyNode* MyParentNode = ParentNodeWeakPtr.Pin().Get();
	check(MyParentNode != nullptr);

	bool OldValue = true;

	const bool bIsSparse = HasNodeFlags(EPropertyNodeFlags::IsSparseClassData);
	
	FComplexPropertyNode* ComplexParentNode = FindComplexParent();

	if (!bIsSparse)
	{
		
		for (int32 Index = 0; Index < ComplexParentNode->GetInstancesNum(); ++Index)
		{
			uint8* ValuePtr = ComplexParentNode->GetValuePtrOfInstance(Index, EditConditionProperty, MyParentNode);

			OldValue &= EditConditionProperty->GetPropertyValue(ValuePtr);
			EditConditionProperty->SetPropertyValue(ValuePtr, !OldValue);
		}
	}
	else
	{
		// The strategy for getting the pointer to the vaue of the edit condition is to use the value pointer of the property
		// that is being conditioned and then walk back that property's offset to find the owning struct's base address. 
		// This owning struct's base address is then offset forward by the edit condition's property to find the edit condition's value pointer.
		// 
		// The assumption is that the edit condition property is on the same struct/class as the property that is being "conditioned".
		// 
		// Since the edit condition is inline, there will not be a PropertyNode available, therefore it is necessary to use
		// the FProperty API to find the value pointer for the edit condition.
		//
		// It is also not possible to use the conditioned property's parent node as above since the parent may not point to the struct that either
		// property is in.  In the case of property directly on the SparseClassData, the parent node is often a category or object node.
		// 
		// Care must also be taken for properties in sparse class data structs that are sub-structs containers within the sparse class data.
		// Therefore, no assumptions can be made about the relationship between the sparse class data's pointer and the offset of the properties
		// that are conditioned or used as edit conditions without walking up that relationship chain.
		FObjectPropertyNode* ObjectPropertyNode = ComplexParentNode->AsObjectNode();
		if (ObjectPropertyNode)
		{
			for (int32 Index = 0; Index < ObjectPropertyNode->GetInstancesNum(); ++Index)
			{
				UObject* Object = ObjectPropertyNode->GetUObject(Index);
				uint8* ConditionedPropertyAddress = GetValueAddressFromObject(Object);
				const int32 ConditionedPropertyOffset = GetProperty()->GetOffset_ForInternal();
				uint8* OwningStructStartAddress = ConditionedPropertyAddress - ConditionedPropertyOffset;
				uint8* ValuePtr = EditConditionProperty->ContainerPtrToValuePtr<uint8>(OwningStructStartAddress);
				OldValue &= EditConditionProperty->GetPropertyValue(ValuePtr);
				EditConditionProperty->SetPropertyValue(ValuePtr, !OldValue);
			}
		}
	}

	// Propagate the value change to any instances if we're editing a template object
	FObjectPropertyNode* ObjectNode = FindObjectItemParent();
	if (ObjectNode != nullptr)
	{
		for (int32 ObjIndex = 0; ObjIndex < ObjectNode->GetNumObjects(); ++ObjIndex)
		{
			TWeakObjectPtr<UObject> ObjectWeakPtr = ObjectNode->GetUObject(ObjIndex);
			UObject* Object = ObjectWeakPtr.Get();
			if (Object != nullptr && Object->IsTemplate())
			{
				TArray<UObject*> ArchetypeInstances;
				Object->GetArchetypeInstances(ArchetypeInstances);
				
				for (int32 InstanceIndex = 0; InstanceIndex < ArchetypeInstances.Num(); ++InstanceIndex)
				{
					uint8* ArchetypeBaseOffset = nullptr;
					if (!bIsSparse)
					{
						ArchetypeBaseOffset = MyParentNode->GetValueAddressFromObject(ArchetypeInstances[InstanceIndex]);
					}
					else
					{
						if(uint8* ConditionedPropertyAddress = GetValueAddressFromObject(ArchetypeInstances[InstanceIndex]))
						{
							const int32 ConditionedPropertyOffset = GetProperty()->GetOffset_ForInternal();
							uint8* OwningStructStartAddress = ConditionedPropertyAddress - ConditionedPropertyOffset;
							ArchetypeBaseOffset = OwningStructStartAddress;
						}
					}
					
					if(ArchetypeBaseOffset != nullptr)
					{
						uint8* ArchetypeValueAddr = EditConditionProperty->ContainerPtrToValuePtr<uint8>(ArchetypeBaseOffset);

						// Only propagate if the current value on the instance matches the previous value on the template.
						const bool CurValue = EditConditionProperty->GetPropertyValue(ArchetypeValueAddr);
						if (OldValue == CurValue)
						{
							EditConditionProperty->SetPropertyValue(ArchetypeValueAddr, !OldValue);
						}
					}
				}
			}
		}
	}
}

bool FPropertyNode::IsOnlyVisibleWhenEditConditionMet() const
{
	static const FName Name_EditConditionHides("EditConditionHides");
	if (Property.IsValid() && Property->HasMetaData(Name_EditConditionHides))
	{
		return HasEditCondition();
	}

	return false;
}

/**
 * Appends my path, including an array index (where appropriate)
 */
bool FPropertyNode::GetQualifiedName( FString& PathPlusIndex, const bool bWithArrayIndex, const FPropertyNode* StopParent, bool bIgnoreCategories ) const
{
	bool bAddedAnything = false;
	const TSharedPtr<FPropertyNode> ParentNode = ParentNodeWeakPtr.Pin();
	if (ParentNode && StopParent != ParentNode.Get())
	{
		bAddedAnything = ParentNode->GetQualifiedName(PathPlusIndex, bWithArrayIndex, StopParent, bIgnoreCategories);
	}

	if (Property.IsValid())
	{
		if (bAddedAnything)
		{
			PathPlusIndex += TEXT(".");
		}

		Property->AppendName(PathPlusIndex);

		if (bWithArrayIndex && (ArrayIndex != INDEX_NONE))
		{
			PathPlusIndex += TEXT("[");
			PathPlusIndex.AppendInt(ArrayIndex);
			PathPlusIndex += TEXT("]");
		}

		bAddedAnything = true;
	}

	return bAddedAnything;
}

bool FPropertyNode::GetReadAddressUncached( const FPropertyNode& InPropertyNode,
									bool InRequiresSingleSelection,
									FReadAddressListData* OutAddresses,
									bool bComparePropertyContents,
									bool bObjectForceCompare,
									bool bArrayPropertiesCanDifferInSize ) const
{
	const TSharedPtr<FPropertyNode> ParentNode = ParentNodeWeakPtr.Pin();
	if (ParentNode.IsValid())
	{
		return ParentNode->GetReadAddressUncached( InPropertyNode, InRequiresSingleSelection, OutAddresses, bComparePropertyContents, bObjectForceCompare, bArrayPropertiesCanDifferInSize );
	}

	return false;
}

bool FPropertyNode::GetReadAddressUncached( const FPropertyNode& InPropertyNode, FReadAddressListData& OutAddresses ) const
{
	const TSharedPtr<FPropertyNode> ParentNode = ParentNodeWeakPtr.Pin();
	if (ParentNode.IsValid())
	{
		return ParentNode->GetReadAddressUncached( InPropertyNode, OutAddresses );
	}
	return false;
}

bool FPropertyNode::GetReadAddress(bool InRequiresSingleSelection,
								   FReadAddressList& OutAddresses,
								   bool bComparePropertyContents,
								   bool bObjectForceCompare,
								   bool bArrayPropertiesCanDifferInSize) const

{
	if (!ParentNodeWeakPtr.IsValid())
	{
		return false;
	}

	// @todo PropertyEditor Nodes which require validation cannot be cached
	if( CachedReadAddresses.Num() && !CachedReadAddresses.bRequiresCache && !HasNodeFlags(EPropertyNodeFlags::RequiresValidation) )
	{
		OutAddresses.ReadAddressListData = &CachedReadAddresses;
		return CachedReadAddresses.bAllValuesTheSame;
	}

	CachedReadAddresses.Reset();

	bool bAllValuesTheSame = GetReadAddressUncached( *this, InRequiresSingleSelection, &CachedReadAddresses, bComparePropertyContents, bObjectForceCompare, bArrayPropertiesCanDifferInSize );
	OutAddresses.ReadAddressListData = &CachedReadAddresses;
	CachedReadAddresses.bAllValuesTheSame = bAllValuesTheSame;
	CachedReadAddresses.bRequiresCache = false;

	return bAllValuesTheSame;
}

/**
 * fills in the OutAddresses array with the addresses of all of the available objects.
 * @param InItem		The property to get objects from.
 * @param OutAddresses	Storage array for all of the objects' addresses.
 */
bool FPropertyNode::GetReadAddress( FReadAddressList& OutAddresses ) const
{
	if (!ParentNodeWeakPtr.IsValid())
	{
		return false;
	}

	// @todo PropertyEditor Nodes which require validation cannot be cached
	if( CachedReadAddresses.Num() && !HasNodeFlags(EPropertyNodeFlags::RequiresValidation) )
	{
		OutAddresses.ReadAddressListData = &CachedReadAddresses;
		return true;
	}

	CachedReadAddresses.Reset();

	bool bSuccess = GetReadAddressUncached( *this, CachedReadAddresses );
	if( bSuccess )
	{
		OutAddresses.ReadAddressListData = &CachedReadAddresses;
	}

	CachedReadAddresses.bRequiresCache = false;

	return bSuccess;
}

FPropertyAccess::Result FPropertyNode::GetSingleReadAddress(uint8*& OutValueAddress) const
{
	OutValueAddress = nullptr;
	FReadAddressList ReadAddresses;
	bool bAllValuesTheSame = GetReadAddress(!!HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ReadAddresses, false, true);

	if ((ReadAddresses.Num() > 0 && bAllValuesTheSame) || ReadAddresses.Num() == 1)
	{
		OutValueAddress = ReadAddresses.GetAddress(0);

		return FPropertyAccess::Success;
	}

	return ReadAddresses.Num() > 1 ? FPropertyAccess::MultipleValues : FPropertyAccess::Fail;
}

FPropertyAccess::Result FPropertyNode::GetSingleObject(UObject*& OutObject) const
{
	OutObject = nullptr;
	FReadAddressList ReadAddresses;
	bool bAllValuesTheSame = GetReadAddress(HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly), ReadAddresses, false, true);

	if ((ReadAddresses.Num() > 0 && bAllValuesTheSame) || ReadAddresses.Num() == 1)
	{
		OutObject = (UObject*)ReadAddresses.GetObject(0);

		return FPropertyAccess::Success;
	}

	return ReadAddresses.Num() > 1 ? FPropertyAccess::MultipleValues : FPropertyAccess::Fail;
}

FPropertyAccess::Result FPropertyNode::GetSingleEditStack(FPropertyNodeEditStack& OutStack) const
{
	UObject* Object = nullptr;
	FPropertyAccess::Result Result = FPropertyAccess::Fail;
	if (GetProperty())
	{
		Result = GetSingleObject(Object);
		if (Result == FPropertyAccess::Success)
		{
			Result = OutStack.Initialize(this, Object);
		}
	}
	return Result;
}

uint8* FPropertyNode::GetStartAddressFromObject(const UObject* Obj) const
{
	if (!Obj)
	{
		return nullptr;
	}

	if (HasNodeFlags(EPropertyNodeFlags::IsSparseClassData))
	{
		return (uint8*)Obj->GetClass()->GetOrCreateSparseClassData();
	}

	return (uint8*)Obj;
}

uint8* FPropertyNode::GetValueBaseAddressFromObject(const UObject* Obj) const
{
	return GetValueBaseAddress(GetStartAddressFromObject(Obj), HasNodeFlags(EPropertyNodeFlags::IsSparseClassData));
}

uint8* FPropertyNode::GetValueAddressFromObject(const UObject* Obj) const
{
	return GetValueAddress(GetStartAddressFromObject(Obj), HasNodeFlags(EPropertyNodeFlags::IsSparseClassData));
}


uint8* FPropertyNode::GetValueBaseAddress(uint8* StartAddress, bool bIsSparseData, bool bIsStruct) const
{
	uint8* Result = NULL;

	if (bIsSparseData)
	{
		Result = StartAddress;
	}
	else
	{
		const TSharedPtr<FPropertyNode> ParentNode = ParentNodeWeakPtr.Pin();
		if (ParentNode.IsValid())
		{
			Result = ParentNode->GetValueAddress(StartAddress, bIsSparseData, bIsStruct);
		}
	}
	return Result;
}

uint8* FPropertyNode::GetValueAddress(uint8* StartAddress, bool bIsSparseData, bool bIsStruct) const
{
	return GetValueBaseAddress(StartAddress, bIsSparseData, bIsStruct);
}


/*-----------------------------------------------------------------------------
	FPropertyItemValueDataTrackerSlate
-----------------------------------------------------------------------------*/
/**
 * Calculates and stores the address for both the current and default value of
 * the associated property and the owning object.
 */
class FPropertyItemValueDataTrackerSlate
{
public:
	/**
	 * A union which allows a single address to be represented as a pointer to a uint8
	 * or a pointer to a UObject.
	 */
	union FPropertyValueRoot
	{
		UObject*	OwnerObject;
		uint8*		ValueAddress;
	};

	void Reset(FPropertyNode* InPropertyNode, UObject* InOwnerObject)
	{
		OwnerObject = InOwnerObject;
		PropertyNode = InPropertyNode;
		bHasDefaultValue = false;
		InnerInitialize();
	}

	void InnerInitialize()
	{
		PropertyValueRoot.OwnerObject = NULL;
		PropertyDefaultValueRoot.OwnerObject = NULL;
		PropertyValueAddress = NULL;
		PropertyValueBaseAddress = NULL;
		PropertyDefaultBaseAddress = NULL;
		PropertyDefaultAddress = NULL;

		PropertyValueRoot.OwnerObject = OwnerObject.Get();
		check(PropertyNode);
		FProperty* Property = PropertyNode->GetProperty();
		check(Property);
		check(PropertyValueRoot.OwnerObject);

		// Do not cache pointers for standalone structures, as we don't have the same guarantees how the provided pointers are invalidated as we have with UObject nodes. 
		// The default value handling for structure nodes is done in FPropertyNode::GetDefaultValueAsString() and FPropertyNode::GetDiffersFromDefault().
		if (PropertyNode->FindStructureItemParent())
		{
			return;
		}
		
		FPropertyNode* ParentNode = PropertyNode->GetParentNode();

		// if the object specified is a class object, transfer to the CDO instead
		if ( Cast<UClass>(PropertyValueRoot.OwnerObject) != NULL )
		{
			PropertyValueRoot.OwnerObject = Cast<UClass>(PropertyValueRoot.OwnerObject)->GetDefaultObject();
		}

		const bool bIsContainerProperty = CastField<FArrayProperty>(Property) || CastField<FSetProperty>(Property) || CastField<FMapProperty>(Property) || CastField<FOptionalProperty>(Property);
		const bool bIsInsideContainerProperty = Property->GetOwner<FArrayProperty>() || Property->GetOwner<FSetProperty>() || Property->GetOwner<FMapProperty>() || Property->GetOwner<FOptionalProperty>();

		FPropertyNode* Node = bIsInsideContainerProperty ? ParentNode : PropertyNode;

		PropertyValueBaseAddress = Node->GetValueBaseAddressFromObject(PropertyValueRoot.OwnerObject);
		PropertyValueAddress = PropertyNode->GetValueAddressFromObject(PropertyValueRoot.OwnerObject);

		if (IsValidTracker())
		{
			bHasDefaultValue = Private_HasDefaultValue();

			// calculate the addresses for the default object if it exists
			if (bHasDefaultValue)
			{
				PropertyDefaultValueRoot.OwnerObject = PropertyValueRoot.OwnerObject ? FPropertyNode::GetArchetype(PropertyValueRoot.OwnerObject) : nullptr;

				PropertyDefaultBaseAddress = Node->GetValueBaseAddressFromObject(PropertyDefaultValueRoot.OwnerObject);
				PropertyDefaultAddress = PropertyNode->GetValueAddressFromObject(PropertyDefaultValueRoot.OwnerObject);

				//////////////////////////
				// If this is a container property, we must take special measures to use the base address of the property's value; for instance,
				// the array property's PropertyDefaultBaseAddress points to an FScriptArray*, while PropertyDefaultAddress points to the 
				// FScriptArray's Data pointer.
				if (bIsContainerProperty)
				{
					PropertyValueAddress = PropertyValueBaseAddress;
					PropertyDefaultAddress = PropertyDefaultBaseAddress;
				}
			}
		}
	}

	/**
	 * Constructor
	 *
	 * @param	InPropItem		the property window item this struct will hold values for
	 * @param	InOwnerObject	the object which contains the property value
	 */
	FPropertyItemValueDataTrackerSlate(FPropertyNode* InPropertyNode, UObject* InOwnerObject)
		: OwnerObject(InOwnerObject)
		, PropertyNode(InPropertyNode)
		, bHasDefaultValue(false)
	{
		InnerInitialize();
	}

	/**
	 * @return Whether or not this tracker has a valid address to a property and object
	 */
	bool IsValidTracker() const
	{
		return PropertyValueBaseAddress != nullptr && OwnerObject.IsValid();
	}

	/**
	 * @return	a pointer to the subobject root (outer-most non-subobject) of the owning object.
	 */
	UObject* GetTopLevelObject()
	{
		check(PropertyNode);
		FObjectPropertyNode* RootNode = PropertyNode->FindRootObjectItemParent();
		check(RootNode);
	
		TArray<UObject*> RootObjects;
		for ( TPropObjectIterator Itor( RootNode->ObjectIterator() ) ; Itor ; ++Itor )
		{
			TWeakObjectPtr<UObject> Object = *Itor;

			if( Object.IsValid() )
			{
				RootObjects.Add(Object.Get());
			}
		}
	
		UObject* Result;
		for ( Result = PropertyValueRoot.OwnerObject; Result; Result = Result->GetOuter() )
		{
			if ( RootObjects.Contains(Result) )
			{
				break;
			}
		}
		
		if( !Result )
		{
			// The result is not contained in the root so it is the top level object
			Result = PropertyValueRoot.OwnerObject;
		}
		return Result;
	}

	/**
	 * Whether or not we have a default value
	 */
	bool HasDefaultValue() const { return bHasDefaultValue; }

	
	/**
	 * @return The property node we are inspecting
	 */
	FPropertyNode* GetPropertyNode() const { return PropertyNode; }

	/**
	 * @return The address of the property's value.
	 */
	uint8* GetPropertyValueAddress() const { return PropertyValueAddress; }

	/**
	 * @return The base address of the property's default value.
	 */
	uint8* GetPropertyDefaultBaseAddress() const { return PropertyDefaultBaseAddress; }

	/**
	 * @return The address of the property's default value.
	 */
	uint8* GetPropertyDefaultAddress() const { return PropertyDefaultAddress; }

	/**
	 * @return The address of the property's owner object.
	 */
	uint8* GetPropertyRootAddress() const { return PropertyValueRoot.ValueAddress; }

	/**
	 * @return The address of the default value owner object.
	 */
	uint8* GetPropertyDefaultRootAddress() const { return PropertyDefaultValueRoot.ValueAddress; }

private:
	/**
	 * Determines whether the property bound to this struct exists in the owning object's archetype.
	 *
	 * @return	true if this property exists in the owning object's archetype; false if the archetype is e.g. a
	 *			CDO for a base class and this property is declared in the owning object's class.
	 */
	bool Private_HasDefaultValue() const
	{
		bool bResult = false;

		if (IsValidTracker())
		{
			UClass* OwnerClass = PropertyValueRoot.OwnerObject->GetClass();
			if (GetPropertyNode()->HasNodeFlags(EPropertyNodeFlags::IsSparseClassData))
			{
				if (OwnerClass)
				{
					UStruct* SparseClassDataStruct = OwnerClass->GetSparseClassDataStruct();
					UStruct* SparseClassDataArchetypeStruct = OwnerClass->GetSparseClassDataArchetypeStruct();

					if (SparseClassDataStruct == SparseClassDataArchetypeStruct)
					{
						bResult = true;
					}
					else
					{
						// Find the member property which contains this item's property
						FPropertyNode* MemberPropertyNode = PropertyNode;
						for (; MemberPropertyNode != nullptr; MemberPropertyNode = MemberPropertyNode->GetParentNode())
						{
							FProperty* MemberProperty = MemberPropertyNode->GetProperty();
							if (MemberProperty != nullptr)
							{
								if (MemberProperty->GetOwner<UClass>() != nullptr)
								{
									break;
								}
							}
						}
						if (MemberPropertyNode != nullptr && MemberPropertyNode->GetProperty())
						{
							// we check to see that this property is in the defaults class
							bResult = MemberPropertyNode->GetProperty()->IsInContainer(SparseClassDataArchetypeStruct);
						}
					}
				}

				return bResult;
			}
			check(PropertyValueBaseAddress);
			check(PropertyValueRoot.OwnerObject);
			UObject* ParentDefault = FPropertyNode::GetArchetype(PropertyValueRoot.OwnerObject);
			check(ParentDefault);
			if (OwnerClass == ParentDefault->GetClass())
			{
				// if the archetype is of the same class, then we must have a default
				bResult = true;
			}
			else
			{
				// Find the member property which contains this item's property
				FPropertyNode* MemberPropertyNode = PropertyNode;
				for ( ;MemberPropertyNode != NULL; MemberPropertyNode = MemberPropertyNode->GetParentNode() )
				{
					FProperty* MemberProperty = MemberPropertyNode->GetProperty();
					if ( MemberProperty != NULL )
					{
						if ( MemberProperty->GetOwner<UClass>() != NULL )
						{
							break;
						}
					}
				}
				if ( MemberPropertyNode != NULL && MemberPropertyNode->GetProperty())
				{
					// we check to see that this property is in the defaults class
					bResult = MemberPropertyNode->GetProperty()->IsInContainer(ParentDefault->GetClass());
				}
			}
		}

		return bResult;
	}

private:
	TWeakObjectPtr<UObject> OwnerObject;

	/** The property node we are inspecting */
	FPropertyNode* PropertyNode;


	/** The address of the owning object */
	FPropertyValueRoot PropertyValueRoot;

	/**
	 * The address of the owning object's archetype
	 */
	FPropertyValueRoot PropertyDefaultValueRoot;

	/**
	 * The address of this property's value.
	 */
	uint8* PropertyValueAddress;

	/**
	 * The base address of this property's value.  i.e. for dynamic arrays, the location of the FScriptArray which
	 * contains the array property's value
	 */
	uint8* PropertyValueBaseAddress;

	/**
	 * The base address of this property's default value (see other comments for PropertyValueBaseAddress)
	 */
	uint8* PropertyDefaultBaseAddress;

	/**
	 * The address of this property's default value.
	 */
	uint8* PropertyDefaultAddress;

	/** Whether or not we have a default value */
	bool bHasDefaultValue;
};


/* ==========================================================================================================
	FPropertyItemComponentCollector

	Given a property and the address for that property's data, searches for references to components and
	keeps a list of any that are found.
========================================================================================================== */
/**
 * Given a property and the address for that property's data, searches for references to components and keeps a list of any that are found.
 */
struct FPropertyItemComponentCollector
{
	/** contains the property to search along with the value address to use */
	const FPropertyItemValueDataTrackerSlate& ValueTracker;

	/** holds the list of instanced objects found */
	TArray<UObject*> Components;

	/** Whether or not we have an edit inline new */
	bool bContainsEditInlineNew;

	/** Constructor */
	FPropertyItemComponentCollector( const FPropertyItemValueDataTrackerSlate& InValueTracker )
	: ValueTracker(InValueTracker)
	, bContainsEditInlineNew( false )
	{
		check(ValueTracker.GetPropertyNode());
		FPropertyNode* PropertyNode = ValueTracker.GetPropertyNode();
		check(PropertyNode);
		FProperty* Prop = PropertyNode->GetProperty();
		if ( PropertyNode->GetArrayIndex() == INDEX_NONE )
		{
			// either the associated property is not an array property, or it's the header for the property (meaning the entire array)
			for ( int32 ArrayIndex = 0; ArrayIndex < Prop->ArrayDim; ArrayIndex++ )
			{
				ProcessProperty(Prop, ValueTracker.GetPropertyValueAddress() + ArrayIndex * Prop->GetElementSize());
			}
		}
		else
		{
			// single element of either a dynamic or static array
			ProcessProperty(Prop, ValueTracker.GetPropertyValueAddress());
		}
	}

	/**
	 * Routes the processing to the appropriate method depending on the type of property.
	 *
	 * @param	Property				the property to process
	 * @param	PropertyValueAddress	the address of the property's value
	 */
	void ProcessProperty( FProperty* Property, uint8* PropertyValueAddress )
	{
		if ( Property != NULL )
		{
			bContainsEditInlineNew |= Property->HasMetaData(TEXT("EditInline")) && ((Property->PropertyFlags & CPF_EditConst) == 0);

			if ( ProcessObjectProperty(CastField<FObjectPropertyBase>(Property), PropertyValueAddress) )
			{
				return;
			}
			if ( ProcessStructProperty(CastField<FStructProperty>(Property), PropertyValueAddress) )
			{
				return;
			}
			if ( ProcessInterfaceProperty(CastField<FInterfaceProperty>(Property), PropertyValueAddress) )
			{
				return;
			}
			if ( ProcessDelegateProperty(CastField<FDelegateProperty>(Property), PropertyValueAddress) )
			{
				return;
			}
			if ( ProcessMulticastDelegateProperty(CastField<FMulticastDelegateProperty>(Property), PropertyValueAddress) )
			{
				return;
			}
			if ( ProcessArrayProperty(CastField<FArrayProperty>(Property), PropertyValueAddress) )
			{
				return;
			}
			if ( ProcessSetProperty(CastField<FSetProperty>(Property), PropertyValueAddress) )
			{
				return;
			}
			if ( ProcessMapProperty(CastField<FMapProperty>(Property), PropertyValueAddress) )
			{
				return;
			}
		}
	}
private:

	/**
	 * FArrayProperty version - invokes ProcessProperty on the array's Inner member for each element in the array.
	 *
	 * @param	ArrayProp				the property to process
	 * @param	PropertyValueAddress	the address of the property's value
	 *
	 * @return	true if the property was handled by this method
	 */
	bool ProcessArrayProperty( FArrayProperty* ArrayProp, uint8* PropertyValueAddress )
	{
		bool bResult = false;

		if ( ArrayProp != NULL )
		{
			FScriptArrayHelper ArrayHelper(ArrayProp, PropertyValueAddress);

			uint8* ArrayValue = (uint8*)ArrayHelper.GetRawPtr();
			int32 ArraySize = ArrayHelper.Num();
			for ( int32 ArrayIndex = 0; ArrayIndex < ArraySize; ArrayIndex++ )
			{
				ProcessProperty(ArrayProp->Inner, ArrayValue + ArrayIndex * ArrayProp->Inner->GetElementSize());
			}

			bResult = true;
		}

		return bResult;
	}

	/**
	 * FSetProperty version - invokes ProcessProperty on the each item in the set
	 *
	 * @param	SetProp					the property to process
	 * @param	PropertyValueAddress	the address of the property's value
	 *
	 * @return	true if the property was handled by this method
	 */
	bool ProcessSetProperty( FSetProperty* SetProp, uint8* PropertyValueAddress )
	{
		bool bResult = false;

		if (SetProp != NULL)
		{
			FScriptSet* SetValuePtr = SetProp->GetPropertyValuePtr(PropertyValueAddress);

			FScriptSetLayout SetLayout = SetValuePtr->GetScriptLayout(SetProp->ElementProp->GetElementSize(), SetProp->ElementProp->GetMinAlignment());
			int32 ItemsLeft = SetValuePtr->Num();

			for (int32 Index = 0; ItemsLeft > 0; ++Index)
			{
				if (SetValuePtr->IsValidIndex(Index))
				{
					--ItemsLeft;
					ProcessProperty(SetProp->ElementProp, (uint8*)SetValuePtr->GetData(Index, SetLayout));
				}
			}

			bResult = true;
		}

		return bResult;
	}

	/**
	 * FMapProperty version - invokes ProcessProperty on each item in the map
	 *
	 * @param	MapProp					the property to process
	 * @param	PropertyValueAddress	the address of the property's value
	 *
	 * @return	true if the property was handled by this method
	 */
	bool ProcessMapProperty( FMapProperty* MapProp, uint8* PropertyValueAddress )
	{
		bool bResult = false;

		if (MapProp != NULL)
		{
			FScriptMapHelper MapHelper(MapProp, PropertyValueAddress);
			for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
			{
				uint8* Data = MapHelper.GetPairPtr(It);

				ProcessProperty(MapProp->KeyProp, MapProp->KeyProp->ContainerPtrToValuePtr<uint8>(Data));
				ProcessProperty(MapProp->ValueProp, MapProp->ValueProp->ContainerPtrToValuePtr<uint8>(Data));
			}

			bResult = true;
		}

		return bResult;
	}

	/**
	 * FStructProperty version - invokes ProcessProperty on each property in the struct
	 *
	 * @param	StructProp				the property to process
	 * @param	PropertyValueAddress	the address of the property's value
	 *
	 * @return	true if the property was handled by this method
	 */
	bool ProcessStructProperty( FStructProperty* StructProp, uint8* PropertyValueAddress )
	{
		bool bResult = false;

		if ( StructProp != NULL )
		{
			for ( FProperty* Prop = StructProp->Struct->PropertyLink; Prop; Prop = Prop->PropertyLinkNext )
			{
				for ( int32 ArrayIndex = 0; ArrayIndex < Prop->ArrayDim; ArrayIndex++ )
				{
					ProcessProperty(Prop, Prop->ContainerPtrToValuePtr<uint8>(PropertyValueAddress, ArrayIndex));
				}
			}
			bResult = true;
		}

		return bResult;
	}

	/**
	 * FObjectProperty version - if the object located at the specified address is instanced, adds the component the list.
	 *
	 * @param	ObjectProp				the property to process
	 * @param	PropertyValueAddress	the address of the property's value
	 *
	 * @return	true if the property was handled by this method
	 */
	bool ProcessObjectProperty( FObjectPropertyBase* ObjectProp, uint8* PropertyValueAddress )
	{
		bool bResult = false;

		if ( ObjectProp != NULL )
		{
			UObject* ObjValue = ObjectProp->GetObjectPropertyValue(PropertyValueAddress);
			if (ObjectProp->PropertyFlags & CPF_InstancedReference)
			{
				Components.AddUnique(ObjValue);
			}

			bResult = true;
		}

		return bResult;
	}

	/**
	 * FInterfaceProperty version - if the FScriptInterface located at the specified address contains a reference to an instance, add the component to the list.
	 *
	 * @param	InterfaceProp			the property to process
	 * @param	PropertyValueAddress	the address of the property's value
	 *
	 * @return	true if the property was handled by this method
	 */
	bool ProcessInterfaceProperty( FInterfaceProperty* InterfaceProp, uint8* PropertyValueAddress )
	{
		bool bResult = false;

		if ( InterfaceProp != NULL )
		{
			FScriptInterface* InterfaceValue = InterfaceProp->GetPropertyValuePtr(PropertyValueAddress);

			UObject* InterfaceObj = InterfaceValue->GetObject();
			if (InterfaceObj && InterfaceObj->IsDefaultSubobject())
			{
				Components.AddUnique(InterfaceValue->GetObject());
			}
			bResult = true;
		}

		return bResult;
	}

	/**
	 * FDelegateProperty version - if the FScriptDelegate located at the specified address contains a reference to an instance, add the component to the list.
	 *
	 * @param	DelegateProp			the property to process
	 * @param	PropertyValueAddress	the address of the property's value
	 *
	 * @return	true if the property was handled by this method
	 */
	bool ProcessDelegateProperty( FDelegateProperty* DelegateProp, uint8* PropertyValueAddress )
	{
		bool bResult = false;

		if ( DelegateProp != NULL )
		{
			FScriptDelegate* DelegateValue = DelegateProp->GetPropertyValuePtr(PropertyValueAddress);
			if (DelegateValue->GetUObject() && DelegateValue->GetUObject()->IsDefaultSubobject())
			{
				Components.AddUnique(DelegateValue->GetUObject());
			}

			bResult = true;
		}

		return bResult;
	}

	/**
	 * FMulticastDelegateProperty version - if the FMulticastScriptDelegate located at the specified address contains a reference to an instance, add the component to the list.
	 *
	 * @param	MulticastDelegateProp	the property to process
	 * @param	PropertyValueAddress	the address of the property's value
	 *
	 * @return	true if the property was handled by this method
	 */
	bool ProcessMulticastDelegateProperty( FMulticastDelegateProperty* MulticastDelegateProp, uint8* PropertyValueAddress )
	{
		bool bResult = false;

		if ( MulticastDelegateProp != NULL )
		{
			if (const FMulticastScriptDelegate* MulticastDelegateValue = MulticastDelegateProp->GetMulticastDelegate(PropertyValueAddress))
			{
				TArray<UObject*> AllObjects = MulticastDelegateValue->GetAllObjects();
				for (TArray<UObject*>::TConstIterator CurObjectIt(AllObjects); CurObjectIt; ++CurObjectIt)
				{
					if ((*CurObjectIt)->IsDefaultSubobject())
					{
						Components.AddUnique((*CurObjectIt));
					}
				}
			}

			bResult = true;
		}

		return bResult;
	}

};


bool FPropertyNode::GetDiffersFromDefault(const uint8* PropertyValueAddress, const uint8* PropertyDefaultAddress, const uint8* DefaultPropertyValueBaseAddress, const FProperty* InProperty, const UObject* TopLevelObject) const
{
	bool bDiffersFromDefaultValue = false;

	if (DefaultPropertyValueBaseAddress != nullptr)
	{
		if (const FArrayProperty* OuterArrayProperty = InProperty->GetOwner<FArrayProperty>())
		{
			// make sure we're not trying to compare against an element that doesn't exist
			FScriptArrayHelper ArrayHelper(OuterArrayProperty, DefaultPropertyValueBaseAddress);
			if (!ArrayHelper.IsValidIndex(GetArrayIndex()))
			{
				bDiffersFromDefaultValue = true;
			}
		}
		else if (const FSetProperty* OuterSetProperty = InProperty->GetOwner<FSetProperty>())
		{
			FScriptSetHelper SetHelper(OuterSetProperty, DefaultPropertyValueBaseAddress);
			if (!SetHelper.IsValidIndex(GetArrayIndex()))
			{
				bDiffersFromDefaultValue = true;
			}
		}
		else if (const FMapProperty* OuterMapProperty = InProperty->GetOwner<FMapProperty>())
		{
			FScriptMapHelper MapHelper(OuterMapProperty, DefaultPropertyValueBaseAddress);
			if (!MapHelper.IsValidIndex(GetArrayIndex()))
			{
				bDiffersFromDefaultValue = true;
			}
		}
		else if (const FOptionalProperty* OuterOptionalProperty = InProperty->GetOwner<FOptionalProperty>())
		{
			if (!OuterOptionalProperty->IsSet(DefaultPropertyValueBaseAddress))
			{
				bDiffersFromDefaultValue = true;
			}
		}
	}

	if (!bDiffersFromDefaultValue)
	{
		if (PropertyValueAddress == nullptr || PropertyDefaultAddress == nullptr)
		{
			// if either are NULL, we had a dynamic array somewhere in our parent chain and the array doesn't
			// have enough elements in either the default or the object
			bDiffersFromDefaultValue = true;
		}
		else
		{
			FString DefaultValue = GetDefaultValueAsString(PropertyDefaultAddress, InProperty, EValueAsStringMode::ForDiff, TopLevelObject);
			FString CurrentValue = GetDefaultValueAsString(PropertyValueAddress, InProperty, EValueAsStringMode::ForDiff, TopLevelObject);
			bDiffersFromDefaultValue = !(DefaultValue.Equals(CurrentValue, ESearchCase::CaseSensitive));
		}
	}

	return bDiffersFromDefaultValue;
}

bool FPropertyNode::GetDiffersFromDefaultForObject( FPropertyItemValueDataTrackerSlate& ValueTracker, FProperty* InProperty )
{	
	check( InProperty );

	const bool bIsValidTracker = ValueTracker.IsValidTracker();
	const bool bHasDefaultValue = ValueTracker.HasDefaultValue();
	const bool bHasParent = GetParentNode() != nullptr;

	if (bIsValidTracker && bHasDefaultValue && bHasParent)
	{
		return GetDiffersFromDefault(ValueTracker.GetPropertyValueAddress(), ValueTracker.GetPropertyDefaultAddress(), ValueTracker.GetPropertyDefaultBaseAddress(), InProperty, ValueTracker.GetTopLevelObject());
	}

	return false;
}

/**
 * If there is a property, sees if it matches.  Otherwise sees if the entire parent structure matches
 */
bool FPropertyNode::GetDiffersFromDefault()
{
	if( bUpdateDiffersFromDefault || UpdateDiffersFromDefaultEpoch != PropertyEditorPolicy::Get().GetPolicyEpoch())
	{
		UpdateDiffersFromDefaultEpoch = PropertyEditorPolicy::Get().GetPolicyEpoch();
		bUpdateDiffersFromDefault = false;
		bDiffersFromDefault = false;

		FProperty* Prop = GetProperty();
		if (!Prop)
		{
			return bDiffersFromDefault;
		}

		if (const FStructurePropertyNode* StructNode = FindStructureItemParent())
		{
			TArray<TSharedPtr<FStructOnScope>> Structs;
			StructNode->GetAllStructureData(Structs);
			
			const bool bIsSparse = HasNodeFlags(EPropertyNodeFlags::IsSparseClassData);
			const bool bIsContainer = CastField<FArrayProperty>(Prop) || CastField<FSetProperty>(Prop) || CastField<FMapProperty>(Prop) || CastField<FOptionalProperty>(Prop);
			const bool bIsInsideContainerProperty = Property->GetOwner<FArrayProperty>() || Property->GetOwner<FSetProperty>() || Property->GetOwner<FMapProperty>() || Property->GetOwner<FOptionalProperty>();
			const FPropertyNode* BaseNode = bIsInsideContainerProperty ? GetParentNode() : this;

			FStructOnScope DefaultStruct;

			const FObjectPropertyNode* TopLevelObjectNode = StructNode->FindObjectItemParent();
			for (int32 Index = 0; !bDiffersFromDefault && Index < Structs.Num(); Index++)
			{
				const TSharedPtr<FStructOnScope>& StructData = Structs[Index];
				// Skip empty data.
				if (!StructData.IsValid())
				{
					continue;
				}
				const UStruct* Struct = StructData->GetStruct();
				if (!Struct)
				{
					continue;
				}

				const uint8* PropertyDefaultAddress;
				const uint8* PropertyDefaultBaseAddress;

				const UObject* TopLevelObject = (TopLevelObjectNode && Index < TopLevelObjectNode->GetNumObjects()) ? TopLevelObjectNode->GetUObject(Index) : nullptr;
				if (TopLevelObject)
				{
					UObject* ObjectArchetype = FPropertyNode::GetArchetype(TopLevelObject);
					check(ObjectArchetype);

					PropertyDefaultAddress = GetValueAddress(StructNode->GetValueAddressFromObject(ObjectArchetype), bIsSparse, /*bIsStruct=*/true);
					PropertyDefaultBaseAddress = BaseNode->GetValueBaseAddress(StructNode->GetValueBaseAddressFromObject(ObjectArchetype), bIsSparse, /*bIsStruct=*/true);
				}
				else
				{
					// Make an instance of the struct to be used as default value to test against.
					if (DefaultStruct.GetStruct() != Struct)
					{
						DefaultStruct.Initialize(Struct);
					}

					check(DefaultStruct.IsValid());

					PropertyDefaultAddress = GetValueAddress(DefaultStruct.GetStructMemory(), bIsSparse, /*bIsStruct=*/true);
					PropertyDefaultBaseAddress = BaseNode->GetValueBaseAddress(DefaultStruct.GetStructMemory(), bIsSparse, /*bIsStruct=*/true);
				}

				const uint8* PropertyValueAddress = GetValueAddress(StructData->GetStructMemory(), bIsSparse, /*bIsStruct=*/true);

				// If this is a container property, we must take special measures to use the base address of the property's value; for instance,
				// the array property's PropertyDefaultBaseAddress points to an FScriptArray*, while PropertyDefaultAddress points to the 
				// FScriptArray's Data pointer.
				if (bIsContainer)
				{
					const uint8* PropertyValueBaseAddress = BaseNode->GetValueBaseAddress(StructData->GetStructMemory(), bIsSparse, /*bIsStruct=*/true);
					PropertyValueAddress = PropertyValueBaseAddress;
					PropertyDefaultAddress = PropertyDefaultBaseAddress;
				}

				bDiffersFromDefault = GetDiffersFromDefault(PropertyValueAddress, PropertyDefaultAddress, PropertyDefaultBaseAddress, Prop, TopLevelObject);
			}
		}
		else if (FObjectPropertyNode* ObjectNode = FindObjectItemParent())
		{
			// Get an iterator for the enclosing objects.
			for (int32 ObjIndex = 0; ObjIndex < ObjectNode->GetNumObjects(); ++ObjIndex)
			{
				UObject* Object = ObjectNode->GetUObject(ObjIndex);

				TSharedPtr<FPropertyItemValueDataTrackerSlate> ValueTracker = GetValueTracker(Object, ObjIndex);

				if (Object && GetDiffersFromDefaultForObject(*ValueTracker, Prop))
				{
					// If any object being observed differs from the result then there is no need to keep searching
					bDiffersFromDefault = true;
					break;
				}
			}
		}
	}

	return bDiffersFromDefault;
}


FString FPropertyNode::GetDefaultValueAsString(const uint8* PropertyDefaultAddress, const FProperty* InProperty, EValueAsStringMode Mode, const UObject* TopLevelObject) const
{
	const bool bUseDisplayName = (Mode == EValueAsStringMode::UseDisplayName);
	FString DefaultValue;

	uint32 PortFlags = PPF_None;
	if (Mode == EValueAsStringMode::UseDisplayName)
	{
		PortFlags |= PPF_PropertyWindow;
	}
	else if (Mode == EValueAsStringMode::ForDiff)
	{
		PortFlags |= PPF_ForDiff;
		if (TopLevelObject && !TopLevelObject->IsTemplate())
		{
			PortFlags |= PPF_ForDiffInstanceOnly;
		}
	}

	if (InProperty->ContainsInstancedObjectProperty())
	{
		PortFlags |= PPF_DeepComparison;
	}

	if (!PropertyDefaultAddress)
	{
		// no default available, fall back on the default value for our primitive:
		uint8* TempComplexPropAddr = (uint8*)FMemory::Malloc(InProperty->GetSize(), InProperty->GetMinAlignment());
		InProperty->InitializeValue(TempComplexPropAddr);
		ON_SCOPE_EXIT
		{
			InProperty->DestroyValue(TempComplexPropAddr);
			FMemory::Free(TempComplexPropAddr);
		};
				
		InProperty->ExportText_Direct(DefaultValue, TempComplexPropAddr, TempComplexPropAddr, nullptr, PortFlags);
	}
	else if ( GetArrayIndex() == INDEX_NONE && InProperty->ArrayDim > 1 )
	{
		FArrayProperty::ExportTextInnerItem(DefaultValue, InProperty, PropertyDefaultAddress, InProperty->ArrayDim,
											PropertyDefaultAddress, InProperty->ArrayDim, nullptr, PortFlags);
	}
	else
	{
		// Port flags will cause enums to display correctly
		InProperty->ExportTextItem_Direct(DefaultValue, PropertyDefaultAddress, PropertyDefaultAddress, nullptr, PortFlags);
	}

	return DefaultValue;
}

FString FPropertyNode::GetDefaultValueAsStringForObject( FPropertyItemValueDataTrackerSlate& ValueTracker, UObject* InObject, FProperty* InProperty, EValueAsStringMode Mode)
{
	check( InObject );
	check( InProperty );

	FString DefaultValue;

	// special case for Object class - no defaults to compare against
	if ( InObject != UObject::StaticClass() && InObject != UObject::StaticClass()->GetDefaultObject() )
	{
		if ( ValueTracker.IsValidTracker() && ValueTracker.HasDefaultValue() )
		{
			DefaultValue = GetDefaultValueAsString(ValueTracker.GetPropertyDefaultAddress(), InProperty, Mode, InObject);
		}
	}

	return DefaultValue;
}

FString FPropertyNode::GetDefaultValueAsString(bool bUseDisplayName)
{
	FString DefaultValue;
	FString DelimitedValue;
	bool bAllSame = true;
	const EValueAsStringMode Mode = bUseDisplayName ? EValueAsStringMode::UseDisplayName : EValueAsStringMode::None;

	FProperty* Prop = GetProperty();
	if (!Prop)
	{
		return DefaultValue;
	}

	if (const FStructurePropertyNode* StructNode = FindStructureItemParent())
	{
		TArray<TSharedPtr<FStructOnScope>> Structs;
		StructNode->GetAllStructureData(Structs);

		const bool bIsSparse = HasNodeFlags(EPropertyNodeFlags::IsSparseClassData);
		const bool bIsContainer = CastField<FArrayProperty>(Prop) || CastField<FSetProperty>(Prop) || CastField<FMapProperty>(Prop) || CastField<FOptionalProperty>(Prop);
		const bool bIsInsideContainerProperty = Property->GetOwner<FArrayProperty>() || Property->GetOwner<FSetProperty>() || Property->GetOwner<FMapProperty>() || Property->GetOwner<FOptionalProperty>();
		const FPropertyNode* BaseNode = bIsInsideContainerProperty ? GetParentNode() : this;

		FStructOnScope DefaultStruct;
		FString NodeDefaultValue;

		const uint8* LastPropertyDefaultAddress = nullptr;
		const FObjectPropertyNode* TopLevelObjectNode = StructNode->FindObjectItemParent();
		for (int32 StructIndex = 0; StructIndex < Structs.Num(); ++StructIndex)
		{
			const TSharedPtr<FStructOnScope>& StructData = Structs[StructIndex];
			if (!StructData.IsValid())
			{
				continue;
			}
			const UStruct* Struct = StructData->GetStruct();
			if (!StructData)
			{
				continue;
			}

			const uint8* PropertyDefaultAddress;
			const uint8* PropertyDefaultBaseAddress;

			const UObject* TopLevelObject = (TopLevelObjectNode && StructIndex < TopLevelObjectNode->GetNumObjects()) ? TopLevelObjectNode->GetUObject(StructIndex) : nullptr;
			if (TopLevelObject)
			{
				UObject* ObjectArchetype = FPropertyNode::GetArchetype(TopLevelObject);
				check(ObjectArchetype);

				PropertyDefaultAddress = GetValueAddress(StructNode->GetValueAddressFromObject(ObjectArchetype), bIsSparse, /*bIsStruct=*/true);
				PropertyDefaultBaseAddress = BaseNode->GetValueBaseAddress(StructNode->GetValueBaseAddressFromObject(ObjectArchetype), bIsSparse, /*bIsStruct=*/true);
			}
			else
			{
				// Make an instance of the struct to be used as default value.
				if (DefaultStruct.GetStruct() != Struct)
				{
					DefaultStruct.Initialize(Struct);
				}

				check(DefaultStruct.IsValid());

				PropertyDefaultAddress = GetValueAddress(DefaultStruct.GetStructMemory(), bIsSparse, /*bIsStruct=*/true);
				PropertyDefaultBaseAddress = BaseNode->GetValueBaseAddress(DefaultStruct.GetStructMemory(), bIsSparse, /*bIsStruct=*/true);
			}

			// If this is a container property, we must take special measures to use the base address of the property's value; for instance,
			// the array property's PropertyDefaultBaseAddress points to an FScriptArray*, while PropertyDefaultAddress points to the 
			// FScriptArray's Data pointer.
			if (bIsContainer)
			{
				PropertyDefaultAddress = PropertyDefaultBaseAddress;
			}

			if (NodeDefaultValue.IsEmpty()
				|| PropertyDefaultAddress != LastPropertyDefaultAddress)
			{
				NodeDefaultValue = GetDefaultValueAsString(PropertyDefaultAddress, Prop, Mode, TopLevelObject);
				LastPropertyDefaultAddress = PropertyDefaultAddress;
			}
			
			if (DefaultValue.IsEmpty())
			{
				DefaultValue = NodeDefaultValue;
			}

			if (DelimitedValue.Len() > 0 && NodeDefaultValue.Len() > 0)
			{
				DelimitedValue += TEXT(", ");
			}
			DelimitedValue += NodeDefaultValue;

			if (!ensureAlwaysMsgf(NodeDefaultValue == DefaultValue, TEXT("Default values differ for different objects of property '%s'. First: \"%s\", Other: \"%s\""), *Prop->GetNameCPP(), *DefaultValue, *NodeDefaultValue))
			{
				bAllSame = false;
			}
		}
	}
	else if (FObjectPropertyNode* ObjectNode = FindObjectItemParent())
	{
		// Get an iterator for the enclosing objects.
		for (int32 ObjIndex = 0; ObjIndex < ObjectNode->GetNumObjects(); ++ObjIndex)
		{
			UObject* Object = ObjectNode->GetUObject( ObjIndex );
			TSharedPtr<FPropertyItemValueDataTrackerSlate> ValueTracker = GetValueTracker(Object, ObjIndex);

			if (Object && ValueTracker.IsValid())
			{
				const FString NodeDefaultValue = GetDefaultValueAsStringForObject( *ValueTracker, Object, Prop, Mode);

				if (DefaultValue.IsEmpty())
				{
					DefaultValue = NodeDefaultValue;
				}

				if (DelimitedValue.Len() > 0 && NodeDefaultValue.Len() > 0)
				{
					DelimitedValue += TEXT(", ");
				}
				DelimitedValue += NodeDefaultValue;

				if (!ensureAlwaysMsgf(NodeDefaultValue == DefaultValue, TEXT("Default values differ for different objects of property '%s'. First: \"%s\", Other: \"%s\""), *Prop->GetNameCPP(), *DefaultValue, *NodeDefaultValue))
				{
					bAllSame = false;
				}
			}
		}
	}

	return bAllSame ? DefaultValue : DelimitedValue; 
}

FText FPropertyNode::GetResetToDefaultLabel()
{
	FString DefaultValue = GetDefaultValueAsString();
	FText OutLabel = GetDisplayName();
	if (DefaultValue.Len())
	{
		const int32 MaxValueLen = 60;

		if (DefaultValue.Len() > MaxValueLen)
		{
			DefaultValue.LeftInline( MaxValueLen, EAllowShrinking::No );
			DefaultValue += TEXT( "..." );
		}

		return FText::Format(NSLOCTEXT("FPropertyNode", "ResetToDefaultLabelFmt", "{0}: {1}"), OutLabel, FText::FromString(MoveTemp(DefaultValue)));
	}

	return OutLabel;
}

bool FPropertyNode::IsReorderable()
{
	FProperty* NodeProperty = GetProperty();
	if (NodeProperty == nullptr)
	{
		return false;
	}
	// It is reorderable if the parent is an array and metadata doesn't prohibit it
	const FArrayProperty* OuterArrayProp = NodeProperty->GetOwner<FArrayProperty>();

	static const FName Name_DisableReordering("EditFixedOrder");
	static const FName NAME_ArraySizeEnum("ArraySizeEnum");
	return OuterArrayProp != nullptr 
		&& !OuterArrayProp->HasMetaData(Name_DisableReordering)
		&& !IsEditConst()
		&& !OuterArrayProp->HasMetaData(NAME_ArraySizeEnum)
		&& !FApp::IsGame();
}

/**
 * Helper function to obtain the display name for an enum property
 * @param InEnum		The enum whose metadata to pull from
 * @param DisplayName	The name of the enum value to adjust
 *
 * @return	true if the DisplayName has been changed
 */
bool FPropertyNode::AdjustEnumPropDisplayName( UEnum *InEnum, FString& DisplayName ) const
{
	// see if we have alternate text to use for displaying the value
	FMetaData& PackageMetaData = InEnum->GetPackage()->GetMetaData();
	FName AltDisplayName = FName(*(DisplayName+TEXT(".DisplayName")));
	FString ValueText = PackageMetaData.GetValue(InEnum, AltDisplayName);
	if (ValueText.Len() > 0)
	{
		// use the alternate text for this enum value
		DisplayName = ValueText;
		return true;
	}

	//DisplayName has been unmodified
	return false;
}

/**Walks up the hierarchy and return true if any parent node is a favorite*/
bool FPropertyNode::IsChildOfFavorite (void) const
{
	for (const FPropertyNode* TestParentNode = GetParentNode(); TestParentNode != NULL; TestParentNode = TestParentNode->GetParentNode())
	{
		if (TestParentNode->HasNodeFlags(EPropertyNodeFlags::IsFavorite))
		{
			return true;
		}
	}
	return false;
}

/**
 * Destroys all node within the hierarchy
 */
void FPropertyNode::DestroyTree(const bool bInDestroySelf)
{
	if (bInDestroySelf)
	{
		bIsDestroyed = true;
	}

	// Marks all the child nodes as destroyed.
	// We cannot call DestroyTree() recursively since some UI code that gets executed
	// on the destroyed nodes (due to unfortunate update order) assume that child nodes are always available.
	for (TSharedPtr<FPropertyNode>& ChildNode : ChildNodes)
	{
		ChildNode->MarkDestroyedRecursive();
	}
	
	ChildNodes.Empty();
}

void FPropertyNode::MarkDestroyedRecursive()
{
	bIsDestroyed = true;
	
	for (TSharedPtr<FPropertyNode>& ChildNode : ChildNodes)
	{
		check(ChildNode.IsValid());
		ChildNode->MarkDestroyedRecursive();
	}
}

/**
 * Marks windows as visible based on the filter strings (EVEN IF normally NOT EXPANDED)
 */
void FPropertyNode::FilterNodes( const TArray<FString>& InFilterStrings, const bool bParentSeenDueToFiltering )
{
	if (const TSharedPtr<FPropertyNode>& KeyNode = GetPropertyKeyNode())
	{
		KeyNode->FilterNodes(InFilterStrings);						
	}

	//clear flags first.  Default to hidden
	SetNodeFlags(EPropertyNodeFlags::IsSeenDueToFiltering | EPropertyNodeFlags::IsSeenDueToChildFiltering | EPropertyNodeFlags::IsParentSeenDueToFiltering, false);
	SetNodeFlags(EPropertyNodeFlags::IsBeingFiltered, InFilterStrings.Num() > 0 );

	//FObjectPropertyNode* ParentPropertyNode = FindObjectItemParent();
	//@todo slate property window
	bool bMultiObjectOnlyShowDiffering = false;/*TopPropertyWindow->HasFlags(EPropertyWindowFlags::ShowOnlyDifferingItems) && (ParentPropertyNode->GetNumObjects()>1)*/;

	if (InFilterStrings.Num() > 0 /*|| (TopPropertyWindow->HasFlags(EPropertyWindowFlags::ShowOnlyModifiedItems)*/ || bMultiObjectOnlyShowDiffering)
	{
		//if filtering, default to NOT-seen
		bool bPassedFilter = false;	//assuming that we aren't filtered

		// Populate name aliases acceptable for searching / filtering
		FText DisplayName = GetDisplayName();
		const FString& DisplayNameStr = DisplayName.ToString();
		TArray <FString> AcceptableNames;
		AcceptableNames.Add(DisplayNameStr);

		// For containers, check if base class metadata in parent includes 'TitleProperty', add corresponding value to filter names if so.
		static const FName TitlePropertyFName = FName(TEXT("TitleProperty"));
		const TSharedPtr<FPropertyNode> ParentNode = ParentNodeWeakPtr.Pin();
		if (ParentNode && ParentNode->GetProperty())
		{
			const FString& TitleProperty = ParentNode->GetProperty()->GetMetaData(TitlePropertyFName);
			if (!TitleProperty.IsEmpty())
			{
				if (TSharedPtr<FPropertyNode> TitlePropertyNode = FindChildPropertyNode(*TitleProperty, true))
				{
					FString TitlePropertyValue;
					if (TitlePropertyNode->GetPropertyValueString(TitlePropertyValue, true /*bAllowAlternateDisplayValue*/) != FPropertyAccess::Result::Fail)
					{
						AcceptableNames.Add(TitlePropertyValue);
					}
				}
			}
		}

		// Check keywords on the property.
		static const FName KeywordPropertyName = FName(TEXT("Keywords"));
		if (GetProperty())
		{
			const FString& KeywordProperty = GetProperty()->GetMetaData(KeywordPropertyName);
			if (!KeywordProperty.IsEmpty())
			{
				AcceptableNames.Add(KeywordProperty);
			}
		}

		// Get the basic name as well of the property
		FProperty* TheProperty = GetProperty();
		if (TheProperty && (TheProperty->GetName() != DisplayNameStr))
		{
			AcceptableNames.Add(TheProperty->GetName());
		}

		bPassedFilter = IsFilterAcceptable(AcceptableNames, InFilterStrings);

		if (bPassedFilter)
		{
			SetNodeFlags(EPropertyNodeFlags::IsSeenDueToFiltering, true);
		} 
		SetNodeFlags(EPropertyNodeFlags::IsParentSeenDueToFiltering, bParentSeenDueToFiltering);
	}
	else
	{
		//indicating that this node should not be force displayed, but opened normally
		SetNodeFlags(EPropertyNodeFlags::IsParentSeenDueToFiltering, true);
	}

	// default to doing only one pass
	int32 StartRecursionPass = HasNodeFlags(EPropertyNodeFlags::IsSeenDueToFiltering) ? 1 : 0;
	//Pass 1, if a pass 1 exists (object or category), is to see if there are any children that pass the filter, if any do, trim the tree to the leaves.
	//	This will stop categories from showing ALL properties if they pass the filter AND a child passes the filter
	//Pass 0, if no child exists that passes the filter OR this node didn't pass the filter
	for (int32 RecursionPass = StartRecursionPass; RecursionPass >= 0; --RecursionPass)
	{
		for (int32 scan = 0; scan < ChildNodes.Num(); ++scan)
		{
			TSharedPtr<FPropertyNode>& ScanNode = ChildNodes[scan];
			check(ScanNode.IsValid());
			//default to telling the children this node is NOT visible, therefore if not in the base pass, only filtered nodes will survive the filtering process.
			bool bChildParamParentVisible = false;
			//if we're at the base pass, tell the children the truth about visibility
			if (RecursionPass == 0)
			{
				bChildParamParentVisible = bParentSeenDueToFiltering || HasNodeFlags(EPropertyNodeFlags::IsSeenDueToFiltering);
			}
			ScanNode->FilterNodes(InFilterStrings, bChildParamParentVisible);

			if (ScanNode->HasNodeFlags(EPropertyNodeFlags::IsSeenDueToFiltering | EPropertyNodeFlags::IsSeenDueToChildFiltering))
			{
				SetNodeFlags(EPropertyNodeFlags::IsSeenDueToChildFiltering, true);
			}
		}

		//now that we've tried a pass at our children, if any of them have been successfully seen due to filtering, just quit now
		if (HasNodeFlags(EPropertyNodeFlags::IsSeenDueToChildFiltering))
		{
			break;
		}
	}
}

void FPropertyNode::ProcessSeenFlags(const bool bParentAllowsVisible )
{
	// Set initial state first
	SetNodeFlags(EPropertyNodeFlags::IsSeen, false);
	SetNodeFlags(EPropertyNodeFlags::IsSeenDueToChildFavorite, false );

	bool bAllowChildrenVisible;

	if ( AsObjectNode() )
	{
		bAllowChildrenVisible = true;
	}
	else
	{
		//can't show children unless they are seen due to child filtering
		bAllowChildrenVisible = !!HasNodeFlags(EPropertyNodeFlags::IsSeenDueToChildFiltering);	
	}

	//process children
	for (int32 scan = 0; scan < ChildNodes.Num(); ++scan)
	{
		TSharedPtr<FPropertyNode>& ScanNode = ChildNodes[scan];
		check(ScanNode.IsValid());
		ScanNode->ProcessSeenFlags(bParentAllowsVisible && bAllowChildrenVisible );	//both parent AND myself have to allow children
	}

	if (HasNodeFlags(EPropertyNodeFlags::IsSeenDueToFiltering | EPropertyNodeFlags::IsSeenDueToChildFiltering))
	{
		SetNodeFlags(EPropertyNodeFlags::IsSeen, true); 
	}
	else 
	{
		//Finally, apply the REAL IsSeen
		SetNodeFlags(EPropertyNodeFlags::IsSeen, bParentAllowsVisible && HasNodeFlags(EPropertyNodeFlags::IsParentSeenDueToFiltering));
	}
}

/**
 * Marks windows as visible based their favorites status
 */
void FPropertyNode::ProcessSeenFlagsForFavorites(void)
{
	if( !HasNodeFlags(EPropertyNodeFlags::IsFavorite) ) 
	{
		bool bAnyChildFavorites = false;
		//process children
		for (int32 scan = 0; scan < ChildNodes.Num(); ++scan)
		{
			TSharedPtr<FPropertyNode>& ScanNode = ChildNodes[scan];
			check(ScanNode.IsValid());
			ScanNode->ProcessSeenFlagsForFavorites();
			bAnyChildFavorites = bAnyChildFavorites || ScanNode->HasNodeFlags(EPropertyNodeFlags::IsFavorite | EPropertyNodeFlags::IsSeenDueToChildFavorite);
		}
		if (bAnyChildFavorites)
		{
			SetNodeFlags(EPropertyNodeFlags::IsSeenDueToChildFavorite, true);
		}
	}
}


void FPropertyNode::NotifyPreChange( FProperty* PropertyAboutToChange, FNotifyHook* InNotifyHook )
{
	TSharedRef<FEditPropertyChain> PropertyChain = BuildPropertyChain( PropertyAboutToChange );
	NotifyPreChangeInternal(PropertyChain, PropertyAboutToChange, InNotifyHook);
}

void FPropertyNode::NotifyPreChange( FProperty* PropertyAboutToChange, FNotifyHook* InNotifyHook, const TSet<UObject*>& AffectedInstances )
{
	TSharedRef<FEditPropertyChain> PropertyChain = BuildPropertyChain( PropertyAboutToChange, AffectedInstances );
	NotifyPreChangeInternal(PropertyChain, PropertyAboutToChange, InNotifyHook);
}

void FPropertyNode::NotifyPreChange( FProperty* PropertyAboutToChange, FNotifyHook* InNotifyHook, TSet<UObject*>&& AffectedInstances )
{
	TSharedRef<FEditPropertyChain> PropertyChain = BuildPropertyChain( PropertyAboutToChange, MoveTemp(AffectedInstances) );
	NotifyPreChangeInternal(PropertyChain, PropertyAboutToChange, InNotifyHook);
}

void FPropertyNode::NotifyPreChangeInternal( TSharedRef<FEditPropertyChain> PropertyChain, FProperty* PropertyAboutToChange, FNotifyHook* InNotifyHook )
{
	// Call through to the property window's notify hook.
	if( InNotifyHook )
	{
		if ( PropertyChain->Num() == 0 )
		{
			InNotifyHook->NotifyPreChange( PropertyAboutToChange );
		}
		else
		{
			InNotifyHook->NotifyPreChange( &PropertyChain.Get() );
		}
	}

	FObjectPropertyNode* ObjectNode = FindObjectItemParent();
	if( ObjectNode )
	{
		FProperty* CurProperty = PropertyAboutToChange;

		// Call PreEditChange on the object chain.
		while ( true )
		{
			for( TPropObjectIterator Itor( ObjectNode->ObjectIterator() ) ; Itor ; ++Itor )
			{
				UObject* Object = Itor->Get();
				if ( ensure( Object ) && PropertyChain->Num() == 0 )
				{
					Object->PreEditChange( Property.Get() );

				}
				else if( ensure( Object ) )
				{
					Object->PreEditChange( *PropertyChain );
				}
			}

			// Pass this property to the parent's PreEditChange call.
			CurProperty = ObjectNode->GetStoredProperty();
			FObjectPropertyNode* PreviousObjectNode = ObjectNode;

			// Traverse up a level in the nested object tree.
			ObjectNode = NotifyFindObjectItemParent( ObjectNode );
			if ( !ObjectNode )
			{
				// We've hit the root -- break.
				break;
			}
			else if ( PropertyChain->Num() > 0 )
			{
				PropertyChain->SetActivePropertyNode( CurProperty->GetOwnerProperty() );
				for ( FPropertyNode* BaseItem = PreviousObjectNode; BaseItem && BaseItem != ObjectNode; BaseItem = BaseItem->GetParentNode())
				{
					FProperty* ItemProperty = BaseItem->GetProperty();
					if ( ItemProperty == NULL )
					{
						// if this property item doesn't have a Property, skip it...it may be a category item or the virtual
						// item used as the root for an inline object
						continue;
					}

					// skip over property window items that correspond to a single element in a static array, or
					// the inner property of another FProperty (e.g. FArrayProperty->Inner)
					if ( BaseItem->ArrayIndex == INDEX_NONE && ItemProperty->GetOwnerProperty() == ItemProperty )
					{
						PropertyChain->SetActiveMemberPropertyNode(ItemProperty);
					}
				}
			}
		}
	}

	// Broadcast the change to any listeners
	BroadcastPropertyPreChangeDelegates();
}

void FPropertyNode::NotifyPostChange( FPropertyChangedEvent& InPropertyChangedEvent, class FNotifyHook* InNotifyHook )
{
	TSharedRef<FEditPropertyChain> PropertyChain = BuildPropertyChain( InPropertyChangedEvent.Property );
	
	// remember the property that was the chain's original active property; this will correspond to the outermost property of struct/array that was modified
	FProperty* const OriginalActiveProperty = PropertyChain->GetActiveMemberNode() ? PropertyChain->GetActiveMemberNode()->GetValue() : nullptr;

	// invalidate the entire chain of objects in the hierarchy
	{
		FComplexPropertyNode* ComplexNode = FindComplexParent();
		while (ComplexNode)
		{
			ComplexNode->InvalidateCachedState();

			// FindComplexParent returns itself if the node is an object, so step up the hierarchy to get to its actual parent object
			FPropertyNode* CurrentParent = ComplexNode->GetParentNode();
			ComplexNode = CurrentParent != nullptr ? CurrentParent->FindComplexParent() : nullptr;
		}
	}

	FObjectPropertyNode* ObjectNode = FindObjectItemParent();
	if( ObjectNode )
	{
		TWeakPtr<FObjectPropertyNode> ObjectNodeAsWeakPtr = ObjectNode->SharedThis<FObjectPropertyNode>(ObjectNode);
		TWeakPtr<FPropertyNode> ThisAsWeakPtr = AsShared();

		FProperty* CurProperty = InPropertyChangedEvent.Property;

		// Fire ULevel::LevelDirtiedEvent when falling out of scope.
		FScopedLevelDirtied	LevelDirtyCallback;

		// Call PostEditChange on the object chain.
		while ( true )
		{
			TArray<FString> ObjectPaths;
			TArray<TWeakObjectPtr<UObject>> WeakObjects;
			// It's possible that PostEditChangeProperty may cause a construction script to re-run
			// which will invalidate the PropObjectIterator. We need to instead cache all of the objects
			// before emitting any change events to ensure there is a PostChange for every PreChange.
			for (TPropObjectIterator Itor(ObjectNode->ObjectIterator()); Itor; ++Itor)
			{
				WeakObjects.Add(*Itor);
				ObjectPaths.Add((*Itor)->GetPathName());
			}

			for (int32 CurrentObjectIndex = 0; CurrentObjectIndex < WeakObjects.Num(); ++CurrentObjectIndex)
			{
				UObject* Object = WeakObjects[CurrentObjectIndex].Get();
				if (Object == nullptr)
				{
					// If our weak pointer has gone out of scope, it means that a prior object has destroyed it, 
					// eg. by causing a blueprint construction script to run (which is triggered by PostEditChangeProperty())
					// Find a new copy now.
					Object = FindObject<UObject>(nullptr, *ObjectPaths[CurrentObjectIndex]);
					if (Object == nullptr)
					{
						continue;
					}
				}

				// Use a scope to ensure that only local variable are use in the loop.
				// Since this object can be destroyed in this loop.
				auto ScopePostEditChange = [&PropertyChain, &InPropertyChangedEvent, &CurProperty, CurrentObjectIndex](UObject* Object)
				{
					// copy the property changed event
					FPropertyChangedEvent ChangedEvent = InPropertyChangedEvent;
					if (CurProperty != InPropertyChangedEvent.Property)
					{
						ChangedEvent.Property = CurProperty;
						ChangedEvent.MemberProperty = CurProperty;
					}
					ChangedEvent.ObjectIteratorIndex = CurrentObjectIndex;

					if (PropertyChain->Num() == 0)
					{
						Object->PostEditChangeProperty(ChangedEvent);
					}
					else
					{
						FPropertyChangedChainEvent ChainEvent(*PropertyChain, ChangedEvent);
						ChainEvent.ObjectIteratorIndex = CurrentObjectIndex;

						Object->PostEditChangeChainProperty(ChainEvent);
					}
				};

				ScopePostEditChange(Object);
				LevelDirtyCallback.Request();
			}

			if (!ThisAsWeakPtr.IsValid())
			{
				UE_LOGF(LogPropertyNode, Error, "The FPropertyNode was destroy while processing the PostEditChangeProperty or PostEditChangeChainProperty.");
				// Redraw viewports
				FEditorSupportDelegates::RedrawAllViewports.Broadcast();
				return;
			}

			if (!ObjectNodeAsWeakPtr.IsValid())
			{
				ObjectNode = nullptr;
				UE_LOGF(LogPropertyNode, Error, "Object for property '%ls, was valid before the PostEditChange callback and now it's invalid", *Property->GetName());
				break;
			}

			// Pass this property to the parent's PostEditChange call.
			CurProperty = ObjectNode->GetStoredProperty();
			FObjectPropertyNode* PreviousObjectNode = ObjectNode;

			// Traverse up a level in the nested object tree.
			ObjectNode = NotifyFindObjectItemParent( ObjectNode );
			if ( !ObjectNode )
			{
				// We've hit the root -- break.
				break;
			}
			else if ( PropertyChain->Num() > 0 )
			{
				PropertyChain->SetActivePropertyNode(CurProperty->GetOwnerProperty());
				for ( FPropertyNode* BaseItem = PreviousObjectNode; BaseItem && BaseItem != ObjectNode; BaseItem = BaseItem->GetParentNode())
				{
					FProperty* ItemProperty = BaseItem->GetProperty();
					if ( ItemProperty == NULL )
					{
						// if this property item doesn't have a Property, skip it...it may be a category item or the virtual
						// item used as the root for an inline object
						continue;
					}

					// skip over property window items that correspond to a single element in a static array, or
					// the inner property of another FProperty (e.g. FArrayProperty->Inner)
					if ( BaseItem->GetArrayIndex() == INDEX_NONE && ItemProperty->GetOwnerProperty() == ItemProperty )
					{
						PropertyChain->SetActiveMemberPropertyNode(ItemProperty);
					}
				}
			}
		}
	}

	// Broadcast the change to any listeners
	BroadcastPropertyChangedDelegates(InPropertyChangedEvent);
	BroadcastPropertyChangedDelegates();

	// Reset these values
	if (PropertyChain->Num() > 0)
	{
		PropertyChain->SetActiveMemberPropertyNode(OriginalActiveProperty);
		PropertyChain->SetActivePropertyNode(InPropertyChangedEvent.Property);
	}

	// Call through to the property window's notify hook.
	if( InNotifyHook )
	{
		if ( PropertyChain->Num() == 0 )
		{
			InNotifyHook->NotifyPostChange( InPropertyChangedEvent, InPropertyChangedEvent.Property );
		}
		else
		{
			InPropertyChangedEvent.SetActiveMemberProperty(OriginalActiveProperty);
			InNotifyHook->NotifyPostChange( InPropertyChangedEvent, &PropertyChain.Get() );
		}
	}


	// For each Property in the Property Chain, see if it has ForceRebuildProperty metadata and find the sibling PropertyNode to rebuild.
	// To do that, we need to match up the FPropertyNode (Editor representation) with the FProperty (Engine representation)
	if(FindObjectItemParent() != nullptr)
	{
		TSharedPtr<FPropertyNode> CurrentPropertyNode = FindObjectItemParent()->AsShared();
		for (auto PropertyChainNode = PropertyChain->GetActiveMemberNode(); PropertyChainNode && CurrentPropertyNode.IsValid() ; PropertyChainNode = PropertyChainNode->GetNextNode())
		{
			if (const FProperty* CurrentProperty = PropertyChainNode->GetValue())
			{
				const static FName NAME_ForceRebuildProperty(TEXT("ForceRebuildProperty"));
				const FString& ForceRebuildPropertyName = CurrentProperty->GetMetaData(NAME_ForceRebuildProperty);
				if (!ForceRebuildPropertyName.IsEmpty())
				{
					constexpr bool bRecursive = true;
					TSharedPtr<FPropertyNode> ForceRebuildNode = CurrentPropertyNode->FindChildPropertyNode(FName(*ForceRebuildPropertyName, FNAME_Find), bRecursive);

					if (ForceRebuildNode.IsValid())
					{
						ForceRebuildNode->RequestRebuildChildren();
					}
					else
					{
						UE_LOGF(LogPropertyNode, Error, "Could not find named property '%ls' referenced from %ls ForceRebuildProperty", *ForceRebuildPropertyName, *CurrentPropertyNode->GetDisplayName().ToString());
					}
				}
				else if (CurrentProperty->HasMetaData(NAME_ForceRebuildProperty))
				{
					CurrentPropertyNode->RequestRebuildChildren();
				}

				CurrentPropertyNode = CurrentPropertyNode->FindChildPropertyNode(CurrentProperty->GetFName());
			}
		}
	}

	// The value has changed so the cached value could be invalid
	// Need to recurse here as we might be editing a struct with child properties that need re-caching
	ClearCachedReadAddresses(true);

	// Redraw viewports
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
}


void FPropertyNode::BroadcastPropertyChangedDelegates()
{
	PropertyValueChangedEvent.Broadcast();

	// Walk through the parents and broadcast
	FPropertyNode* LocalParentNode = GetParentNode();
	while( LocalParentNode )
	{
		if( LocalParentNode->OnChildPropertyValueChanged().IsBound() )
		{
			LocalParentNode->OnChildPropertyValueChanged().Broadcast();
		}

		LocalParentNode = LocalParentNode->GetParentNode();
	}
}

void FPropertyNode::BroadcastPropertyChangedDelegates(const FPropertyChangedEvent& Event)
{
	PropertyValueChangedDelegate.Broadcast(Event);

	// Walk through the parents and broadcast
	FPropertyNode* LocalParentNode = GetParentNode();
	while( LocalParentNode )
	{
		if( LocalParentNode->OnChildPropertyValueChangedWithData().IsBound() )
		{
			LocalParentNode->OnChildPropertyValueChangedWithData().Broadcast(Event);
		}

		LocalParentNode = LocalParentNode->GetParentNode();
	}
}

void FPropertyNode::BroadcastPropertyPreChangeDelegates()
{
	PropertyValuePreChangeEvent.Broadcast();

	// Walk through the parents and broadcast
	FPropertyNode* LocalParentNode = GetParentNode();
	while (LocalParentNode)
	{
		if (LocalParentNode->OnChildPropertyValuePreChange().IsBound())
		{
			LocalParentNode->OnChildPropertyValuePreChange().Broadcast();
		}

		LocalParentNode = LocalParentNode->GetParentNode();
	}

}

void FPropertyNode::BroadcastPropertyResetToDefault()
{
	PropertyResetToDefaultEvent.Broadcast();
}

void FPropertyNode::GetExpandedChildPropertyPaths(TSet<FString>& OutExpandedChildPropertyPaths) const
{
	TArray<const FPropertyNode*> RecursiveStack;
	RecursiveStack.Add(this);

	do
	{
		const FPropertyNode* SearchNode = RecursiveStack.Pop();
		if (SearchNode->HasNodeFlags(EPropertyNodeFlags::Expanded))
		{
			OutExpandedChildPropertyPaths.Add(SearchNode->PropertyPath);

			for (int32 Index = 0; Index < SearchNode->GetNumChildNodes(); ++Index)
			{
				TSharedPtr<FPropertyNode> ChildNode = SearchNode->GetChildNode(Index);
				if (ChildNode.IsValid())
				{
					RecursiveStack.Push(ChildNode.Get());
				}
			}
		}
	} while (RecursiveStack.Num() > 0);
}

void FPropertyNode::SetExpandedChildPropertyNodes(const TSet<FString>& InNodesToExpand)
{
	TArray<FPropertyNode*> RecursiveStack;
	RecursiveStack.Add(this);

	do
	{
		FPropertyNode* SearchNode = RecursiveStack.Pop();
		if (InNodesToExpand.Contains(SearchNode->PropertyPath))
		{
			SearchNode->SetNodeFlags(EPropertyNodeFlags::Expanded, true);

			// Lets recurse over this nodes children to see if they need to be expanded
			for (int32 Index = 0; Index < SearchNode->GetNumChildNodes(); ++Index)
			{
				TSharedPtr<FPropertyNode> ChildNode = SearchNode->GetChildNode(Index);
				if (ChildNode.IsValid())
				{
					RecursiveStack.Push(ChildNode.Get());
				}
			}
		}
		else
		{
			// Collapse the target node if its not within the list of expanded nodes.
			SearchNode->SetNodeFlags(EPropertyNodeFlags::Expanded, false);
		}
	} while (RecursiveStack.Num() > 0);
}

void FPropertyNode::SetIgnoreInstancedReference()
{
	bIgnoreInstancedReference = true;
}

bool FPropertyNode::IsIgnoringInstancedReference() const
{
	return bIgnoreInstancedReference;
}

bool FPropertyNode::IsDestroyed() const
{
	return bIsDestroyed;
}

FDelegateHandle FPropertyNode::SetOnRebuildChildren(const FSimpleDelegate& InOnRebuildChildren)
{
	return OnRebuildChildrenEvent.Add(InOnRebuildChildren);
}

TSharedPtr< FPropertyItemValueDataTrackerSlate > FPropertyNode::GetValueTracker( UObject* Object, uint32 ObjIndex )
{
	ensure( AsItemPropertyNode() );

	TSharedPtr< FPropertyItemValueDataTrackerSlate > RetVal;

	if( Object && Object != UObject::StaticClass() && Object != UObject::StaticClass()->GetDefaultObject() )
	{
		if( !ObjectDefaultValueTrackers.IsValidIndex(ObjIndex) )
		{
			uint32 NumToAdd = (ObjIndex - ObjectDefaultValueTrackers.Num()) + 1;
			while( NumToAdd > 0 )
			{
				ObjectDefaultValueTrackers.Add( TSharedPtr<FPropertyItemValueDataTrackerSlate> () );
				--NumToAdd;
			}
		}

		TSharedPtr<FPropertyItemValueDataTrackerSlate>& ValueTracker = ObjectDefaultValueTrackers[ObjIndex];
		if( !ValueTracker.IsValid() )
		{
			ValueTracker = MakeShareable( new FPropertyItemValueDataTrackerSlate( this, Object ) );
		}
		else
		{
			ValueTracker->Reset(this, Object);
		}
		RetVal = ValueTracker;

	}

	return RetVal;

}

TSharedRef<FEditPropertyChain> FPropertyNode::BuildPropertyChain( FProperty* InProperty ) const
{
	TSharedRef<FEditPropertyChain> PropertyChain( MakeShareable( new FEditPropertyChain ) );

	const FPropertyNode* ItemNode = this;

	const FComplexPropertyNode* ComplexNode = FindObjectItemParent();
	FProperty* MemberProperty = InProperty;

	do
	{
		if (ItemNode == ComplexNode && PropertyChain->GetHead())
		{
			MemberProperty = PropertyChain->GetHead()->GetValue();
		}

		FProperty* TheProperty	= ItemNode->Property.Get();
		if ( TheProperty )
		{
			// Skip over property window items that correspond to a single element in a static array,
			// or the inner property of another FProperty (e.g. FArrayProperty->Inner).
			if ( ItemNode->GetArrayIndex() == INDEX_NONE && TheProperty->GetOwnerProperty() == TheProperty )
			{
				PropertyChain->AddHead( TheProperty );
			}
		}
		ItemNode = ItemNode->GetParentNode();
	} 
	while( ItemNode != NULL );

	// If the modified property was a property of the object at the root of this property window, the member property will not have been set correctly
	if (ItemNode == ComplexNode && PropertyChain->GetHead())
	{
		MemberProperty = PropertyChain->GetHead()->GetValue();
	}

	PropertyChain->SetActivePropertyNode( InProperty );
	PropertyChain->SetActiveMemberPropertyNode( MemberProperty );

	return PropertyChain;
}

TSharedRef<FEditPropertyChain> FPropertyNode::BuildPropertyChain( FProperty* InProperty, const TSet<UObject*>& InAffectedArchetypeInstances ) const
{
	TSharedRef<FEditPropertyChain> PropertyChain = BuildPropertyChain(InProperty);
	PropertyChain->SetAffectedArchetypeInstances(InAffectedArchetypeInstances);
	return PropertyChain;
}

TSharedRef<FEditPropertyChain> FPropertyNode::BuildPropertyChain( FProperty* InProperty, TSet<UObject*>&& InAffectedArchetypeInstances ) const
{
	TSharedRef<FEditPropertyChain> PropertyChain = BuildPropertyChain(InProperty);
	PropertyChain->SetAffectedArchetypeInstances(MoveTemp(InAffectedArchetypeInstances));
	return PropertyChain;
}

FPropertyChangedEvent& FPropertyNode::FixPropertiesInEvent(FPropertyChangedEvent& Event)
{
	ensure(Event.Property);

	TSharedRef<FEditPropertyChain> PropertyChain = BuildPropertyChain(Event.Property);
	auto MemberProperty = PropertyChain->GetActiveMemberNode() ? PropertyChain->GetActiveMemberNode()->GetValue() : NULL;
	if (ensure(MemberProperty))
	{
		Event.SetActiveMemberProperty(MemberProperty);
	}

	return Event;
}

void FPropertyNode::SetInstanceMetaData(const FName& Key, const FString& Value)
{
	InstanceMetaData.Add(Key, Value);
}

const FString* FPropertyNode::GetInstanceMetaData(const FName& Key) const
{
	return InstanceMetaData.Find(Key);
}

const TMap<FName, FString>* FPropertyNode::GetInstanceMetaDataMap() const
{
	return &InstanceMetaData;
}

bool FPropertyNode::ParentOrSelfHasMetaData(const FName& MetaDataKey) const
{
	if (Property.IsValid() && Property->HasMetaData(MetaDataKey))
	{
		return true;
	}
	
	const TSharedPtr<FPropertyNode> ParentNode = ParentNodeWeakPtr.Pin();
	if (ParentNode.IsValid() && ParentNode->ParentOrSelfHasMetaData(MetaDataKey))
	{
		return true;
	}

	return false;
}

FProperty* FPropertyNode::GetMetaDataProperty()
{
	FProperty* MetaDataProperty = GetProperty();

	// If we are part of an array, we need to take our meta-data from the array property
	if (GetArrayIndex() != INDEX_NONE)
	{
		FPropertyNode* ParentNode = GetParentNode();
		if (ParentNode)
		{
			MetaDataProperty = ParentNode->GetProperty();
		}
	}
	
	return MetaDataProperty;
}

void FPropertyNode::InvalidateCachedState()
{
	bUpdateDiffersFromDefault = true;
	bUpdateEditConstState = true;

	for( TSharedPtr<FPropertyNode>& ChildNode : ChildNodes )
	{
		ChildNode->InvalidateCachedState();
	}
}

/**
 * Does the string compares to ensure this Name is acceptable to the filter that is passed in
 * @return		Return True if this property should be displayed.  False if it should be culled
 */
bool FPropertyNode::IsFilterAcceptable(const TArray<FString>& InAcceptableNames, const TArray<FString>& InFilterStrings)
{
	bool bCompleteMatchFound = true;
	if (InFilterStrings.Num())
	{
		//we have to make sure one name matches all criteria
		for (int32 TestNameIndex = 0; TestNameIndex < InAcceptableNames.Num(); ++TestNameIndex)
		{
			bCompleteMatchFound = true;

			FString TestName = InAcceptableNames[TestNameIndex];
			for (int32 scan = 0; scan < InFilterStrings.Num(); scan++)
			{
				if (!TestName.Contains(InFilterStrings[scan])) 
				{
					bCompleteMatchFound = false;
					break;
				}
			}
			if (bCompleteMatchFound)
			{
				break;
			}
		}
	}
	return bCompleteMatchFound;
}

void FPropertyNode::PropagateContainerPropertyChange( UObject* ModifiedObject, const void* OriginalContainerAddr, EPropertyArrayChangeType::Type ChangeType, int32 Index, int32 SwapIndex /*= INDEX_NONE*/)
{
	TArray<UObject*> AffectedInstances;
	GatherInstancesAffectedByContainerPropertyChange(ModifiedObject, OriginalContainerAddr, ChangeType, AffectedInstances);
	PropagateContainerPropertyChange(ModifiedObject, OriginalContainerAddr, AffectedInstances, ChangeType, Index, SwapIndex);
}

void FPropertyNode::GatherInstancesAffectedByContainerPropertyChange(UObject* ModifiedObject, const void* OriginalContainerAddr, EPropertyArrayChangeType::Type ChangeType, TArray<UObject*>& OutAffectedInstances)
{
	check(OriginalContainerAddr);

	FProperty* NodeProperty = GetProperty();

	FPropertyNode* ParentPropertyNode = GetParentNode();
	
	FComplexPropertyNode* ComplexParentNode = FindComplexParent();

	FProperty* ConvertedProperty = NULL;

	if (ChangeType == EPropertyArrayChangeType::Add || ChangeType == EPropertyArrayChangeType::Clear)
	{
		ConvertedProperty = NodeProperty;
	}
	else
	{
		ConvertedProperty = NodeProperty->GetOwner<FProperty>();
	}

	TArray<UObject*> ArchetypeInstances, ObjectsToChange;
	FPropertyNode* SubobjectPropertyNode = NULL;
	UObject* Object = ModifiedObject;

	if (Object->HasAnyFlags(RF_ClassDefaultObject|RF_ArchetypeObject))
	{
		// Object is a default subobject, collect all instances.
		Object->GetArchetypeInstances(ArchetypeInstances);
	}
	else if (Object->HasAnyFlags(RF_DefaultSubObject) && Object->GetOuter()->HasAnyFlags(RF_ClassDefaultObject|RF_ArchetypeObject))
	{
		// Object is a default subobject of a default object. Get the subobject property node and use its owner instead.
		for (SubobjectPropertyNode = FindObjectItemParent(); SubobjectPropertyNode && !SubobjectPropertyNode->GetProperty(); SubobjectPropertyNode = SubobjectPropertyNode->GetParentNode());
		if (SubobjectPropertyNode != NULL)
		{
			// Switch the object to the owner default object and collect its instances.
			Object = Object->GetOuter();	
			Object->GetArchetypeInstances(ArchetypeInstances);
		}
	}

	ObjectsToChange.Push(Object);

	while (ObjectsToChange.Num() > 0)
	{
		check(ObjectsToChange.Num() > 0);

		// Pop the first object to change
		UObject* ObjToChange = ObjectsToChange[0];
		UObject* ActualObjToChange = NULL;
		ObjectsToChange.RemoveAt(0);
		
		if (SubobjectPropertyNode)
		{
			// If the original object is a subobject, get the current object's subobject too.
			// In this case we're not going to modify ObjToChange but its default subobject.
			ActualObjToChange = *(UObject**)SubobjectPropertyNode->GetValueBaseAddressFromObject(ObjToChange);
		}
		else
		{
			ActualObjToChange = ObjToChange;
		}

		if (ActualObjToChange != ModifiedObject)
		{
			uint8* Addr = NULL;

			if (ChangeType == EPropertyArrayChangeType::Add || ChangeType == EPropertyArrayChangeType::Clear)
			{
				Addr = GetValueBaseAddressFromObject(ActualObjToChange);
			}
			else
			{
				Addr = ParentPropertyNode->GetValueBaseAddressFromObject(ActualObjToChange);
			}

			if (Addr != nullptr)
			{
				if (OriginalContainerAddr == Addr)
				{
					if (HasNodeFlags(EPropertyNodeFlags::IsSparseClassData) || (ComplexParentNode && ComplexParentNode->AsStructureNode()))
					{
						// SparseClassData and StructureNodes will always return the same address from GetValueBaseAddressFromObject 
						// (see FPropertyNode::GetStartAddressFromObject and FStructurePropertyNode::GetValueBaseAddress)
						continue;
					}

					checkf(false, TEXT("PropagateContainerPropertyChange tried to propagate a change onto itself!"));
				}

				const bool bIsDefaultContainerContent = ConvertedProperty->Identical(OriginalContainerAddr, Addr, PPF_DeepComparison);
				if (bIsDefaultContainerContent)
				{
					OutAffectedInstances.Add(ActualObjToChange);
				}
			}
		}

		for (int32 i=0; i < ArchetypeInstances.Num(); ++i)
		{
			UObject* Obj = ArchetypeInstances[i];

			if (GetArchetype(Obj) == ObjToChange)
			{
				ObjectsToChange.Push(Obj);
				ArchetypeInstances.RemoveAt(i--);
			}
		}
	}
}

void FPropertyNode::DuplicateArrayEntry(UObject* ModifiedObject, FProperty* NodeProperty, FScriptArrayHelper& ArrayHelper, int32 Index)
{
	ArrayHelper.InsertValues(Index);

	void* SrcAddress = ArrayHelper.GetRawPtr(Index + 1);
	void* DestAddress = ArrayHelper.GetRawPtr(Index);

	check(SrcAddress && DestAddress);

	// Copy the selected item's value to the new item.
	NodeProperty->CopyCompleteValue(DestAddress, SrcAddress);

	bool bHasDynamicInstancedReferences = false;
	if (ModifiedObject && ModifiedObject->GetClass()->ShouldUseDynamicSubobjectInstancing())
	{
		bHasDynamicInstancedReferences = true;
	}

	auto IsDynamicInstancedReference = [bHasDynamicInstancedReferences, ModifiedObject](const UObject* ReferencedObject)
	{
		return bHasDynamicInstancedReferences && ReferencedObject->IsInOuter(ModifiedObject);
	};

	// If this is a property with instanced references within it the new entry will reference the old subobjects
	// Go through and duplicate the subobjects so that each new entry has unique instances
	FPropertyVisitorPath Path;
	Path.Push(FPropertyVisitorInfo(NodeProperty));
	FPropertyVisitorData Data(SrcAddress, nullptr);
	FPropertyVisitorContext VisitorContext(Path, Data, FPropertyVisitorContext::EScope::ObjectRefs);
	NodeProperty->Visit(VisitorContext, [&IsDynamicInstancedReference](const FPropertyVisitorContext& Context) -> EPropertyVisitorControlFlow
	{
		if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Context.Path.Top().Property))
		{
			UObject* ReferencedObject = ObjectProperty->GetObjectPropertyValue(Context.Data.PropertyData);
			if (ReferencedObject && (ObjectProperty->HasAnyPropertyFlags(CPF_InstancedReference) || IsDynamicInstancedReference(ReferencedObject)))
			{
				// Make a deep copy
				UObject* DuplicatedObject = DuplicateObject(ReferencedObject, ReferencedObject->GetOuter());
				ObjectProperty->SetObjectPropertyValue(Context.Data.PropertyData, DuplicatedObject);
			}
			
			// Continue on to the next property/item.
			return EPropertyVisitorControlFlow::StepOver;
		}

		// In the case of container/optional properties, this will recurse into any inner value(s).
		return EPropertyVisitorControlFlow::StepInto;
	});
}

void FPropertyNode::PropagateContainerPropertyChange( UObject* ModifiedObject, const void* OriginalContainerAddr, const TArray<UObject*>& AffectedInstances, EPropertyArrayChangeType::Type ChangeType, int32 Index, int32 SwapIndex /*= INDEX_NONE*/)
{
	check(OriginalContainerAddr);

	FProperty* NodeProperty = GetProperty();

	FPropertyNode* ParentPropertyNode = GetParentNode();
	
	FProperty* ConvertedProperty = NULL;

	if (ChangeType == EPropertyArrayChangeType::Add || ChangeType == EPropertyArrayChangeType::Clear)
	{
		ConvertedProperty = NodeProperty;
	}
	else
	{
		ConvertedProperty = NodeProperty->GetOwner<FProperty>();
	}

	FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ConvertedProperty);
	FSetProperty* SetProperty = CastField<FSetProperty>(ConvertedProperty);
	FMapProperty* MapProperty = CastField<FMapProperty>(ConvertedProperty);

	check(ArrayProperty || SetProperty || MapProperty);

	FPropertyNode* SubobjectPropertyNode = NULL;

	UObject* Object = ModifiedObject;

	if (Object->HasAnyFlags(RF_ClassDefaultObject|RF_ArchetypeObject))
	{
		// Object is a default subobject
	}
	else if (Object->HasAnyFlags(RF_DefaultSubObject) && Object->GetOuter()->HasAnyFlags(RF_ClassDefaultObject|RF_ArchetypeObject))
	{
		// Object is a default subobject of a default object. Get the subobject property node and use its owner instead.
		for (SubobjectPropertyNode = FindObjectItemParent(); SubobjectPropertyNode && !SubobjectPropertyNode->GetProperty(); SubobjectPropertyNode = SubobjectPropertyNode->GetParentNode());
		if (SubobjectPropertyNode != NULL)
		{
			// Switch the object to the owner default object
			Object = Object->GetOuter();	
		}
	}
	
	for (UObject* InstanceToChange : AffectedInstances)
	{
		uint8* Addr = NULL;

		if (ChangeType == EPropertyArrayChangeType::Add || ChangeType == EPropertyArrayChangeType::Clear)
		{
			Addr = GetValueBaseAddressFromObject(InstanceToChange);
		}
		else
		{
			Addr = ParentPropertyNode->GetValueBaseAddressFromObject(InstanceToChange);
		}

		if (ArrayProperty)
		{
			FScriptArrayHelper ArrayHelper(ArrayProperty, Addr);

			int32 ElementToInitialize = -1;
			switch (ChangeType)
			{
			case EPropertyArrayChangeType::Add:
				ElementToInitialize = ArrayHelper.AddValue();
				break;
			case EPropertyArrayChangeType::Clear:
				ArrayHelper.EmptyValues();
				break;
			case EPropertyArrayChangeType::Insert:
				ArrayHelper.InsertValues(ArrayIndex, 1);
				ElementToInitialize = ArrayIndex;
				break;
			case EPropertyArrayChangeType::Delete:
				ArrayHelper.RemoveValues(ArrayIndex, 1);
				break;
			case EPropertyArrayChangeType::Duplicate:
				DuplicateArrayEntry(InstanceToChange, NodeProperty, ArrayHelper, ArrayIndex);
				break;
			case EPropertyArrayChangeType::Swap:
				if (SwapIndex != INDEX_NONE)
				{
					ArrayHelper.SwapValues(Index, SwapIndex);
				}
				break;
			}
		}	// End Array

		else if (SetProperty)
		{
			FScriptSetHelper SetHelper(SetProperty, Addr);

			int32 ElementToInitialize = -1;
			switch (ChangeType)
			{
			case EPropertyArrayChangeType::Add:
				ElementToInitialize = SetHelper.AddDefaultValue_Invalid_NeedsRehash();
				SetHelper.Rehash();
				break;
			case EPropertyArrayChangeType::Clear:
				SetHelper.EmptyElements();
				break;
			case EPropertyArrayChangeType::Insert:
				check(false);	// Insert is not supported for sets
				break;
			case EPropertyArrayChangeType::Delete:
				SetHelper.RemoveAt(SetHelper.FindInternalIndex(ArrayIndex));
				SetHelper.Rehash();
				break;
			case EPropertyArrayChangeType::Duplicate:
				check(false);	// Duplicate not supported on sets
				break;
			}
		}	// End Set
		else if (MapProperty)
		{
			FScriptMapHelper MapHelper(MapProperty, Addr);

			// Check if the original value was the default value and change it only then
			int32 ElementToInitialize = -1;
			switch (ChangeType)
			{
			case EPropertyArrayChangeType::Add:
				ElementToInitialize = MapHelper.AddDefaultValue_Invalid_NeedsRehash();
				MapHelper.Rehash();
				break;
			case EPropertyArrayChangeType::Clear:
				MapHelper.EmptyValues();
				break;
			case EPropertyArrayChangeType::Insert:
				check(false);	// Insert is not supported for maps
				break;
			case EPropertyArrayChangeType::Delete:
				MapHelper.RemoveAt(MapHelper.FindInternalIndex(ArrayIndex));
				MapHelper.Rehash();
				break;
			case EPropertyArrayChangeType::Duplicate:
				check(false);	// Duplicate is not supported for maps
				break;
			}
		}	// End Map
	}
}

void FPropertyNode::PropagatePropertyChange( UObject* ModifiedObject, const TCHAR* NewValue, const FString& PreviousValue )
{
	TArray<UObject*> ArchetypeInstances, ObjectsToChange;
	FPropertyNode* SubobjectPropertyNode = NULL;
	UObject* Object = ModifiedObject;

	if (HasNodeFlags(EPropertyNodeFlags::IsSparseClassData))
	{
		// Propagate only to child types with the CDO serving as a 'dummy' object to identify
		// the class (and consequently edit the SCD)
		if (ensure(Object->HasAnyFlags(RF_ClassDefaultObject)))
		{
			TArray<UClass*> Children;
			GetDerivedClasses(Object->GetClass(), Children);
			for (UClass* ChildClass : Children)
			{
				if (ChildClass->GetDefaultObject(false) &&
					!ChildClass->GetPackage()->HasAnyFlags(RF_Transient))
				{
					ArchetypeInstances.Add(ChildClass->GetDefaultObject(false));
				}
			}
		}
	}
	else if (Object->HasAnyFlags(RF_ClassDefaultObject|RF_ArchetypeObject))
	{
		// Object is a default subobject, collect all instances.
		Object->GetArchetypeInstances(ArchetypeInstances);
	}
	else if (Object->HasAnyFlags(RF_DefaultSubObject) && Object->GetOuter()->HasAnyFlags(RF_ClassDefaultObject|RF_ArchetypeObject))
	{
		// Object is a default subobject of a default object. Get the subobject property node and use its owner instead.
		for (SubobjectPropertyNode = FindObjectItemParent(); SubobjectPropertyNode && !SubobjectPropertyNode->GetProperty(); SubobjectPropertyNode = SubobjectPropertyNode->GetParentNode());
		if (SubobjectPropertyNode != NULL)
		{
			// Switch the object to the owner default object and collect its instances.
			Object = Object->GetOuter();	
			Object->GetArchetypeInstances(ArchetypeInstances);
		}
	}

	static const FName FNAME_EditableWhenInherited = GET_MEMBER_NAME_CHECKED(UActorComponent,bEditableWhenInherited);
	if (GetProperty()->GetFName() == FNAME_EditableWhenInherited && ModifiedObject->IsA<UActorComponent>() && FString(TEXT("False")) == NewValue)
	{
		FBlueprintEditorUtils::HandleDisableEditableWhenInherited(ModifiedObject, ArchetypeInstances);
	}

	FPropertyNode*  Parent          = GetParentNode();
	FProperty*      ParentProp      = Parent->GetProperty();
	FArrayProperty* ParentArrayProp = CastField<FArrayProperty>(ParentProp);
	FMapProperty*   ParentMapProp   = CastField<FMapProperty>(ParentProp);
	FSetProperty*	ParentSetProp	= CastField<FSetProperty>(ParentProp);
	FProperty*      Prop            = GetProperty();

	if (ParentArrayProp && ParentArrayProp->Inner != Prop)
	{
		ParentArrayProp = nullptr;
	}

	if (ParentMapProp && ParentMapProp->KeyProp != Prop && ParentMapProp->ValueProp != Prop)
	{
		ParentMapProp = nullptr;
	}

	if (ParentSetProp && ParentSetProp->ElementProp != Prop)
	{
		ParentSetProp = nullptr;
	}

	ObjectsToChange.Push(Object);

	while (ObjectsToChange.Num() > 0)
	{
		check(ObjectsToChange.Num() > 0);

		// Pop the first object to change
		UObject* ObjToChange = ObjectsToChange[0];
		UObject* ActualObjToChange = NULL;
		ObjectsToChange.RemoveAt(0);
		
		if (SubobjectPropertyNode)
		{
			// If the original object is a subobject, get the current object's subobject too.
			// In this case we're not going to modify ObjToChange but its default subobject.
			ActualObjToChange = *(UObject**)SubobjectPropertyNode->GetValueBaseAddressFromObject(ObjToChange);
		}
		else
		{
			ActualObjToChange = ObjToChange;
		}

		if (ActualObjToChange != ModifiedObject)
		{
			uint8* DestSimplePropAddr = GetValueBaseAddressFromObject(ActualObjToChange);
			if (DestSimplePropAddr != nullptr)
			{
				FProperty* ComplexProperty = Prop;
				TSharedPtr<FPropertyNode> ComplexPropertyNode = AsShared();
				if (ParentArrayProp || ParentMapProp || ParentSetProp)
				{
					ComplexProperty = ParentProp;
					ComplexPropertyNode = ParentNodeWeakPtr.Pin();
				}
				
				const uint8* DestComplexPropAddr = ComplexPropertyNode->GetValueBaseAddressFromObject(ActualObjToChange);

				FString ActualCurrentValue;
				ComplexProperty->ExportText_Direct(ActualCurrentValue, DestComplexPropAddr, DestComplexPropAddr, ActualObjToChange, PPF_ForDiff);

				const bool bShouldImport = ActualCurrentValue.Equals(PreviousValue, ESearchCase::CaseSensitive);

				// Only import if the value matches the previous value of the property that changed
				if (bShouldImport)
				{
					Prop->ImportText_Direct(NewValue, DestSimplePropAddr, ActualObjToChange, PPF_InstanceSubobjects);
				}
			}
		}

		for (int32 InstanceIndex = 0; InstanceIndex < ArchetypeInstances.Num(); ++InstanceIndex)
		{
			UObject* Obj = ArchetypeInstances[InstanceIndex];

			if (GetArchetype(Obj) == ObjToChange)
			{
				ObjectsToChange.Push(Obj);
				ArchetypeInstances.RemoveAt(InstanceIndex--);
			}
		}
	}
}

void FPropertyNode::AddRestriction( TSharedRef<const class FPropertyRestriction> Restriction )
{
	Restrictions.AddUnique(Restriction);
}

bool FPropertyNode::IsHidden(const FString& Value, TArray<FText>* OutReasons) const
{
	bool bIsHidden = false;
	for( auto It = Restrictions.CreateConstIterator() ; It ; ++It )
	{
		TSharedRef<const FPropertyRestriction> Restriction = (*It);
		if( Restriction->IsValueHidden(Value) )
		{
			bIsHidden = true;
			if (OutReasons)
			{
				OutReasons->Add(Restriction->GetReason());
			}
			else
			{
				break;
			}
		}
	}

	return bIsHidden;
}

bool FPropertyNode::IsDisabled(const FString& Value, TArray<FText>* OutReasons) const
{
	bool bIsDisabled = false;
	for (const TSharedRef<const FPropertyRestriction>& Restriction : Restrictions)
	{
		if( Restriction->IsValueDisabled(Value) )
		{
			bIsDisabled = true;
			if (OutReasons)
			{
				OutReasons->Add(Restriction->GetReason());
			}
			else
			{
				break;
			}
		}
	}

	return bIsDisabled;
}

bool FPropertyNode::IsRestricted(const FString& Value, TArray<FText>& OutReasons) const
{
	const bool bIsHidden = IsHidden(Value, &OutReasons);
	const bool bIsDisabled = IsDisabled(Value, &OutReasons);
	return (bIsHidden || bIsDisabled);
}

bool FPropertyNode::GenerateRestrictionToolTip(const FString& Value, FText& OutTooltip) const
{
	static FText ToolTipFormat = NSLOCTEXT("PropertyRestriction", "TooltipFormat ", "{0}{1}");
	static FText MultipleRestrictionsToolTopAdditionFormat = NSLOCTEXT("PropertyRestriction", "MultipleRestrictionToolTipAdditionFormat ", "({0} restrictions...)");

	TArray<FText> Reasons;
	const bool bRestricted = IsRestricted(Value, Reasons);

	FText Ret;
	if( bRestricted && Reasons.Num() > 0 )
	{
		if( Reasons.Num() > 1 )
		{
			FText NumberOfRestrictions = FText::AsNumber(Reasons.Num());

			OutTooltip = FText::Format(ToolTipFormat, Reasons[0], FText::Format(MultipleRestrictionsToolTopAdditionFormat,NumberOfRestrictions));
		}
		else
		{
			OutTooltip = FText::Format(ToolTipFormat, Reasons[0], FText());
		}
	}
	return bRestricted;
}

void FComplexPropertyNode::SetDisplayNameOverride(const FText& InDisplayNameOverride)
{
	DisplayNameOverride = InDisplayNameOverride;
}

FText FComplexPropertyNode::GetDisplayName() const
{
	if (!DisplayNameOverride.IsEmpty())
	{
		return DisplayNameOverride;
	}

	return FPropertyNode::GetDisplayName();
}

#undef LOCTEXT_NAMESPACE
