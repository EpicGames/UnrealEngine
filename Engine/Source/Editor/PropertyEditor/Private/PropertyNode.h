// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"
#include "PropertyPath.h"
#include "PropertyEditorModule.h"
#include "EditConditionParser.h"

class FCategoryPropertyNode;
class FComplexPropertyNode;
class FDetailTreeNode;
class FEditConditionContext;
class FEditConditionExpression;
class FItemPropertyNode;
class FNotifyHook;
class FObjectPropertyNode;
class FPropertyItemValueDataTrackerSlate;
class FPropertyNode;
class FPropertyRestriction;
class FStructurePropertyNode;

namespace PropertyEditorPolicy
{
	class IEditConstPolicy;
	class IArchetypePolicy;
}

DECLARE_LOG_CATEGORY_EXTERN(LogPropertyNode, Log, All);

namespace EPropertyNodeFlags
{
	typedef uint32 Type;

	inline const Type	IsSeen							= 1 << 0;	/** true if this node can be seen based on current parent expansion.  Does not take into account clipping*/
	inline const Type	IsSeenDueToFiltering			= 1 << 1;	/** true if this node has been accepted by the filter*/
	inline const Type	IsSeenDueToChildFiltering		= 1 << 2;	/** true if this node or one of it's children is seen due to filtering.  It will then be forced on as well.*/
	inline const Type	IsParentSeenDueToFiltering		= 1 << 3;	/** True if the parent was visible due to filtering*/
	inline const Type	IsSeenDueToChildFavorite		= 1 << 4;	/** True if this node is seen to it having a favorite as a child */
	
	inline const Type	Expanded						= 1 << 5;	/** true if this node should display its children*/
	inline const Type	CanBeExpanded					= 1 << 6;	/** true if this node is able to be expanded */

	inline const Type	EditInlineNew					= 1 << 7;	/** true if the property can be expanded into the property window. */

	inline const Type	SingleSelectOnly				= 1 << 8;	/** true if only a single object is selected. */
	inline const Type  ShowCategories					= 1 << 9;	/** true if this node should show categories.  Different*/

	inline const Type  HasEverBeenExpanded				= 1 << 10;	/** true if expand has ever been called on this node */

	inline const Type	IsBeingFiltered					= 1 << 11;	/** true if the node is being filtered. If this is true, seen flags should be checked for visibility.  If this is false the node has no filter and is visible */

	inline const Type  IsFavorite						= 1 << 12;	/** true if this item has been dubbed a favorite by the user */

	inline const Type  NoChildrenDueToCircularReference= 1 << 13;	/** true if this node has no children (but normally would) due to circular referencing */

	inline const Type	AutoExpanded					= 1 << 14;	/** true if this node was autoexpanded due to being filtered */
	inline const Type	ShouldShowHiddenProperties		= 1 << 15;	/** true if this node should all properties not just those with the correct flag(s) to be shown in the editor */
	inline const Type	IsAdvanced						= 1 << 16;	/** true if the property node is advanced (i.e it only shows up in advanced sections) */
	inline const Type	IsCustomized					= 1 << 17;	/** true if this node's visual representation has been customized by the editor */
	
	inline const Type	RequiresValidation				= 1 << 18;	/** true if this node could unexpectedly change (array changes, editinlinenew changes) */

	inline const Type	ShouldShowDisableEditOnInstance = 1 << 19;	/** true if this node should show child properties marked CPF_DisableEditOnInstance */

	inline const Type	IsReadOnly						= 1 << 20;	/** true if this node is overridden to appear as read-only */

	inline const Type	SkipChildValidation				= 1 << 21;	/** true if this node should skip child validation */

	inline const Type  ShowInnerObjectProperties		= 1 << 22;

	inline const Type	HasCustomResetToDefault			= 1 << 23;	/** true if this node's visual representation of reset to default has been customized*/

	inline const Type	IsSparseClassData				= 1 << 24;	/** true if the property on this node is part of a sparse class data structure */

	inline const Type	ShouldShowInViewport			= 1 << 25;	/** true if the property should be shown in the viewport context menu */

	inline const Type   ResolveInstanceDataObjects      = 1 << 26;  /** Object properties will resolve to the instance data objects if they exist */

	inline const Type   SupportsDynamicInstancing 		= 1 << 27;  /** This property could be point to a reference, an instance or null */
	
	inline const Type   DynamicInstance					= 1 << 28;  /** This property is pointing to an instance object.
																		Convenience and optimization without requiring inspecting the outer of the child Objects
																		*/
	
	inline const Type 	NoFlags							= 0;

};

namespace FPropertyNodeConstants
{
	inline const int32 NoDepthRestrictions = -1;

	/** Character used to deliminate sub-categories in category path names */
	inline const TCHAR CategoryDelimiterChar = TCHAR( '|' );
};

class FPropertySettings
{
public:
	static FPropertySettings& Get();
	bool ShowFriendlyPropertyNames() const { return bShowFriendlyPropertyNames; }
	bool ShowHiddenProperties() const { return bShowHiddenProperties; }
	bool ExpandDistributions() const { return bExpandDistributions; }
private:
	FPropertySettings();
private:
	bool bShowFriendlyPropertyNames;
	bool bExpandDistributions;
	bool bShowHiddenProperties;
};

struct FAddressPair
{
	FAddressPair(const UObject* InObject, uint8* Address, bool bInIsStruct)
		: Object( InObject)
		, ReadAddress( Address )
		, bIsStruct(bInIsStruct)
	{}
	TWeakObjectPtr<const UObject> Object;
	uint8* ReadAddress;
	bool bIsStruct;
};

template<> struct TIsPODType<FAddressPair> { enum { Value = true }; };

struct FReadAddressListData
{
public:
	FReadAddressListData()
		: bAllValuesTheSame( false )
		, bRequiresCache( true )
	{
	}
	void Add(const UObject* Object, uint8* Address, bool bIsStruct = false)
	{
		ReadAddresses.Add(FAddressPair(Object, Address, bIsStruct));
	}

	int32 Num() const
	{
		return ReadAddresses.Num();
	}

	uint8* GetAddress( int32 Index )
	{
		const FAddressPair& Pair = ReadAddresses[Index];
		return (Pair.Object.IsValid() || Pair.bIsStruct) ? Pair.ReadAddress : 0;
	}

	const UObject* GetObject(int32 Index)
	{
		const FAddressPair& Pair = ReadAddresses[Index];
		return Pair.Object.Get();
	}

	bool IsValidIndex( int32 Index ) const
	{
		return ReadAddresses.IsValidIndex(Index);
	}

	void Reset()
	{
		ReadAddresses.Reset();
		bAllValuesTheSame = false;
		bRequiresCache = true;
	}

	bool bAllValuesTheSame;
	bool bRequiresCache;
private:
	TArray<FAddressPair> ReadAddresses;
};

/**
 * A list of read addresses for a property node which contains the address for the nodes FProperty on each object
 */
class FReadAddressList
{
	friend class FPropertyNode;
public:
	FReadAddressList()
		: ReadAddressListData(nullptr)
	{}

	int32 Num() const
	{
		return (ReadAddressListData != nullptr) ? ReadAddressListData->Num() : 0;
	}

	uint8* GetAddress( int32 Index )
	{
		return ReadAddressListData->GetAddress( Index );
	}
	
	const UObject* GetObject(int32 Index)
	{
		return ReadAddressListData->GetObject(Index);
	}

	bool IsValidIndex( int32 Index ) const
	{
		return ReadAddressListData->IsValidIndex( Index );
	}

	void Reset()
	{
		if (ReadAddressListData != nullptr)
		{
			ReadAddressListData->Reset();
		}
	}

private:
	FReadAddressListData* ReadAddressListData;
};



/**
 * Parameters for initializing a property node
 */
struct FPropertyNodeInitParams
{
	enum class EIsSparseDataProperty : uint8
	{
		False,
		True,
		Inherit,
	};

	/** The parent of the property node */
	TSharedPtr<FPropertyNode> ParentNode;
	/** The property that the node observes and modifies*/
	FProperty* Property;
	/** Offset to the property data within either a fixed array or a dynamic array */
	int32 ArrayOffset;
	/** Index of the property in its array parent */
	int32 ArrayIndex;
	/** Whether or not to create any children */
	bool bAllowChildren;
	/** Whether or not to allow hidden properties (ones without CPF_Edit) to be visible */
	bool bForceHiddenPropertyVisibility;
	/** Whether or not to create category nodes (note: this setting is only valid for the root node of a property tree. The setting will propagate to children) */
	bool bCreateCategoryNodes;
	/** Whether or not to create nodes for properties marked CPF_DisableEditOnInstance */
	bool bCreateDisableEditOnInstanceNodes;
	/** Resolve Objects to their IDOs */
	bool bResolveInstanceDataObjects;
	
	/** Whether or not this property is sparse data */
	EIsSparseDataProperty IsSparseProperty;

	FPropertyNodeInitParams()
		: ParentNode(nullptr)
		, Property(nullptr)
		, ArrayOffset(0)
		, ArrayIndex( INDEX_NONE )
		, bAllowChildren( true )
		, bForceHiddenPropertyVisibility( false )
		, bCreateCategoryNodes( true )
		, bCreateDisableEditOnInstanceNodes( true )
		, bResolveInstanceDataObjects(false)
		, IsSparseProperty( EIsSparseDataProperty::Inherit )
	{}
};

/** Describes in which way an array property change has happend. This is used
	for propagation of array property changes to the instances of archetype objects. */
struct EPropertyArrayChangeType
{
	enum Type
	{
		/** A value was added to the array */
		Add,
		/** The array was cleared */
		Clear,
		/** A new item has been inserted. */
		Insert,
		/** An item has been deleted */
		Delete,
		/** An item has been duplicated */
		Duplicate,
		/** Two items have been swapped */
		Swap,
	};
};

enum EPropertyDataValidationResult : uint8
{
	/** The object(s) being viewed are now invalid */
	ObjectInvalid,
	/** Non dynamic array property nodes were added or removed that would require a refresh */
	PropertiesChanged,
	/** An edit inline new value changed,  In the tree this rebuilds the nodes, in the details view we don't need to do this */
	EditInlineNewValueChanged,
	/** The size of an array changed (delete,insert,add) */
	ArraySizeChanged,
	/** An internal node's children were rebuilt for some reason */
	ChildrenRebuilt,
	/** All data is valid */
	DataValid,
};

/** Helper class for modifying property values with setters and getters */
class FPropertyNodeEditStack
{
	struct FMemoryFrame
	{
		FMemoryFrame() = default;
		FMemoryFrame(const FProperty* InProperty, uint8* InMemory)
			: Property(InProperty)
			, Memory(InMemory)
		{
		}
		/** Property that points to the memory in this frame */
		const FProperty* Property = nullptr;
		/** Property address */
		uint8* Memory = nullptr;
	};
public:

	/**
	* Constructs property stack for the specified node
	* InNode Property node to construct the stack for
	* InObj Optional Object instance that contains the property being modified (if not provided the root container pointer will be acquired from the provided node hierarchy)
	*/
	FPropertyNodeEditStack(const FPropertyNode* InNode, const UObject* InObj = nullptr);
	FPropertyNodeEditStack() = default;
	~FPropertyNodeEditStack();

	FPropertyNodeEditStack& operator = (const FPropertyNodeEditStack& Other) = delete;
	FPropertyNodeEditStack(const FPropertyNodeEditStack& Other) = delete;

	/**
	* Initializes property stack for the specified node
	* InNode Property node to construct the stack for
	* InObj Object instance that contains the property being modified
	*/
	FPropertyAccess::Result Initialize(const FPropertyNode* InNode, const UObject* InObj);

	/**
	* Returns the address of the property being modified.
	* If anywhere in the property stack is a property with a setter or getter this will point to a temporarily allocated memory.
	*/
	uint8* GetDirectPropertyAddress()
	{
		return MemoryStack.Last().Memory;
	}

	/**
	* Commits all modifications to temporarily allocated property values back to the actual member variables using setters and getters where available
	*/
	void CommitChanges();

	/** Checks if this edit stack is valid */
	bool IsValid() const
	{
		return MemoryStack.Num() > 0;
	}

private:

	FPropertyAccess::Result InitializeInternal(const FPropertyNode* InNode, const UObject* InObj);
	void Cleanup();

	TArray<FMemoryFrame> MemoryStack;
};

/**
 * The base class for all property nodes
 */
class FPropertyNode : public TSharedFromThis<FPropertyNode>
{
public:

	FPropertyNode();
	virtual ~FPropertyNode();

	/**
	 * Init Tree Node internally (used only derived classes to pass through variables that are common to all nodes
	 * @param InitParams	Parameters for how the node should be initialized
	 */
	void InitNode(const FPropertyNodeInitParams& InitParams);

	/**
	 * Indicates that children of this node should be rebuilt next tick.  Some topology changes will require this
	 */
	void RequestRebuildChildren() { bRebuildChildrenRequested = true; }

	/**
	 * Used for rebuilding this nodes children
	 */
	void RebuildChildren();

	/**
	 * Mark this and all children as having been rebuilt.
	 */
	void MarkChildrenAsRebuilt();

	/**
	 * For derived windows to be able to add their nodes to the child array
	 */
	void AddChildNode(TSharedPtr<FPropertyNode> InNode);

	/**
	 * Destroys all child nodes under this node
	 */
	void RemoveAllChildNodes();

	/**
	 * Clears cached read address data
	 */
	void ClearCachedReadAddresses(bool bRecursive = true);

	/**
	 * Interface function to get at the derived FObjectPropertyNode class
	 */
	virtual FObjectPropertyNode* AsObjectNode() { return nullptr; }
	virtual const FObjectPropertyNode* AsObjectNode() const { return nullptr; }

	/**
	 * Interface function to get at the derived FComplexPropertyNode class
	 */
	virtual FComplexPropertyNode* AsComplexNode() { return nullptr; }
	virtual const FComplexPropertyNode* AsComplexNode() const { return nullptr; }

	/**
	 * Interface function to get at the derived FCategoryPropertyNode class
	 */
	virtual FCategoryPropertyNode* AsCategoryNode() { return nullptr; }
	virtual const FCategoryPropertyNode* AsCategoryNode() const { return nullptr; }

	/**
	 * Interface function to get at the derived FItemPropertyNode class
	 */
	virtual FItemPropertyNode* AsItemPropertyNode() { return nullptr; }
	virtual const FItemPropertyNode* AsItemPropertyNode() const { return nullptr; }

	/**
	 * Follows the chain of items upwards until it finds the complex property that houses this item.
	 */
	FComplexPropertyNode* FindComplexParent();
	const FComplexPropertyNode* FindComplexParent() const;

	/**
	 * Follows the chain of items upwards until it finds the object property that houses this item.
	 */
	FObjectPropertyNode* FindObjectItemParent();
	const FObjectPropertyNode* FindObjectItemParent() const;

	/**
	 * Follows the chain of items upwards until it finds the structure property that houses this item.
	 */
	FStructurePropertyNode* FindStructureItemParent();
	const FStructurePropertyNode* FindStructureItemParent() const;

	/**
	 * Follows the top-most object window that contains this property window item.
	 */
	FObjectPropertyNode* FindRootObjectItemParent();

	/**
	 * Used to see if any data has been destroyed from under the property tree.  Should only be called during Tick
	 */
	virtual EPropertyDataValidationResult EnsureDataIsValid();

	//////////////////////////////////////////////////////////////////////////
	// Text

	/**
	 * @param OutText						The property formatted in a string
	 * @param bAllowAlternateDisplayValue	Allow the function to potentially use an alternate form more suitable for display in the UI
	 * @param PortFlags						Determines how the property's value is accessed. Defaults to PPF_PropertyWindow
	 * @return true if the value was retrieved successfully
	 */
	FPropertyAccess::Result GetPropertyValueString(FString& OutString, const bool bAllowAlternateDisplayValue, EPropertyPortFlags PortFlags = PPF_PropertyWindow) const;

	/**
	 * @param OutText			The property formatted in text
	 * @param bAllowAlternateDisplayValue Allow the function to potentially use an alternate form more suitable for display in the UI
	 * @return true if the value was retrieved successfully
	 */
	FPropertyAccess::Result GetPropertyValueText(FText& OutText, const bool bAllowAlternateDisplayValue) const;

	//////////////////////////////////////////////////////////////////////////
	//Flags
	bool HasNodeFlags(const EPropertyNodeFlags::Type InTestFlags) const { return (PropertyNodeFlags & InTestFlags) != 0; }
	/**
	 * Sets the flags used by the window and the root node
	 * @param InFlags - flags to turn on or off
	 * @param InOnOff - whether to toggle the bits on or off
	 */
	void SetNodeFlags(const EPropertyNodeFlags::Type InFlags, const bool InOnOff);

	/**
	 * Finds a child of this property node
	 *
	 * @param InPropertyName	The name of the property to find
	 * @param bRecurse		true if we should recurse into children's children and so on.
	 */
	TSharedPtr<FPropertyNode> FindChildPropertyNode(const FName InPropertyName, bool bRecurse = false);

	/**
	 * Returns the parent node in the hierarchy
	 */
	FPropertyNode*			GetParentNode() { return ParentNodeWeakPtr.Pin().Get(); }
	const FPropertyNode*	GetParentNode() const { return ParentNodeWeakPtr.Pin().Get(); }
	TSharedPtr<FPropertyNode> GetParentNodeSharedPtr() { return ParentNodeWeakPtr.Pin(); }
	/**
	 * Returns the Property this Node represents
	 */
	FProperty*			GetProperty() { return Property.Get(); }
	const FProperty*	GetProperty() const { return Property.Get(); }

	/**
	 * Accessor functions for internals
	 */
	int32 GetArrayOffset() const { return ArrayOffset; }
	int32 GetArrayIndex() const { return ArrayIndex; }

	/**
	 * Return number of children that survived being filtered
	 */
	int32 GetNumChildNodes() const { return ChildNodes.Num(); }

	/**
	 * Returns the matching Child node
	 */
	TSharedPtr<FPropertyNode>& GetChildNode(const int32 ChildIndex)
	{
		check(ChildNodes[ChildIndex].IsValid());
		return ChildNodes[ChildIndex];
	}

	/**
	 * Returns the matching Child node
	 */
	const TSharedPtr<FPropertyNode>& GetChildNode(const int32 ChildIndex) const
	{
		check(ChildNodes[ChildIndex].IsValid());
		return ChildNodes[ChildIndex];
	}

	/**
	 * Returns the Child node whose ArrayIndex matches the supplied ChildIndex
	 */
	bool GetChildNode(const int32 ChildArrayIndex, TSharedPtr<FPropertyNode>& OutChildNode);

	/**
	* Returns the Child node whose ArrayIndex matches the supplied ChildIndex
	*/
	bool GetChildNode(const int32 ChildArrayIndex, TSharedPtr<FPropertyNode>& OutChildNode) const;

	/**
	 * Returns whether this window's property is read only or has the CPF_EditConst flag.
	 */
	bool IsPropertyConst() const;

	/** @return whether this window's property is constant (can't be edited by the user). If bIncludeEditCondition is false, the property is considered editable regardless of the EditCondition result. */
	bool IsEditConst(const bool bIncludeEditCondition = true) const;

	/**
	 * Returns whether this window's property should not be serialized (determined by the CPF_SkipSerialization flag).
	 */
	bool ShouldSkipSerialization() const;

	/**
	 * Gets the full name of this node
	 * @param PathPlusIndex - return value with full path of node
	 * @param bWithArrayIndex - If True, adds an array index (where appropriate)
	 * @param StopParent	- Stop at this parent (if any). Does NOT include it in the path
	 * @param bIgnoreCategories - Skip over categories
	 */
	virtual bool GetQualifiedName(FString& PathPlusIndex, const bool bWithArrayIndex, const FPropertyNode* StopParent = nullptr, bool bIgnoreCategories = false) const;

	// The bArrayPropertiesCanDifferInSize flag is an override for array properties which want to display
	// e.g. the "Clear" and "Empty" buttons, even though the array properties may differ in the number of elements.
	bool GetReadAddress(
		bool InRequiresSingleSelection,
		FReadAddressList& OutAddresses,
		bool bComparePropertyContents = true,
		bool bObjectForceCompare = false,
		bool bArrayPropertiesCanDifferInSize = false) const;

	/**
	 * fills in the OutAddresses array with the addresses of all of the available objects.
	 * @param OutAddresses	Storage array for all of the objects' addresses.
	 */
	bool GetReadAddress(FReadAddressList& OutAddresses) const;

	/**
	 * Fills in the OutValueAddress with the address of the value of all the available objects.
	 * If multiple items are selected, this will return a null address unless they are all the same value.
	 * @param OutValueAddress	The address of the item
	 */
	FPropertyAccess::Result GetSingleReadAddress(uint8*& OutValueAddress) const;

	/**
	 * Fills in the OutObject with the address of the object of all the available objects.
	 * If multiple items are selected, this will return a null address unless they are all the same value.
	 * @param OutObject	The address of the Object
	 */
	FPropertyAccess::Result GetSingleObject(UObject*& OutObject) const;

	/**
	 * Fills in the OutContainer with the address of the container (struct or UObject instance) that owns the property this node represents.
	 * @param OutContainer	The address of the container instance
	 */
	FPropertyAccess::Result GetSingleEditStack(FPropertyNodeEditStack& OutStack) const;

	/**
	 * Gets read addresses without accessing cached data.  Is less efficient but gets the must up to date data
	 */
	virtual bool GetReadAddressUncached(const FPropertyNode& InNode, bool InRequiresSingleSelection, FReadAddressListData* OutAddresses, bool bComparePropertyContents = true, bool bObjectForceCompare = false, bool bArrayPropertiesCanDifferInSize = false) const;
	virtual bool GetReadAddressUncached(const FPropertyNode& InNode, FReadAddressListData& OutAddresses) const;

	/**
	 * Calculates the memory address for the data associated with this item's property.  This is typically the value of a FProperty or a UObject address.
	 *
	 * @param	StartAddress	the location to use as the starting point for the calculation; typically the address of the object that contains this property.
	 * @param	bIsSparseData	True if StartAddress is pointing to a sidecar structure containing sparse class data, false otherwise
	 *
	 * @return	a pointer to a FProperty value or UObject.  (For dynamic arrays, you'd cast this value to an FArray*)
	 */
	virtual uint8* GetValueBaseAddress(uint8* StartAddress, bool bIsSparseData, bool bIsStruct = false) const;

	/**
	 * Calculates the memory address for the data associated with this item's value.  For most properties, identical to GetValueBaseAddress.  For items corresponding
	 * to dynamic array elements, the pointer returned will be the location for that element's data.
	 *
	 * @param	StartAddress	the location to use as the starting point for the calculation; typically the address of the object that contains this property.
	 * @param	bIsSparseData	True if StartAddress is pointing to a sidecar structure containing sparse class data, false otherwise
	 *
	 * @return	a pointer to a FProperty value or UObject.  (For dynamic arrays, you'd cast this value to whatever type is the Inner for the dynamic array)
	 */
	virtual uint8* GetValueAddress(uint8* StartAddress, bool bIsSparseData, bool bIsStruct = false) const;

	/**
	 * Caclulates the memory address for the starting point of the structure that contains the property this node uses.
	 * This will often be Obj but may also point to a sidecar data structure.
	 */
	uint8* GetStartAddressFromObject(const UObject* Obj) const;

	/**
	 * Calculates the memory address for the data associated with this item's property.  This is typically the value of a FProperty or a UObject address.
	 *
	 * @param	Obj	The object that contains this property; used as the starting point for the calculation
	 *
	 * @return	a pointer to a FProperty value or UObject.  (For dynamic arrays, you'd cast this value to an FArray*)
	 */
	uint8* GetValueBaseAddressFromObject(const UObject* Obj) const;

	/**
	 * Calculates the memory address for the data associated with this item's value.  For most properties, identical to GetValueBaseAddress.  For items corresponding
	 * to dynamic array elements, the pointer returned will be the location for that element's data.
	 *
	 * @param	Obj	The object that contains this property; used as the starting point for the calculation
	 *
	 * @return	a pointer to a FProperty value or UObject.  (For dynamic arrays, you'd cast this value to whatever type is the Inner for the dynamic array)
	 */
	uint8* GetValueAddressFromObject(const UObject* Obj) const;

	/**
	 * Sets the display name override to use instead of the display name
	 */
	virtual void SetDisplayNameOverride(const FText& InDisplayNameOverride) {}

	/**
	* @return true if the property is mark as a favorite
	*/
	virtual void SetFavorite(bool FavoriteValue) {}

	/**
	* @return true if the property is mark as a favorite
	*/
	virtual bool IsFavorite() const { return false; }

	/**
	 * @return The formatted display name for the property in this node
	 */
	virtual FText GetDisplayName() const { return FText::GetEmpty(); }

	/**
	 * Sets the tooltip override to use instead of the property tooltip
	 */
	virtual void SetToolTipOverride(const FText& InToolTipOverride) {}

	/**
	 * @return The tooltip for the property in this node
	 */
	virtual FText GetToolTipText() const { return FText::GetEmpty(); }

	/**
	 * If there is a property, sees if it matches.  Otherwise sees if the entire parent structure matches
	 */
	bool GetDiffersFromDefault();

	/**
	 * @return The label for displaying a reset to default value
	 */
	FText GetResetToDefaultLabel();

	/**
	 * @return If this property node is associated with a property that can be reordered within an array
	 */
	bool IsReorderable();

	/**Walks up the hierarchy and return true if any parent node is a favorite*/
	bool IsChildOfFavorite() const;

	void NotifyPreChange(FProperty* PropertyAboutToChange, FNotifyHook* InNotifyHook);
	void NotifyPreChange(FProperty* PropertyAboutToChange, FNotifyHook* InNotifyHook, const TSet<UObject*>& AffectedInstances);
	void NotifyPreChange(FProperty* PropertyAboutToChange, FNotifyHook* InNotifyHook, TSet<UObject*>&& AffectedInstances);

	void NotifyPostChange(FPropertyChangedEvent& InPropertyChangedEvent, FNotifyHook* InNotifyHook);

	DECLARE_EVENT(FPropertyNode, FPropertyChildrenRebuiltEvent);
	FDelegateHandle SetOnRebuildChildren(const FSimpleDelegate& InOnRebuildChildren);
	FPropertyChildrenRebuiltEvent& OnRebuildChildren() { return OnRebuildChildrenEvent; }

	/**
	 * Propagates the property change to all instances of an archetype
	 *
	 * @param	ModifiedObject	Object which property has been modified
	 * @param	NewValue		New value of the property
	 * @param	PreviousValue	Value of the property before the modification
	 */
	void PropagatePropertyChange(UObject* ModifiedObject, const TCHAR* NewValue, const FString& PreviousValue);

	/**
	 * Propagates the property change of a container property to all instances of an archetype
	 *
	 * @param	ModifiedObject				Object which property has been modified
	 * @param	OriginalContainerAddr		Original address holding the container value before the modification
	 * @param	ChangeType					In which way was the container modified
	 * @param	Index						Index of the modified item
	 */
	void PropagateContainerPropertyChange(UObject* ModifiedObject, const void* OriginalContainerAddr,
		EPropertyArrayChangeType::Type ChangeType, int32 Index, int32 SwapIndex = INDEX_NONE);

	/**
	 * Helper function to centralize logic for duplcating an array entry and ensuring that instanced object references are correctly handled
	 */
	static void DuplicateArrayEntry(UObject* ModifiedObject, FProperty* NodeProperty, FScriptArrayHelper& ArrayHelper, int32 Index);

	/**
	 * Gather the list of all instances that will be affected by a container property change
	 *
	 * @param	ModifiedObject				Object which property has been modified
	 * @param	OriginalContainerAddr		Original address holding the container value before the modification
	 * @param	ChangeType					In which way is the container modified
	 * @param	OutAffectedInstances		Instances affected by the property change
	 */
	void GatherInstancesAffectedByContainerPropertyChange(UObject* ModifiedObject, const void* OriginalContainerAddr, EPropertyArrayChangeType::Type ChangeType, TArray<UObject*>& OutAffectedInstances);

	/**
	 * Propagates the property change of a container property to the provided archetype instances
	 *
	 * @param	ModifiedObject				Object which property has been modified
	 * @param	OriginalContainerAddr		Original address holding the container value before the modification
	 * @param	AffectedInstances			Instances affected by the property change
	 * @param	ChangeType					In which way was the container modified
	 * @param	Index						Index of the modified item
	 */
	void PropagateContainerPropertyChange(UObject* ModifiedObject, const void* OriginalContainerAddr, const TArray<UObject*>& AffectedInstances,
		EPropertyArrayChangeType::Type ChangeType, int32 Index, int32 SwapIndex = INDEX_NONE);

	/** Broadcasts when a property value changes */
	DECLARE_EVENT(FPropertyNode, FPropertyValueChangedEvent);
	/** Broadcasts when a property value changes, but additionally includes the property changed event.*/
	DECLARE_MULTICAST_DELEGATE_OneParam(FPropertyValueChangedWithData, const FPropertyChangedEvent&)
	FPropertyValueChangedEvent& OnPropertyValueChanged() { return PropertyValueChangedEvent; }
	FPropertyValueChangedWithData& OnPropertyValueChangedWithData() { return PropertyValueChangedDelegate; }
	
	/** Broadcasts when a child of this property changes */
	FPropertyValueChangedEvent& OnChildPropertyValueChanged() { return ChildPropertyValueChangedEvent; }
	FPropertyValueChangedWithData& OnChildPropertyValueChangedWithData() { return ChildPropertyValueChangedDelegate; }

	/** Broadcasts when a property value changes */
	DECLARE_EVENT(FPropertyNode, FPropertyValuePreChangeEvent);
	FPropertyValuePreChangeEvent& OnPropertyValuePreChange() { return PropertyValuePreChangeEvent; }

	/** Broadcasts when a child of this property changes */
	FPropertyValuePreChangeEvent& OnChildPropertyValuePreChange() { return ChildPropertyValuePreChangeEvent; }

	/** Broadcasts when this property is reset to default */
	DECLARE_EVENT(FPropertyNode, FPropertyResetToDefaultEvent);
	FPropertyResetToDefaultEvent& OnPropertyResetToDefault() { return PropertyResetToDefaultEvent; }

	/**
	 * Broadcasts before an item in a container is being mutated, giving it a chance to back out.
	 * If an error occurs, it's sent by ChangeBlockedEvent.
	 */
	FCanPropertyItemChange& OnCanItemChangeEvent() { return CanItemChangeEvent; }

	/** Broadcasts before an item in a container is being mutated, giving it a chance to back out */
	FPropertyChangeBlocked& OnChangeBlockedEvent() { return ChangeBlockedEvent; }

	/**
	 * Marks window's seem due to filtering flags
	 * @param InFilterStrings	- List of strings that must be in the property name in order to display
	 *					Empty InFilterStrings means display as normal
	 *					All strings must match in order to be accepted by the filter
	 * @param bParentAllowsVisible - is NOT true for an expander that is NOT expanded
	 */
	void FilterNodes(const TArray<FString>& InFilterStrings, const bool bParentSeenDueToFiltering = false);

	/**
	 * Marks windows as visible based on the filter strings or standard visibility
	 *
	 * @param bParentAllowsVisible - is NOT true for an expander that is NOT expanded
	 */
	void ProcessSeenFlags(const bool bParentAllowsVisible);

	/**
	 * Marks windows as visible based their favorites status
	 */
	void ProcessSeenFlagsForFavorites();

	/**
	 * @return true if this node should be visible in a tree
	 */
	bool IsVisible() const { return HasNodeFlags(EPropertyNodeFlags::IsBeingFiltered) == 0 || HasNodeFlags(EPropertyNodeFlags::IsSeen) || HasNodeFlags(EPropertyNodeFlags::IsSeenDueToChildFiltering); };

	static TSharedRef< FPropertyPath > CreatePropertyPath(const TSharedRef< FPropertyNode >& PropertyNode)
	{
		TArray< FPropertyInfo > Properties;
		FPropertyNode* CurrentNode = &PropertyNode.Get();

		if (CurrentNode != nullptr && CurrentNode->AsCategoryNode() != nullptr)
		{
			TSharedRef< FPropertyPath > NewPath = MakeShared<FPropertyPath>();
			return NewPath;
		}

		while (CurrentNode != nullptr)
		{
			if (CurrentNode->AsItemPropertyNode() != nullptr)
			{
				FPropertyInfo NewPropInfo;
				NewPropInfo.Property = CurrentNode->GetProperty();
				if (!NewPropInfo.Property.IsValid())
				{
					return MakeShared<FPropertyPath>();
				}
				NewPropInfo.ArrayIndex = CurrentNode->GetArrayIndex();

				Properties.Add(NewPropInfo);
			}

			CurrentNode = CurrentNode->GetParentNode();
		}

		TSharedRef< FPropertyPath > NewPath = MakeShared<FPropertyPath>();

		for (int PropertyIndex = Properties.Num() - 1; PropertyIndex >= 0; --PropertyIndex)
		{
			NewPath->AddProperty(Properties[PropertyIndex]);
		}

		return NewPath;
	}

	static TSharedPtr< FPropertyNode > FindPropertyNodeByPath(const TSharedPtr< FPropertyPath > Path, const TSharedRef< FPropertyNode >& StartingNode)
	{
		if (!Path.IsValid() || Path->GetNumProperties() == 0)
		{
			return StartingNode;
		}

		bool FailedToFindProperty = false;
		TSharedPtr< FPropertyNode > PropertyNode = StartingNode;
		for (int PropertyIndex = 0; PropertyIndex < Path->GetNumProperties() && !FailedToFindProperty; PropertyIndex++)
		{
			FailedToFindProperty = true;
			const FPropertyInfo& PropInfo = Path->GetPropertyInfo(PropertyIndex);

			TArray< TSharedRef< FPropertyNode > > ChildrenStack;
			ChildrenStack.Push(PropertyNode.ToSharedRef());
			while (ChildrenStack.Num() > 0)
			{
				const TSharedRef< FPropertyNode > CurrentNode = ChildrenStack.Pop();

				for (int32 ChildIndex = 0; ChildIndex < CurrentNode->GetNumChildNodes(); ++ChildIndex)
				{
					const TSharedPtr< FPropertyNode > ChildNode = CurrentNode->GetChildNode(ChildIndex);

					if (ChildNode->AsItemPropertyNode() == nullptr)
					{
						ChildrenStack.Add(ChildNode.ToSharedRef());
					}
					else if (ChildNode.IsValid() &&
						ChildNode->GetProperty() == PropInfo.Property.Get() &&
						ChildNode->GetArrayIndex() == PropInfo.ArrayIndex)
					{
						PropertyNode = ChildNode;
						FailedToFindProperty = false;
						break;
					}
				}
			}
		}

		if (FailedToFindProperty)
		{
			PropertyNode = nullptr;
		}

		return PropertyNode;
	}


	static TArray< FPropertyInfo > GetPossibleExtensionsForPath(const TSharedPtr< FPropertyPath > Path, const TSharedRef< FPropertyNode >& StartingNode)
	{
		TArray< FPropertyInfo > PossibleExtensions;
		TSharedPtr< FPropertyNode > PropertyNode = FindPropertyNodeByPath(Path, StartingNode);

		if (!PropertyNode.IsValid())
		{
			return PossibleExtensions;
		}

		for (int32 ChildIndex = 0; ChildIndex < PropertyNode->GetNumChildNodes(); ++ChildIndex)
		{
			TSharedPtr< FPropertyNode > CurrentNode = PropertyNode->GetChildNode(ChildIndex);

			if (CurrentNode.IsValid() && CurrentNode->AsItemPropertyNode() != nullptr)
			{
				FPropertyInfo NewPropInfo;
				NewPropInfo.Property = CurrentNode->GetProperty();
				NewPropInfo.ArrayIndex = CurrentNode->GetArrayIndex();

				bool AlreadyExists = false;
				for (auto ExtensionIter = PossibleExtensions.CreateConstIterator(); ExtensionIter; ++ExtensionIter)
				{
					if (*ExtensionIter == NewPropInfo)
					{
						AlreadyExists = true;
						break;
					}
				}

				if (!AlreadyExists)
				{
					PossibleExtensions.Add(NewPropInfo);
				}
			}
		}

		return PossibleExtensions;
	}

	static UObject* GetArchetype(const UObject* Object);

	static void RegisterArchetypePolicy(PropertyEditorPolicy::IArchetypePolicy* ArchetypePolicy);
	static void UnregisterArchetypePolicy(PropertyEditorPolicy::IArchetypePolicy* ArchetypePolicy);
	
	static void RegisterEditConstPolicy(PropertyEditorPolicy::IEditConstPolicy* EditConstPolicy);
	static void UnregisterEditConstPolicy(PropertyEditorPolicy::IEditConstPolicy* EditConstPolicy);

	static bool IsPropertyEditConst(const FEditPropertyChain& PropertyChain, UObject* Object);

	static bool IsPropertyEditConst(const FProperty* Property, UObject* Object);

	/**
	 * Adds a restriction to the possible values for this property.
	 * @param Restriction	The restriction being added to this property.
	 */
	virtual void AddRestriction(TSharedRef<const FPropertyRestriction> Restriction);

	/**
	* Tests if a value is hidden for this property
	* @param Value			The value to test for being hidden.
	* @return				True if this value is hidden.
	*/
	bool IsHidden(const FString& Value) const
	{
		return IsHidden(Value, nullptr);
	}

	/**
	 * Tests if a value is disabled for this property
	 * @param Value			The value to test for being disabled.
	 * @return				True if this value is disabled.
	 */
	bool IsDisabled(const FString& Value) const
	{
		return IsDisabled(Value, nullptr);
	}

	bool IsRestricted(const FString& Value) const
	{
		return IsHidden(Value) || IsDisabled(Value);
	}

	/**
	* Tests if a value is hidden for this property.
	* @param Value			The value to test for being hidden.
	* @param OutReasons		If hidden, the reasons why.
	* @return				True if this value is hidden.
	*/
	virtual bool IsHidden(const FString& Value, TArray<FText>* OutReasons) const;

	/**
	 * Tests if a value is disabled for this property.
	 * @param Value			The value to test for being disabled.
	 * @param OutReasons	If disabled, the reasons why.
	 * @return				True if this value is disabled.
	 */
	virtual bool IsDisabled(const FString& Value, TArray<FText>* OutReasons) const;

	/**
	* Tests if a value is restricted for this property.
	* @param Value			The value to test for being restricted.
	* @param OutReasons		If restricted, the reasons why.
	* @return				True if this value is restricted.
	*/
	bool IsRestricted(const FString& Value, TArray<FText>& OutReasons) const;

	/**
	 * Generates a consistent tooltip describing this restriction for use in the editor.
	 * @param Value			The value to test for restriction and generate the tooltip from.
	 * @param OutTooltip	The tooltip describing why this value is restricted.
	 * @return				True if this value is restricted.
	 */
	virtual bool GenerateRestrictionToolTip(const FString& Value, FText& OutTooltip)const;

	const TArray<TSharedRef<const FPropertyRestriction>>& GetRestrictions() const
	{
		return Restrictions;
	}

	FPropertyChangedEvent& FixPropertiesInEvent(FPropertyChangedEvent& Event);

	/** Set metadata value for 'Key' to 'Value' on this property instance (as opposed to the class) */
	void SetInstanceMetaData(const FName& Key, const FString& Value);

	/**
	 * Get metadata value for 'Key' for this property instance (as opposed to the class)
	 *
	 * @return Pointer to metadata value; nullptr if Key not found
	 */
	const FString* GetInstanceMetaData(const FName& Key) const;

	/**
	 * Get metadata map for this property instance (as opposed to the class)
	 *
	 * @return Map ptr containing metadata pairs
	 */
	const TMap<FName, FString>* GetInstanceMetaDataMap() const;

	bool ParentOrSelfHasMetaData(const FName& MetaDataKey) const;

	/**
	 * Gets the property we should use to read meta-data
	 */
	FProperty* GetMetaDataProperty();

	/**
	 * Invalidates the cached state of this node in all children;
	 */
	void InvalidateCachedState();

	static void SetupKeyValueNodePair( TSharedPtr<FPropertyNode>& KeyNode, TSharedPtr<FPropertyNode>& ValueNode )
	{
		check( KeyNode.IsValid() && ValueNode.IsValid() );
		check( !KeyNode->PropertyKeyNode.IsValid() && !ValueNode->PropertyKeyNode.IsValid() );

		ValueNode->PropertyKeyNode = KeyNode;
	}

	TSharedPtr<FPropertyNode>& GetPropertyKeyNode() { return PropertyKeyNode; }

	const TSharedPtr<FPropertyNode>& GetPropertyKeyNode() const { return PropertyKeyNode; }

	/**
	* Gets the default value of the property as string.
	*/
	FString GetDefaultValueAsString(bool bUseDisplayName = true);

	/**
	 * Broadcasts reset to default property changes
	 */
	void BroadcastPropertyResetToDefault();

	/** @return Whether this property should have an edit condition toggle. */
	bool SupportsEditConditionToggle() const;

	/** Toggle the current state of the edit condition if this SupportsEditConditionToggle() */
	void ToggleEditConditionState();

	/**	@return Whether the property has a condition which must be met before allowing editing of it's value */
	bool HasEditCondition() const;

	/**	@return Whether the condition has been met to allow editing of this property's value */
	bool IsEditConditionMet() const;

	/**	@return Whether this property derives its visibility from its edit condition */
	bool IsOnlyVisibleWhenEditConditionMet() const;


	/**
	 * Helper to fetch a list of child property nodes that are expanded
	 */
	void GetExpandedChildPropertyPaths(TSet<FString>& OutExpandedChildPropertyPaths) const;

	/**
	 * Helper to set the expansion state of a list of child property nodes
	 */
	void SetExpandedChildPropertyNodes(const TSet<FString>& InNodesToExpand);

	/**
	 * Helper to fetch a PropertyPath 
	 */
	const FString& GetPropertyPath() const { return PropertyPath; }

	/** Marks this property node as ignoring CPF_InstancedReference */
	void SetIgnoreInstancedReference();

	/** Queries whether the node would like to ignore CPF_InstancedReference semantics */
	bool IsIgnoringInstancedReference() const;

	/** Return true if DestroyTree() has been called on this node. */
	bool IsDestroyed() const;

	/**
	 * Sets bIsDestroyed on all nodes within the hierarchy
	 */
	void MarkDestroyedRecursive();

	TSharedRef<FEditPropertyChain> BuildPropertyChain( FProperty* PropertyAboutToChange ) const;
	
protected:
	TSharedRef<FEditPropertyChain> BuildPropertyChain( FProperty* PropertyAboutToChange, const TSet<UObject*>& InAffectedArchetypeInstances ) const;
	TSharedRef<FEditPropertyChain> BuildPropertyChain( FProperty* PropertyAboutToChange, TSet<UObject*>&& InAffectedArchetypeInstances ) const;

	void NotifyPreChangeInternal(TSharedRef<FEditPropertyChain> PropertyChain, FProperty* PropertyAboutToChange, FNotifyHook* InNotifyHook);

	/**
	 * Destroys all node within the hierarchy
	 */
	void DestroyTree(const bool bInDestroySelf=true);

	/**
	 * Interface function for Custom Setup of Node (prior to node flags being set)
	 */
	virtual void InitBeforeNodeFlags() {};

	/**
	 * Interface function for Custom expansion Flags.  Default is objects and categories which always expand
	 */
	virtual void InitExpansionFlags() { SetNodeFlags(EPropertyNodeFlags::CanBeExpanded, true); };

	/**
	 * Interface function for Creating Child Nodes
	 */
	virtual void InitChildNodes() = 0;

	/**
	 * Does the string compares to ensure this Name is acceptable to the filter that is passed in
	 */
	bool IsFilterAcceptable(const TArray<FString>& InAcceptableNames, const TArray<FString>& InFilterStrings);
	/**
	 * Make sure that parent nodes are expanded
	 */
	void ExpandParent( bool bInRecursive );

	/** @return		The property stored at this node, to be passed to Pre/PostEditChange. */
	FProperty*		GetStoredProperty()		{ return nullptr; }

	bool GetDiffersFromDefault(const uint8* PropertyValueAddress, const uint8* PropertyDefaultAddress, const uint8* DefaultPropertyValueBaseAddress, const FProperty* InProperty, const UObject* TopLevelObject) const;
	bool GetDiffersFromDefaultForObject( FPropertyItemValueDataTrackerSlate& ValueTracker, FProperty* InProperty );

	enum class EValueAsStringMode
	{
		None,
		UseDisplayName,
		ForDiff,
	};
	FString GetDefaultValueAsString(const uint8* PropertyDefaultAddress, const FProperty* InProperty, EValueAsStringMode Mode, const UObject* TopLevelObject) const;
	FString GetDefaultValueAsStringForObject( FPropertyItemValueDataTrackerSlate& ValueTracker, UObject* InObject, FProperty* InProperty, EValueAsStringMode Mode);
	
	/**
	 * Helper function to obtain the display name for an enum property
	 * @param InEnum		The enum whose metadata to pull from
	 * @param DisplayName	The name of the enum value to adjust
	 *
	 * @return	true if the DisplayName has been changed
	 */
	bool AdjustEnumPropDisplayName(UEnum* InEnum, FString& DisplayName) const;

	/**
	 * Helper function for derived members to be able to
	 * broadcast property changed notifications
	 */
	void BroadcastPropertyChangedDelegates();

	/**
	 * Helper function for derived members to be able to 
	 * broadcast property changed notifications including property changed event data
	 */
	void BroadcastPropertyChangedDelegates(const FPropertyChangedEvent& Event);


	/**
	* Helper function for derived members to be able to
	* broadcast property pre-change notifications
	*/
	void BroadcastPropertyPreChangeDelegates();

	/**
	 * Gets a value tracker for the default of this property in the passed in object
	 *
	 * @param Object	The object to get the value for
	 * @param ObjIndex	The index of the object in the parent property node's object array (for caching)
	 */
	TSharedPtr< FPropertyItemValueDataTrackerSlate > GetValueTracker( UObject* Object, uint32 ObjIndex );

	/**
	 * Updates and caches the current edit const state of this property
	 */
	void UpdateEditConstState();

	/**
	 * Checks to see if the supplied property of a child node requires validation
	 * @param	InChildProp		The property of the child node
	 * @return	True if the property requires validation, false otherwise
	 */
	static bool DoesChildPropertyRequireValidation(FProperty* InChildProp);

protected:

	static FEditConditionParser EditConditionParser;

	/**
	 * The node that is the parent of this node or nullptr for the root
	 */
	TWeakPtr<FPropertyNode> ParentNodeWeakPtr;

	/**	The property node, if any, that serves as the key value for this node */
	TSharedPtr<FPropertyNode> PropertyKeyNode;

	/** Cached read addresses for this property node */
	mutable FReadAddressListData CachedReadAddresses;

	/** List of per object default value trackers associated with this property node */
	TArray< TSharedPtr<FPropertyItemValueDataTrackerSlate> > ObjectDefaultValueTrackers;

	/** List of all child nodes this node is responsible for */
	TArray< TSharedPtr<FPropertyNode> > ChildNodes;

	/** Called when this node's children are rebuilt */
	FPropertyChildrenRebuiltEvent OnRebuildChildrenEvent;

	/** Called when this node's property value is about to change (called during NotifyPreChange) */
	FPropertyValuePreChangeEvent PropertyValuePreChangeEvent;

	/** Called when a child's property value is about to change */
	FPropertyValuePreChangeEvent ChildPropertyValuePreChangeEvent;

	/** Called when this node's property value has changed (called during NotifyPostChange) */
	FPropertyValueChangedEvent PropertyValueChangedEvent;
	/** Called when this node's property value has changed with the property changed event data as payload (called during NotifyPostChange) */
	FPropertyValueChangedWithData PropertyValueChangedDelegate;
	
	/** Called when a child's property value has changed */
	FPropertyValueChangedEvent ChildPropertyValueChangedEvent;
	/** Called when a child's property value has changed with the property changed event data as payload */
	FPropertyValueChangedWithData ChildPropertyValueChangedDelegate;

	/** Called when the property is reset to default */
	FPropertyResetToDefaultEvent PropertyResetToDefaultEvent;

	/** Called before a container element is changed */
	FCanPropertyItemChange CanItemChangeEvent;

	/** Called when an attempted property change has been disallowed */
	FPropertyChangeBlocked ChangeBlockedEvent;

	/** The property being displayed/edited. */
	TWeakFieldPtr<FProperty> Property;

	/** Offset to the property data within either a fixed array or a dynamic array */
	int32 ArrayOffset;

	/** The index of the property if it is inside an array, set, or map (internally, we'll use set/map helpers that store element indices in an array) */
	int32 ArrayIndex;

	/** Safety Value representing Depth in the property tree used to stop diabolical topology cases
	 * -1 = No limit on children
	 *  0 = No more children are allowed.  Do not process child nodes
	 *  >0 = A limit has been set by the property and will tick down for successive children
	 */
	int32 MaxChildDepthAllowed;

	/**
	 * Used for flags to determine if the node is seen (if not seen and never been created, don't create it on display)
	 */
	EPropertyNodeFlags::Type PropertyNodeFlags;

	/** If true, children of this node will be rebuilt next tick. */
	bool bRebuildChildrenRequested;

	/** Set to true when RebuildChildren is called on the node */
	bool bChildrenRebuilt;

	/** Set to true when we want to ignore CPF_InstancedReference */
	bool bIgnoreInstancedReference;

	/** If true, DestroyTree() has been called on the node. */
	bool bIsDestroyed = false;
	
	/** An array of restrictions limiting this property's potential values in property editors.*/
	TArray<TSharedRef<const FPropertyRestriction>> Restrictions;

	/** Optional reference to a tree node that is displaying this property */
	TWeakPtr<FDetailTreeNode> TreeNode;

	/**
	 * Stores metadata for this instance of the property (in contrast
	 * to regular metadata, which is stored per-class)
	 */
	TMap<FName, FString> InstanceMetaData;

	/**
	* The property path for this property
	*/
	FString PropertyPath;

	/** Edit condition expression used to determine if this property editor can modify its property */
	TSharedPtr<FEditConditionExpression> EditConditionExpression;
	TSharedPtr<FEditConditionContext> EditConditionContext;

	/**
	* Cached state of flags that are expensive to update
	* These update when values are changed in the details panel
	*/
	mutable bool bIsEditConst; // Includes EditCondition state
	mutable bool bIsEditConstWithoutCondition; // Ignores EditCondition state
	mutable bool bUpdateEditConstState;
	mutable int32 UpdateEditConstStateEpoch;
	mutable bool bDiffersFromDefault;
	mutable bool bUpdateDiffersFromDefault;
	mutable int32 UpdateDiffersFromDefaultEpoch;
};

class FComplexPropertyNode : public FPropertyNode
{
public:

	enum EPropertyType
	{
		EPT_Object,
		EPT_StandaloneStructure,
	};

	FComplexPropertyNode() : FPropertyNode() {}
	virtual ~FComplexPropertyNode() = default;

	virtual FComplexPropertyNode* AsComplexNode() override { return this; }
	virtual const FComplexPropertyNode* AsComplexNode() const override { return this; }

	virtual FStructurePropertyNode* AsStructureNode() { return nullptr; }
	virtual const FStructurePropertyNode* AsStructureNode() const { return nullptr; }

	virtual void SetDisplayNameOverride(const FText& InDisplayNameOverride) override;
	virtual FText GetDisplayName() const override;

	virtual UStruct* GetBaseStructure() = 0;
	virtual const UStruct* GetBaseStructure() const = 0;

	// Returns the base struct as well as any sidecar data structs
	virtual TArray<UStruct*> GetAllStructures() = 0;
	virtual TArray<const UStruct*> GetAllStructures() const = 0;

	virtual int32 GetInstancesNum() const = 0;
	virtual uint8* GetMemoryOfInstance(int32 Index) const = 0;

	/**
	 * Returns a pointer to the stored value of InProperty on InParentNode's Index'th instance.
	 */
	virtual uint8* GetValuePtrOfInstance(int32 Index, const FProperty* InProperty, const FPropertyNode* InParentNode) const = 0;
	virtual TWeakObjectPtr<UObject> GetInstanceAsUObject(int32 Index) const = 0;
	virtual EPropertyType GetPropertyType() const = 0;

	virtual void Disconnect() = 0;

	/** Generates a single child from the provided property name.  Any existing children are destroyed */
	virtual TSharedPtr<FPropertyNode> GenerateSingleChild(FName ChildPropertyName) = 0;

private:
	/** Display name to use instead of the fully qualified name */
	FText DisplayNameOverride;
};
