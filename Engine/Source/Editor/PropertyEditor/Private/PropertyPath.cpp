// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyPath.h"

#include "UObject/PropertyPathName.h"
#include "UObject/PropertyTypeName.h"
#include "UObject/PropertyPathFunctions.h"

TArray<UE::FPropertyPathName> FPropertyPath::ToPropertyPathName() const
{
	using namespace UE;

	TArray<FPropertyPathName> PathNames;
	FPropertyPathName PathName;

	for (int32 PropIndex = 0; PropIndex < Properties.Num(); ++PropIndex)
	{
		const FPropertyInfo& Info = Properties[PropIndex];
		FPropertyTypeNameBuilder TypeName;
		Info.Property->SaveTypeName(TypeName);
		int32 ArrayIndex = Info.ArrayIndex;

		if (Info.Property->IsA<FArrayProperty>() || Info.Property->IsA<FSetProperty>() || Info.Property->IsA<FMapProperty>())
		{
			// FPropertyPaths store the index of containers on separate entries but FPropertyPathNames don't
			++PropIndex;

			if (PropIndex < Properties.Num())
			{
				ArrayIndex = Properties[PropIndex].ArrayIndex;
			}
		}

		PathName.Push({ Info.Property->GetFName(), TypeName.Build(), ArrayIndex });

		if (PropIndex < Properties.Num())
		{
			if (FMapProperty* AsMapProperty = CastField<FMapProperty>(Info.Property.Get()))
			{
				// add either the key or value flag to the path
				const FPropertyInfo& InnerInfo = Properties[PropIndex];
				if (InnerInfo.Property.Get() == AsMapProperty->KeyProp)
				{
					PathName.Push({ UE::NAME_Key });
				}
				else if (InnerInfo.Property.Get() == AsMapProperty->ValueProp)
				{
					PathName.Push({ UE::NAME_Value });
				}
			}
		}

		if (Info.Property->IsA<FObjectPropertyBase>())
		{
			PathNames.Emplace(MoveTemp(PathName));
			PathName.Reset();
		}
	}

	if (!PathName.IsEmpty())
	{
		PathNames.Emplace(MoveTemp(PathName));
	}

	return PathNames;
}
