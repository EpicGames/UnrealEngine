// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyPermissionList.h"

#include "Editor.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CoreDelegates.h"
#include "UObject/UnrealType.h"
#include "DetailTreeNode.h"

DEFINE_LOG_CATEGORY(LogPropertyEditorPermissionList);

namespace PropertyEditorPermissionList
{
	const FName PropertyPermissionListOwner = "PropertyPermissionList";
	static const FName NAME_CallInEditor("CallInEditor");

	static bool bSupportAllowAllFunctions = true;
	static FAutoConsoleVariableRef CVarSupportAllowAllFunctions(
		TEXT("PropertyEditor.PropertyEditorPermissionList.SupportAllowAllFunctions"),
		bSupportAllowAllFunctions,
		TEXT("Enable allowing all zero parameter call in editor functions when allowing all properties"));

	static FString ClassToDebug;
	static FAutoConsoleVariableRef CVarClassToDebug(
		TEXT("PropertyEditor.PropertyEditorPermissionList.ClassToDebug"),
		ClassToDebug,
		TEXT("Debug when a class that contains this string is cached"));
}

bool operator==(const FPropertyPermissionList::FPermissionListUpdate& A, const FPropertyPermissionList::FPermissionListUpdate& B)
{
	return A.ObjectStruct == B.ObjectStruct && A.OwnerName == B.OwnerName;
}

uint32 GetTypeHash(const FPropertyPermissionList::FPermissionListUpdate& PermisisonList)
{
	return HashCombine(
		GetTypeHash(PermisisonList.ObjectStruct), 
		GetTypeHash(PermisisonList.OwnerName));
}

FPropertyPermissionList::FPropertyPermissionList()
{
	if (!IsRunningCommandlet()) // Commandlets don't Tick
	{
		OnTickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FPropertyPermissionList::Tick), 1.0f);
	}
	FCoreUObjectDelegates::OnObjectPostCDOCompiled.AddRaw(this, &FPropertyPermissionList::OnObjectPostCDOCompiled);
}

FPropertyPermissionList::~FPropertyPermissionList()
{
	if (OnTickHandle.IsValid())
	{
		FTSTicker::RemoveTicker(OnTickHandle);
	}
	FCoreUObjectDelegates::OnObjectPostCDOCompiled.RemoveAll(this);
}

bool FPropertyPermissionList::Tick(float DeltaTime)
{
	TArray<FPermissionListUpdate> PendingUpdatesCopy;
	{
		FWriteScopeLock _(PermissionListRW);
		if (PendingUpdates.Num() == 0)
		{
			return true;
		}

		PendingUpdatesCopy = PendingUpdates.Array();
		PendingUpdates.Reset();
	}

	for (const FPermissionListUpdate& PermissionListUpdate : PendingUpdatesCopy)
	{
		PermissionListUpdatedDelegate.Broadcast(PermissionListUpdate.ObjectStruct, PermissionListUpdate.OwnerName);
	}
	return true;
}

void FPropertyPermissionList::OnObjectPostCDOCompiled(UObject* CDO, const FObjectPostCDOCompiledContext& Context)
{
	if (Context.bIsRegeneratingOnLoad)
	{
		// Can skip compile-on-load as the class layout can't have changed from what was cached
		return;
	}

	{
		FWriteScopeLock _(PermissionListRW);
		ClearCacheForClass_NoLock(CDO->GetClass());
	}
}

void FPropertyPermissionList::ClearCacheAndQueueBroadcast_NoLock(TSoftObjectPtr<const UStruct> ObjectStruct, FName OwnerName)
{
	// The cache isn't too expensive to recompute, so it is cleared
	// and lazily repopulated any time the raw PermissionList changes.
	CachedPropertyPermissionList.Reset();

	if (!bSuppressUpdateDelegate && OnTickHandle.IsValid())
	{
		PendingUpdates.Add({ObjectStruct, OwnerName});
	}
}

void FPropertyPermissionList::ClearCacheForClass_NoLock(const UClass* Class)
{
	if (CachedPropertyPermissionList.Remove(Class) == 0)
	{
		// If this class wasn't cached, then none of its derived types can be either
		return;
	}

	TArray<UClass*> DerivedClasses;
	GetDerivedClasses(Class, DerivedClasses, /*bRecursive*/false);
	for (const UClass* DerivedClass : DerivedClasses)
	{
		ClearCacheForClass_NoLock(DerivedClass);
	}
}

void FPropertyPermissionList::AddPermissionList(TSoftObjectPtr<const UStruct> Struct, const FNamePermissionList& PermissionList, const EPropertyPermissionListRules Rules, const TConstArrayView<FName> InAdditionalOwnerNames)
{
	FWriteScopeLock _(PermissionListRW);

	FPropertyPermissionListEntry& Entry = RawPropertyPermissionList.FindOrAdd(Struct);
	Entry.PermissionList = PermissionList;

	// Track additional owners to prevent entry from being removed by FPropertyPermissionList::UnregisterOwner()
	// PermissionList without AllowList/DenyList/DenyAll entries will have no owners and be deleted from RawPropertyPermissionList unless it provides AdditionalOwnerNames here
	Entry.AdditionalOwnerNames = InAdditionalOwnerNames;

	// Always use the most permissive rule previously set
	if (Entry.Rules > Rules)
	{
		Entry.Rules = Rules;
	}

	ClearCacheAndQueueBroadcast_NoLock(Struct);
}

void FPropertyPermissionList::AddPermissionList(TSoftObjectPtr<const UStruct> Struct, const FNamePermissionList& PermissionList, const EPropertyPermissionListRules Rules)
{
	AddPermissionList(Struct, PermissionList, Rules, {});
}

void FPropertyPermissionList::RemovePermissionList(TSoftObjectPtr<const UStruct> Struct)
{
	FWriteScopeLock _(PermissionListRW);

	if (RawPropertyPermissionList.Remove(Struct) > 0)
	{
		ClearCacheAndQueueBroadcast_NoLock(Struct);
	}
}

void FPropertyPermissionList::ClearPermissionList()
{
	FWriteScopeLock _(PermissionListRW);

	RawPropertyPermissionList.Reset();
	ClearCacheAndQueueBroadcast_NoLock();
}

void FPropertyPermissionList::UnregisterOwner(const FName Owner)
{
	FWriteScopeLock _(PermissionListRW);

	TArray<TSoftObjectPtr<const UStruct>> StructsToRemove;

	for (TPair<TSoftObjectPtr<const UStruct>, FPropertyPermissionListEntry>& Pair : RawPropertyPermissionList)
	{
		Pair.Value.PermissionList.UnregisterOwner(Owner);
		Pair.Value.AdditionalOwnerNames.Remove(Owner);
		if (Pair.Value.AdditionalOwnerNames.Num() == 0 && Pair.Value.PermissionList.GetOwnerNames().Num() == 0)
		{
			StructsToRemove.Add(Pair.Key);
		}
	}

	for (const TSoftObjectPtr<const UStruct>& StructToRemove : StructsToRemove)
	{
		RawPropertyPermissionList.Remove(StructToRemove);
	}

	ClearCacheAndQueueBroadcast_NoLock(nullptr, Owner);
}

void FPropertyPermissionList::AddToAllowList(TSoftObjectPtr<const UStruct> Struct, const FName PropertyName, const FName Owner)
{
	FWriteScopeLock _(PermissionListRW);

	FPropertyPermissionListEntry& Entry = RawPropertyPermissionList.FindOrAdd(Struct);
	if (Entry.PermissionList.AddAllowListItem(Owner, PropertyName))
	{
		ClearCacheAndQueueBroadcast_NoLock(Struct, Owner);
	}
}

void FPropertyPermissionList::AddToAllowList(TSoftObjectPtr<const UStruct> Struct, const TArray<FName>& PropertyNames, const FName Owner)
{
	FWriteScopeLock _(PermissionListRW);

	FPropertyPermissionListEntry& Entry = RawPropertyPermissionList.FindOrAdd(Struct);
	bool bAddedItem = false;
	for (const FName& PropertyName : PropertyNames)
	{
		if (Entry.PermissionList.AddAllowListItem(Owner, PropertyName))
		{
			bAddedItem = true;
		}
	}

	if (bAddedItem)
	{
		ClearCacheAndQueueBroadcast_NoLock(Struct, Owner);
	}
}

void FPropertyPermissionList::RemoveFromAllowList(TSoftObjectPtr<const UStruct> Struct, const FName PropertyName, const FName Owner)
{
	FWriteScopeLock _(PermissionListRW);

	FPropertyPermissionListEntry& Entry = RawPropertyPermissionList.FindOrAdd(Struct);
	if (Entry.PermissionList.RemoveAllowListItem(Owner, PropertyName))
	{
		ClearCacheAndQueueBroadcast_NoLock(Struct, Owner);
	}
}

void FPropertyPermissionList::AddToDenyList(TSoftObjectPtr<const UStruct> Struct, const FName PropertyName, const FName Owner)
{
	FWriteScopeLock _(PermissionListRW);

	FPropertyPermissionListEntry& Entry = RawPropertyPermissionList.FindOrAdd(Struct);
	if (Entry.PermissionList.AddDenyListItem(Owner, PropertyName))
	{
		ClearCacheAndQueueBroadcast_NoLock(Struct, Owner);
	}
}

void FPropertyPermissionList::RemoveFromDenyList(TSoftObjectPtr<const UStruct> Struct, const FName PropertyName, const FName Owner)
{
	FWriteScopeLock _(PermissionListRW);

	FPropertyPermissionListEntry& Entry = RawPropertyPermissionList.FindOrAdd(Struct);
	if (Entry.PermissionList.RemoveDenyListItem(Owner, PropertyName))
	{
		ClearCacheAndQueueBroadcast_NoLock(Struct, Owner);
	}
}

void FPropertyPermissionList::RegisterCustomDetailsTreeNodePermissionDelegate(TSoftObjectPtr<const UStruct> Struct, FCustomDetailTreeNodePermissionDelegate InDelegate)
{
	if(Struct && InDelegate.IsBound())
	{
		CustomDetailTreeNodePermissionDelegates.Add(Struct, InDelegate);
	}
}

void FPropertyPermissionList::UnregisterCustomDetailsTreeNodePermissionDelegate(TSoftObjectPtr<const UStruct> Struct)
{
	if(Struct && CustomDetailTreeNodePermissionDelegates.Contains(Struct))
	{
		CustomDetailTreeNodePermissionDelegates.Remove(Struct);
	}
}

void FPropertyPermissionList::SetEnabled(bool bEnable, bool bAnnounce)
{
	EnablePermissionList = FDelegateEnablePathFilter::CreateLambda([bEnable]()
		{
			return bEnable;
		});

	if (bAnnounce)
	{
		PermissionListEnabledDelegate.Broadcast();
	}
}

void FPropertyPermissionList::SetEnabled(const FDelegateEnablePathFilter& bEnable, bool bAnnounce)
{
	EnablePermissionList = bEnable;

	if (bAnnounce)
	{
		PermissionListEnabledDelegate.Broadcast();
	}
}

void FPropertyPermissionList::ClearCache()
{
	FWriteScopeLock _(PermissionListRW);

	CachedPropertyPermissionList.Reset();
}

bool FPropertyPermissionList::HasFiltering(const UStruct* ObjectStruct) const
{
	if (ObjectStruct && IsEnabled())
	{
		TSharedRef<const FCachedPermissionList> CachedPermissionList = GetCachedPermissionListEntryForStruct(ObjectStruct);
		return CachedPermissionList->PermissionList->HasFiltering();
	}
	return false;
}

bool FPropertyPermissionList::DoesPropertyPassFilter(const UStruct* ObjectStruct, FName PropertyName) const
{
	if (ObjectStruct && IsEnabled())
	{
		TSharedRef<const FCachedPermissionList> CachedPermissionList = GetCachedPermissionListEntryForStruct(ObjectStruct);
		return CachedPermissionList->PermissionList->PassesFilter(PropertyName);
	}
	return true;
}

bool FPropertyPermissionList::DoesDetailTreeNodePassFilter(const UStruct* ObjectStruct, TSharedRef<FDetailTreeNode> DetailTreeNode)
{
	if (ObjectStruct && IsEnabled())
	{
		if(CustomDetailTreeNodePermissionDelegates.Contains(ObjectStruct) && CustomDetailTreeNodePermissionDelegates[ObjectStruct].IsBound())
		{
			TOptional<bool> CustomPermissionResult = CustomDetailTreeNodePermissionDelegates[ObjectStruct].Execute(DetailTreeNode);
			if(CustomPermissionResult.IsSet())
			{
				return CustomPermissionResult.GetValue();
			}
		}

		TSharedRef<const FCachedPermissionList> CachedPermissionList = GetCachedPermissionListEntryForStruct(ObjectStruct);
		return CachedPermissionList->PermissionList->PassesFilter(DetailTreeNode->GetNodeName());
	}
	
	return true;
}

TSharedPtr<const FNamePermissionList> FPropertyPermissionList::GetCachedPermissionListForStructIfEnabled(const UStruct* Struct) const
{
	if (Struct && IsEnabled())
	{
		return GetCachedPermissionListEntryForStruct(Struct)->PermissionList;
	}
	return nullptr;
}

TSharedRef<const FPropertyPermissionList::FCachedPermissionList> FPropertyPermissionList::GetCachedPermissionListEntryForStruct(const UStruct* Struct) const
{
	check(Struct);

#if !UE_BUILD_SHIPPING
	if (!PropertyEditorPermissionList::ClassToDebug.IsEmpty())
	{
		FNameBuilder StructPathName;
		Struct->GetPathName(nullptr, StructPathName);
		if (StructPathName.ToView().Contains(PropertyEditorPermissionList::ClassToDebug))
		{
			bool bSetBreakpointHere = true;
		}
	}
#endif

	TOptional<FPropertyPermissionListEntry> EntryCopy;
	{
		FReadScopeLock _(PermissionListRW);

		// Is this struct already cached? If so, we can just return its result
		if (const TSharedRef<const FCachedPermissionList>* CachedPermissionListPtr = CachedPropertyPermissionList.Find(Struct))
		{
			return *CachedPermissionListPtr;
		}

		// Otherwise snapshot the raw entry so we don't need a second lock
		if (const FPropertyPermissionListEntry* Found = RawPropertyPermissionList.Find(Struct))
		{
			EntryCopy.Emplace(*Found);
		}
	}

	FNamePermissionList NewPermissionList;

	auto AllowAllFields =
		[&NewPermissionList](const UStruct* InStruct, const EFieldIterationFlags InIterationFlags)
		{
			for (TFieldIterator<FProperty> Property(InStruct, InIterationFlags); Property; ++Property)
			{
				NewPermissionList.AddAllowListItem(PropertyEditorPermissionList::PropertyPermissionListOwner, Property->GetFName());
			}

			if (PropertyEditorPermissionList::bSupportAllowAllFunctions)
			{
				for (TFieldIterator<UFunction> FunctionIter(InStruct, InIterationFlags); FunctionIter; ++FunctionIter)
				{
					UFunction* TestFunction = *FunctionIter;
					if ((TestFunction->ParmsSize == 0) && TestFunction->GetBoolMetaData(PropertyEditorPermissionList::NAME_CallInEditor))
					{
						NewPermissionList.AddAllowListItem(PropertyEditorPermissionList::PropertyPermissionListOwner, TestFunction->GetFName());
					}
				}
			}
		};

	// Recursively fill the cache for all parent structs
	UStruct* SuperStruct = Struct->GetSuperStruct();
	EPropertyPermissionListRules SuperEntryRule = EPropertyPermissionListRules::UseExistingPermissionList;
	if (SuperStruct)
	{
		TSharedRef<const FCachedPermissionList> SuperPermissionList = GetCachedPermissionListEntryForStruct(SuperStruct);
		SuperEntryRule = SuperPermissionList->Rules;
		NewPermissionList.Append(*SuperPermissionList->PermissionList);
	}

	// Resolve the permission list rule for this entry
	const FPropertyPermissionListEntry* Entry = EntryCopy.GetPtrOrNull();
	EPropertyPermissionListRules EntryRule = EPropertyPermissionListRules::UseExistingPermissionList;
	if (Entry)
	{
		// This causes an issue in the case where a struct should have no AllowList properties but wants use AllowListAllSubclassProperties
		// In this case, simply add a dummy AllowList entry that (likely) won't ever collide with a real property name
		if (Entry->PermissionList.GetAllowList().Num() == 0 || Entry->Rules == EPropertyPermissionListRules::AllowListAllProperties)
		{
			EntryRule = EPropertyPermissionListRules::AllowListAllProperties;
		}
		else if (Entry->Rules == EPropertyPermissionListRules::AllowListAllSubclassProperties)
		{
			EntryRule = EPropertyPermissionListRules::AllowListAllSubclassProperties;
		}
	}
	else
	{
		// If we don't have a raw entry then we propagate the rule from our parent struct to avoid breaking AllowListAllProperties or AllowListAllSubclassProperties chains
		EntryRule = SuperEntryRule;
	}

	// If this entry has explicit rules, append them on-top of any permission rules inherited from our parent(s)
	if (Entry)
	{
		if (EntryRule == EPropertyPermissionListRules::AllowListAllProperties)
		{
			// Allow all fields if requested
			// If the allow list inherited from our parent(s) is empty then that already implies all fields are visible, so we can just keep the list as empty
			if (NewPermissionList.GetAllowList().Num() > 0)
			{
				AllowAllFields(Struct, EFieldIterationFlags::None);
			}

			// If the AllowList is empty, we only want to append the DenyLists
			FNamePermissionList DuplicatePermissionList = Entry->PermissionList;
			// Hack to get around the fact that there's no easy way to only clear an AllowList
			TMap<FName, FPermissionListOwners>& AllowList = const_cast<TMap<FName, FPermissionListOwners>&>(DuplicatePermissionList.GetAllowList());
			AllowList.Empty();
			NewPermissionList.Append(DuplicatePermissionList);
		}
		else
		{
			check(Entry->PermissionList.GetAllowList().Num() > 0);

			// If the parent struct is explicitly allowing all properties but has an empty allow list, then we need to explicitly allow all of its fields before appending our permissions
			if (SuperEntryRule == EPropertyPermissionListRules::AllowListAllProperties && NewPermissionList.GetAllowList().Num() == 0)
			{
				check(SuperStruct);
				AllowAllFields(SuperStruct, EFieldIterationFlags::IncludeSuper);
			}
			NewPermissionList.Append(Entry->PermissionList);
		}
	}

	// Did our super class ask for sub-classes to expose all their properties? If so, respect that request unless we define our own permission rules
	EPropertyPermissionListRules EntryRuleToPropagate = EntryRule;
	if ((!Entry || EntryRule == EPropertyPermissionListRules::AllowListAllProperties) && SuperEntryRule == EPropertyPermissionListRules::AllowListAllSubclassProperties)
	{
		EntryRuleToPropagate = EPropertyPermissionListRules::AllowListAllSubclassProperties;
		if (NewPermissionList.GetAllowList().Num() > 0)
		{
			AllowAllFields(Struct, EFieldIterationFlags::None);
		}
	}

	{
		FWriteScopeLock _(PermissionListRW);

		// Check the cache again in case another thread beat us to it
		if (const TSharedRef<const FCachedPermissionList>* CachedPermissionListPtr = CachedPropertyPermissionList.Find(Struct))
		{
			return *CachedPermissionListPtr;
		}

		return CachedPropertyPermissionList.Add(Struct, MakeShared<FCachedPermissionList>(
			MoveTemp(NewPermissionList),
			EntryRuleToPropagate
			));
	}
}

bool FPropertyPermissionList::HasSpecificList(const UStruct* ObjectStruct) const
{
	FReadScopeLock _(PermissionListRW);
	return RawPropertyPermissionList.Find(ObjectStruct) != nullptr;
}

bool FPropertyPermissionList::IsSpecificPropertyAllowListed(const UStruct* ObjectStruct, FName PropertyName) const
{
	FReadScopeLock _(PermissionListRW);
	if (const FPropertyPermissionListEntry* Entry = RawPropertyPermissionList.Find(ObjectStruct))
	{
		return Entry->PermissionList.GetAllowList().Contains(PropertyName);
	}
	return false;
}

bool FPropertyPermissionList::IsSpecificPropertyDenyListed(const UStruct* ObjectStruct, FName PropertyName) const
{
	FReadScopeLock _(PermissionListRW);
	if (const FPropertyPermissionListEntry* Entry = RawPropertyPermissionList.Find(ObjectStruct))
	{
		return Entry->PermissionList.GetDenyList().Contains(PropertyName);
	}
	return false;
}

FPropertyEditorPermissionList& FPropertyEditorPermissionList::Get()
{
	static FPropertyEditorPermissionList PermissionList;
	return PermissionList;
}

bool FPropertyEditorPermissionList::IsEnabledForPackage(const UPackage* Package) const
{
	return !IsEnabledForPackageDelegate.IsBound() || IsEnabledForPackageDelegate.Execute(Package);
}

FHiddenPropertyPermissionList& FHiddenPropertyPermissionList::Get()
{
	static FHiddenPropertyPermissionList PermissionList;
	return PermissionList;
}


void FEnumValuePermissionList::AddPermissionList(TSoftObjectPtr<const UEnum> Enum, const FNamePermissionList& InPermissionList)
{
	FNamePermissionList& Entry = PermissionList.FindOrAdd(Enum);
	Entry = InPermissionList;
}

void FEnumValuePermissionList::RemovePermissionList(TSoftObjectPtr<const UEnum> Enum)
{
	PermissionList.Remove(Enum);
}

void FEnumValuePermissionList::ClearPermissionList()
{
	PermissionList.Reset();
}

bool FEnumValuePermissionList::DoesEnumValuePassFilter(const UEnum* Enum, FName ValueName) const
{
	if (!bEnablePermissionList)
	{
		return true;
	}

	const FNamePermissionList* Entry = PermissionList.Find(Enum);
	if (Entry)
	{
		return Entry->PassesFilter(ValueName);
	}
	return true;
}

bool FEnumValuePermissionList::HasFiltering(const UEnum* Enum) const
{
	if (!bEnablePermissionList)
	{
		return false;
	}

	const FNamePermissionList* Entry = PermissionList.Find(Enum);
	return Entry != nullptr;
}

FEnumValuePermissionList& FEnumValuePermissionList::Get()
{
	static FEnumValuePermissionList Instance;
	return Instance;
}
