// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyRestriction.h"

#include "HAL/PlatformCrt.h"
#include "Templates/UnrealTemplate.h"

bool FPropertyRestriction::IsValueHidden(const FString& InValue) const
{
	for (const FString& Value : HiddenValues)
	{
		if (InValue == Value)
		{
			return true;
		}
	}
	return false;
}

bool FPropertyRestriction::IsValueDisabled(const FString& InValue)const
{
	for (const FString& Value : DisabledValues)
	{
		if (InValue == Value)
		{
			return true;
		}
	}
	return false;
}

void FPropertyRestriction::AddHiddenValue(FString InValue)
{
	HiddenValues.Add(MoveTemp(InValue));
}

void FPropertyRestriction::AddDisabledValue(FString InValue)
{
	DisabledValues.Add(MoveTemp(InValue));
}

void FPropertyRestriction::AddClassFilter(TSharedRef<IClassViewerFilter> InValue)
{
	ClassViewFilter.Add(MoveTemp(InValue));
}

void FPropertyRestriction::RemoveHiddenValue(FString Value)
{
	HiddenValues.Remove(Value);
}

void FPropertyRestriction::RemoveDisabledValue(FString Value)
{
	DisabledValues.Remove(Value);
}

void FPropertyRestriction::RemoveClassFilter(TSharedRef<IClassViewerFilter> Value)
{
	ClassViewFilter.Remove(Value);
}

void FPropertyRestriction::RemoveAll()
{
	HiddenValues.Empty();
	DisabledValues.Empty();
	ClassViewFilter.Empty();
}
