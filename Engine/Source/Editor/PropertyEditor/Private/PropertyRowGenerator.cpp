// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyRowGenerator.h"
#include "PropertyNode.h"
#include "ObjectPropertyNode.h"
#include "StructurePropertyNode.h"
#include "DetailLayoutBuilderImpl.h"
#include "CategoryPropertyNode.h"
#include "UserInterface/PropertyEditor/SPropertyEditorEditInline.h"
#include "DetailCategoryBuilderImpl.h"
#include "Modules/ModuleManager.h"
#include "DetailLayoutHelpers.h"
#include "PropertyHandleImpl.h"
#include "IPropertyGenerationUtilities.h"
#include "EditConditionParser.h"
#include "UObject/StructOnScope.h"
#include "ThumbnailRendering/ThumbnailManager.h"

class FPropertyRowGeneratorUtilities : public IPropertyUtilities
{
public:
	FPropertyRowGeneratorUtilities(FPropertyRowGenerator& InGenerator)
		: Generator(&InGenerator)
	{
	}

	void ResetGenerator()
	{
		Generator = nullptr;
	}

	/** IPropertyUtilities interface */
	virtual class FNotifyHook* GetNotifyHook() const override
	{
		return Generator != nullptr ? Generator->GetNotifyHook() : nullptr;
	}
	virtual bool AreFavoritesEnabled() const override
	{
		return false;
	}

	virtual void ToggleFavorite(const TSharedRef< class FPropertyEditor >& PropertyEditor) const override {}
	virtual void CreateColorPickerWindow(const TSharedRef< class FPropertyEditor >& PropertyEditor, bool bUseAlpha) const override {}
	virtual void EnqueueDeferredAction(FSimpleDelegate DeferredAction) override
	{
		checkf(Generator != nullptr, TEXT("Can not enqueue action, generator is no longer valid"));
		Generator->EnqueueDeferredAction(DeferredAction);
	}
	virtual bool IsPropertyEditingEnabled() const override
	{
		return Generator != nullptr && Generator->IsPropertyEditingEnabled();
	}

	virtual void ForceRefresh() override
	{
		if (Generator != nullptr)
		{
			Generator->ForceRefresh();
		}
	}
	virtual void RequestRefresh() override 
	{ 
		if (Generator != nullptr)
		{
			Generator->RequestRefresh();
		}
	}
	virtual void RequestForceRefresh() override
	{
		// RequestRefresh is already a deferred ForceRefresh
		RequestRefresh();
	}

	virtual TSharedPtr<class FAssetThumbnailPool> GetThumbnailPool() const override
	{
		return Generator != nullptr
			? Generator->GetThumbnailPool()
			: TSharedPtr<class FAssetThumbnailPool>();
	}

	virtual void NotifyStartedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
	{
		if (Generator != nullptr)
		{
			Generator->OnNotifyStartedChangingProperties().Broadcast(PropertyChangedEvent);
		}
	}

	virtual void NotifyFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) override 
	{
		if (Generator)
		{
			Generator->OnFinishedChangingProperties().Broadcast(PropertyChangedEvent);
		}
	}

	virtual bool DontUpdateValueWhileEditing() const override { return false; }

	const TArray<TWeakObjectPtr<UObject>>& GetSelectedObjects() const override
	{
		if (Generator != nullptr)
		{
			return Generator->GetSelectedObjects();
		}
		else
		{
			static TArray<TWeakObjectPtr<UObject>> NullSelectedObjects;
			return NullSelectedObjects;
		}
	}

	virtual bool HasClassDefaultObject() const override
	{
		return Generator != nullptr && Generator->HasClassDefaultObject();
	}

	virtual const TArray<TSharedRef<class IClassViewerFilter>>& GetClassViewerFilters() const override
	{
		if (Generator != nullptr)
		{
			return Generator->GetClassViewerFilters();
		}
		else
		{
			static TArray<TSharedRef<class IClassViewerFilter>> NullFilters;
			return NullFilters;
		}
	}

private:
	FPropertyRowGenerator* Generator;
};


class FPropertyRowGeneratorGenerationUtilities : public IPropertyGenerationUtilities
{
public:
	FPropertyRowGeneratorGenerationUtilities(FPropertyRowGenerator& InGenerator)
		: Generator(&InGenerator)
	{
	}

	void ResetGenerator()
	{
		Generator = nullptr;
	}

	virtual const FCustomPropertyTypeLayoutMap& GetInstancedPropertyTypeLayoutMap() const override
	{
		return Generator->GetInstancedPropertyTypeLayoutMap();
	}

	virtual void RebuildTreeNodes() override
	{
		if (Generator != nullptr)
		{
			Generator->UpdateDetailRows();
		}
	}

private:
	FPropertyRowGenerator* Generator;
};

FPropertyRowGenerator::FPropertyRowGenerator(const FPropertyRowGeneratorArgs& InArgs)
	: Args(InArgs)
	, PropertyUtilities(new FPropertyRowGeneratorUtilities(*this))
	, PropertyGenerationUtilities(new FPropertyRowGeneratorGenerationUtilities(*this))
{
	CurrentFilter.bShowAllAdvanced = true;
}

FPropertyRowGenerator::FPropertyRowGenerator(const FPropertyRowGeneratorArgs& InArgs, TSharedPtr<FAssetThumbnailPool> InThumbnailPool)
	: Args(InArgs)
	, PropertyUtilities(new FPropertyRowGeneratorUtilities(*this))
	, PropertyGenerationUtilities(new FPropertyRowGeneratorGenerationUtilities(*this))
{
	CurrentFilter.bShowAllAdvanced = true;
}

FPropertyRowGenerator::~FPropertyRowGenerator()
{
	StaticCastSharedRef<FPropertyRowGeneratorUtilities>(PropertyUtilities)->ResetGenerator();
	StaticCastSharedRef<FPropertyRowGeneratorGenerationUtilities>(PropertyGenerationUtilities)->ResetGenerator();
}

void FPropertyRowGenerator::SetObjects(const TArray<UObject*>& InObjects)
{
	PreSetObject(InObjects.Num(), /*bHasStructRoots=*/false);
	
	bViewingClassDefaultObject = InObjects.Num() > 0 ? true : false;
	
	SelectedObjects.Reset(InObjects.Num());

	for (int32 ObjectIndex = 0; ObjectIndex < InObjects.Num(); ++ObjectIndex)
	{
		UObject* Object = InObjects[ObjectIndex];

		SelectedObjects.Add(Object);

		bViewingClassDefaultObject &= Object->HasAnyFlags(RF_ClassDefaultObject);

		if (Args.bAllowMultipleTopLevelObjects)
		{
			check(RootPropertyNodes.Num() == InObjects.Num());
			RootPropertyNodes[ObjectIndex]->AsObjectNode()->AddObject(Object);
		}
		else
		{
			RootPropertyNodes[0]->AsObjectNode()->AddObject(Object);
		}
	}

	PostSetObject();
}

void FPropertyRowGenerator::SetStructure(const TSharedPtr<FStructOnScope>& InStruct)
{
	PreSetObject(1, /*bHasStructRoots=*/true);

	check(RootPropertyNodes.Num() == 1);
	RootPropertyNodes[0]->AsStructureNode()->SetStructure(InStruct);

	PostSetObject();
}

void FPropertyRowGenerator::SetStructure(const TSharedPtr<IStructureDataProvider>& InStructProvider)
{
	PreSetObject(1, /*bHasStructRoots=*/true);

	check(RootPropertyNodes.Num() == 1);
	RootPropertyNodes[0]->AsStructureNode()->SetStructure(InStructProvider);

	PostSetObject();
}

const TArray<TSharedRef<IDetailTreeNode>>& FPropertyRowGenerator::GetRootTreeNodes() const
{
	return RootTreeNodes;
}

TSharedPtr<IDetailTreeNode> FPropertyRowGenerator::FindTreeNode(TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
	{
		TArray<TSharedPtr<IDetailTreeNode>> NodesToCheck;
		NodesToCheck.Append(RootTreeNodes);
		TSharedPtr<FPropertyNode> PropertyNodeWeAreAfter = StaticCastSharedPtr<FPropertyHandleBase>(PropertyHandle)->GetPropertyNode();

		TArray<TSharedRef<IDetailTreeNode>> Children;
		while(NodesToCheck.Num())
		{
			TSharedPtr<IDetailTreeNode> Node = NodesToCheck.Pop(EAllowShrinking::No);
			TSharedPtr<FDetailTreeNode> TreeNodeImpl = StaticCastSharedPtr<FDetailTreeNode>(Node);
			TSharedPtr<FPropertyNode> PropertyNode = TreeNodeImpl->GetPropertyNode();

			if (UNLIKELY(PropertyNode.IsValid() && PropertyNode == PropertyNodeWeAreAfter))
			{
				return Node;
			}

			Node->GetChildren(Children);	// will nix the existing Children contents
			NodesToCheck.Append(Children);
		}
	}
	return nullptr;
}

TArray<TSharedPtr<IDetailTreeNode>> FPropertyRowGenerator::FindTreeNodes(const TArray<TSharedPtr<IPropertyHandle>>& PropertyHandles) const
{
	TArray<TSharedPtr<IDetailTreeNode>> NodesToCheck;
	NodesToCheck.Append(RootTreeNodes);

	// Property Node to Array Index mapping
	TMap<TSharedPtr<FPropertyNode>, int32> PropertyNodesWeAreAfter;
	for (int32 Index = 0, NumHandle = PropertyHandles.Num(); Index < NumHandle; ++Index)
	{
		TSharedPtr<FPropertyNode> PropertyNode = StaticCastSharedPtr<FPropertyHandleBase>(PropertyHandles[Index])->GetPropertyNode();
		// we assume no duplicates in the input param
		ensure(!PropertyNodesWeAreAfter.Contains(PropertyNode));
		PropertyNodesWeAreAfter.Add(MoveTemp(PropertyNode), Index);
	}
	TArray<TSharedPtr<IDetailTreeNode>> Results;
	Results.AddDefaulted(PropertyHandles.Num());

	int32 NumNotFound = Results.Num();

	TArray<TSharedRef<IDetailTreeNode>> Children;
	while (NodesToCheck.Num())
	{
		TSharedPtr<IDetailTreeNode> Node = NodesToCheck.Pop(EAllowShrinking::No);
		TSharedPtr<FDetailTreeNode> TreeNodeImpl = StaticCastSharedPtr<FDetailTreeNode>(Node);
		TSharedPtr<FPropertyNode> PropertyNode = TreeNodeImpl->GetPropertyNode();

		if (PropertyNode.IsValid())
		{
			if (int32* HandleIndex = PropertyNodesWeAreAfter.Find(PropertyNode))
			{
				Results[*HandleIndex] = Node;
				--NumNotFound;
			}

			check(NumNotFound >= 0);
			if (NumNotFound == 0)
			{
				break;
			}
		}

		Node->GetChildren(Children);	// will nix the existing Children contents
		NodesToCheck.Append(Children);
	}

	return Results;
}

void FPropertyRowGenerator::RegisterInstancedCustomPropertyLayout(UStruct* Class, FOnGetDetailCustomizationInstance DetailLayoutDelegate)
{
	check(Class);

	FDetailLayoutCallback Callback;
	Callback.DetailLayoutDelegate = DetailLayoutDelegate;
	Callback.Order = InstancedClassToDetailLayoutMap.Num();

	InstancedClassToDetailLayoutMap.Add(Class, Callback);
}

void FPropertyRowGenerator::RegisterInstancedCustomPropertyTypeLayout(FName PropertyTypeName, FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate, TSharedPtr<IPropertyTypeIdentifier> Identifier /*= nullptr*/)
{
	FPropertyTypeLayoutCallback Callback;
	Callback.PropertyTypeLayoutDelegate = PropertyTypeLayoutDelegate;
	Callback.PropertyTypeIdentifier = Identifier;

	FPropertyTypeLayoutCallbackList* LayoutCallbacks = InstancedTypeToLayoutMap.Find(PropertyTypeName);
	if (LayoutCallbacks)
	{
		LayoutCallbacks->Add(Callback);
	}
	else
	{
		FPropertyTypeLayoutCallbackList NewLayoutCallbacks;
		NewLayoutCallbacks.Add(Callback);
		InstancedTypeToLayoutMap.Add(PropertyTypeName, NewLayoutCallbacks);
	}
}

void FPropertyRowGenerator::UnregisterInstancedCustomPropertyLayout(UStruct* Class)
{
	check(Class);

	InstancedClassToDetailLayoutMap.Remove(Class);
}

void FPropertyRowGenerator::UnregisterInstancedCustomPropertyTypeLayout(FName PropertyTypeName, TSharedPtr<IPropertyTypeIdentifier> Identifier /*= nullptr*/)
{
	FPropertyTypeLayoutCallbackList* LayoutCallbacks = InstancedTypeToLayoutMap.Find(PropertyTypeName);

	if (LayoutCallbacks)
	{
		LayoutCallbacks->Remove(Identifier);
	}
}

void FPropertyRowGenerator::InvalidateCachedState()
{
	for (const TSharedPtr<FComplexPropertyNode>& ComplexRootNode : RootPropertyNodes)
	{
		ComplexRootNode->InvalidateCachedState();
	}
}

void FPropertyRowGenerator::FilterNodes(const TArray<FString>& InFilterStrings)
{
	CurrentFilter.FilterStrings = InFilterStrings;
	UpdateDetailRows();
}

void FPropertyRowGenerator::Tick(float DeltaTime)
{
	for (TSharedPtr<IDetailCustomization>& Customization : CustomizationClassInstancesPendingDelete)
	{
		ensure(Customization.IsUnique());
	}

	// Release any pending kill nodes.
	for (TSharedPtr<FComplexPropertyNode>& PendingKillNode : RootNodesPendingKill)
	{
		if (PendingKillNode.IsValid())
		{
			PendingKillNode->Disconnect();
			PendingKillNode.Reset();
		}
	}

	RootNodesPendingKill.Empty();
	CustomizationClassInstancesPendingDelete.Empty();

	if (DeferredActions.Num() > 0)
	{
		// Execute any deferred actions
		for (FSimpleDelegate& Action : DeferredActions)
		{
			Action.ExecuteIfBound();
		}
		DeferredActions.Empty();
	}

	if (bRefreshPending)
	{
		bRefreshPending = false;
		
		ForceRefresh();
	}

	bool bFullRefresh = ValidatePropertyNodes(RootPropertyNodes);

	for (FDetailLayoutData& LayoutData : DetailLayouts)
	{
		if (LayoutData.DetailLayout.IsValid())
		{
			if (!bFullRefresh)
			{
				ValidatePropertyNodes(LayoutData.DetailLayout->GetExternalRootPropertyNodes());
			}
			LayoutData.DetailLayout->Tick(DeltaTime);
		}
	}
}

TStatId FPropertyRowGenerator::GetStatId() const 
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FPropertyRowGenerator, STATGROUP_Tickables);
}

void FPropertyRowGenerator::EnqueueDeferredAction(FSimpleDelegate DeferredAction)
{
	DeferredActions.Add(DeferredAction);
}

bool FPropertyRowGenerator::IsPropertyEditingEnabled() const
{
	if (!Args.bAllowEditingClassDefaultObjects)
	{
		return !bViewingClassDefaultObject;
	}

	return true;
}

void FPropertyRowGenerator::ForceRefresh()
{
	TArray<UObject*> NewObjectList;
	TSharedPtr<IStructureDataProvider> StructureProvider = nullptr;

	for (const TSharedPtr<FComplexPropertyNode>& ComplexRootNode : RootPropertyNodes)
	{
		if (FObjectPropertyNode* RootNode = ComplexRootNode->AsObjectNode())
		{
			// Simply re-add the same existing objects to cause a refresh
			for (TPropObjectIterator Itor(RootNode->ObjectIterator()); Itor; ++Itor)
			{
				TWeakObjectPtr<UObject> Object = *Itor;
				if (Object.IsValid())
				{
					NewObjectList.Add(Object.Get());
				}
			}
		}
		else if (FStructurePropertyNode* StructRootNode = ComplexRootNode->AsStructureNode())
		{
			StructureProvider = StructRootNode->GetStructProvider();
		}
	}

	if (StructureProvider)
	{
		SetStructure(StructureProvider);
	}
	else
	{
		SetObjects(NewObjectList);
	}
}

TSharedPtr<class FAssetThumbnailPool> FPropertyRowGenerator::GetThumbnailPool() const
{
	return UThumbnailManager::Get().GetSharedThumbnailPool();
}

const TArray<TSharedRef<class IClassViewerFilter>>& FPropertyRowGenerator::GetClassViewerFilters() const
{
	// not implemented
	static TArray<TSharedRef<class IClassViewerFilter>> NotImplemented;
	return NotImplemented;
}

void FPropertyRowGenerator::SetPropertyGenerationAllowListPaths(const TSet<FString>& InPropertyPathsToGeneratePropertyNodes)
{
	PropertyGenerationAllowListPaths = InPropertyPathsToGeneratePropertyNodes;
}

void FPropertyRowGenerator::PreSetObject(int32 NumNewObjects, bool bHasStructRoots)
{
	// Save existing expanded items first
	for (TSharedPtr<FComplexPropertyNode>& RootNode : RootPropertyNodes)
	{
		RootNodesPendingKill.Add(RootNode);
		if (FObjectPropertyNode* RootObjectNode = RootNode->AsObjectNode())
		{
			RootObjectNode->RemoveAllObjects();
			RootObjectNode->ClearObjectPackageOverrides();
		}
		else
		{
			FStructurePropertyNode* RootStructNode = RootNode->AsStructureNode();
			RootStructNode->RemoveStructure();
		}
		RootNode->ClearCachedReadAddresses(true);
	}

	RootPropertyNodes.Empty(NumNewObjects);

	if(!bHasStructRoots)
	{
		if (Args.bAllowMultipleTopLevelObjects)
		{
			for (int32 NewRootIndex = 0; NewRootIndex < NumNewObjects; ++NewRootIndex)
			{
				RootPropertyNodes.Add(MakeShared<FObjectPropertyNode>());
			}
		}
		else
		{
			RootPropertyNodes.Add(MakeShared<FObjectPropertyNode>());
		}
	}
	else
	{
		RootPropertyNodes.Add(MakeShared<FStructurePropertyNode>());
	}

}

void FPropertyRowGenerator::PostSetObject()
{
	FPropertyNodeInitParams InitParams;
	InitParams.ParentNode = nullptr;
	InitParams.Property = nullptr;
	InitParams.ArrayOffset = 0;
	InitParams.ArrayIndex = INDEX_NONE;
	InitParams.bAllowChildren = true;
	InitParams.bForceHiddenPropertyVisibility = FPropertySettings::Get().ShowHiddenProperties() || Args.bShouldShowHiddenProperties;
	
	switch (Args.DefaultsOnlyVisibility)
	{
	case EEditDefaultsOnlyNodeVisibility::Hide:
		InitParams.bCreateDisableEditOnInstanceNodes = false;
		break;
	case EEditDefaultsOnlyNodeVisibility::Show:
		InitParams.bCreateDisableEditOnInstanceNodes = true;
		break;
	case EEditDefaultsOnlyNodeVisibility::Automatic:
		InitParams.bCreateDisableEditOnInstanceNodes = HasClassDefaultObject();
		break;
	default:
		check(false);
	}

	for (TSharedPtr<FComplexPropertyNode>& ComplexRootNode : RootPropertyNodes)
	{
		ComplexRootNode->InitNode(InitParams);
	}

	UpdatePropertyMaps();

	UpdateDetailRows();
}

const FCustomPropertyTypeLayoutMap& FPropertyRowGenerator::GetInstancedPropertyTypeLayoutMap() const
{
	return InstancedTypeToLayoutMap;
}

void FPropertyRowGenerator::UpdateDetailRows()
{
	RootTreeNodes.Reset();

	FDetailNodeList InitialRootNodeList;

	//NumVisbleTopLevelObjectNodes = 0;

	for (int32 RootNodeIndex = 0; RootNodeIndex < RootPropertyNodes.Num(); ++RootNodeIndex)
	{
		TSharedPtr<FComplexPropertyNode>& RootPropertyNode = RootPropertyNodes[RootNodeIndex];
		if (RootPropertyNode.IsValid())
		{
			RootPropertyNode->FilterNodes(CurrentFilter.FilterStrings);
			RootPropertyNode->ProcessSeenFlags(true);

			TSharedPtr<FDetailLayoutBuilderImpl>& DetailLayout = DetailLayouts[RootNodeIndex].DetailLayout;
			if (DetailLayout.IsValid())
			{
				const FRootPropertyNodeList& ExternalPropertyNodeList = DetailLayout->GetExternalRootPropertyNodes();
				for (int32 NodeIndex = 0; NodeIndex < ExternalPropertyNodeList.Num(); ++NodeIndex)
				{
					TSharedPtr<FPropertyNode> PropertyNode = ExternalPropertyNodeList[NodeIndex];

					if (PropertyNode.IsValid())
					{
						PropertyNode->FilterNodes(CurrentFilter.FilterStrings);
						PropertyNode->ProcessSeenFlags(true);
					}
				}

				DetailLayout->FilterDetailLayout(CurrentFilter);

				FDetailNodeList& LayoutRoots = DetailLayout->GetFilteredRootTreeNodes();
				if (LayoutRoots.Num() > 0)
				{
					// A top level object nodes has a non-filtered away root so add one to the total number we have
					//++NumVisbleTopLevelObjectNodes;

					InitialRootNodeList.Append(LayoutRoots);
				}
			}
		}
	}


	// for multiple top level object we need to do a secondary pass on top level object nodes after we have determined if there is any nodes visible at all.  If there are then we ask the details panel if it wants to show childen
	for (TSharedRef<class FDetailTreeNode> RootNode : InitialRootNodeList)
	{
		if (RootNode->ShouldShowOnlyChildren())
		{
			FDetailNodeList TreeNodes;
			RootNode->GetChildren(TreeNodes);
			for (auto& Node : TreeNodes)
			{
				RootTreeNodes.Add(Node);
			}
		}
		else
		{
			RootTreeNodes.Add(RootNode);
		}
	}

	RowsRefreshedDelegate.Broadcast();

}

void FPropertyRowGenerator::UpdatePropertyMaps()
{
	RootTreeNodes.Empty();

	for (FDetailLayoutData& LayoutData : DetailLayouts)
	{
		// Check uniqueness.  It is critical that detail layouts can be destroyed
		// We need to be able to create a new detail layout and properly clean up the old one in the process
		check(!LayoutData.DetailLayout.IsValid() || LayoutData.DetailLayout.IsUnique());

		// All the current customization instances need to be deleted when it is safe
		CustomizationClassInstancesPendingDelete.Append(LayoutData.CustomizationClassInstances);

		for (auto ExternalRootNode : LayoutData.DetailLayout->GetExternalRootPropertyNodes())
		{
			if (ExternalRootNode.IsValid())
			{
				FComplexPropertyNode* ComplexNode = ExternalRootNode->AsComplexNode();
				if (ComplexNode)
				{
					ComplexNode->Disconnect();
				}
			}
		}
	}

	DetailLayouts.Empty(RootPropertyNodes.Num());

	// There should be one detail layout for each root node
	DetailLayouts.AddDefaulted(RootPropertyNodes.Num());

	for (int32 RootNodeIndex = 0; RootNodeIndex < RootPropertyNodes.Num(); ++RootNodeIndex)
	{
		FDetailLayoutData& LayoutData = DetailLayouts[RootNodeIndex];
		UpdateSinglePropertyMap(RootPropertyNodes[RootNodeIndex], LayoutData);
	}
}

void FPropertyRowGenerator::UpdateSinglePropertyMap(TSharedPtr<FComplexPropertyNode> InRootPropertyNode, FDetailLayoutData& LayoutData)
{
	// Reset everything
	LayoutData.ClassToPropertyMap.Empty();

	TSharedPtr<FDetailLayoutBuilderImpl> DetailLayout = MakeShareable(new FDetailLayoutBuilderImpl(InRootPropertyNode, LayoutData.ClassToPropertyMap, PropertyUtilities, PropertyGenerationUtilities, nullptr, false));
	DetailLayout->AddNodeVisibilityChangedHandler(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FPropertyRowGenerator::LayoutNodeVisibilityChanged));
	LayoutData.DetailLayout = DetailLayout;
	LayoutData.DetailLayout->SetPropertyGenerationAllowListPaths(PropertyGenerationAllowListPaths);

	TSharedPtr<FComplexPropertyNode> RootPropertyNode = InRootPropertyNode;
	check(RootPropertyNode.IsValid());

	DetailLayoutHelpers::FUpdatePropertyMapArgs LayoutArgs;

	LayoutArgs.LayoutData = &LayoutData;
	LayoutArgs.InstancedPropertyTypeToDetailLayoutMap = &InstancedTypeToLayoutMap;
	LayoutArgs.bEnableFavoriteSystem = false;
	LayoutArgs.bUpdateFavoriteSystemOnly = false;

	DetailLayoutHelpers::UpdateSinglePropertyMapRecursive(*RootPropertyNode, NAME_None, RootPropertyNode.Get(), LayoutArgs);

	DetailLayoutHelpers::QueryCustomDetailLayout(LayoutData, InstancedClassToDetailLayoutMap, FOnGetDetailCustomizationInstance());

	LayoutData.DetailLayout->GenerateDetailLayout();
}


bool FPropertyRowGenerator::ValidatePropertyNodes(const FRootPropertyNodeList& PropertyNodeList)
{
	if (CustomValidatePropertyNodesFunction.IsBound())
	{
		return CustomValidatePropertyNodesFunction.Execute(PropertyNodeList);
	}
	
	bool bFullRefresh = false;

	for (const TSharedPtr<FComplexPropertyNode>& RootPropertyNode : RootPropertyNodes)
	{
		// Purge any objects that are marked pending kill from the object list
		if (FObjectPropertyNode* ObjectRoot = RootPropertyNode->AsObjectNode())
		{
			ObjectRoot->PurgeKilledObjects();
		}

		EPropertyDataValidationResult Result = RootPropertyNode->EnsureDataIsValid();
		if (Result == EPropertyDataValidationResult::PropertiesChanged || Result == EPropertyDataValidationResult::EditInlineNewValueChanged)
		{
			UpdatePropertyMaps();
			UpdateDetailRows();
			break;
		}
		else if (Result == EPropertyDataValidationResult::ArraySizeChanged || Result == EPropertyDataValidationResult::ChildrenRebuilt)
		{
			UpdateDetailRows();
		}
		else if (Result == EPropertyDataValidationResult::ObjectInvalid)
		{
			ForceRefresh();
			bFullRefresh = true;
			break;
		}
	}

	return bFullRefresh;
}

TSharedPtr<IDetailTreeNode> FPropertyRowGenerator::FindTreeNodeRecursive(const TSharedPtr<IDetailTreeNode>& StartNode, TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	TSharedPtr<FDetailTreeNode> TreeNodeImpl = StaticCastSharedPtr<FDetailTreeNode>(StartNode);
	
	TSharedPtr<FPropertyNode> PropertyNode = TreeNodeImpl->GetPropertyNode();
	if (PropertyNode.IsValid() && PropertyNode == StaticCastSharedPtr<FPropertyHandleBase>(PropertyHandle)->GetPropertyNode())
	{
		return StartNode;
	}

	TArray<TSharedRef<IDetailTreeNode>> Children;
	StartNode->GetChildren(Children);
	for (TSharedRef<IDetailTreeNode>& Child : Children)
	{
		TSharedPtr<IDetailTreeNode> FoundNode = FindTreeNodeRecursive(Child, PropertyHandle);
		if (FoundNode.IsValid())
		{
			return FoundNode;
		}
	}

	return nullptr;
}

void FPropertyRowGenerator::LayoutNodeVisibilityChanged()
{
	UpdateDetailRows();
}

