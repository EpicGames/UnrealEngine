// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DetailFilter.h"
#include "UObject/WeakObjectPtr.h"
#include "PropertyPath.h"
#include "TickableEditorObject.h"
#include "IPropertyUtilities.h"
#include "PropertyEditorModule.h"
#include "IPropertyRowGenerator.h"

class FDetailCategoryImpl;
class FDetailLayoutBuilderImpl;
class FPropertyNode;
class IDetailCustomization;
class IDetailNodeTree;
class FComplexPropertyNode;
class FDetailTreeNode;
class IPropertyGenerationUtilities;
class FStructOnScope;
class IStructureDataProvider;

struct FPropertyNodeMap
{
	FPropertyNodeMap()
		: ParentProperty(NULL)
	{}

	/** Object property node which contains the properties in the node map */
	FPropertyNode* ParentProperty;

	/** Property name to property node map */
	TMap<FName, TSharedPtr<FPropertyNode> > PropertyNameToNode;

	bool Contains(FName PropertyName) const
	{
		return PropertyNameToNode.Contains(PropertyName);
	}

	void Add(FName PropertyName, TSharedPtr<FPropertyNode>& PropertyNode)
	{
		PropertyNameToNode.Add(PropertyName, PropertyNode);
	}
};


/** Mapping of categories to all top level item property nodes in that category */
typedef TMap<FName, TSharedPtr<FDetailCategoryImpl> > FCategoryMap;

/** Class to properties in that class */
typedef TMap<FName, FPropertyNodeMap> FClassInstanceToPropertyMap;

/** Class to properties in that class */
typedef TMap<FName, FClassInstanceToPropertyMap> FClassToPropertyMap;

struct FDetailLayoutData
{
	TSharedPtr<FDetailLayoutBuilderImpl> DetailLayout;

	FClassToPropertyMap ClassToPropertyMap;

	/** A  unique classes being viewed */
	TSet<TWeakObjectPtr<UStruct>> ClassesWithProperties;

	/** Customization class instances currently active in this view */
	TArray<TSharedPtr<IDetailCustomization>> CustomizationClassInstances;
};

class FPropertyRowGenerator : public IPropertyRowGenerator, public FTickableEditorObject, public TSharedFromThis<FPropertyRowGenerator>
{
public:
	FPropertyRowGenerator(const FPropertyRowGeneratorArgs& InArgs);

	UE_DEPRECATED(5.0, "FPropertyRowGenerator which takes in a thumbnail pool parameter is no longer necessary.")
	FPropertyRowGenerator(const FPropertyRowGeneratorArgs& InArgs, TSharedPtr<FAssetThumbnailPool> InThumbnailPool);

	~FPropertyRowGenerator();

	DECLARE_DERIVED_EVENT(FPropertyRowGenerator, IPropertyRowGenerator::FOnRowsRefreshed, FOnRowsRefreshed);

	/** IPropertyRowGenerator interface */
	virtual void SetObjects(const TArray<UObject*>& InObjects) override;
	virtual void SetStructure(const TSharedPtr<FStructOnScope>& InStruct) override;
	virtual void SetStructure(const TSharedPtr<IStructureDataProvider>& InStructProvider) override;
	virtual const TArray<TWeakObjectPtr<UObject>>& GetSelectedObjects() const override { return SelectedObjects; }
	virtual const TArray<TSharedRef<IDetailTreeNode>>& GetRootTreeNodes() const override;
	virtual TSharedPtr<IDetailTreeNode> FindTreeNode(TSharedPtr<IPropertyHandle> PropertyHandle) const override;
	virtual TArray<TSharedPtr<IDetailTreeNode>> FindTreeNodes(const TArray<TSharedPtr<IPropertyHandle>>& PropertyHandles) const override;
	virtual FOnRowsRefreshed& OnRowsRefreshed() override { return RowsRefreshedDelegate; }
	virtual void RegisterInstancedCustomPropertyLayout(UStruct* Class, FOnGetDetailCustomizationInstance DetailLayoutDelegate) override;
	virtual void RegisterInstancedCustomPropertyTypeLayout(FName PropertyTypeName, FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate, TSharedPtr<IPropertyTypeIdentifier> Identifier = nullptr) override;
	virtual void UnregisterInstancedCustomPropertyLayout(UStruct* Class) override;
	virtual void UnregisterInstancedCustomPropertyTypeLayout(FName PropertyTypeName, TSharedPtr<IPropertyTypeIdentifier> Identifier = nullptr) override;
	virtual void SetCustomValidatePropertyNodesFunction(FOnValidatePropertyRowGeneratorNodes InCustomValidatePropertyNodesFunction) override
	{
		CustomValidatePropertyNodesFunction = MoveTemp(InCustomValidatePropertyNodesFunction);
	}
	/** This function sets property paths to generate PropertyNodes.This improves the performance for cases where PropertyView is only showing a few properties of the object by not generating all other PropertyNodes */
	virtual void SetPropertyGenerationAllowListPaths(const TSet<FString>& InPropertyGenerationAllowListPaths) override;
	virtual void InvalidateCachedState() override;
	virtual void FilterNodes(const TArray<FString>& InFilterStrings) override;

	/** FTickableEditorObject interface */
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override;

	/** IPropertyUtilities interface */
	virtual class FNotifyHook* GetNotifyHook() const { return Args.NotifyHook; }
	virtual void EnqueueDeferredAction(FSimpleDelegate DeferredAction);	
	virtual bool IsPropertyEditingEnabled() const;
	virtual void ForceRefresh();
	virtual void RequestRefresh() { bRefreshPending = true; }
	virtual TSharedPtr<class FAssetThumbnailPool> GetThumbnailPool() const;
	virtual bool HasClassDefaultObject() const { return bViewingClassDefaultObject; }
	virtual const TArray<TSharedRef<class IClassViewerFilter>>& GetClassViewerFilters() const;

	const FCustomPropertyTypeLayoutMap& GetInstancedPropertyTypeLayoutMap() const;
	void UpdateDetailRows();
	virtual FOnNotifyStartedChangingProperties& OnNotifyStartedChangingProperties() override { return OnNotifyStartedChangingPropertiesDelegate; }
	virtual FOnFinishedChangingProperties& OnFinishedChangingProperties() override { return OnFinishedChangingPropertiesDelegate; }

private:
	void PreSetObject(int32 NumNewObjects, bool bHasStructRoots);
	void PostSetObject();
	void UpdatePropertyMaps();
	void UpdateSinglePropertyMap(TSharedPtr<FComplexPropertyNode> InRootPropertyNode, FDetailLayoutData& LayoutData);
	bool ValidatePropertyNodes(const FRootPropertyNodeList& PropertyNodeList);
	TSharedPtr<IDetailTreeNode> FindTreeNodeRecursive(const TSharedPtr<IDetailTreeNode>& StartNode, TSharedPtr<IPropertyHandle> PropertyHandle) const;
	void LayoutNodeVisibilityChanged();

private:
	const FPropertyRowGeneratorArgs Args;
	/** The root property nodes of the property tree for a specific set of UObjects */
	FRootPropertyNodeList RootPropertyNodes;
	/** Root tree nodes that needs to be destroyed when safe */
	FRootPropertyNodeList RootNodesPendingKill;
	/** Root tree nodes visible in the tree */
	TArray<TSharedRef<IDetailTreeNode>> RootTreeNodes;
	/** The current detail layout based on objects in this details panel.  There is one layout for each top level object node.*/
	TArray<FDetailLayoutData> DetailLayouts;
	/** Customization instances that need to be destroyed when safe to do so */
	TArray<TSharedPtr<IDetailCustomization>> CustomizationClassInstancesPendingDelete;
	/** Actions that should be executed next tick */
	TArray<FSimpleDelegate> DeferredActions;
	/** Currently viewed objects */
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	/** Delegate to call when the user of this generator needs to know the rows are invalid */
	FOnRowsRefreshed RowsRefreshedDelegate;
	/** A mapping of type names to detail layout delegates, called when querying for custom detail layouts in this instance of the details view only */
	FCustomPropertyTypeLayoutMap InstancedTypeToLayoutMap;
	/** A mapping of classes to detail layout delegates, called when querying for custom detail layouts in this instance of the details view only*/
	FCustomDetailLayoutMap InstancedClassToDetailLayoutMap;
	/** Utility class for accessing commonly used helper methods from customizations */
	TSharedRef<IPropertyUtilities> PropertyUtilities;
	/** Utility class for accessing internal helper methods */
	TSharedRef<IPropertyGenerationUtilities> PropertyGenerationUtilities;
	/** Delegate called when the details panel finishes editing a property (after post edit change is called) */
	FOnFinishedChangingProperties OnFinishedChangingPropertiesDelegate;
	/** Delegate called before the details panel finishes editing a property (after pre edit change is called) */
	FOnNotifyStartedChangingProperties OnNotifyStartedChangingPropertiesDelegate;
	/** The ValidatePropertyNodes function can be overridden with this member, if set. Useful if your implementation doesn't require this kind of validation each Tick. */
	FOnValidatePropertyRowGeneratorNodes CustomValidatePropertyNodesFunction;
	/** If specified only nodes for these properties will be generated */
	TSet<FString> PropertyGenerationAllowListPaths;
	/** Filter to apply on generated rows nodes */
	FDetailFilter CurrentFilter;

	bool bViewingClassDefaultObject = false;
	bool bRefreshPending = false;
};
