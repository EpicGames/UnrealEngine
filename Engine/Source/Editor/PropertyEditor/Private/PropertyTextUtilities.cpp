// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyTextUtilities.h"

#include "CoreTypes.h"
#include "PropertyHandleImpl.h"
#include "PropertyNode.h"
#include "UObject/UnrealType.h"

class FString;
class UObject;

void FPropertyTextUtilities::PropertyToTextHelper(FString& OutString, const FPropertyNode* InPropertyNode, const FProperty* Property, const uint8* ValueAddress, UObject* Object, EPropertyPortFlags PortFlags)
{
	if (InPropertyNode->GetArrayIndex() != INDEX_NONE || Property->ArrayDim == 1)
	{
		Property->ExportText_Direct(OutString, ValueAddress, ValueAddress, Object, PortFlags);
	}
	else
	{
		FArrayProperty::ExportTextInnerItem(OutString, Property, ValueAddress, Property->ArrayDim, ValueAddress, Property->ArrayDim, Object, PortFlags);
	}
}

void FPropertyTextUtilities::PropertyToTextHelper(FString& OutString, const FPropertyNode* InPropertyNode, const FProperty* Property, const FObjectBaseAddress& ObjectAddress, EPropertyPortFlags PortFlags)
{
	PropertyToTextHelper(OutString, InPropertyNode, Property, ObjectAddress.BaseAddress, ObjectAddress.Object, PortFlags);
}

void FPropertyTextUtilities::TextToPropertyHelper(const TCHAR* Buffer, const FPropertyNode* InPropertyNode, const FProperty* Property, uint8* ValueAddress, UObject* Object, EPropertyPortFlags PortFlags)
{
	if (InPropertyNode->GetArrayIndex() != INDEX_NONE || Property->ArrayDim == 1)
	{
		Property->ImportText_Direct(Buffer, ValueAddress, Object, PortFlags);
	}
	else
	{
		FArrayProperty::ImportTextInnerItem(Buffer, Property, ValueAddress, PortFlags, Object);
	}
}

void FPropertyTextUtilities::TextToPropertyHelper(const TCHAR* Buffer, const FPropertyNode* InPropertyNode, const FProperty* Property, const FObjectBaseAddress& ObjectAddress, EPropertyPortFlags PortFlags)
{
	TextToPropertyHelper(Buffer, InPropertyNode, Property, ObjectAddress.BaseAddress, ObjectAddress.Object, PortFlags);
}
