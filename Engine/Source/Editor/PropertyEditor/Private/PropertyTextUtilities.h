// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "UObject/PropertyPortFlags.h"

class FObjectBaseAddress;
class FProperty;
class FPropertyNode;
class FString;
class UObject;

class FPropertyTextUtilities
{
public:
	static void PropertyToTextHelper(FString& OutString, const FPropertyNode* InPropertyNode, const FProperty* Property, const uint8* ValueAddress, UObject* Object, EPropertyPortFlags PortFlags);
	static void PropertyToTextHelper(FString& OutString, const FPropertyNode* InPropertyNode, const FProperty* Property, const FObjectBaseAddress& ObjectAddress, EPropertyPortFlags PortFlags);
	static void TextToPropertyHelper(const TCHAR* Buffer, const FPropertyNode* InPropertyNode, const FProperty* Property, uint8* ValueAddress, UObject* Object, EPropertyPortFlags PortFlags = PPF_None);
	static void TextToPropertyHelper(const TCHAR* Buffer, const FPropertyNode* InPropertyNode, const FProperty* Property, const FObjectBaseAddress& ObjectAddress, EPropertyPortFlags PortFlags = PPF_None);
};