// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConstrainedBox.h"

#include "HAL/PlatformCrt.h"
#include "Layout/Children.h"
#include "Math/UnrealMathUtility.h"
#include "Widgets/SWidget.h"

void SConstrainedBox::Construct(const FArguments& InArgs)
{
	MinWidth = InArgs._MinWidth;
	MaxWidth = InArgs._MaxWidth;

	ChildSlot
	[
		InArgs._Content.Widget
	];
}

FVector2D SConstrainedBox::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	const float MinWidthVal = MinWidth.Get().Get(0.0f);
	const float MaxWidthVal = MaxWidth.Get().Get(0.0f);

	if (MinWidthVal == 0.0f && MaxWidthVal == 0.0f)
	{
		return SCompoundWidget::ComputeDesiredSize(LayoutScaleMultiplier);
	}
	else
	{
		FVector2D ChildSize = ChildSlot.GetWidget()->GetDesiredSize();

		double XVal = FMath::Max(MinWidthVal, ChildSize.X);
		if (MaxWidthVal > MinWidthVal)
		{
			XVal = FMath::Min(MaxWidthVal, XVal);
		}

		return FVector2D(XVal, ChildSize.Y);
	}
}
