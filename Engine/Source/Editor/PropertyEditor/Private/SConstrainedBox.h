// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SWidget;

class SConstrainedBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SConstrainedBox)
		: _MinWidth()
		, _MaxWidth()
	{}
	SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_ATTRIBUTE(TOptional<float>, MinWidth)
		SLATE_ATTRIBUTE(TOptional<float>, MaxWidth)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;

private:
	TAttribute< TOptional<float> > MinWidth;
	TAttribute< TOptional<float> > MaxWidth;
};
