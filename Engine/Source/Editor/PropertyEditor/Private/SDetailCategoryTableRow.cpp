// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDetailCategoryTableRow.h"

#include "Algo/AnyOf.h"
#include "Algo/Compare.h"
#include "Async/ParallelFor.h"
#include "DetailCategoryBuilderImpl.h"
#include "PropertyEditorClipboard.h"
#include "PropertyEditorClipboardPrivate.h"
#include "ScopedTransaction.h"
#include "SDetailExpanderArrow.h"
#include "SDetailRowIndent.h"
#include "DetailsViewStyle.h"
#include "SDetailsView.h"
#include "Serialization/JsonSerializer.h"
#include "UserInterface/Categories/CategoryMenuComboButtonBuilder.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SSeparator.h"
#include "Brushes/SlateColorBrush.h"
#include "ToolMenus.h"

void SDetailCategoryTableRow::Construct(const FArguments& InArgs, TSharedRef<FDetailTreeNode> InOwnerTreeNode, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	OwnerTreeNode = InOwnerTreeNode;

	DisplayName = InArgs._DisplayName;
	bIsInnerCategory = InArgs._InnerCategory;
	bShowBorder = InArgs._ShowBorder;
	bShowHeader = InArgs._ShowHeader;
	bIsEmpty = InArgs._IsEmpty;

	TSharedPtr<IDetailsViewPrivate> DetailsView = InOwnerTreeNode->GetDetailsViewSharedPtr();
	FDetailColumnSizeData& ColumnSizeData = DetailsView->GetColumnSizeData();
	DisplayManager = DetailsView->GetDisplayManager();

	if (DisplayManager.IsValid() && DisplayManager->GetDetailsViewStyle())
	{
		DetailsViewStyle = DisplayManager->GetDetailsViewStyle();
	}

	if (DisplayManager.IsValid())
	{
		DisplayManager->SetIsOuterCategory( !bIsInnerCategory );
	}

	PulseAnimation.AddCurve(0.0f, UE::PropertyEditor::Private::PulseAnimationLength, ECurveEaseFunction::CubicInOut);
	CopyAction.ExecuteAction = FExecuteAction::CreateSP(this, &SDetailCategoryTableRow::OnCopyCategory);
	CopyAction.CanExecuteAction = FCanExecuteAction::CreateSP(this, &SDetailCategoryTableRow::CanCopyCategory);

	if (InArgs._PasteFromText.IsValid())
	{
		OnPasteFromTextDelegate = InArgs._PasteFromText;
	}
	if (InArgs._CopyToText.IsValid())
	{
		OnCopyToTextDelegate = InArgs._CopyToText;
	}
	
	PasteAction.ExecuteAction = FExecuteAction::CreateSP(this, &SDetailCategoryTableRow::OnPasteCategory);
	PasteAction.CanExecuteAction = FCanExecuteAction::CreateSP(this, &SDetailCategoryTableRow::CanPasteCategory);

	ResetToDefault.ExecuteAction = FExecuteAction::CreateSP(this, &SDetailCategoryTableRow::OnResetToDefaultCategory);
	ResetToDefault.CanExecuteAction = FCanExecuteAction::CreateSP(this, &SDetailCategoryTableRow::CanResetToDefaultCategory);
	
	TSharedRef<SHorizontalBox> HeaderBox = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Fill)
		.AutoWidth()
		[
			SNew(SDetailRowIndent, SharedThis(this))
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(2.0f, 0.0f, 0.0f, 0.0f)
		.AutoWidth()
		[
			SNew(SDetailExpanderArrow, SharedThis(this))
				//if this is a stub category
				.Visibility_Lambda([this]
				{
					if (bIsEmpty)
					{
						return EVisibility::Hidden;
					}
					return EVisibility::Visible;
				})
		];
	
	if (!InArgs._WholeRowHeaderContent)
	{
		HeaderBox->AddSlot()
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			.FillWidth(1)
			[
				SNew(STextBlock)
				.Text(InArgs._DisplayName)
				.Font(FAppStyle::Get().GetFontStyle(bIsInnerCategory ? PropertyEditorConstants::PropertyFontStyle : PropertyEditorConstants::CategoryFontStyle))
				.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
			];
	}

	if (InArgs._HeaderContent.IsValid())
	{
		HeaderBox->AddSlot()
			.VAlign(VAlign_Center)
			.FillWidth(1)
			[
				InArgs._HeaderContent.ToSharedRef()
			];
	}

	OwnerTableViewWeak = InOwnerTableView;
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (DisplayManager->CanConstructPropertyUpdatedWidgetBuilder())
	{
		TSharedPtr<FPropertyNode> Node = InOwnerTreeNode->GetPropertyNode();
		FConstructPropertyUpdatedWidgetBuilderArgs Args;
		Args.InvalidateCachedState = FExecuteAction::CreateSPLambda(this, [DetailsView]
		{
			if (DetailsView.IsValid())
			{
				DetailsView->InvalidateCachedState();
			}
		});

		Args.ResetToDefaultAction = FExecuteAction::CreateSPLambda(this, [this]
		{
			if(ResetToDefault.CanExecute())
			{
				ResetToDefault.Execute();
			}

			if (TSharedPtr<FDetailTreeNode> OwnerTreeNodePinned = OwnerTreeNode.Pin())
			{
				if (TSharedPtr<IDetailsViewPrivate> DetailsView = OwnerTreeNodePinned->GetDetailsViewSharedPtr())
				{
					DetailsView->RequestForceRefresh();
				}
			}
		});
		Args.PropertyPath = nullptr;
		Args.Category = InOwnerTreeNode->GetNodeName();
		Args.Objects = InArgs._Objects;
		
		PropertyUpdatedWidgetBuilder = DisplayManager->ConstructPropertyUpdatedWidgetBuilder(Args);
		if (PropertyUpdatedWidgetBuilder.IsValid())
		{
			TAttribute<bool> IsHovered = TAttribute<bool>::CreateSP( this, &SDetailCategoryTableRow::IsHovered);
			PropertyUpdatedWidgetBuilder->Bind_IsRowHovered(IsHovered);
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("DetailsView.GridLine"))
		.Padding_Lambda([this]
		{
			return DetailsViewStyle ? DetailsViewStyle->GetRowPadding(!bIsInnerCategory) : 0;
		})
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SBorder)
				.BorderImage(this, &SDetailCategoryTableRow::GetBackgroundImage)
				.BorderBackgroundColor(this, &SDetailCategoryTableRow::GetInnerBackgroundColor)
				.Padding(0.0f)
				.Visibility(bShowHeader ? EVisibility::Visible : EVisibility::Collapsed)
				[
					SNew(SBox)
					.MinDesiredHeight(PropertyEditorConstants::PropertyRowHeight)
					[
						HeaderBox
					]
				]
			]
			 + SOverlay::Slot()
			.HAlign(HAlign_Right)
			[
			SNew(SBorder)
			.BorderImage(new FSlateColorBrush(FLinearColor::Transparent))
			.Visibility(EVisibility::Visible)
			.Padding_Lambda([this]
			{
				return DetailsViewStyle ? DetailsViewStyle->GetCategoryButtonsMargin() : 0;
			})
			[

			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				PropertyUpdatedWidgetBuilder.IsValid() ?
					PropertyUpdatedWidgetBuilder->
						Bind_IsVisible(TAttribute<EVisibility>::CreateLambda([this]
						{
							return EVisibility::Visible;
						}))
						.GenerateWidget().ToSharedRef() :
						SNullWidget::NullWidget
			]
			]
			] 
		]
	];

	STableRow< TSharedPtr< FDetailTreeNode > >::ConstructInternal(
		STableRow::FArguments()
		.Style(FAppStyle::Get(), "DetailsView.TreeView.TableRow")
		.ShowSelection(false),
		InOwnerTableView
	);
}

FReply SDetailCategoryTableRow::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetModifierKeys().IsShiftDown())
	{
		bool bIsHandled = false;
		if (CopyAction.CanExecute() && MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			CopyAction.Execute();
			PulseAnimation.Play(SharedThis(this));
			bIsHandled = true;
		}
		else if (PasteAction.CanExecute() && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			PasteAction.Execute();
			PulseAnimation.Play(SharedThis(this));
			bIsHandled = true;
		}

		if (bIsHandled)
		{
			return FReply::Handled();
		}
	}

	return SDetailTableRowBase::OnMouseButtonUp(MyGeometry, MouseEvent);
}

void SDetailCategoryTableRow::PopulateContextMenu(UToolMenu* ToolMenu)
{
	SDetailTableRowBase::PopulateContextMenu(ToolMenu);

	FToolMenuSection& EditSection = ToolMenu->FindOrAddSection(TEXT("Edit"));
	{
		// Don't add anything if neither actions are bound
		if (CopyAction.IsBound() && PasteAction.IsBound())
		{
			bool bLongDisplayName = false;

			{
				// Copy
				{
					FToolMenuEntry& CopyMenuEntry = EditSection.AddMenuEntry(
						TEXT("Copy"),
						NSLOCTEXT("PropertyView", "CopyCategoryProperties", "Copy All Properties in Category"),
						TAttribute<FText>::CreateLambda([this]()
						{
							return CanCopyCategory()
								? NSLOCTEXT("PropertyView", "CopyCategoryProperties_ToolTip", "Copy all properties in this category")
								: NSLOCTEXT("PropertyView", "CantCopyCategoryProperties_ToolTip", "None of the properties in this category can be copied");
						}),
						FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
						CopyAction);

					CopyMenuEntry.InputBindingLabel = FInputChord(EModifierKey::Shift, EKeys::RightMouseButton).GetInputText(bLongDisplayName);
				}

				// Paste
				{
					FToolMenuEntry& PasteMenuEntry = EditSection.AddMenuEntry(
						TEXT("Paste"),
						NSLOCTEXT("PropertyView", "PasteCategoryProperties", "Paste All Properties in Category"),
						TAttribute<FText>::CreateLambda([this]()
						{
							return CanPasteCategory()
								? NSLOCTEXT("PropertyView", "PasteCategoryProperties_ToolTip", "Paste the copied property values here")
								// @note: this is specific to the constraint that the destination category has to match the source category (copied from) exactly 
								: NSLOCTEXT("PropertyView", "CantPasteCategoryProperties_ToolTip", "The properties in this category don't match the contents of the clipboard, or the properties aren't editable");
						}),
						FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Paste"),
						PasteAction);

					PasteMenuEntry.InputBindingLabel = FInputChord(EModifierKey::Shift, EKeys::LeftMouseButton).GetInputText(bLongDisplayName);
				}
			}
		}
	}
}

bool SDetailCategoryTableRow::CanFavoriteRow(bool bInAddToFavorite)
{
	const TSharedPtr<FDetailTreeNode> OwnerNode = OwnerTreeNode.Pin();
	if (!OwnerNode)
	{
		return false;
	}

	// Checks if node is a favorite or included in favorite category
	TFunction<bool(const TSharedPtr<FDetailTreeNode>&, bool)> IsFavoriteCategoryNode = [&IsFavoriteCategoryNode](const TSharedPtr<FDetailTreeNode>& InNode, bool bInRecurse)
		{
			if (!InNode)
			{
				return false;
			}

			if (InNode->GetNodeType() == EDetailNodeType::Category)
			{
				const TSharedPtr<FDetailCategoryImpl> CategoryNode = StaticCastSharedPtr<FDetailCategoryImpl>(InNode);
				if (CategoryNode->IsFavoriteCategory())
				{
					return true;
				}
			}

			return bInRecurse ? IsFavoriteCategoryNode(InNode->GetParentNode().Pin(), bInRecurse) : false;
		};

	// Disallow adding Favorites category to Favorites...
	// Disallow adding subcategories (group) of Favorites to Favorites
	const bool bIsRowInFavorites = IsFavoriteCategoryNode(OwnerNode, /** Recurse */true);
	if (bInAddToFavorite && bIsRowInFavorites)
	{
		return false;
	}

	bool bCanFavoriteRow = false;
	ForEachImmediateChildPropertyNode([bInAddToFavorite, &bCanFavoriteRow](const TSharedPtr<FPropertyNode>& PropertyNode)
		{
			if (PropertyNode->IsFavorite() != bInAddToFavorite)
			{
				bCanFavoriteRow = true;
				return false; // Break
			}

			return true; // Continue
		}
	);

	return bCanFavoriteRow;
}

void SDetailCategoryTableRow::OnFavoriteRow(bool bInAddToFavorite)
{
	bool bRefresh = false;
	ForEachImmediateChildPropertyNode([bInAddToFavorite, &bRefresh](const TSharedPtr<FPropertyNode>& PropertyNode)
		{
			PropertyNode->SetFavorite(bInAddToFavorite);
			bRefresh = true;
			return true; // Continue
		}
	);

	if (bRefresh)
	{
		// Refresh the tree
		ForceRefresh();
	}
}

void SDetailCategoryTableRow::ForEachImmediateChildPropertyNode(TFunctionRef<bool(TSharedPtr<FPropertyNode>&)> InFunctor)
{
	const TSharedPtr<FDetailTreeNode> TreeNode = OwnerTreeNode.Pin();
	if (!TreeNode)
	{
		return;
	}

	TArray<TSharedRef<IDetailTreeNode>> ChildNodes;
	TreeNode->GetChildren(ChildNodes);
	if (ChildNodes.IsEmpty())
	{
		return;
	}

	for (const TSharedRef<IDetailTreeNode>& Child : ChildNodes)
	{
		if (TSharedPtr<IPropertyHandle> PropertyHandle = Child->CreatePropertyHandle())
		{
			for (TSharedPtr<FPropertyNode>& PropertyNode : GetPropertyNodesFromHandles({PropertyHandle}))
			{
				if (PropertyNode.IsValid())
				{
					const bool bContinue = InFunctor(PropertyNode);
				
					if (!bContinue)
					{
						return;
					}
				}
			}
		}
	}
}

EVisibility SDetailCategoryTableRow::IsSeparatorVisible() const
{
	return bIsInnerCategory || IsItemExpanded() ? EVisibility::Collapsed : EVisibility::Visible;
}

/* TODO ~ refactor to not require styling of row and scrollbar well separately */
const FSlateBrush* SDetailCategoryTableRow::GetBackgroundImage() const
{
	if (bShowBorder)
	{
		if (bIsInnerCategory)
		{
			return FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle");
		}

		const bool bIsScrollBarNeeded = IsScrollBarVisible(OwnerTableViewWeak);
		DisplayManager->SetIsScrollBarNeeded(bIsScrollBarNeeded);
		const bool bIsCategoryExpanded = IsItemExpanded() && !bIsEmpty;
		return DetailsViewStyle ? DetailsViewStyle->GetBackgroundImageForCategoryRow(bShowBorder, bIsInnerCategory, bIsCategoryExpanded) : nullptr;
	}
	return nullptr;
}

const FSlateBrush* SDetailCategoryTableRow::GetBackgroundImageForScrollBarWell() const
{
	if (bShowBorder)
	{
		const bool bIsCategoryExpanded = IsItemExpanded() && !bIsEmpty;
		const bool bIsScrollBarVisible = IsScrollBarVisible(OwnerTableViewWeak);
		return DetailsViewStyle ? DetailsViewStyle->GetBackgroundImageForScrollBarWell(
			bShowBorder, bIsInnerCategory, bIsCategoryExpanded, bIsScrollBarVisible) : nullptr;
	}
	return nullptr;
}

FSlateColor SDetailCategoryTableRow::GetInnerBackgroundColor() const
{
	FSlateColor Color = FSlateColor(FLinearColor::White);
	
	if (bShowBorder && bIsInnerCategory)
	{
		int32 IndentLevel = -1;
		if (OwnerTablePtr.IsValid())
		{
			IndentLevel = GetIndentLevel();
		}

		IndentLevel = FMath::Max(IndentLevel - 1, 0);

		Color = PropertyEditorConstants::GetRowBackgroundColor(IndentLevel, this->IsHovered());
	}

	if (PulseAnimation.IsPlaying())
	{
		float Lerp = PulseAnimation.GetLerp();
		return FMath::Lerp(FAppStyle::Get().GetSlateColor("Colors.Hover2").GetSpecifiedColor(), Color.GetSpecifiedColor(), Lerp);
	}

	return Color;
}

FSlateColor SDetailCategoryTableRow::GetOuterBackgroundColor() const
{
	if (IsHovered())
	{
		return FAppStyle::Get().GetSlateColor("Colors.Header");
	}

	return FAppStyle::Get().GetSlateColor("Colors.Panel");
}

void SDetailCategoryTableRow::OnCopyCategory()
{
	if (!OwnerTreeNode.IsValid())
	{
		return;
	}

	if (TArray<TSharedPtr<IPropertyHandle>> CategoryProperties = GetPropertyHandles(true);
		!CategoryProperties.IsEmpty())
	{
		TArray<FString> PropertiesNotCopied;
		PropertiesNotCopied.Reserve(CategoryProperties.Num());
		
		TMap<FString, FString> PropertyValues;
		PropertyValues.Reserve(CategoryProperties.Num());

		const FGuid OperationGuid = FGuid::NewGuid();			

		for (TSharedPtr<IPropertyHandle> PropertyHandle : CategoryProperties)
		{
			if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
			{
				FString PropertyPath = UE::PropertyEditor::GetPropertyPath(PropertyHandle);
				
				FString PropertyValueStr;

				// First check if there is some custom copy paste that handles this property
				if (OnCopyToTextDelegate.IsValid())
				{
					OnCopyToTextDelegate->Broadcast(PropertyHandle, PropertyValueStr, OperationGuid);
				}

				if (!PropertyValueStr.IsEmpty())
				{
					PropertyValues.Add(PropertyPath, PropertyValueStr);
				}
				else if (PropertyHandle->GetValueAsFormattedString(PropertyValueStr, PPF_Copy) == FPropertyAccess::Success)
				{
					PropertyValues.Add(PropertyPath, PropertyValueStr);
				}
				else
				{
					PropertiesNotCopied.Add(PropertyHandle->GetPropertyDisplayName().ToString());
				}
			}
		}

		if (!PropertiesNotCopied.IsEmpty())
		{
			UE_LOGF(
				LogPropertyNode,
				Warning,
				"One or more of the properties in category \"%ls\" was not copied:\n%ls",
				*DisplayName.ToString(),
				*FString::Join(PropertiesNotCopied, TEXT("\n")));
		}

		FPropertyEditorClipboard::ClipboardCopy([&PropertyValues](TMap<FName, FString>& OutTaggedClipboard)
		{
			for (const TPair<FString, FString>& PropertyValuePair : PropertyValues)
			{
				OutTaggedClipboard.Add(FName(PropertyValuePair.Key), PropertyValuePair.Value);
			}
		});

		PulseAnimation.Play(SharedThis(this));
	}
}

bool SDetailCategoryTableRow::CanCopyCategory() const
{
	if (!OwnerTreeNode.IsValid())
	{
		return false;
	}

	TArray<TSharedPtr<IPropertyHandle>> PropertyHandles = GetPropertyHandles(true);
	return !PropertyHandles.IsEmpty();
}

void SDetailCategoryTableRow::OnPasteCategory()
{
	if (!OwnerTreeNode.IsValid()
		|| !CanPasteCategory())
	{
		return;
	}

	if (const TArray<TSharedPtr<IPropertyHandle>> CategoryProperties = GetPropertyHandles(true);
		!CategoryProperties.IsEmpty())
	{
		FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "PasteCategoryProperties", "Paste Category Properties"));

		TArray<FString> PropertiesNotPasted;
		PropertiesNotPasted.Reserve(CategoryProperties.Num());

		{
			// Assign a unique id to this paste operation, enabling batching of various operations
			const FGuid OperationGuid = FGuid::NewGuid();			
			for (const TPair<FName, FString>& KVP : PreviousClipboardData.PropertyValues)
			{
				if (OnPasteFromTextDelegate.IsValid())
				{
					OnPasteFromTextDelegate->Broadcast(KVP.Key.ToString(), KVP.Value, OperationGuid);
				}
			}
		}
		
		if (!PropertiesNotPasted.IsEmpty())
		{
			UE_LOGF(
				LogPropertyNode,
				Warning,
				"One or more of the properties in category \"%ls\" was not pasted:\n%ls",
				*DisplayName.ToString(),
				*FString::Join(PropertiesNotPasted, TEXT("\n")));
		}

		TSharedPtr<IDetailsViewPrivate> DetailsView = OwnerTreeNode.Pin()->GetDetailsViewSharedPtr();
		DetailsView->ForceRefresh();
	}
}

bool SDetailCategoryTableRow::CanPasteCategory()
{
	if (!OwnerTreeNode.IsValid())
	{
		return false;
	}

	const TArray<TSharedPtr<IPropertyHandle>> PropertyHandles = GetPropertyHandles(true);
	const TArray<TSharedPtr<FPropertyNode>> PropertyNodes = GetPropertyNodesFromHandles(PropertyHandles);

	// @note: We allow pasting to properties that are disabled due to an EditCondition, but not those that are never editable (ie. VisibleAnywhere).
	const bool bHasEditables = Algo::AnyOf(PropertyNodes, [](const TSharedPtr<FPropertyNode>& InPropertyNode)
	{
		constexpr bool bIncludeEditConditionForConstCheck = false;
		return !InPropertyNode->IsEditConst(bIncludeEditConditionForConstCheck);
	});

	// No editable properties to write to
	if (!bHasEditables)
	{
		return false;
	}

	FString ClipboardContent;
	FPropertyEditorClipboard::ClipboardPaste(ClipboardContent);

	const bool bChildrenHaveChanged = PreviousClipboardData.PreviousPropertyHandleNum != PropertyHandles.Num();

	// If child property count hasn't changed, and content same as last, return previously resolved applicability
	if (!bChildrenHaveChanged
		&& PreviousClipboardData.Content.Get({}).Equals(ClipboardContent))
	{
		return PreviousClipboardData.bIsApplicable;
	}

	// New clipboard contents, non-applicable by default
	PreviousClipboardData.Reset();

	// Can't be empty, must be json
	if (!UE::PropertyEditor::Internal::IsJsonString(ClipboardContent))
	{
		return false;
	}

	PreviousClipboardData.Reserve(PropertyHandles.Num());

	if (!UE::PropertyEditor::Internal::TryParseClipboard(ClipboardContent, PreviousClipboardData.PropertyValues))
	{
		return false;
	}

	PreviousClipboardData.PropertyValues.GenerateKeyArray(PreviousClipboardData.PropertyNames);

	TArray<FString> PropertyNames;
	Algo::Transform(PropertyHandles, PropertyNames, [](const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	{
		return UE::PropertyEditor::GetPropertyPath(InPropertyHandle);
	});

	PreviousClipboardData.PropertyNames.Sort(FNameLexicalLess());
	PropertyNames.Sort();

	// @note: properties must all match to be applicable
	PreviousClipboardData.Content = ClipboardContent;

	PreviousClipboardData.PreviousPropertyHandleNum = PropertyHandles.Num();
	return PreviousClipboardData.bIsApplicable = Algo::Compare(PreviousClipboardData.PropertyNames, PropertyNames);
}

void SDetailCategoryTableRow::OnResetToDefaultCategory()
{
	if (!OwnerTreeNode.IsValid())
	{
		return;
	}

	if (TArray<TSharedPtr<IPropertyHandle>> CategoryProperties = GetPropertyHandles(true);
		!CategoryProperties.IsEmpty())
	{
		for (TSharedPtr<IPropertyHandle> PropertyHandle : CategoryProperties)
		{
			PropertyHandle->ResetToDefault();
		}
	}
}

bool SDetailCategoryTableRow::CanResetToDefaultCategory() const
{
	if (!OwnerTreeNode.IsValid())
	{
		return false;
	}

	TArray<TSharedPtr<IPropertyHandle>> PropertyHandles = GetPropertyHandles(true);
	return !PropertyHandles.IsEmpty();
}

FReply SDetailCategoryTableRow::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetModifierKeys().IsShiftDown())
	{
		// Allows this to be used for the paste properties shortcut
		return FReply::Unhandled();
	}
	
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		ToggleExpansion();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SDetailCategoryTableRow::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return OnMouseButtonDown(InMyGeometry, InMouseEvent);
}
