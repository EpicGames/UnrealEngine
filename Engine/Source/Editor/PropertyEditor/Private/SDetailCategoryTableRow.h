// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailsViewStyle.h"
#include "DetailTreeNode.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "PropertyEditorClipboardPrivate.h"
#include "SDetailTableRowBase.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

class SDetailCategoryTableRow : public SDetailTableRowBase
{
public:
	SLATE_BEGIN_ARGS(SDetailCategoryTableRow)
		: _InnerCategory(false)
		, _WholeRowHeaderContent(false)
		, _ShowBorder(true)
		, _ShowHeader(true)
		, _IsEmpty(false)
		, _Objects(nullptr)
	{}
		SLATE_ARGUMENT(FText, DisplayName)
		SLATE_ARGUMENT(bool, InnerCategory)
		SLATE_ARGUMENT(TSharedPtr<SWidget>, HeaderContent)
		SLATE_ARGUMENT(bool, WholeRowHeaderContent)
		SLATE_ARGUMENT(bool, ShowBorder)
		SLATE_ARGUMENT(bool, ShowHeader)
		
		/** If true, this Category should have no UProperty data associated with it, and will be shown as an 
		* empty stub with no expansion arrow */
		SLATE_ARGUMENT(bool, IsEmpty)
		SLATE_ARGUMENT(TSharedPtr<FOnPasteFromText>, PasteFromText)
		SLATE_ARGUMENT(TSharedPtr<FOnCopyToText>, CopyToText)
		SLATE_ARGUMENT(TArray<TWeakObjectPtr<UObject>>*, Objects)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FDetailTreeNode> InOwnerTreeNode, const TSharedRef<STableViewBase>& InOwnerTableView);

	//~ Begin SWidget
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	//~ End SWidget

protected:
	//~ Begin SDetailTableRowBase
	virtual void PopulateContextMenu(UToolMenu* ToolMenu) override;
	virtual bool CanFavoriteRow(bool bInAddToFavorite) override;
	virtual void OnFavoriteRow(bool bInAddToFavorite) override;
	//~ End SDetailTableRowBase

private:
	/** Process immediate children property nodes of this category, return false to break */
	void ForEachImmediateChildPropertyNode(TFunctionRef<bool(TSharedPtr<FPropertyNode>&)> InFunctor);

	EVisibility IsSeparatorVisible() const;

	/**
	* Gets the @code const FSlateBrush* @endcode which holds the background image for the Category Row, minus the ScrollBar Well
	*/
	const FSlateBrush* GetBackgroundImage() const;

	/**
	* Gets the @code const FSlateBrush* @endcode which holds the background image for the ScrollBar Well
	*/
	const FSlateBrush* GetBackgroundImageForScrollBarWell() const;
	FSlateColor GetInnerBackgroundColor() const;
	FSlateColor GetOuterBackgroundColor() const;

	void OnCopyCategory();
	bool CanCopyCategory() const;
	void OnPasteCategory();
	bool CanPasteCategory();
	void OnResetToDefaultCategory();
	bool CanResetToDefaultCategory() const;

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

private:
	/** Cached category name. */
	FText DisplayName;

	/**
	* The style of the details view. This holds information and methods to get things like the row border image,
	* table padding, and so forth.
	*/
	const FDetailsViewStyle* DetailsViewStyle = nullptr;

	/** Previously parsed clipboard data. */
	UE::PropertyEditor::Internal::FClipboardData PreviousClipboardData;

	bool bIsInnerCategory = false;
	bool bShowBorder = false;
	bool bShowHeader = true;

	/**
	* If true, this Category should have no UProperty data associated with it, and will be shown as an empty stub
	* with no expansion arrow
	*/
	bool bIsEmpty = false;

	FUIAction CopyAction;
	FUIAction PasteAction;
	FUIAction ResetToDefault;

	/** Delegate handling pasting an optionally tagged text snippet */
	TSharedPtr<FOnPasteFromText> OnPasteFromTextDelegate;

	/** Delegate handling pasting an optionally tagged text snippet */
	TSharedPtr<FOnCopyToText> OnCopyToTextDelegate;

	/** Animation curve for displaying pulse */
	FCurveSequence PulseAnimation;
};