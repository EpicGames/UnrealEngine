// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDetailExpanderArrow.h"
#include "SDetailTableRowBase.h"
#include "SConstrainedBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

void SDetailExpanderArrow::Construct(const FArguments& InArgs, TSharedRef<SDetailTableRowBase> DetailsRow)
{
	Row = DetailsRow;

	ChildSlot
	[
		SNew(SConstrainedBox)
		.MinWidth(12.0f)
		.Visibility(this, &SDetailExpanderArrow::GetExpanderVisibility)
		[
			SAssignNew(ExpanderArrow, SButton)
			.ButtonStyle(FCoreStyle::Get(), "NoBorder")
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.ClickMethod(EButtonClickMethod::MouseDown)
			.OnClicked(this, &SDetailExpanderArrow::OnExpanderClicked)
			.ContentPadding(0.0f)
			.IsFocusable(false)
			[
				SNew(SImage)
				.Image(this, &SDetailExpanderArrow::GetExpanderImage)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			]
		]
	];
}

EVisibility SDetailExpanderArrow::GetExpanderVisibility() const
{
	TSharedPtr<SDetailTableRowBase> RowPtr = Row.Pin();
	if (!RowPtr.IsValid())
	{
		return EVisibility::Collapsed;
	}

	return RowPtr->DoesItemHaveChildren() ? EVisibility::Visible : EVisibility::Hidden;
}

const FSlateBrush* SDetailExpanderArrow::GetExpanderImage() const
{
	TSharedPtr<SDetailTableRowBase> RowPtr = Row.Pin();
	if (!RowPtr.IsValid())
	{
		return FAppStyle::Get().GetBrush("NoBrush");
	}

	const bool bIsItemExpanded = RowPtr->IsItemExpanded();

	FName ResourceName;
	if (bIsItemExpanded)
	{
		if (ExpanderArrow->IsHovered())
		{
			static const FName ExpandedHoveredName = "TreeArrow_Expanded_Hovered";
			ResourceName = ExpandedHoveredName;
		}
		else
		{
			static const FName ExpandedName = "TreeArrow_Expanded";
			ResourceName = ExpandedName;
		}
	}
	else
	{
		if (ExpanderArrow->IsHovered())
		{
			static const FName CollapsedHoveredName = "TreeArrow_Collapsed_Hovered";
			ResourceName = CollapsedHoveredName;
		}
		else
		{
			static const FName CollapsedName = "TreeArrow_Collapsed";
			ResourceName = CollapsedName;
		}
	}

	return FAppStyle::Get().GetBrush(ResourceName);
}

FReply SDetailExpanderArrow::OnExpanderClicked()
{
	TSharedPtr<SDetailTableRowBase> RowPtr = Row.Pin();
	if (!RowPtr.IsValid())
	{
		return FReply::Unhandled();
	}

	// Recurse the expansion if "shift" is being pressed
	const FModifierKeysState ModKeyState = FSlateApplication::Get().GetModifierKeys();
	if (ModKeyState.IsShiftDown())
	{
		RowPtr->Private_OnExpanderArrowShiftClicked();
	}
	else
	{
		RowPtr->ToggleExpansion();
	}

	return FReply::Handled();
}