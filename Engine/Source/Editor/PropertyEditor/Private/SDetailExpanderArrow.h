// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SButton;
class SDetailTableRowBase;

class SDetailExpanderArrow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDetailExpanderArrow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<SDetailTableRowBase> DetailsRow);

private:

	EVisibility GetExpanderVisibility() const;
	const FSlateBrush* GetExpanderImage() const;
	FReply OnExpanderClicked();
	
private:
	TWeakPtr<SDetailTableRowBase> Row;
	TSharedPtr<SButton> ExpanderArrow;
};