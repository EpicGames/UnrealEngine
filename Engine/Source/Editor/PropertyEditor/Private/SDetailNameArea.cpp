// Copyright Epic Games, Inc. All Rights Reserved.


#include "SDetailNameArea.h"

#include "AssetSelection.h"
#include "Components/ActorComponent.h"
#include "EditorClassUtils.h"
#include "Styling/AppStyle.h"
#include "EditorWidgetsModule.h"
#include "Engine/World.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "DetailNameAreaObjectFilterRegistry.h"

#define LOCTEXT_NAMESPACE "SDetailsView"

void SDetailNameArea::Construct( const FArguments& InArgs, const TArray< TWeakObjectPtr<UObject> >* SelectedObjects )
{
	OnLockButtonClicked = InArgs._OnLockButtonClicked;
	IsLocked = InArgs._IsLocked;
	SelectionTip = InArgs._SelectionTip;
	bShowLockButton = InArgs._ShowLockButton;
	bShowObjectLabel = InArgs._ShowObjectLabel;
	CustomContent = SNullWidget::NullWidget;
}

void SDetailNameArea::Refresh( const TArray< TWeakObjectPtr<UObject> >& SelectedObjects, int32 NameAreaSettings )
{
	TArray< TWeakObjectPtr<UObject> > FinalSelectedObjects;
	FinalSelectedObjects.Reserve(SelectedObjects.Num());

	// Apply the name area filter in priority order
	if ((NameAreaSettings & FDetailsViewArgs::ActorsUseNameArea) && FinalSelectedObjects.Num() == 0)
	{
		for (const TWeakObjectPtr<UObject>& Object : SelectedObjects)
		{
			if (AActor* Actor = Cast<AActor>(Object.Get()))
			{
				FinalSelectedObjects.Add(Actor);
			}
		}
	}
	if ((NameAreaSettings & FDetailsViewArgs::ComponentsAndActorsUseNameArea) && FinalSelectedObjects.Num() == 0)
	{
		for (const TWeakObjectPtr<UObject>& Object : SelectedObjects)
		{
			if (UActorComponent* ActorComp = Cast<UActorComponent>(Object.Get()))
			{
				if (AActor* Actor = ActorComp->GetOwner())
				{
					FinalSelectedObjects.AddUnique(Actor);
				}
			}
		}
	}
	if ((NameAreaSettings & FDetailsViewArgs::ObjectsUseNameArea) && FinalSelectedObjects.Num() == 0)
	{
		FinalSelectedObjects = SelectedObjects;
	}

	const FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");
	EditorWidgetsModule.GetDetailNameAreaObjectFilterRegistry()->ApplyAllFiltersToSelectedObjects(FinalSelectedObjects);

	ChildSlot
	[
		BuildObjectNameArea( FinalSelectedObjects )
	];
}

const FSlateBrush* SDetailNameArea::OnGetLockButtonImageResource() const
{
	if( IsLocked.Get() )
	{
		return FAppStyle::GetBrush(TEXT("PropertyWindow.Locked"));
	}
	else
	{
		return FAppStyle::GetBrush(TEXT("PropertyWindow.Unlocked"));
	}
}

TSharedRef< SWidget > SDetailNameArea::BuildObjectNameArea( const TArray< TWeakObjectPtr<UObject> >& SelectedObjects )
{
	// Get the common base class of the selected objects
	UClass* BaseClass = NULL;
	for( int32 ObjectIndex = 0; ObjectIndex < SelectedObjects.Num(); ++ObjectIndex )
	{
		TWeakObjectPtr<UObject> ObjectWeakPtr = SelectedObjects[ObjectIndex];
		if( ObjectWeakPtr.IsValid() )
		{
			UClass* ObjClass = ObjectWeakPtr->GetClass();

			if (!BaseClass)
			{
				BaseClass = ObjClass;
			}

			while (!ObjClass->IsChildOf(BaseClass))
			{
				BaseClass = BaseClass->GetSuperClass();
			}
		}
	}

	TSharedRef< SHorizontalBox > ObjectNameArea = SNew( SHorizontalBox );

	if (BaseClass)
	{
		// Get selection icon based on actor(s) classes and add before the selection label
		const FSlateBrush* ActorIcon = FSlateIconFinder::FindIconBrushForClass(BaseClass);

		ObjectNameArea->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(0.0f)
		[
			SNew(SImage)
			.Image(ActorIcon)
			.ToolTip(FEditorClassUtils::GetTooltip(BaseClass))
		];
	}


	// Add the selected object(s) type name, along with buttons for either opening C++ code or editing blueprints
	const int32 NumSelectedSurfaces = AssetSelectionUtils::GetNumSelectedSurfaces( GWorld );
	if( SelectedObjects.Num() > 0 )
	{
		if ( bShowObjectLabel )
		{
			FEditorWidgetsModule& EdWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>(TEXT("EditorWidgets"));
			TSharedRef<IObjectNameEditableTextBox> ObjectNameBox = EdWidgetsModule.CreateObjectNameEditableTextBox(SelectedObjects);

			ObjectNameArea->AddSlot()
				.FillWidth(1.0f)
				.Padding(8.0f, 0.0f, 3.0f, 0.0f)
				[
					SNew(SBox)
					.WidthOverride(200.0f)
					.VAlign(VAlign_Center)
					[
						ObjectNameBox
					]
				];
		}

		const TWeakObjectPtr< UObject > ObjectWeakPtr = SelectedObjects.Num() == 1 ? SelectedObjects[0] : NULL;
		BuildObjectNameAreaSelectionLabel( ObjectNameArea, ObjectWeakPtr, SelectedObjects.Num() );

		ObjectNameArea->AddSlot()
		.Expose(CustomContentSlot)
		.AutoWidth()
		[
			CustomContent.ToSharedRef()
		];

		if( bShowLockButton )
		{
			ObjectNameArea->AddSlot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(4.0f,0.0f,4.0f,0.0f)
				.AutoWidth()
				[
					SNew( SButton )
					.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
					.OnClicked(	OnLockButtonClicked )
					.ContentPadding(FMargin(4,2))
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.ToolTipText( LOCTEXT("LockSelectionButton_ToolTip", "Locks the current selection into the Details panel") )
					[
						SNew( SImage )
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image( this, &SDetailNameArea::OnGetLockButtonImageResource )
					]
				];
		}
	}
	else
	{
		if ( SelectionTip.Get() && NumSelectedSurfaces == 0 )
		{
			ObjectNameArea->AddSlot()
			.FillWidth( 1.0f )
			.HAlign( HAlign_Center )
			.Padding( 2.0f, 24.0f, 2.0f, 2.0f )
			[
				SNew( STextBlock )
				.Text( LOCTEXT("NoObjectsSelected", "Select an object to view details.") )
				.TextStyle( FAppStyle::Get(), "HintText")
			];
		}
		else
		{
			// Fill the empty space
			ObjectNameArea->AddSlot();
		}
	}
	
	return ObjectNameArea;
}

void SDetailNameArea::BuildObjectNameAreaSelectionLabel( TSharedRef< SHorizontalBox > SelectionLabelBox, const TWeakObjectPtr<UObject> ObjectWeakPtr, const int32 NumSelectedObjects ) 
{
	check( NumSelectedObjects > 1 || ObjectWeakPtr.IsValid() );

	if (NumSelectedObjects > 1)
	{
		const FText SelectionText = FText::Format( LOCTEXT("MultipleObjectsSelectedFmt", "{0} objects"), FText::AsNumber(NumSelectedObjects) );
		SelectionLabelBox->AddSlot()
		.VAlign(VAlign_Center)
		.HAlign( HAlign_Left )
		.FillWidth( 1.0f )
		[
			SNew(STextBlock)
			.Text( SelectionText )
		];
	}
}


void SDetailNameArea::SetCustomContent(TSharedRef<SWidget>& InCustomContent)
{
	CustomContent = InCustomContent;

	if (CustomContentSlot != nullptr)
	{
		SHorizontalBox::FSlot& T = *CustomContentSlot;
		T[InCustomContent];
	}

}

#undef LOCTEXT_NAMESPACE

