// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "IDetailsView.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/SBoxPanel.h"

class AActor;
class UBlueprint;

/** 
 * Displays the name area which is not recreated when the detail view is refreshed
 */
class SDetailNameArea : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SDetailNameArea ){}
		SLATE_EVENT( FOnClicked, OnLockButtonClicked )
		SLATE_ARGUMENT( bool, ShowLockButton )
		SLATE_ARGUMENT( bool, ShowObjectLabel )
		SLATE_ATTRIBUTE( bool, IsLocked )
		SLATE_ATTRIBUTE( bool, SelectionTip )
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const TArray< TWeakObjectPtr<UObject> >* SelectedObjects );

	/**
	 * Refreshes the name area when selection changes
	 *
	 * @param SelectedObjects	the new list of selected objects
	 */
	void Refresh( const TArray< TWeakObjectPtr<UObject> >& SelectedObjects, int32 NameAreaSettings );

	/**
	 * Inserts Custom Content (typically tool buttons) before the lock
	 */
	virtual void SetCustomContent(TSharedRef<SWidget>& InCustomContent);

private:
	/** @return the Slate brush to use for the lock image */
	const FSlateBrush* OnGetLockButtonImageResource() const;

	TSharedRef< SWidget > BuildObjectNameArea( const TArray< TWeakObjectPtr<UObject> >& SelectedObjects );

	void BuildObjectNameAreaSelectionLabel( TSharedRef< SHorizontalBox > SelectionLabelBox, const TWeakObjectPtr<UObject> ObjectWeakPtr, const int32 NumSelectedObjects );

	void OnEditBlueprintClicked( TWeakObjectPtr<UBlueprint> InBlueprint, TWeakObjectPtr<UObject> InAsset );
private:
	FOnClicked OnLockButtonClicked;
	TAttribute<bool> IsLocked;
	TAttribute<bool> SelectionTip;

	TSharedPtr<SWidget> CustomContent;

	bool bShowLockButton;
	bool bShowObjectLabel;

	/** Area where the customs content resides */
	SHorizontalBox::FSlot* CustomContentSlot;
};
