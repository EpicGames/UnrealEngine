// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDetailRowIndent.h"
#include "SConstrainedBox.h"
#include "SDetailTableRowBase.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "Widgets/Layout/SBox.h"

void SDetailRowIndent::Construct(const FArguments& InArgs, TSharedRef<SDetailTableRowBase> DetailsRow)
{
	Row = DetailsRow;

	ChildSlot
	[
		SNew(SBox)
		.WidthOverride(this, &SDetailRowIndent::GetIndentWidth)
	];
}

int32 SDetailRowIndent::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	TSharedPtr<SDetailTableRowBase> RowPtr = Row.Pin();
	if (!RowPtr.IsValid())
	{
		return LayerId;
	}

	const FSlateBrush* BackgroundBrush = FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle");
	const FSlateBrush* DropShadowBrush = FAppStyle::Get().GetBrush("DetailsView.ArrayDropShadow");

	int32 IndentLevel = RowPtr->GetIndentLevelForBackgroundColor();
	for (int32 i = 0; i < IndentLevel; ++i)
	{
		FSlateColor BackgroundColor = GetRowBackgroundColor(i);

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(FVector2f(16.f, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform(FVector2f(16.f * i, 0.f))),
			BackgroundBrush,
			ESlateDrawEffect::None,
			BackgroundColor.GetColor(InWidgetStyle)
		);

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + 1,
			AllottedGeometry.ToPaintGeometry(FVector2f(16.f, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform(FVector2f(16.f * i, 0.f))),
			DropShadowBrush
		);
	}
		
	return LayerId + 1;
}

FOptionalSize SDetailRowIndent::GetIndentWidth() const
{
	int32 IndentLevel = 0;

	TSharedPtr<SDetailTableRowBase> RowPtr = Row.Pin();
	if (RowPtr.IsValid())
	{
		IndentLevel = RowPtr->GetIndentLevelForBackgroundColor();
	}

	return IndentLevel * 16.0f;
}

FSlateColor SDetailRowIndent::GetRowBackgroundColor(int32 IndentLevel) const
{
	TSharedPtr<SDetailTableRowBase> RowPtr = Row.Pin();
	return PropertyEditorConstants::GetRowBackgroundColor(IndentLevel, RowPtr.IsValid() && RowPtr->IsHovered());
}
