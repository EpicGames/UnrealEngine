// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateColor.h"
#include "Types/SlateStructs.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SDetailTableRowBase;
class SHorizontalBox;

class SDetailRowIndent : public SCompoundWidget
{ 
public:
	SLATE_BEGIN_ARGS(SDetailRowIndent) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<SDetailTableRowBase> DetailsRow);

private:
	int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const;
	FOptionalSize GetIndentWidth() const;
	FSlateColor GetRowBackgroundColor(int32 IndentLevel) const;

private:
	TWeakPtr<SDetailTableRowBase> Row;
};
