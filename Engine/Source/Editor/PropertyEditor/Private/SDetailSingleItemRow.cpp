// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDetailSingleItemRow.h"

#include "Algo/AnyOf.h"
#include "Algo/Compare.h"
#include "DetailGroup.h"
#include "DetailPropertyRow.h"
#include "DetailsNameWidgetOverrideCustomization.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "IDetailDragDropHandler.h"
#include "IDetailPropertyExtensionHandler.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "ObjectPropertyNode.h"
#include "PropertyEditorClipboard.h"
#include "PropertyEditorClipboardPrivate.h"
#include "PropertyEditorCopyPastePrivate.h"
#include "PropertyEditorModule.h"
#include "PropertyHandleImpl.h"
#include "PropertyPermissionList.h"
#include "SConstrainedBox.h"
#include "SDetailExpanderArrow.h"
#include "SDetailRowIndent.h"
#include "Styling/StyleColors.h"
#include "UObject/Field.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "DetailsViewStyle.h"
#include "SDetailsView.h"
#include "ToolMenus.h"

namespace DetailWidgetConstants
{
	const FMargin LeftRowPadding( 20.0f, 0.0f, 10.0f, 0.0f );
	const FMargin RightRowPadding( 12.0f, 0.0f, 2.0f, 0.0f );
}

namespace SDetailSingleItemRow_Helper
{
	// Get the node item number in case it is expand we have to recursively count all expanded children
	void RecursivelyGetItemShow(TSharedRef<FDetailTreeNode> ParentItem, int32& ItemShowNum)
	{
		if (ParentItem->GetVisibility() == ENodeVisibility::Visible)
		{
			ItemShowNum++;
		}

		if (ParentItem->ShouldBeExpanded())
		{
			TArray< TSharedRef<FDetailTreeNode> > Childrens;
			ParentItem->GetChildren(Childrens);
			for (TSharedRef<FDetailTreeNode> ItemChild : Childrens)
			{
				RecursivelyGetItemShow(ItemChild, ItemShowNum);
			}
		}
	}
}

void SDetailSingleItemRow::OnArrayOrCustomDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDecoratedDragDropOp> DecoratedOp = DragDropEvent.GetOperationAs<FDecoratedDragDropOp>();
	if (DecoratedOp.IsValid())
	{
		DecoratedOp->ResetToDefaultToolTip();
	}
}

/** Compute the new index to which to move the item when it's dropped onto the given index/drop zone. */
static int32 ComputeNewIndex(int32 OriginalIndex, int32 DropOntoIndex, EItemDropZone DropZone)
{
	check(DropZone != EItemDropZone::OntoItem);

	int32 NewIndex = DropOntoIndex;
	if (DropZone == EItemDropZone::BelowItem)
	{
		// If the drop zone is below, then we actually move it to the next item's index
		NewIndex++;
	}
	if (OriginalIndex < NewIndex)
	{
		// If the item is moved down the list, then all the other elements below it are shifted up one
		NewIndex--;
	}

	return ensure(NewIndex >= 0) ? NewIndex : 0;
}

bool SDetailSingleItemRow::CheckValidDrop(const TSharedPtr<SDetailSingleItemRow> RowPtr, EItemDropZone DropZone) const
{
	// Can't drop onto another array item; need to drop above or below
	if (DropZone == EItemDropZone::OntoItem)
	{
		return false;
	}

	TSharedPtr<FPropertyNode> SwappingPropertyNode = RowPtr->SwappablePropertyNode;
	if (SwappingPropertyNode.IsValid() && SwappablePropertyNode.IsValid() && SwappingPropertyNode != SwappablePropertyNode)
	{
		const int32 OriginalIndex = SwappingPropertyNode->GetArrayIndex();
		const int32 NewIndex = ComputeNewIndex(OriginalIndex, SwappablePropertyNode->GetArrayIndex(), DropZone);

		if (OriginalIndex != NewIndex)
		{
			TSharedPtr<IDetailsViewPrivate> DetailsView = OwnerTreeNode.Pin()->GetDetailsViewSharedPtr();
			TSharedPtr<IPropertyHandle> SwappingHandle = PropertyEditorHelpers::GetPropertyHandle(SwappingPropertyNode.ToSharedRef(), DetailsView->GetNotifyHook(), DetailsView->GetPropertyUtilities());
			TSharedPtr<IPropertyHandleArray> ParentHandle = SwappingHandle->GetParentHandle()->AsArray();

			if (ParentHandle.IsValid() && SwappablePropertyNode->GetParentNode() == SwappingPropertyNode->GetParentNode())
			{
				return true;
			}
		}
	}
	return false;
}

FReply SDetailSingleItemRow::OnArrayAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FDetailTreeNode> TargetItem)
{
	TSharedPtr<FArrayRowDragDropOp> ArrayDropOp = DragDropEvent.GetOperationAs< FArrayRowDragDropOp >();
	if (!ArrayDropOp.IsValid())
	{
		return FReply::Unhandled();
	}

	TSharedPtr<SDetailSingleItemRow> RowPtr = ArrayDropOp->Row.Pin();
	if (!RowPtr.IsValid())
	{
		return FReply::Unhandled();
	}

	if (!CheckValidDrop(RowPtr, DropZone))
	{
		return FReply::Unhandled();
	}

	TSharedPtr<IDetailsViewPrivate> DetailsView = OwnerTreeNode.Pin()->GetDetailsViewSharedPtr();

	TSharedPtr<FPropertyNode> SwappingPropertyNode = RowPtr->SwappablePropertyNode;
	TSharedPtr<IPropertyHandle> SwappingHandle = PropertyEditorHelpers::GetPropertyHandle(SwappingPropertyNode.ToSharedRef(), DetailsView->GetNotifyHook(), DetailsView->GetPropertyUtilities());
	TSharedPtr<IPropertyHandleArray> ParentHandle = SwappingHandle->GetParentHandle()->AsArray();
	const int32 OriginalIndex = SwappingPropertyNode->GetArrayIndex();
	const int32 NewIndex = ComputeNewIndex(OriginalIndex, SwappablePropertyNode->GetArrayIndex(), DropZone);

	// Need to swap the moving and target expansion states before saving
	bool bOriginalSwappableExpansion = SwappablePropertyNode->HasNodeFlags(EPropertyNodeFlags::Expanded) != 0;
	bool bOriginalSwappingExpansion = SwappingPropertyNode->HasNodeFlags(EPropertyNodeFlags::Expanded) != 0;
	SwappablePropertyNode->SetNodeFlags(EPropertyNodeFlags::Expanded, bOriginalSwappingExpansion);
	SwappingPropertyNode->SetNodeFlags(EPropertyNodeFlags::Expanded, bOriginalSwappableExpansion);

	DetailsView->SaveExpandedItems(SwappablePropertyNode->GetParentNodeSharedPtr().ToSharedRef());
	FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MoveRow", "Move Row"));

	SwappingHandle->GetParentHandle()->NotifyPreChange();

	ParentHandle->MoveElementTo(OriginalIndex, NewIndex);

	FPropertyChangedEvent MoveEvent(SwappingHandle->GetParentHandle()->GetProperty(), EPropertyChangeType::ArrayMove);
	SwappingHandle->GetParentHandle()->NotifyPostChange(EPropertyChangeType::ArrayMove);
	if (DetailsView->GetPropertyUtilities().IsValid())
	{
		DetailsView->GetPropertyUtilities()->NotifyFinishedChangingProperties(MoveEvent);
	}

	return FReply::Handled();
}

TOptional<EItemDropZone> SDetailSingleItemRow::OnArrayCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FDetailTreeNode> TargetItem)
{
	TSharedPtr<FArrayRowDragDropOp> ArrayDropOp = DragDropEvent.GetOperationAs< FArrayRowDragDropOp >();
	if (!ArrayDropOp.IsValid())
	{
		return TOptional<EItemDropZone>();
	}

	TSharedPtr<SDetailSingleItemRow> RowPtr = ArrayDropOp->Row.Pin();
	if (!RowPtr.IsValid())
	{
		return TOptional<EItemDropZone>();
	}

	// Can't drop onto another array item, so recompute our own drop zone to ensure it's above or below
	const FGeometry& Geometry = GetTickSpaceGeometry();
	const float LocalPointerY = Geometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition()).Y;
	const EItemDropZone OverrideDropZone = LocalPointerY < Geometry.GetLocalSize().Y * 0.5f ? EItemDropZone::AboveItem : EItemDropZone::BelowItem;

	const bool IsValidDrop = CheckValidDrop(RowPtr, OverrideDropZone);

	ArrayDropOp->SetValidTarget(IsValidDrop);

	if (!IsValidDrop)
	{
		return TOptional<EItemDropZone>();
	}

	return OverrideDropZone;
}

FReply SDetailSingleItemRow::OnArrayHeaderAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr< FDetailTreeNode > Type)
{
	OnArrayOrCustomDragLeave(DragDropEvent);
	return FReply::Handled();
}

FReply SDetailSingleItemRow::OnCustomAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FDetailTreeNode> TargetItem)
{
	// This should only be registered as a delegate if there's a custom handler
	if (ensure(WidgetRow.CustomDragDropHandler))
	{
		if (WidgetRow.CustomDragDropHandler->AcceptDrop(DragDropEvent, DropZone))
		{
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}

TOptional<EItemDropZone> SDetailSingleItemRow::OnCustomCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FDetailTreeNode> Type)
{
	// This should only be registered as a delegate if there's a custom handler
	if (ensure(WidgetRow.CustomDragDropHandler))
	{
		return WidgetRow.CustomDragDropHandler->CanAcceptDrop(DragDropEvent, DropZone);
	}
	return TOptional<EItemDropZone>();
}

TSharedPtr<FPropertyNode> SDetailSingleItemRow::GetPropertyNode() const
{
	TSharedPtr<FPropertyNode> PropertyNode = Customization->GetPropertyNode();
	if (!PropertyNode.IsValid() && Customization->DetailGroup.IsValid())
	{
		PropertyNode = Customization->DetailGroup->GetHeaderPropertyNode();
	}

	// See if a custom builder has an associated node
	if (!PropertyNode.IsValid() && Customization->HasCustomBuilder())
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = Customization->CustomBuilderRow->GetPropertyHandle();
		if (PropertyHandle.IsValid())
		{
			PropertyNode = StaticCastSharedPtr<FPropertyHandleBase>(PropertyHandle)->GetPropertyNode();
		}
	}

	return PropertyNode;
}

TSharedPtr<IPropertyHandle> SDetailSingleItemRow::GetPropertyHandle() const
{
	TSharedPtr<IPropertyHandle> Handle;
	if (const TSharedPtr<FPropertyNode> PropertyNode = GetPropertyNode())
	{
		if (const TSharedPtr<FDetailTreeNode> OwnerTreeNodePtr = OwnerTreeNode.Pin())
		{
			if (TSharedPtr<IDetailsViewPrivate> DetailsView = OwnerTreeNodePtr->GetDetailsViewSharedPtr())
			{
				Handle = PropertyEditorHelpers::GetPropertyHandle(PropertyNode.ToSharedRef(), DetailsView->GetNotifyHook(), DetailsView->GetPropertyUtilities());
			}
		}
	}
	else if (WidgetRow.PropertyHandles.Num() > 0)
	{
		// @todo: Handle more than 1 property handle?
		Handle = WidgetRow.PropertyHandles[0];
	}

	return Handle;
}

void SDetailSingleItemRow::UpdateResetToDefault()
{
	bCachedResetToDefaultVisible = false;

	TSharedPtr<IPropertyHandle> PropertyHandle = GetPropertyHandle();
	if (WidgetRow.CustomResetToDefault.IsSet())
	{
		bCachedResetToDefaultVisible = WidgetRow.CustomResetToDefault.GetValue().IsResetToDefaultVisible(PropertyHandle);
		return;
	}

	if (PropertyHandle.IsValid())
	{
		if (PropertyHandle->HasMetaData("NoResetToDefault") || PropertyHandle->GetInstanceMetaData("NoResetToDefault"))
		{
			bCachedResetToDefaultVisible = false;
			return;
		}

		bCachedResetToDefaultVisible = PropertyHandle->CanResetToDefault();
	}
}

void SDetailSingleItemRow::Construct( const FArguments& InArgs, FDetailLayoutCustomization* InCustomization, bool bHasMultipleColumns, TSharedRef<FDetailTreeNode> InOwnerTreeNode, const TSharedRef<STableViewBase>& InOwnerTableView )
{
	OwnerTreeNode = InOwnerTreeNode;
	bAllowFavoriteSystem = InArgs._AllowFavoriteSystem;
	Customization = InCustomization;

	TSharedRef<SWidget> Widget = SNullWidget::NullWidget;

	FOnTableRowDragLeave DragLeaveDelegate;
	FOnAcceptDrop AcceptDropDelegate;
	FOnCanAcceptDrop CanAcceptDropDelegate;

	TSharedPtr<IDetailsViewPrivate> DetailsView = InOwnerTreeNode->GetDetailsViewSharedPtr();
	const FDetailColumnSizeData& ColumnSizeData = DetailsView ? DetailsView->GetColumnSizeData() : FDetailColumnSizeData();

	PulseAnimation.AddCurve(0.0f, UE::PropertyEditor::Private::PulseAnimationLength, ECurveEaseFunction::CubicInOut);

	// Play on construction if animation was started from a behavior the re-constructs this widget
	const TSharedPtr<FPropertyNode> AssociatedPropertyNode = GetPropertyNode();
	if (DetailsView && DetailsView->IsNodeAnimating(AssociatedPropertyNode))
	{
		// Resume from the stored animation time if it's still animating
		float AnimationTime = DetailsView->GetNodeAnimationTime(AssociatedPropertyNode);
		PulseAnimation.Play(SharedThis(this), false, AnimationTime);
	}

	TSharedPtr<FDetailCategoryImpl> Category = InOwnerTreeNode->GetParentCategory();

	const bool bIsValidTreeNode = InOwnerTreeNode->GetParentCategory().IsValid() && InOwnerTreeNode->GetParentCategory()->IsParentLayoutValid();
	if (bIsValidTreeNode)
	{
		if (Customization->IsValidCustomization())
		{
			TSharedPtr<FDetailGroup> Group = nullptr;
			WidgetRow = Customization->GetWidgetRow();

			// Populate the extension content in the WidgetRow if there's an extension handler.
			PopulateExtensionWidget();

			// Setup copy / paste actions
			{
				if (WidgetRow.IsCopyPasteBound())
				{
					CopyAction = WidgetRow.CopyMenuAction;
					PasteAction = WidgetRow.PasteMenuAction;
				}
				else
				{
					TSharedPtr<FPropertyNode> PropertyNode = GetPropertyNode();
					static const FName DisableCopyPasteMetaDataName("DisableCopyPaste");
					if (PropertyNode.IsValid() && !PropertyNode->ParentOrSelfHasMetaData(DisableCopyPasteMetaDataName))
					{
						CopyAction.ExecuteAction = FExecuteAction::CreateSP(this, &SDetailSingleItemRow::OnCopyProperty);
						PasteAction.ExecuteAction = FExecuteAction::CreateSP(this, &SDetailSingleItemRow::OnPasteProperty);
						PasteAction.CanExecuteAction = FCanExecuteAction::CreateSP(this, &SDetailSingleItemRow::CanPasteProperty);
					}
					else if (Group = Customization->DetailGroup;
						Group.IsValid())
					{
						CopyAction.ExecuteAction = FExecuteAction::CreateSP(this, &SDetailSingleItemRow::OnCopyGroup);
						CopyAction.CanExecuteAction = FCanExecuteAction::CreateSP(this, &SDetailSingleItemRow::CanCopyGroup);

						PasteAction.ExecuteAction = FExecuteAction::CreateSP(this, &SDetailSingleItemRow::OnPasteGroup);
						PasteAction.CanExecuteAction = FCanExecuteAction::CreateSP(this, &SDetailSingleItemRow::CanPasteGroup);
					}
					else
					{
						CopyAction.ExecuteAction = FExecuteAction::CreateLambda([]() {});
						CopyAction.CanExecuteAction = FCanExecuteAction::CreateLambda([]() { return false; });
						PasteAction.ExecuteAction = FExecuteAction::CreateLambda([]() {});
						PasteAction.CanExecuteAction = FCanExecuteAction::CreateLambda([]() { return false; });
					}
				}

				if (WidgetRow.IsCopyToTextBound())
				{
					OnCopyToTextDelegate = WidgetRow.OnCopyToTextDelegate.Pin();					
				}
				else if (Category.IsValid())
				{
					OnCopyToTextDelegate = Category->OnCopyToText();
				}
				// If still not set, but this is a group, initialize
				else if (Group.IsValid())
				{
					if (TSharedPtr<FOnCopyToText> GroupOnCopyToTextDelegate = Group->OnCopyToText())
					{
						OnCopyToTextDelegate = GroupOnCopyToTextDelegate;
					}
				}

				bool bIncludeEditCondition = true;
				
				if (WidgetRow.IsPasteFromTextBound())
				{
					OnPasteFromTextDelegate = WidgetRow.OnPasteFromTextDelegate.Pin();					
				}
				else if (Category.IsValid())
				{
					OnPasteFromTextDelegate = Category->OnPasteFromText();

					// When pasting an entire category, an edit condition property might come after the property that uses it.
					// Consider the case where the value on the clipboard enables the edit condition but that it may have been disabled prior to the paste.
					// Therefore, the edit condition should not be included when checking if a pasted property IsEditConst.
					bIncludeEditCondition = false;
				}
				// If still not set, but this is a group, initialize
				else if (Group.IsValid())
				{
					if (TSharedPtr<FOnPasteFromText> GroupOnPasteFromTextDelegate = Group->OnPasteFromText())
					{
						OnPasteFromTextDelegate = GroupOnPasteFromTextDelegate;
					}
				}

				// Note: We currently only animate the row on paste but if there is other custom behavior in OnPasteFromText we should
				// also bind to OnCopyToText to match it
				if (OnPasteFromTextDelegate.IsValid())
				{
					OnPasteFromTextDelegate->AddSP(this, &SDetailSingleItemRow::OnPasteFromText, bIncludeEditCondition);
				}
			}

			TSharedPtr<SWidget> NameWidget = WidgetRow.NameWidget.Widget;

			TSharedPtr<SWidget> ValueWidget =
				SNew(SConstrainedBox)
				.MinWidth(WidgetRow.ValueWidget.MinWidth)
				.MaxWidth(WidgetRow.ValueWidget.MaxWidth)
				[
					WidgetRow.ValueWidget.Widget
				];

			TSharedPtr<SWidget> ExtensionWidget = WidgetRow.ExtensionWidget.Widget;

			// copies of attributes for lambda captures
			TAttribute<bool> PropertyEnabledAttribute = InOwnerTreeNode->IsPropertyEditingEnabled();
			TAttribute<bool> RowEditConditionAttribute = WidgetRow.EditConditionValue;
			TAttribute<bool> RowIsEnabledAttribute = WidgetRow.IsEnabledAttr;

			TAttribute<bool> IsEnabledAttribute = TAttribute<bool>::CreateLambda(
				[PropertyEnabledAttribute, RowIsEnabledAttribute, RowEditConditionAttribute]()
				{
					return PropertyEnabledAttribute.Get(true) && RowIsEnabledAttribute.Get(true) && RowEditConditionAttribute.Get(true);
				});

			TAttribute<bool> RowIsValueEnabledAttribute = WidgetRow.IsValueEnabledAttr;
			TAttribute<bool> IsValueEnabledAttribute = TAttribute<bool>::CreateLambda(
				[IsEnabledAttribute, RowIsValueEnabledAttribute]()
				{
					return IsEnabledAttribute.Get() && RowIsValueEnabledAttribute.Get(true);
				});

			NameWidget->SetEnabled(IsEnabledAttribute);
			ValueWidget->SetEnabled(IsValueEnabledAttribute);
			ExtensionWidget->SetEnabled(IsEnabledAttribute);

			TSharedRef<SSplitter> Splitter = SNew(SSplitter)
					.Style(FAppStyle::Get(), "DetailsView.Splitter")
					.PhysicalSplitterHandleSize(1.0f)
					.HitDetectionSplitterHandleSize(5.0f)
					.HighlightedHandleIndex(ColumnSizeData.GetHoveredSplitterIndex())
					.OnHandleHovered(ColumnSizeData.GetOnSplitterHandleHovered());

			Widget = SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle"))
				.BorderBackgroundColor(this, &SDetailSingleItemRow::GetInnerBackgroundColor)
				.Padding(0.0f)
				[
					Splitter
				];

			// create Name column:
			// | Name | Value | Right |
			TSharedRef<SHorizontalBox> NameColumnBox = SNew(SHorizontalBox)
				.Clipping(EWidgetClipping::OnDemand);

			// indentation and expander arrow
			NameColumnBox->AddSlot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Fill)
				.Padding(0.0f)
				.AutoWidth()
				[
					SNew(SDetailRowIndent, SharedThis(this))
				];

			if (WidgetRow.CustomDragDropHandler)
			{
				TSharedPtr<SDetailSingleItemRow> InRow = SharedThis(this);

				// Let the handler subclasses decide if they want the handle widget. 
				if (WidgetRow.CustomDragDropHandler->UseHandleWidget())
				{
					TSharedRef<SWidget> ReorderHandle = PropertyEditorHelpers::MakePropertyReorderHandle(InRow, IsEnabledAttribute);

					NameColumnBox->AddSlot()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						.Padding(-4.0f, 0.0f, -10.0f, 0.0f)
						.AutoWidth()
						[
							ReorderHandle
						];
				}

				DragLeaveDelegate = FOnTableRowDragLeave::CreateSP(this, &SDetailSingleItemRow::OnArrayOrCustomDragLeave);
				AcceptDropDelegate = FOnAcceptDrop::CreateSP(this, &SDetailSingleItemRow::OnCustomAcceptDrop);
				CanAcceptDropDelegate = FOnCanAcceptDrop::CreateSP(this, &SDetailSingleItemRow::OnCustomCanAcceptDrop);
			}
			else if (TSharedPtr<FPropertyNode> PropertyNode = Customization->GetPropertyNode())
			{
				if (PropertyNode->IsReorderable())
				{
					TSharedPtr<SDetailSingleItemRow> InRow = SharedThis(this);
					TSharedRef<SWidget> ArrayHandle = PropertyEditorHelpers::MakePropertyReorderHandle(InRow, IsEnabledAttribute);

					NameColumnBox->AddSlot()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						.Padding(-4.0f, 0.0f, -10.0f, 0.0f)
						.AutoWidth()
						[
							ArrayHandle
						];

					SwappablePropertyNode = PropertyNode;
				}
				
				if (PropertyNode->IsReorderable() || 
					(CastField<FArrayProperty>(PropertyNode->GetProperty()) != nullptr && 
					CastField<FObjectProperty>(CastField<FArrayProperty>(PropertyNode->GetProperty())->Inner) != nullptr)) // Is an object array
				{
					DragLeaveDelegate = FOnTableRowDragLeave::CreateSP(this, &SDetailSingleItemRow::OnArrayOrCustomDragLeave);
					AcceptDropDelegate = FOnAcceptDrop::CreateSP(this, PropertyNode->IsReorderable() ? &SDetailSingleItemRow::OnArrayAcceptDrop : &SDetailSingleItemRow::OnArrayHeaderAcceptDrop);
					CanAcceptDropDelegate = FOnCanAcceptDrop::CreateSP(this, &SDetailSingleItemRow::OnArrayCanAcceptDrop);
				}
			}

			NameColumnBox->AddSlot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(2.0f,0.0f,0.0f,0.0f)
				.AutoWidth()
				[
					SNew(SDetailExpanderArrow, SharedThis(this))
				];

			NameColumnBox->AddSlot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(2.0f,0.0f,0.0f,0.0f)
				.AutoWidth()
				[
					SNew(SEditConditionWidget)
					.EditConditionValue(WidgetRow.EditConditionValue)
					.OnEditConditionValueChanged(WidgetRow.OnEditConditionValueChanged)
				];

			if (bHasMultipleColumns)
			{
				NameColumnBox->AddSlot()
					.HAlign(WidgetRow.NameWidget.HorizontalAlignment)
					.VAlign(WidgetRow.NameWidget.VerticalAlignment)
					.Padding(2.0f,0.0f,0.0f,0.0f)
					[
						NameWidget.ToSharedRef()
					];

				// create Name column:
				// | Name | Value | Right |
				Splitter->AddSlot()
					.Value(ColumnSizeData.GetNameColumnWidth())
					.OnSlotResized(ColumnSizeData.GetOnNameColumnResized())
					[
						GetNameWidget(NameColumnBox, GetPropertyNode())
					];

				// create Value column:
				// | Name | Value | Right |
				Splitter->AddSlot()
					.Value(ColumnSizeData.GetValueColumnWidth())
					.OnSlotResized(ColumnSizeData.GetOnValueColumnResized())
					[
						SNew(SHorizontalBox)
						.Clipping(EWidgetClipping::OnDemand)
						+ SHorizontalBox::Slot()
						.HAlign(WidgetRow.ValueWidget.HorizontalAlignment)
						.VAlign(WidgetRow.ValueWidget.VerticalAlignment)
						.Padding(DetailWidgetConstants::RightRowPadding)
						[
							ValueWidget.ToSharedRef()
						]
						// extension widget
						+ SHorizontalBox::Slot()
						.HAlign(WidgetRow.ExtensionWidget.HorizontalAlignment)
						.VAlign(WidgetRow.ExtensionWidget.VerticalAlignment)
						.Padding(5.0f,0.0f,0.0f,0.0f)
						.AutoWidth()
						[
							ExtensionWidget.ToSharedRef()
						]
					];
			}
			else
			{
				// create whole row widget, which takes up both the Name and Value columns:
				// | Name | Value | Right |
				NameColumnBox->SetEnabled(IsEnabledAttribute);
				NameColumnBox->AddSlot()
					.HAlign(WidgetRow.WholeRowWidget.HorizontalAlignment)
					.VAlign(WidgetRow.WholeRowWidget.VerticalAlignment)
					.Padding(2.0f,0.0f,0.0f,0.0f)
					[
						WidgetRow.WholeRowWidget.Widget
					];

				Splitter->AddSlot()
					.Value(ColumnSizeData.GetWholeRowColumnWidth())
					.OnSlotResized(ColumnSizeData.GetOnWholeRowColumnResized())
					[
						NameColumnBox
					];
			}

			TSharedPtr<SWidget> RightmostWidget;
			if (GetPropertyNode().IsValid() &&
					DetailsView &&
					DetailsView->GetDisplayManager().IsValid())
			{
				TSharedPtr<FDetailsDisplayManager> DisplayManagerLocal = DetailsView->GetDisplayManager();
				
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (DisplayManagerLocal->CanConstructPropertyUpdatedWidgetBuilder())
				{
					DisplayManager = MoveTemp(DisplayManagerLocal);

					auto FindParentUObject = [](FPropertyNode* Cursor) -> FObjectPropertyNode*
					{
						while(Cursor)
						{
							FObjectPropertyNode* ParentObjectNode = Cursor->AsObjectNode();
							if (ParentObjectNode)
							{
								return ParentObjectNode;
							}
							else
							{
								Cursor = Cursor->GetParentNode();
							}
						}
						return nullptr;
					};

					FObjectPropertyNode* OwningObjectNode = FindParentUObject(Customization->GetPropertyNode().Get());
					TArray<TWeakObjectPtr<UObject>> Objects;
					if (OwningObjectNode)
					{
						Objects.Reserve(OwningObjectNode->GetNumObjects());
						for(int32 Index = 0, End = OwningObjectNode->GetNumObjects(); Index < End; ++Index)
						{
							if (UObject* CategoryObject = OwningObjectNode->GetUObject(Index))
							{
								Objects.Add(CategoryObject);
							}
						}	
					}

					TSharedPtr<FPropertyNode> PropertyNode = InOwnerTreeNode->GetPropertyNode();
					TStrongObjectPtr<UObject> PinnedObject;
					if (!Objects.IsEmpty())
					{
						PinnedObject = Objects[0].Pin();
					}
					if (PropertyNode && PinnedObject.IsValid())
					{
						TSharedRef<FPropertyPath> PropertyPath = FPropertyNode::CreatePropertyPath( PropertyNode.ToSharedRef() );
						
						if (PropertyPath->IsValid())
						{
							FConstructPropertyUpdatedWidgetBuilderArgs Args;
							Args.ResetToDefaultAction =	FExecuteAction::CreateSPLambda(this, [this]
							{
								this->OnResetToDefaultClicked();

								if (TSharedPtr<FDetailTreeNode> OwnerTreeNodePinned = OwnerTreeNode.Pin())
								{
									if (TSharedPtr<IDetailsViewPrivate> DetailsView = OwnerTreeNodePinned->GetDetailsViewSharedPtr())
									{
										DetailsView->RequestForceRefresh();
									}
								}
							});
							Args.Objects = &Objects;
							Args.PropertyPath = PropertyPath;
							Args.Category = Category.IsValid() ? Category->GetCategoryName() : NAME_None;
						
							PropertyUpdatedWidgetBuilder = DisplayManager->ConstructPropertyUpdatedWidgetBuilder(Args);

							if (PropertyUpdatedWidgetBuilder.IsValid())
							{
								TAttribute<bool> IsHovered = TAttribute<bool>::CreateSP( this, &SDetailSingleItemRow::IsHovered);
								PropertyUpdatedWidgetBuilder->Bind_IsRowHovered(IsHovered);
							}
						}
					}
				}
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}

			if(PropertyUpdatedWidgetBuilder.IsValid())
			{
				RightmostWidget = PropertyUpdatedWidgetBuilder->GenerateWidget();
			}

			// also perform the two additional cases below in case
			// PropertyUpdatedWidgetBuilder->GenerateWidget() return nullptr
			if(RightmostWidget.IsValid())
			{
				// nothing to do
			}
			else if(WidgetRow.HasResetToDefaultContent())
			{
				RightmostWidget = WidgetRow.ResetToDefaultWidget.Widget;
			}
			else
			{
				TArray<FPropertyRowExtensionButton> ExtensionButtons;

				UpdateResetToDefault();
				FPropertyRowExtensionButton& ResetToDefault = ExtensionButtons.AddDefaulted_GetRef();
				ResetToDefault.Label = NSLOCTEXT("PropertyEditor", "ResetToDefault", "Reset to Default");
				ResetToDefault.UIAction = FUIAction(
					FExecuteAction::CreateSP(this, &SDetailSingleItemRow::OnResetToDefaultClicked),
					FCanExecuteAction::CreateLambda([this, IsValueEnabledAttribute]()
						{
							return IsResetToDefaultVisible() && IsValueEnabledAttribute.Get(true);
						})
				);

				// We could just collapse the Reset to Default button by setting the FIsActionButtonVisible delegate,
				// but this would cause the reset to defaults not to reserve space in the toolbar and not be aligned across all rows.
				// Instead, we show an empty icon and tooltip and disable the button.
				static FSlateIcon EnabledResetToDefaultIcon(FAppStyle::Get().GetStyleSetName(), "PropertyWindow.DiffersFromDefault");
				static FSlateIcon DisabledResetToDefaultIcon(FAppStyle::Get().GetStyleSetName(), "NoBrush");
				ResetToDefault.Icon = TAttribute<FSlateIcon>::Create([this]()
				{
					return IsResetToDefaultVisible() ?
						EnabledResetToDefaultIcon :
						DisabledResetToDefaultIcon;
				});

				ResetToDefault.ToolTip = TAttribute<FText>::Create([this]() 
				{
					return IsResetToDefaultVisible() ?
						NSLOCTEXT("PropertyEditor", "ResetToDefaultPropertyValueToolTip", "Reset this property to its default value.") :
						FText::GetEmpty();
				});

				CreateGlobalExtensionWidgets(ExtensionButtons);

				FSlimHorizontalToolBarBuilder ToolbarBuilder(TSharedPtr<FUICommandList>(), FMultiBoxCustomization::None);
				ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);
				ToolbarBuilder.SetStyle(&FAppStyle::Get(), "DetailsView.ExtensionToolBar");
				ToolbarBuilder.SetIsFocusable(false);

				for (const FPropertyRowExtensionButton& Extension : ExtensionButtons)
				{
					ToolbarBuilder.AddToolBarButton(Extension.UIAction, NAME_None, Extension.Label, Extension.ToolTip, Extension.Icon);
				}

				RightmostWidget = ToolbarBuilder.MakeWidget();
			}

			Splitter->AddSlot()
				.Value(ColumnSizeData.GetRightColumnWidth())
				.OnSlotResized(ColumnSizeData.GetOnRightColumnResized())
				.MinSize(ColumnSizeData.GetRightColumnMinWidth())
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle"))
				.BorderBackgroundColor(this, &SDetailSingleItemRow::GetOuterBackgroundColor)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(0.0f)
				[
					RightmostWidget.ToSharedRef()
				]
			];
		}
	}
	else
	{
		// details panel layout became invalid.  This is probably a scenario where a widget is coming into view in the parent tree but some external event previous in the frame has invalidated the contents of the details panel.
		// The next frame update of the details panel will fix it
		Widget = SNew(SSpacer);
	}

	OwnerTableViewWeak = InOwnerTableView;
	auto GetScrollbarWellBrush = [this]()
	{
		return SDetailTableRowBase::IsScrollBarVisible(OwnerTableViewWeak) ?
			FAppStyle::Get().GetBrush("DetailsView.GridLine") : 
			FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle");
	};

	auto GetScrollbarWellTint = [this]()
	{
		return SDetailTableRowBase::IsScrollBarVisible(OwnerTableViewWeak) ?
			FStyleColors::White : 
			this->GetOuterBackgroundColor();
	};

	auto GetHighlightBorderPadding = [this]()
	{
		return this->IsHighlighted() ? FMargin(1) : FMargin(0);
	};

	static const FDetailsViewStyleKey& PrimaryKey = SDetailsView::GetPrimaryDetailsViewStyleKey();

	// If this is a stub category with no UProperty data, just show a null widget, we don't have anything useful to show here
	if (Category.IsValid() && Category->IsEmpty())
	{
		this->ChildSlot
		[ 
			SNullWidget::NullWidget
		];
	}
	else
	{
		TSharedRef<SWidget> ContentBorder = SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("DetailsView.GridLine"))
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 1.0f))
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SNew(SBox)
				.MinDesiredHeight(PropertyEditorConstants::PropertyRowHeight)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("DetailsView.Highlight"))
						.Padding_Lambda(GetHighlightBorderPadding)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle"))
							.BorderBackgroundColor(this, &SDetailSingleItemRow::GetOuterBackgroundColor)
							.Padding(0.0f)
							[
								Widget
							]
						]
					]
				]
			];

		if (Category.IsValid() && Category->GetColor().IsSet())
		{
			this->ChildSlot
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(0.f)
				.MaxWidth(2.f)
				[
					SNew(SColorBlock)
					.Color(Category->GetColor().GetValue())
				]
				+ SHorizontalBox::Slot()
				.Padding(0.f)
				.FillContentWidth(1.0f)
				[
					ContentBorder
				]
			];
		}
		else
		{
			this->ChildSlot
			[
				ContentBorder
			];
		}
	}

	STableRow< TSharedPtr< FDetailTreeNode > >::ConstructInternal(
		STableRow::FArguments()
			.Style(FAppStyle::Get(), "DetailsView.TreeView.TableRow")
			.ShowSelection(false)
			.OnDragLeave(DragLeaveDelegate)
			.OnAcceptDrop(AcceptDropDelegate)
			.OnCanAcceptDrop(CanAcceptDropDelegate),
		InOwnerTableView
	);
}

void SDetailSingleItemRow::SetPropertyColorationTarget(bool bSelectActors /*=false*/)
{
	const TSharedPtr<FPropertyNode> PropertyNode = GetPropertyNode();
	const TSharedPtr<IPropertyHandle> PropertyHandle = GetPropertyHandle();		

	if (PropertyNode.IsValid() && PropertyHandle.IsValid())
	{
		TSharedPtr<IDetailsViewPrivate> DetailsView = OwnerTreeNode.Pin()->GetDetailsViewSharedPtr();
		TSharedPtr<IPropertyHandle> Handle = PropertyEditorHelpers::GetPropertyHandle(PropertyNode.ToSharedRef(), DetailsView->GetNotifyHook(), DetailsView->GetPropertyUtilities());

		FString Value;
		if (Handle->GetValueAsFormattedString(Value, PPF_Copy) == FPropertyAccess::Success)
		{
			FProperty* Property = PropertyHandle->GetProperty();

			TSharedRef<FEditPropertyChain> PropertyChain = PropertyNode->BuildPropertyChain(Property);
			check(PropertyChain.IsUnique());

			FProperty* TopProperty = PropertyChain->GetHead()->GetValue();
			GEditor->SetPropertyColorationTarget(GWorld, Value, Property, TopProperty->GetOwnerClass(), &PropertyChain, bSelectActors);
		}
	}
}

FReply SDetailSingleItemRow::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetModifierKeys().IsShiftDown())
	{
		bool bIsHandled = false;
		if (CopyAction.CanExecute() && MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			CopyAction.Execute();
			PulseAnimation.Play(SharedThis(this));
			bIsHandled = true;
		}
		// Paste is disabled if property editing is disabled
		else if (PasteAction.CanExecute() && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && OwnerTreeNode.Pin()->GetDetailsViewSharedPtr()->IsPropertyEditingEnabled())
		{
			PasteAction.Execute();
			PulseAnimation.Play(SharedThis(this));
			if(TSharedPtr<FPropertyNode> PropertyNode = GetPropertyNode())
			{
				// Mark property node as animating so we will animate after any potential re-construction
				TSharedPtr<IDetailsViewPrivate> DetailsView = OwnerTreeNode.Pin()->GetDetailsViewSharedPtr();
				DetailsView->MarkNodeAnimating(PropertyNode, UE::PropertyEditor::Private::PulseAnimationLength);
			}
			bIsHandled = true;
		}

		if (bIsHandled)
		{
			return FReply::Handled();
		}
	}
	else if (MouseEvent.GetModifierKeys().IsControlDown() && GEditor && GWorld)
	{
		SetPropertyColorationTarget();
	}

	return SDetailTableRowBase::OnMouseButtonUp(MyGeometry, MouseEvent);
}

bool SDetailSingleItemRow::IsResetToDefaultVisible() const
{
	return bCachedResetToDefaultVisible;
}

void SDetailSingleItemRow::OnResetToDefaultClicked() const
{
	TSharedPtr<IPropertyHandle> PropertyHandle = GetPropertyHandle();
	if (WidgetRow.CustomResetToDefault.IsSet())
	{
		WidgetRow.CustomResetToDefault.GetValue().OnResetToDefaultClicked(PropertyHandle);
	}
	else if (PropertyHandle.IsValid())
	{
		TSharedPtr<IPropertyHandleOptional> OptionalHandle = PropertyHandle->AsOptional();
		if (!OptionalHandle.IsValid())
		{
			// This could be the value calling as we share UI between options and their value
			OptionalHandle = PropertyHandle->GetParentHandle()->AsOptional();
			if (OptionalHandle.IsValid())
			{
				PropertyHandle = PropertyHandle->GetParentHandle();
			}
		}
		
		PropertyHandle->ResetToDefault();
		
	}
}

/** Get the background color of the outer part of the row, which contains the edit condition and extension widgets. */
FSlateColor SDetailSingleItemRow::GetOuterBackgroundColor() const
{
	if (IsHighlighted() || DragOperation.IsValid())
	{
		return FAppStyle::Get().GetSlateColor("Colors.Hover");
	}

	return PropertyEditorConstants::GetRowBackgroundColor(0, this->IsHovered());
}

/** Get the background color of the inner part of the row, which contains the name and value widgets. */
FSlateColor SDetailSingleItemRow::GetInnerBackgroundColor() const
{
	FSlateColor Color;

	if (IsHighlighted())
	{
		Color = FAppStyle::Get().GetSlateColor("Colors.Hover");
	}
	else
	{
		const int32 IndentLevel = GetIndentLevelForBackgroundColor();
		Color = PropertyEditorConstants::GetRowBackgroundColor(IndentLevel, this->IsHovered());
	}

	if (PulseAnimation.IsPlaying())
	{
		float Lerp = PulseAnimation.GetLerp();
		return FMath::Lerp(FAppStyle::Get().GetSlateColor("Colors.Hover2").GetSpecifiedColor(), Color.GetSpecifiedColor(), Lerp);
	}

	return Color;
}

void SDetailSingleItemRow::OnCopyGroup()
{
	if (!OwnerTreeNode.IsValid())
	{
		return;
	}

	if (TArray<TSharedPtr<IPropertyHandle>> GroupProperties = GetPropertyHandles(true);
		!GroupProperties.IsEmpty())
	{
		TArray<FString> PropertiesNotCopied;
		PropertiesNotCopied.Reserve(GroupProperties.Num());
		
		TMap<FString, FString> PropertyValues;
		PropertyValues.Reserve(GroupProperties.Num());

		const FGuid OperationGuid = FGuid::NewGuid();			
		
		for (TSharedPtr<IPropertyHandle> PropertyHandle : GroupProperties)
		{
			if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
			{
				FString PropertyPath = UE::PropertyEditor::GetPropertyPath(PropertyHandle);
				
				FString PropertyValueStr;

				// First check if there is some custom copy paste that handles this property
				if (OnCopyToTextDelegate.IsValid())
				{
					OnCopyToTextDelegate->Broadcast(PropertyHandle, PropertyValueStr, OperationGuid);
				}
				
				if (!PropertyValueStr.IsEmpty())
				{
					PropertyValues.Add(PropertyPath, PropertyValueStr);
				}
				else if (PropertyHandle->GetValueAsFormattedString(PropertyValueStr, PPF_Copy) == FPropertyAccess::Success)
				{
					PropertyValues.Add(PropertyPath, PropertyValueStr);
				}
				else
				{
					PropertiesNotCopied.Add(PropertyHandle->GetPropertyDisplayName().ToString());
				}
			}
		}

		if (!PropertiesNotCopied.IsEmpty())
		{
			UE_LOGF(
				LogPropertyNode,
				Warning,
				"One or more of the properties in group \"%ls\" was not copied:\n%ls",
				*GetRowNameText(),
				*FString::Join(PropertiesNotCopied, TEXT("\n")));
		}

		FPropertyEditorClipboard::ClipboardCopy([&PropertyValues](TMap<FName, FString>& OutTaggedClipboard)
		{
			for (const TPair<FString, FString>& PropertyValuePair : PropertyValues)
			{
				OutTaggedClipboard.Add(FName(PropertyValuePair.Key), PropertyValuePair.Value);
			}
		});

		PulseAnimation.Play(SharedThis(this));
	}
}

bool SDetailSingleItemRow::CanCopyGroup() const
{
	if (!OwnerTreeNode.IsValid())
	{
		return false;
	}

	TArray<TSharedPtr<IPropertyHandle>> GroupPropertyHandles = GetPropertyHandles(true);
	return !GroupPropertyHandles.IsEmpty();
}

void SDetailSingleItemRow::OnPasteGroup()
{
	if (!OwnerTreeNode.IsValid()
		|| !CanPasteGroup())
	{
		return;
	}

	if (OnPasteFromTextDelegate.IsValid())
	{
		if (const TArray<TSharedPtr<IPropertyHandle>> GroupProperties = GetPropertyHandles(true);
			!GroupProperties.IsEmpty())
		{
			FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "PasteGroupProperties", "Paste Group Properties"));

			TArray<FString> PropertiesNotPasted;
			PropertiesNotPasted.Reserve(GroupProperties.Num());

			{
				const FGuid OperationGuid = FGuid::NewGuid();
			
				for (const TPair<FName, FString>& PropertyNameValuePair : PreviousClipboardData.PropertyValues)
				{
					OnPasteFromTextDelegate->Broadcast(PropertyNameValuePair.Key.ToString(), PropertyNameValuePair.Value, OperationGuid);
				}
			}
		
			if (!PropertiesNotPasted.IsEmpty())
			{
				UE_LOGF(
					LogPropertyNode,
					Warning,
					"One or more of the properties in group \"%ls\" was not pasted:\n%ls",
					*GetRowNameText(),
					*FString::Join(PropertiesNotPasted, TEXT("\n")));
			}

			ForceRefresh();
		}
	}
}

bool SDetailSingleItemRow::CanPasteGroup()
{
	if (!OwnerTreeNode.IsValid())
	{
		return false;
	}

	const TArray<TSharedPtr<IPropertyHandle>> GroupPropertyHandles = GetPropertyHandles(true);
	const TArray<TSharedPtr<FPropertyNode>> GroupPropertyNodes = GetPropertyNodesFromHandles(GroupPropertyHandles);

	// @note: We allow pasting to properties that are disabled due to an EditCondition, but not those that are never editable (ie. VisibleAnywhere).
	const bool bHasEditables = Algo::AnyOf(GroupPropertyNodes, [](const TSharedPtr<FPropertyNode>& InPropertyNode)
	{
		constexpr bool bIncludeEditConditionForConstCheck = false;
		return !InPropertyNode->IsEditConst(bIncludeEditConditionForConstCheck);
	});

	// No editable properties to write to
	if (!bHasEditables)
	{
		return false;
	}

	FString ClipboardContent;
	FPropertyEditorClipboard::ClipboardPaste(ClipboardContent);

	// If same as last, return previously resolved applicability
	if (PreviousClipboardData.Content.Get({}).Equals(ClipboardContent))
	{
		return PreviousClipboardData.bIsApplicable;
	}

	// New clipboard contents, non-applicable by default
	PreviousClipboardData.Reset();

	// Can't be empty, must be json
	if (!UE::PropertyEditor::Internal::IsJsonString(ClipboardContent))
	{
		return false;
	}

	PreviousClipboardData.Reserve(GroupPropertyHandles.Num());

	if (!UE::PropertyEditor::Internal::TryParseClipboard(ClipboardContent, PreviousClipboardData.PropertyValues))
	{
		return false;
	}

	PreviousClipboardData.PropertyValues.GenerateKeyArray(PreviousClipboardData.PropertyNames);

	TArray<FString> PropertyNames;
	Algo::Transform(GroupPropertyHandles, PropertyNames, [](const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	{
		return UE::PropertyEditor::GetPropertyPath(InPropertyHandle);
	});

	PreviousClipboardData.PropertyNames.Sort(FNameLexicalLess());
	PropertyNames.Sort();

	// @note: properties must all match to be applicable
	PreviousClipboardData.Content = ClipboardContent;

	return PreviousClipboardData.bIsApplicable = Algo::Compare(PreviousClipboardData.PropertyNames, PropertyNames);
}

void SDetailSingleItemRow::PopulateContextMenu(UToolMenu* ToolMenu)
{
	SDetailTableRowBase::PopulateContextMenu(ToolMenu);

	TSharedPtr<IDetailsViewPrivate> OwningDetailsView;
	if (TSharedPtr<FDetailTreeNode> OwnerTreeNodePtr = OwnerTreeNode.Pin())
	{
		OwningDetailsView = OwnerTreeNodePtr->GetDetailsViewSharedPtr();
	}
	
	FToolMenuSection& EditSection = ToolMenu->FindOrAddSection(TEXT("Edit"));
	{
		if (CopyAction.IsBound() && PasteAction.IsBound())
		{
			constexpr bool bLongDisplayName = false;
			const bool bIsGroup = Customization->IsValidCustomization() && Customization->DetailGroup.IsValid();

			// Copy
			{
				TAttribute<FText> Label;
				TAttribute<FText> ToolTip;
				
				if (bIsGroup)
				{
					Label = NSLOCTEXT("PropertyView", "CopyGroupProperties", "Copy All Properties in Group");
					ToolTip = TAttribute<FText>::CreateLambda([this]()
					{
						return CanCopyGroup()
							? NSLOCTEXT("PropertyView", "CopyGroupProperties_ToolTip", "Copy all properties in this group")
							: NSLOCTEXT("PropertyView", "CantCopyGroupProperties_ToolTip", "None of the properties in this group can be copied");
					});
				}
				else
				{
					Label = NSLOCTEXT("PropertyView", "CopyProperty", "Copy");
					ToolTip = NSLOCTEXT("PropertyView", "CopyProperty_ToolTip", "Copy this property value");
				}
				
				// Used to trigger animation after a copy
				FUIAction WrappedCopyAction(CopyAction);
				WrappedCopyAction.ExecuteAction = FExecuteAction::CreateSPLambda(this, 
					[this]()
					{
						if (CopyAction.Execute())
						{
							PulseAnimation.Play(SharedThis(this));
						}
					}
				);

				FToolMenuEntry& CopyMenuEntry = EditSection.AddMenuEntry(
					TEXT("Copy"),
					Label,
					ToolTip,
					FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
					WrappedCopyAction);

				CopyMenuEntry.InputBindingLabel = FInputChord(EModifierKey::Shift, EKeys::RightMouseButton).GetInputText(bLongDisplayName);
			}

			// Paste
			{
				// Paste is only enabled if property editing is enabled
				if (OwningDetailsView && OwningDetailsView->IsPropertyEditingEnabled() && WidgetRow.EditConditionValue.Get(true /*DefaultValue*/))
				{
					TAttribute<FText> Label;
					TAttribute<FText> ToolTip;

					if (bIsGroup)
					{
						Label = NSLOCTEXT("PropertyView", "PasteGroupProperties", "Paste All Properties in Group");
						ToolTip = TAttribute<FText>::CreateLambda([this]()
						{
							return CanPasteGroup()
								? NSLOCTEXT("PropertyView", "PasteGroupProperties_ToolTip", "Paste the copied property values here")
								// @note: this is specific to the constraint that the destination group has to match the source group (copied from) exactly 
								: NSLOCTEXT("PropertyView", "CantPasteGroupProperties_ToolTip", "The properties in this group don't match the contents of the clipboard, or the properties aren't editable");
						});
					}
					else
					{
						Label = NSLOCTEXT("PropertyView", "PasteProperty", "Paste");
						ToolTip = NSLOCTEXT("PropertyView", "PasteProperty_ToolTip", "Paste the copied value here");	
					}
					
					// Used to trigger animation after a paste
					FUIAction WrappedPasteAction(PasteAction);
					WrappedPasteAction.ExecuteAction = FExecuteAction::CreateSPLambda(this, 
						[this]()
						{
							if (PasteAction.Execute())
							{
								PulseAnimation.Play(SharedThis(this));
							}
						}
					);
																
					FToolMenuEntry& PasteMenuEntry = EditSection.AddMenuEntry(
						TEXT("Paste"),
						Label,
						ToolTip,
						FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Paste"),
						WrappedPasteAction);

					PasteMenuEntry.InputBindingLabel = FInputChord(EModifierKey::Shift, EKeys::LeftMouseButton).GetInputText(bLongDisplayName);	
				}
			}
		}

		// Copy Display Name
		{
			FUIAction CopyDisplayNameAction = FExecuteAction::CreateSP(this, &SDetailSingleItemRow::OnCopyPropertyDisplayName);
			CopyDisplayNameAction.CanExecuteAction = FCanExecuteAction::CreateSP(this, &SDetailSingleItemRow::CanCopyPropertyDisplayName);

			static const FTextFormat TooltipFormat = NSLOCTEXT("PropertyView_Single", "CopyPropertyDisplayName_ToolTip", "Copy the display name of this property to the system clipboard:\n{0}");

			EditSection.AddMenuEntry(
				TEXT("CopyDisplayName"),
				NSLOCTEXT("PropertyView", "CopyPropertyDisplayName", "Copy Display Name"),
				FText::Format(TooltipFormat, GetPropertyDisplayName()),
				FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
				CopyDisplayNameAction);
		}

		// Copy Internal Name
		{
			FUIAction CopyInternalNameAction = FExecuteAction::CreateSP(this, &SDetailSingleItemRow::OnCopyPropertyInternalName);
			CopyInternalNameAction.CanExecuteAction = FCanExecuteAction::CreateSP(this, &SDetailSingleItemRow::CanCopyPropertyInternalName);

			static const FTextFormat TooltipFormat = NSLOCTEXT("PropertyView_Single", "CopyPropertyInternalName_ToolTip", "Copy the internal name of this property to the system clipboard:\n{0}");

			EditSection.AddMenuEntry(
				TEXT("CopyInternalName"),
				NSLOCTEXT("PropertyView", "CopyPropertyInternalName", "Copy Internal Name"),
				FText::Format(TooltipFormat, FText::FromString(GetPropertyInternalName())),
				FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
				CopyInternalNameAction);
		}

		// Select By Property
		{
			FUIAction SelectByPropertyAction = FExecuteAction::CreateSP(this, &SDetailSingleItemRow::SetPropertyColorationTarget, true);
			SelectByPropertyAction.IsActionVisibleDelegate = FIsActionButtonVisible::CreateSP(this, &SDetailSingleItemRow::CanSelectByProperty);

			static const FTextFormat TooltipFormat = NSLOCTEXT("PropertyView_Single", "SelectActorsByProperty_ToolTip", "Select all actors with the same value for the property named:\n{0}");

			EditSection.AddMenuEntry(
				TEXT("SelectByProperty"),
				NSLOCTEXT("PropertyView", "SelectActorsByProperty", "Select Actors by Property"),
				FText::Format(TooltipFormat, GetPropertyDisplayName()),
				FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.SelectAll"),
				SelectByPropertyAction);
		}

		if (FPropertyEditorPermissionList::Get().ShouldShowMenuEntries())
		{
			// Hide separator line if it only contains the SearchWidget, making the next 2 elements the top of the list
			if (EditSection.Blocks.Num() > 1)
			{
				EditSection.AddSeparator(NAME_None);
			}
                    
			EditSection.AddMenuEntry(
				TEXT("CopyRowName"),
				NSLOCTEXT("PropertyView", "CopyRowName", "Copy internal row name"),
				NSLOCTEXT("PropertyView", "CopyRowName_ToolTip", "Copy the row's parent struct and internal name to use in the property editor's allow/deny lists."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SDetailSingleItemRow::CopyRowNameText)));

			EditSection.AddMenuEntry(
				TEXT("AddAllowList"),
				NSLOCTEXT("PropertyView", "AddAllowList", "Add to Allowed"),
				NSLOCTEXT("PropertyView", "AddAllowList_ToolTip", "Add this row to the property editor's allowed properties list."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SDetailSingleItemRow::OnToggleAllowList),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &SDetailSingleItemRow::IsAllowListChecked)),
				EUserInterfaceActionType::Check,
				NAME_None);

			EditSection.AddMenuEntry(
				TEXT("AddDenyList"),
				NSLOCTEXT("PropertyView", "AddDenyList", "Add to Denied"),
				NSLOCTEXT("PropertyView", "AddDenyList_ToolTip", "Add this row to the property editor's denied properties list."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SDetailSingleItemRow::OnToggleDenyList),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &SDetailSingleItemRow::IsDenyListChecked)),
				EUserInterfaceActionType::Check,
				NAME_None);
		}

		if (WidgetRow.CustomMenuItems.Num() > 0)
		{
			// Hide separator line if it only contains the SearchWidget, making the next 2 elements the top of the list
			if (EditSection.Blocks.Num() > 1)
			{
				EditSection.AddSeparator(NAME_None);
			}

			for (const FDetailWidgetRow::FCustomMenuData& CustomMenuData : WidgetRow.CustomMenuItems)
			{
				// Add the menu entry
				EditSection.AddMenuEntry(
					CustomMenuData.GetEntryName(),
					CustomMenuData.Name,
					CustomMenuData.Tooltip,
					CustomMenuData.SlateIcon,
					CustomMenuData.Action);
			}
		}
	}
}

TArray<TSharedPtr<IPropertyHandle>> SDetailSingleItemRow::GetPropertyHandles(const bool bRecursive) const
{
	if (TArray<TSharedPtr<IPropertyHandle>> PropertyHandles = SDetailTableRowBase::GetPropertyHandles(bRecursive);
		!PropertyHandles.IsEmpty())
	{
		return PropertyHandles;
	}

	return WidgetRow.GetPropertyHandles();
}

TSharedPtr<IPropertyHandle> SDetailSingleItemRow::GetPrimaryPropertyHandle() const
{
	TSharedPtr<IPropertyHandle> PrimaryPropertyHandle = SDetailTableRowBase::GetPrimaryPropertyHandle();
	if (PrimaryPropertyHandle.IsValid())
	{
		return PrimaryPropertyHandle;
	}

	if (const TArray<TSharedPtr<IPropertyHandle>>& WidgetPropertyHandles = WidgetRow.GetPropertyHandles();
		!WidgetPropertyHandles.IsEmpty())
	{
		return WidgetPropertyHandles[0];
	}

	return nullptr;
}

bool SDetailSingleItemRow::CanFavoriteRow(bool bInAddToFavorite)
{
	if (bInAddToFavorite == IsFavorite())
	{
		return false;		
	}

	if (Customization->HasPropertyNode())
	{
		return true;
	}

	if (Customization->HasCustomBuilder())
	{
		const TSharedPtr<IPropertyHandle> PropertyHandle = Customization->CustomBuilderRow->GetPropertyHandle();
		const FString& OriginalPath = Customization->CustomBuilderRow->GetOriginalPath();
		return PropertyHandle.IsValid() || !OriginalPath.IsEmpty();
	}

	return false;
}

void SDetailSingleItemRow::OnFavoriteRow(bool bInAddToFavorite)
{
	const TSharedPtr<FDetailTreeNode> OwnerTreeNodePinned = OwnerTreeNode.Pin();
	if (!OwnerTreeNodePinned.IsValid())
	{
		return;
	}

	const TSharedPtr<IDetailsViewPrivate> DetailsView = OwnerTreeNodePinned->GetDetailsViewSharedPtr();
	if (!DetailsView)
	{
		return;
	}

	if (Customization->HasPropertyNode())
	{
		const TSharedPtr<FPropertyNode> PropertyNode = Customization->GetPropertyNode();
		PropertyNode->SetFavorite(bInAddToFavorite);
	}
	else if (Customization->HasCustomBuilder())
	{
		const TSharedPtr<IPropertyHandle> PropertyHandle = Customization->CustomBuilderRow->GetPropertyHandle();
		const FString& OriginalPath = Customization->CustomBuilderRow->GetOriginalPath();

		if (PropertyHandle.IsValid())
		{
			StaticCastSharedPtr<FPropertyHandleBase>(PropertyHandle)->GetPropertyNode()->SetFavorite(bInAddToFavorite); 
		}
		else if (!OriginalPath.IsEmpty())
		{
			DetailsView->SetCustomBuilderFavorite(OriginalPath, bInAddToFavorite);
		}
	}

	// Calculate the scrolling offset (by item) to make sure the mouse stay over the same property
	int32 ExpandSize = 0;
	if (OwnerTreeNodePinned->ShouldBeExpanded())
	{
		SDetailSingleItemRow_Helper::RecursivelyGetItemShow(OwnerTreeNodePinned.ToSharedRef(), ExpandSize);
	}
	else
	{
		// if the item is not expand count is 1
		ExpandSize = 1;
	}

	// Apply the calculated offset
	DetailsView->MoveScrollOffset(bInAddToFavorite ? ExpandSize : -ExpandSize);

	// Refresh the tree
	ForceRefresh();
}

void SDetailSingleItemRow::OnCopyProperty()
{
	if (OwnerTreeNode.IsValid())
	{
		TSharedPtr<FPropertyNode> PropertyNode = GetPropertyNode();
		if (PropertyNode.IsValid())
		{
			TSharedPtr<IDetailsViewPrivate> DetailsView = OwnerTreeNode.Pin()->GetDetailsViewSharedPtr();
			TSharedPtr<IPropertyHandle> Handle = PropertyEditorHelpers::GetPropertyHandle(PropertyNode.ToSharedRef(), DetailsView->GetNotifyHook(), DetailsView->GetPropertyUtilities());

			FString Value;
			if (Handle->GetValueAsFormattedString(Value, PPF_Copy) == FPropertyAccess::Success)
			{
				FPropertyEditorClipboard::ClipboardCopy(*Value);
				PulseAnimation.Play(SharedThis(this));
			}
		}
	}
}

FText SDetailSingleItemRow::GetPropertyDisplayName() const
{
	if (!OwnerTreeNode.IsValid())
	{
		return { };
	}

	const TSharedPtr<FPropertyNode> PropertyNode = GetPropertyNode();
	if (!PropertyNode.IsValid())
	{
		return { };
	}

	if (PropertyEditorHelpers::IsChildOfOption(*PropertyNode))
	{
		return PropertyNode->GetParentNode()->GetDisplayName();
	}

	return PropertyNode->GetDisplayName();
}

void SDetailSingleItemRow::OnCopyPropertyDisplayName()
{
	FPropertyEditorClipboard::ClipboardCopy(*GetPropertyDisplayName().ToString());
}

bool SDetailSingleItemRow::CanCopyPropertyDisplayName()
{
	return !GetPropertyDisplayName().IsEmpty();
}

FString SDetailSingleItemRow::GetPropertyInternalName() const
{
	if (!OwnerTreeNode.IsValid())
	{
		return { };
	}

	const TSharedPtr<FPropertyNode> PropertyNode = GetPropertyNode();
	if (!PropertyNode.IsValid())
	{
		return { };
	}

	const FProperty* Property = PropertyEditorHelpers::IsChildOfOption(*PropertyNode) ? PropertyEditorHelpers::GetOptionParent(*PropertyNode) : PropertyNode->GetProperty();
	if (!Property)
	{
		return { };
	}

	const UStruct* OwnerStruct = Property->GetOwnerStruct();
	if (!OwnerStruct)
	{
		return { };
	}

	return OwnerStruct->GetAuthoredNameForField(Property);
}

void SDetailSingleItemRow::OnCopyPropertyInternalName()
{
	const FString InternalName = GetPropertyInternalName();
	if (!InternalName.IsEmpty())
	{
		FPropertyEditorClipboard::ClipboardCopy(*InternalName);
	}
}

bool SDetailSingleItemRow::CanCopyPropertyInternalName()
{
	return !GetPropertyInternalName().IsEmpty();
}

bool SDetailSingleItemRow::CanSelectByProperty()
{
	return GEditor && GEditor->IsPropertyColorationModeActive();
}

bool SDetailSingleItemRow::CanPasteProperty() const
{
	TSharedPtr<FPropertyNode> PropertyNode = GetPropertyNode();
	check(PropertyNode.IsValid());

	// Check if the property is editable first (a failed EditCondition is still considered to be Editable, to make copying to PostProcess settings etc. practical)
	constexpr bool bIncludeEditConditionForConstCheck = false;
	if (PropertyNode->IsEditConst(bIncludeEditConditionForConstCheck)) // Ignore EditCondition state
	{
		return false;
	}
	
	FString ClipboardContent;
	FPropertyEditorClipboard::ClipboardPaste(ClipboardContent);

	return CanPasteFromText(TEXT(""), ClipboardContent);
}

TSharedRef<SWidget> SDetailSingleItemRow::GetNameWidget(TSharedRef<SWidget> NameWidget, const TSharedPtr<FPropertyNode>& Node) const
{
	 if (Node.IsValid() && OwnerTreeNode.IsValid() )
     {
	 	const TSharedPtr<FDetailTreeNode> DetailTreeNodeSP = OwnerTreeNode.Pin();
	 	
	     if (DetailTreeNodeSP.IsValid())
	     {
			 if (TSharedPtr<IDetailsViewPrivate> DetailsView = DetailTreeNodeSP->GetDetailsViewSharedPtr())
			 {
				 const TSharedPtr<FDetailsNameWidgetOverrideCustomization> DetailsNameWidgetOverrideCustomization =
					 DetailsView->GetDetailsNameWidgetOverrideCustomization();

				 if (DetailsNameWidgetOverrideCustomization.IsValid())
				 {
					 const TSharedRef< FPropertyPath > Path = FPropertyNode::CreatePropertyPath(Node.ToSharedRef());
					 return DetailsNameWidgetOverrideCustomization->CustomizeName(NameWidget, Path.Get());
				 }
			 }
	     }
     }
	return NameWidget;
}

bool SDetailSingleItemRow::CanPasteFromText(const FString& InTag, const FString& InText, const bool bIncludeEditCondition) const
{
	if (InText.IsEmpty())
	{
		return false;
	}

	const TSharedPtr<FPropertyNode> PropertyNode = GetPropertyNode();

	// Prevent paste if we cannot find the property node to paste into.
	if (!PropertyNode.IsValid())
	{
		return false;
	}

	const TSharedPtr<IPropertyHandle> PropertyHandle = GetPropertyHandle();

	// We won't be able to paste without a property handle.
	if (!PropertyHandle.IsValid())
	{
		return false;
	}

	if (const bool bIsTagged = !InTag.IsEmpty();
		bIsTagged)
	{
		const FString PropertyPath = UE::PropertyEditor::Private::GetPropertyPath(
			[&]() { return GetPropertyHandle(); },
			[&]() { return PropertyNode; });
		
		// Ensure that if tag is specified, that it matches the subscriber.
		if (!InTag.Equals(PropertyPath))
		{
			return false;
		}
	}

	// Prevent paste from working if the property's edit condition is not met.
	// Allow paste if no property row can be found.
	{
		TSharedPtr<FDetailPropertyRow> PropertyRow = Customization->PropertyRow;
		
		if (!PropertyRow.IsValid() && Customization->DetailGroup.IsValid())
		{
			PropertyRow = Customization->DetailGroup->GetHeaderPropertyRow();
		}

		if (PropertyRow.IsValid())
		{
			if (const FPropertyEditor* PropertyEditor = PropertyRow->GetPropertyEditor().Get())
			{
				return !PropertyEditor->IsEditConst(bIncludeEditCondition);
			}
		}
	}

	return true;
}

void SDetailSingleItemRow::OnPasteProperty()
{
	FString ClipboardContent;

	const TSharedPtr<FPropertyNode> PropertyNode = GetPropertyNode();
	const FString PropertyPath = UE::PropertyEditor::Private::GetPropertyPath(
		[&]() { return GetPropertyHandle(); },
		[&]() { return PropertyNode; });

	if (PropertyPath.IsEmpty())
	{
		FPropertyEditorClipboard::ClipboardPaste(ClipboardContent);
	}
	else
	{
		FPropertyEditorClipboard::ClipboardPaste(ClipboardContent, FName(PropertyPath));
	}

	if (PasteFromText(TEXT(""), ClipboardContent))
	{
		// Mark property node as animating so we will animate after re-construction
		TSharedPtr<IDetailsViewPrivate> DetailsView = OwnerTreeNode.Pin()->GetDetailsViewSharedPtr();
		DetailsView->MarkNodeAnimating(PropertyNode, UE::PropertyEditor::Private::PulseAnimationLength);

		// Need to refresh the details panel in case a property was pasted over another.
		ForceRefresh();
	}
}

void SDetailSingleItemRow::OnPasteFromText(const FString& InTag, const FString& InText, const TOptional<FGuid>& InOperationId, const bool bIncludeEditCondition)
{
	if (PasteFromText(InTag, InText, bIncludeEditCondition))
	{
		TSharedPtr<FPropertyNode> PropertyNode = GetPropertyNode();
		if (PropertyNode.IsValid())
		{
			// Mark property node as animating so we will animate after re-construction
			TSharedPtr<IDetailsViewPrivate> DetailsView = OwnerTreeNode.Pin()->GetDetailsViewSharedPtr();
			DetailsView->MarkNodeAnimating(PropertyNode, UE::PropertyEditor::Private::PulseAnimationLength, InOperationId);
		}
	}
}

bool SDetailSingleItemRow::PasteFromText(const FString& InTag, const FString& InText, const bool bIncludeEditCondition)
{
	if (!CanPasteFromText(InTag, InText, bIncludeEditCondition))
	{
		return false;
	}

	// The logic below is largely taken from SDisplayClusterColorGradingColorWheel::CommitColor,
	// which avoids writing to trashed objects

	FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "PasteProperty", "Paste Property"));
	
	EPropertyValueSetFlags::Type PropertyValueSetFlags = EPropertyValueSetFlags::InstanceObjects;

	const bool bIsTagged = !InTag.IsEmpty();

	// If tagged, skip individual property transactions. Instead, a single undo will revert all changes in the batch paste.
	// @todo: would be better to indicate that this is a batched paste rather than checking for a tag
	if (bIsTagged)
	{
		PropertyValueSetFlags |= EPropertyValueSetFlags::NotTransactable;
	}

	const TSharedPtr<IPropertyHandle> PropertyHandle = GetPropertyHandle();
	if (!PropertyHandle.IsValid())
	{
		return false;
	}

	if (PropertyHandle->SetValueFromFormattedString(InText, PropertyValueSetFlags) != FPropertyAccess::Success)
	{
		return false;
	}

	if (bIsTagged)
	{
		TArray<UObject*> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);
		for (UObject* Object : OuterObjects)
		{
			if (!Object->HasAnyFlags(RF_Transactional))
			{
				Object->SetFlags(RF_Transactional);
			}

			SaveToTransactionBuffer(Object, false);
			SnapshotTransactionBuffer(Object);
		}
	}

	return true;
}

// Helper function to determine which parent Struct a property comes from. If PropertyName is not a real property,
// it will return the passed-in Struct (eg a DetailsCustomization with a custom name is a "fake" property).
const UStruct* GetExactStructForProperty(const UStruct* MostDerivedStruct, const FName PropertyName)
{
	if (FProperty* Property = MostDerivedStruct->FindPropertyByName(PropertyName))
	{
		return Property->GetOwnerStruct();
	}
	return MostDerivedStruct;
}

FString SDetailSingleItemRow::GetRowNameText() const
{
	if (const TSharedPtr<FDetailTreeNode> Owner = OwnerTreeNode.Pin())
	{
		const UStruct* BaseStructure = Owner->GetParentBaseStructure();
		if (BaseStructure)
		{
			const UStruct* ExactStruct = GetExactStructForProperty(Owner->GetParentBaseStructure(), Owner->GetNodeName());
			return FString::Printf(TEXT("(%s, %s)"), *FSoftObjectPtr(ExactStruct).ToString(), *Owner->GetNodeName().ToString());
		}
	}
	return FString();
}

void SDetailSingleItemRow::CopyRowNameText() const
{
	const FString RowNameText = GetRowNameText();
	if (!RowNameText.IsEmpty())
	{
		FPropertyEditorClipboard::ClipboardCopy(*RowNameText);
	}
}

void SDetailSingleItemRow::OnToggleAllowList() const
{
	const TSharedPtr<FDetailTreeNode> Owner = OwnerTreeNode.Pin();
	if (Owner)
	{
		const FName OwnerName = "DetailRowContextMenu";
		const UStruct* ExactStruct = GetExactStructForProperty(Owner->GetParentBaseStructure(), Owner->GetNodeName());
		if (IsAllowListChecked())
		{
			FPropertyEditorPermissionList::Get().RemoveFromAllowList(ExactStruct, Owner->GetNodeName(), OwnerName);
			UE_LOGF(LogPropertyEditorPermissionList, Log, "Removing %ls from AllowList", *GetRowNameText());
		}
		else
		{
			FPropertyEditorPermissionList::Get().AddToAllowList(ExactStruct, Owner->GetNodeName(), OwnerName);
			UE_LOGF(LogPropertyEditorPermissionList, Log, "Adding %ls to AllowList", *GetRowNameText());
		}
	}
}

bool SDetailSingleItemRow::IsAllowListChecked() const
{
	if (const TSharedPtr<FDetailTreeNode> Owner = OwnerTreeNode.Pin())
	{
		const UStruct* ExactStruct = GetExactStructForProperty(Owner->GetParentBaseStructure(), Owner->GetNodeName());
		return FPropertyEditorPermissionList::Get().IsSpecificPropertyAllowListed(ExactStruct, Owner->GetNodeName());
	}
	return false;
}

void SDetailSingleItemRow::OnToggleDenyList() const
{
	const TSharedPtr<FDetailTreeNode> Owner = OwnerTreeNode.Pin();
	if (Owner)
	{
		const FName OwnerName = "DetailRowContextMenu";
		const UStruct* ExactStruct = GetExactStructForProperty(Owner->GetParentBaseStructure(), Owner->GetNodeName());
		if (IsDenyListChecked())
		{
			FPropertyEditorPermissionList::Get().RemoveFromDenyList(ExactStruct, Owner->GetNodeName(), OwnerName);
			UE_LOGF(LogPropertyEditorPermissionList, Log, "Removing %ls from DenyList", *GetRowNameText());
		}
		else
		{
			FPropertyEditorPermissionList::Get().AddToDenyList(ExactStruct, Owner->GetNodeName(), OwnerName);
			UE_LOGF(LogPropertyEditorPermissionList, Log, "Adding %ls to DenyList", *GetRowNameText());
		}
	}
}

bool SDetailSingleItemRow::IsDenyListChecked() const
{
	if (const TSharedPtr<FDetailTreeNode> Owner = OwnerTreeNode.Pin())
    {
    	return FPropertyEditorPermissionList::Get().IsSpecificPropertyDenyListed(Owner->GetParentBaseStructure(), Owner->GetNodeName());
    }
	return false;
}

void SDetailSingleItemRow::PopulateExtensionWidget()
{
	TSharedPtr<FDetailTreeNode> OwnerTreeNodePinned = OwnerTreeNode.Pin();
	if (OwnerTreeNodePinned.IsValid())
	{ 
		TSharedPtr<IDetailsViewPrivate> DetailsView = OwnerTreeNodePinned->GetDetailsViewSharedPtr();
		TSharedPtr<IDetailPropertyExtensionHandler> ExtensionHandler = DetailsView->GetExtensionHandler();
		if (Customization->HasPropertyNode() && ExtensionHandler.IsValid())
		{
			TSharedPtr<IPropertyHandle> Handle = PropertyEditorHelpers::GetPropertyHandle(Customization->GetPropertyNode().ToSharedRef(), nullptr, nullptr);
			const UClass* ObjectClass = Handle->GetOuterBaseClass();
			if (Handle->IsValidHandle() && ExtensionHandler->IsPropertyExtendable(ObjectClass, *Handle))
			{
				IDetailLayoutBuilder& DetailLayout = OwnerTreeNodePinned->GetParentCategory()->GetParentLayout();
				ExtensionHandler->ExtendWidgetRow(WidgetRow, DetailLayout, ObjectClass, Handle);
			}
		}
	}
}

bool SDetailSingleItemRow::IsFavorite() const
{
	if (Customization->HasPropertyNode())
	{
		return Customization->GetPropertyNode()->IsFavorite();
	}

	if (Customization->HasCustomBuilder())
	{
		TSharedPtr<FDetailTreeNode> OwnerTreeNodePinned = OwnerTreeNode.Pin();
		if (OwnerTreeNodePinned.IsValid())
		{
			TSharedPtr<FDetailCategoryImpl> ParentCategory = OwnerTreeNodePinned->GetParentCategory();
			if (ParentCategory.IsValid() && ParentCategory->IsFavoriteCategory())
			{
				return true;
			}

			TSharedPtr<IPropertyHandle> PropertyHandle = Customization->CustomBuilderRow->GetPropertyHandle();
			if (PropertyHandle.IsValid())
			{
				return PropertyHandle->IsFavorite();
			}

			const FString& OriginalPath = Customization->CustomBuilderRow->GetOriginalPath();
			if (!OriginalPath.IsEmpty())
			{
				return OwnerTreeNodePinned->GetDetailsViewSharedPtr()->IsCustomBuilderFavorite(OriginalPath);
			}
		}
	}

	return false;
}

void SDetailSingleItemRow::CreateGlobalExtensionWidgets(TArray<FPropertyRowExtensionButton>& OutExtensions) const
{
	// fetch global extension widgets 
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FOnGenerateGlobalRowExtensionArgs Args;
	Args.OwnerTreeNode = OwnerTreeNode;

	if (Customization->HasPropertyNode())
	{
		Args.PropertyHandle = PropertyEditorHelpers::GetPropertyHandle(Customization->GetPropertyNode().ToSharedRef(), nullptr, nullptr);
	}
	else if (const TSharedPtr<IPropertyHandle> PrimaryPropertyHandle = GetPrimaryPropertyHandle();
		PrimaryPropertyHandle.IsValid() && PrimaryPropertyHandle->IsValidHandle())
	{
		Args.PropertyHandle = PrimaryPropertyHandle;
	}

	PropertyEditorModule.GetGlobalRowExtensionDelegate().Broadcast(Args, OutExtensions);
}

bool SDetailSingleItemRow::IsHighlighted() const
{
	TSharedPtr<FDetailTreeNode> OwnerTreeNodePtr = OwnerTreeNode.Pin();
	return OwnerTreeNodePtr.IsValid() ? OwnerTreeNodePtr->IsHighlighted() : false;
}

TSharedPtr<FDragDropOperation> SDetailSingleItemRow::CreateDragDropOperation()
{
	if (WidgetRow.CustomDragDropHandler)
	{
		TSharedPtr<FDragDropOperation> DragOp = WidgetRow.CustomDragDropHandler->CreateDragDropOperation();
		DragOperation = DragOp;
		return DragOp;
	}
	else
	{
		TSharedPtr<FArrayRowDragDropOp> ArrayDragOp = MakeShareable(new FArrayRowDragDropOp(SharedThis(this)));
		ArrayDragOp->Init();
		DragOperation = ArrayDragOp;
		return ArrayDragOp;
	}
}

void SDetailSingleItemRow::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	UpdateResetToDefault();
}

void SArrayRowHandle::Construct(const FArguments& InArgs)
{
	ParentRow = InArgs._ParentRow;

	ChildSlot
	[
		InArgs._Content.Widget
	];
}

FReply SArrayRowHandle::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		TSharedPtr<FDragDropOperation> DragDropOp = ParentRow.Pin()->CreateDragDropOperation();
		if (DragDropOp.IsValid())
		{
			return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
		}
	}

	return FReply::Unhandled();
}

FArrayRowDragDropOp::FArrayRowDragDropOp(TSharedPtr<SDetailSingleItemRow> InRow)
{
	check(InRow.IsValid());
	Row = InRow;
	MouseCursor = EMouseCursor::GrabHandClosed;
}

void FArrayRowDragDropOp::Init()
{
	SetValidTarget(false);
	SetupDefaults();
	Construct();
}

void FArrayRowDragDropOp::SetValidTarget(bool IsValidTarget)
{
	if (IsValidTarget)
	{
		CurrentHoverText = NSLOCTEXT("ArrayDragDrop", "PlaceRowHere", "Place Row Here");
		CurrentIconBrush = FAppStyle::GetBrush("Graph.ConnectorFeedback.OK");
	}
	else
	{
		CurrentHoverText = NSLOCTEXT("ArrayDragDrop", "CannotPlaceRowHere", "Cannot Place Row Here");
		CurrentIconBrush = FAppStyle::GetBrush("Graph.ConnectorFeedback.Error");
	}
}
