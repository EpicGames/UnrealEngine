// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DetailTreeNode.h"
#include "PropertyCustomizationHelpers.h"
#include "SDetailTableRowBase.h"
#include "SDetailsViewBase.h"
#include "ScopedTransaction.h"

#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Framework/Commands/Commands.h"
#include "Input/Reply.h"
#include "PropertyEditorClipboardPrivate.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableViewBase.h"

class IDetailKeyframeHandler;
struct FDetailLayoutCustomization;
class SDetailSingleItemRow;

class SArrayRowHandle : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SArrayRowHandle)
	{}
	SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_ARGUMENT(TSharedPtr<SDetailSingleItemRow>, ParentRow)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	};

	FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

private:
	TWeakPtr<SDetailSingleItemRow> ParentRow;
};

/**
 * A widget for details that span the entire tree row and have no columns                                                              
 */
class SDetailSingleItemRow : public SDetailTableRowBase
{
public:
	SLATE_BEGIN_ARGS( SDetailSingleItemRow )
		: _ColumnSizeData() {}
		SLATE_ARGUMENT( FDetailColumnSizeData, ColumnSizeData )
		SLATE_ARGUMENT( bool, AllowFavoriteSystem)
	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 */
	void Construct( const FArguments& InArgs, FDetailLayoutCustomization* InCustomization, bool bHasMultipleColumns, TSharedRef<FDetailTreeNode> InOwnerTreeNode, const TSharedRef<STableViewBase>& InOwnerTableView );

	//~ Begin SWidget
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget

	TSharedPtr<FDragDropOperation> CreateDragDropOperation();

protected:
	//~ Begin SDetailTableRowBase
	virtual void PopulateContextMenu(UToolMenu* ToolMenu) override;
	virtual TArray<TSharedPtr<IPropertyHandle>> GetPropertyHandles(const bool bRecursive = false) const override;
	virtual TSharedPtr<IPropertyHandle> GetPrimaryPropertyHandle() const override;
	virtual bool CanFavoriteRow(bool bInAddToFavorite) override;
	virtual void OnFavoriteRow(bool bInAddToFavorite) override;
	//~ End SDetailTableRowBase

private:
	void OnCopyProperty();
	void OnCopyPropertyDisplayName();
	bool CanCopyPropertyDisplayName();
	void OnCopyPropertyInternalName();
	bool CanCopyPropertyInternalName();
	bool CanSelectByProperty();
	void OnPasteProperty();
	bool CanPasteProperty() const;

	void OnCopyGroup();
	bool CanCopyGroup() const;
	void OnPasteGroup();
	bool CanPasteGroup();

	FString GetPropertyInternalName() const;
	FText GetPropertyDisplayName() const;

	/**
	 * Returns the TSharedRef<SWidget> for the Name column widget
	 * 
	 * @param NameWidget 
	 * @param Node 
	 * @return 
	 */
	TSharedRef<SWidget> GetNameWidget(TSharedRef<SWidget> NameWidget, const TSharedPtr<FPropertyNode>& Node) const;

	/**
	 * @return True if the (optionally tagged) input contents can be pasted
	 */
	bool CanPasteFromText(const FString& InTag, const FString& InText, const bool bIncludeEditCondition = true) const;

	/**
	 * Delegate handling pasting an optionally tagged text snippet
	 */
	void OnPasteFromText(const FString& InTag, const FString& InText, const TOptional<FGuid>& InOperationId, const bool bIncludeEditCondition);

	/**
	 * Handle pasting an optionally tagged text snippet
	 */
	bool PasteFromText(const FString& InTag, const FString& InText, const bool bIncludeEditCondition = true);
	
	FSlateColor GetOuterBackgroundColor() const;
	FSlateColor GetInnerBackgroundColor() const;

	void CreateGlobalExtensionWidgets(TArray<FPropertyRowExtensionButton>& ExtensionButtons) const;
	void PopulateExtensionWidget();

	void UpdateResetToDefault();
	void OnResetToDefaultClicked() const;
	bool IsResetToDefaultVisible() const;

	bool IsHighlighted() const;
	bool IsFavorite() const;
	
	/** UIActions to help populate the PropertyEditorPermissionList, which must first be turned on through FPropertyEditorPermissionList::Get().SetShouldShowMenuEntries */
	FString GetRowNameText() const;
	void CopyRowNameText() const;
	void OnToggleAllowList() const;
	bool IsAllowListChecked() const;
	void OnToggleDenyList() const;
	bool IsDenyListChecked() const;

	void OnArrayOrCustomDragLeave(const FDragDropEvent& DragDropEvent);
	FReply OnArrayAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FDetailTreeNode> TargetItem);
	FReply OnArrayHeaderAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FDetailTreeNode> TargetItem);

	TOptional<EItemDropZone> OnArrayCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr< FDetailTreeNode > Type);

	FReply OnCustomAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FDetailTreeNode> TargetItem);
	TOptional<EItemDropZone> OnCustomCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FDetailTreeNode> Type);

	/** Checks if the current drop event is being dropped into a valid location
	 */
	bool CheckValidDrop(const TSharedPtr<SDetailSingleItemRow> RowPtr, EItemDropZone DropZone) const;
	
	TSharedPtr<FPropertyNode> GetPropertyNode() const;
	TSharedPtr<IPropertyHandle> GetPropertyHandle() const;

	void SetPropertyColorationTarget(bool bSelectActors = false);
private:
	/** Customization for this widget */
	FDetailLayoutCustomization* Customization = nullptr;
	FDetailWidgetRow WidgetRow;
	bool bAllowFavoriteSystem = false;
	bool bCachedResetToDefaultVisible = false;
	TSharedPtr<FPropertyNode> SwappablePropertyNode;
	TSharedPtr<SButton> ExpanderArrow;
	TWeakPtr<FDragDropOperation> DragOperation; // last drag initiated by this widget
	FUIAction CopyAction;
	FUIAction PasteAction;

	/** Previously parsed clipboard data. */
	UE::PropertyEditor::Internal::FClipboardData PreviousClipboardData;

	/** Delegate handling pasting an optionally tagged text snippet */
	TSharedPtr<FOnPasteFromText> OnPasteFromTextDelegate;

	/** Delegate handling copying to a text snippet */
	TSharedPtr<FOnCopyToText> OnCopyToTextDelegate;

	/** Animation curve for displaying pulse */
	FCurveSequence PulseAnimation;
};

class FArrayRowDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FArrayRowDragDropOp, FDecoratedDragDropOp)

	FArrayRowDragDropOp(TSharedPtr<SDetailSingleItemRow> InRow);

	/** Inits the tooltip, needs to be called after constructing */
	void Init();

	/** Update the drag tool tip indicating whether the current drop target is valid */
	void SetValidTarget(bool IsValidTarget);

	TWeakPtr<SDetailSingleItemRow> Row;
};
