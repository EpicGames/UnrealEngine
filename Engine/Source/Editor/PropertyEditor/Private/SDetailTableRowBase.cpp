// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDetailTableRowBase.h"

#include "DetailRowMenuContext.h"
#include "DetailRowMenuContextPrivate.h"
#include "PropertyHandleImpl.h"
#include "ToolMenus.h"

const float SDetailTableRowBase::ScrollBarPadding = 16.0f;

FReply SDetailTableRowBase::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if( OwnerTreeNode.IsValid() && MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && !StaticCastSharedRef<STableViewBase>( OwnerTablePtr.Pin()->AsWidget() )->IsRightClickScrolling() )
	{
		FDetailNodeList VisibleChildren;
		OwnerTreeNode.Pin()->GetChildren( VisibleChildren );

		// Open context menu if this node can be expanded 
		bool bShouldOpenMenu = true;

		if( bShouldOpenMenu )
		{
			if (UToolMenus* ToolMenus = UToolMenus::Get())
			{
				if (UToolMenu* ToolMenu = ToolMenus->FindMenu(UE::PropertyEditor::RowContextMenuName))
				{
					const FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();

					UDetailRowMenuContext* RowMenuContext = NewObject<UDetailRowMenuContext>();					
					RowMenuContext->PropertyHandles = GetPropertyHandles(true);
					RowMenuContext->DetailsView = OwnerTreeNode.Pin()->GetDetailsViewSharedPtr();
					RowMenuContext->ForceRefreshWidget().AddSPLambda(this, [this]
					{
						ForceRefresh();						
					});

					UDetailRowMenuContextPrivate* RowMenuContextPrivate = NewObject<UDetailRowMenuContextPrivate>();
					RowMenuContextPrivate->Row = SharedThis(this);

					FToolMenuContext MenuContext;
					MenuContext.AddObject(RowMenuContext);
					MenuContext.AddObject(RowMenuContextPrivate);

					const TSharedRef<SWidget> ToolMenuWidget = ToolMenus->GenerateWidget(UE::PropertyEditor::RowContextMenuName, MenuContext);

					FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, ToolMenuWidget, MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect::ContextMenu);
				}
			}

			return FReply::Handled();
		}
	}

	return STableRow< TSharedPtr< FDetailTreeNode > >::OnMouseButtonUp( MyGeometry, MouseEvent );
}

int32 SDetailTableRowBase::GetIndentLevelForBackgroundColor() const
{
	int32 IndentLevel = 0; 
	if (OwnerTablePtr.IsValid())
	{
		// every item is in a category, but we don't want to show an indent for "top-level" properties
		IndentLevel = GetIndentLevel() - 1;
	}

	if (TSharedPtr<FDetailTreeNode> DetailTreeNode = OwnerTreeNode.Pin())
	{
		if (TSharedPtr<IDetailsViewPrivate> DetailsView = DetailTreeNode->GetDetailsViewSharedPtr())
		{
			if (DetailsView->ContainsMultipleTopLevelObjects())
			{
				// if the row is in a multiple top level object display (eg. Project Settings), don't display an indent for the initial level
				--IndentLevel;
			}
		}
	}

	return FMath::Max(0, IndentLevel);
}

bool SDetailTableRowBase::IsScrollBarVisible(TWeakPtr<STableViewBase> OwnerTableViewWeak)
{
	TSharedPtr<STableViewBase> OwnerTableView = OwnerTableViewWeak.Pin();
	if (OwnerTableView.IsValid())
	{
		return OwnerTableView->GetScrollbarVisibility() == EVisibility::Visible;
	}
	return false;
}

void SDetailTableRowBase::PopulateContextMenu(UToolMenu* ToolMenu)
{
	const TSharedPtr<FDetailTreeNode> DetailTreeNode = OwnerTreeNode.Pin();
	
	if (!DetailTreeNode)
	{
		return;
	}

	FToolMenuSection& ExpansionSection = ToolMenu->FindOrAddSection(TEXT("Expansion"));
	{
		FDetailNodeList VisibleChildren;
		DetailTreeNode->GetChildren( VisibleChildren );

		// Open context menu if this node can be expanded 
		if( VisibleChildren.Num() )
		{
			const FUIAction CollapseAllAction( FExecuteAction::CreateSP( this, &SDetailTableRowBase::OnCollapseAllClicked ) );
			ExpansionSection.AddMenuEntry(
				TEXT("CollapseAll"),
				NSLOCTEXT("PropertyView", "CollapseAll", "Collapse All"),
				NSLOCTEXT("PropertyView", "CollapseAll_ToolTip", "Collapses this item and all children"),
				FSlateIcon(),
				CollapseAllAction);
				
			const FUIAction ExpandAllAction( FExecuteAction::CreateSP( this, &SDetailTableRowBase::OnExpandAllClicked ) );
			ExpansionSection.AddMenuEntry(
				TEXT("ExpandAll"),
				 NSLOCTEXT("PropertyView", "ExpandAll", "Expand All"),
				NSLOCTEXT("PropertyView", "ExpandAll_ToolTip", "Expands this item and all children"),
				FSlateIcon(),
				ExpandAllAction);
		}
	}
	
	TSharedPtr<IDetailsViewPrivate> DetailsPanel = DetailTreeNode->GetDetailsViewSharedPtr();
	if (DetailsPanel && DetailsPanel->IsFavoritingEnabled())
	{
		FToolMenuSection& FavoritesSection = ToolMenu->FindOrAddSection(TEXT("Favorites"));

		bool bAddFavorites = true;
		FavoritesSection.AddMenuEntry(
			TEXT("FavoriteAll"),
			NSLOCTEXT("PropertyView", "FavoriteRow", "Add To Favorites"),
			NSLOCTEXT("PropertyView", "FavoriteRow_ToolTip", "Add this row to favorites"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "DetailsView.PropertyIsFavorite"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SDetailTableRowBase::OnFavoriteRow, bAddFavorites),
				FCanExecuteAction(),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP(this, &SDetailTableRowBase::CanFavoriteRow, bAddFavorites)
			)
		);
		
		bAddFavorites = false;
		FavoritesSection.AddMenuEntry(
			TEXT("UnFavoriteAll"),
			NSLOCTEXT("PropertyView", "UnFavoriteRow", "Remove From Favorites"),
			NSLOCTEXT("PropertyView", "UnFavoriteRow_ToolTip", "Remove this row from favorites"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "DetailsView.PropertyIsNotFavorite"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SDetailTableRowBase::OnFavoriteRow, bAddFavorites),
				FCanExecuteAction(),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP(this, &SDetailTableRowBase::CanFavoriteRow, bAddFavorites)
			)
		);
	}
}

TArray<TSharedPtr<FPropertyNode>> SDetailTableRowBase::GetPropertyNodes(const bool bRecursive) const
{
	TArray<TSharedPtr<IPropertyHandle>> PropertyHandles = GetPropertyHandles(bRecursive);
	return GetPropertyNodesFromHandles(PropertyHandles);
}

TArray<TSharedPtr<FPropertyNode>> SDetailTableRowBase::GetPropertyNodesFromHandles(const TConstArrayView<TSharedPtr<IPropertyHandle>>& InPropertyHandles) const
{
	TArray<TSharedPtr<FPropertyNode>> PropertyNodes;
	PropertyNodes.Reserve(InPropertyHandles.Num());
	
	for (const TSharedPtr<IPropertyHandle>& PropertyHandle : InPropertyHandles)
	{
		if (PropertyHandle->IsValidHandle())
		{
			PropertyNodes.Add(StaticCastSharedPtr<FPropertyHandleBase>(PropertyHandle)->GetPropertyNode());
		}
	}

	return PropertyNodes;
}

TArray<TSharedPtr<IPropertyHandle>> SDetailTableRowBase::GetPropertyHandles(const bool bRecursive) const
{
	TFunction<bool(const TSharedPtr<IDetailTreeNode>&, TArray<TSharedPtr<IPropertyHandle>>&)> AppendPropertyHandles;
	AppendPropertyHandles = [&AppendPropertyHandles, bRecursive]
		(const TSharedPtr<IDetailTreeNode>& InParent, TArray<TSharedPtr<IPropertyHandle>>& OutPropertyHandles)
	{
		// Parent in first call is actually parent of this row, so these are the nodes for this row 
		TArray<TSharedRef<IDetailTreeNode>> ChildNodes;
		InParent->GetChildren(ChildNodes, true);

		if (ChildNodes.IsEmpty() || !bRecursive)
		{
			return false;
		}

		OutPropertyHandles.Reserve(OutPropertyHandles.Num() + ChildNodes.Num());		
		for (const TSharedRef<IDetailTreeNode>& ChildNode : ChildNodes)
		{
			// @fixme: this won't return when there's multiple properties in a single row, or the row is custom 
			TSharedPtr<IPropertyHandle> ChildPropertyHandle = ChildNode->CreatePropertyHandle();
			if (ChildPropertyHandle.IsValid() && ChildPropertyHandle->IsValidHandle())
			{
				OutPropertyHandles.Add(ChildPropertyHandle);
			}
			AppendPropertyHandles(ChildNode, OutPropertyHandles);
		}

		return true;
	};

	TArray<TSharedPtr<IPropertyHandle>> PropertyHandles;
	if (const TSharedPtr<FDetailTreeNode> OwnerTreeNodePtr = OwnerTreeNode.Pin())
	{
		AppendPropertyHandles(OwnerTreeNodePtr, PropertyHandles);
	}

	PropertyHandles.Remove(nullptr);

	return PropertyHandles;
}

TSharedPtr<IPropertyHandle> SDetailTableRowBase::GetPrimaryPropertyHandle() const
{
	if (const TSharedPtr<FDetailTreeNode> OwnerTreeNodePtr = OwnerTreeNode.Pin())
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = OwnerTreeNodePtr->CreatePropertyHandle();
		if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
		{
			return PropertyHandle;
		}

		// If the primary property handle can't be retrieved from the associated node, try to get it from the children
		// This is less than ideal - the primary property handle should always be available from the node, except for category headers, etc.
		const TArray<TSharedPtr<IPropertyHandle>> PropertyHandles = GetPropertyHandles(false);
		if (!PropertyHandles.IsEmpty())
		{
			return PropertyHandles[0];
		}
	}

	return nullptr;
}

void SDetailTableRowBase::ForceRefresh()
{
	if (const TSharedPtr<FDetailTreeNode> Owner = OwnerTreeNode.Pin();
		Owner.IsValid())
	{
		if (TSharedPtr<IDetailsViewPrivate> DetailsView = Owner->GetDetailsViewSharedPtr())
		{
			DetailsView->ForceRefresh();
		}
	}
}

void SDetailTableRowBase::OnFavoriteRow(bool bInAddToFavorite)
{
	
}

bool SDetailTableRowBase::CanFavoriteRow(bool bInAddToFavorite)
{
	return false;
}
