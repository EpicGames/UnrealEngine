// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailTreeNode.h"
#include "IDetailsViewPrivate.h"
#include "InputCoreTypes.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Input/Reply.h"
#include "Layout/WidgetPath.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

class UToolMenu;
class UDetailRowMenuContext;

class SDetailTableRowBase : public STableRow< TSharedPtr< FDetailTreeNode > >
{
public:
	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

	int32 GetIndentLevelForBackgroundColor() const;

	static bool IsScrollBarVisible(TWeakPtr<STableViewBase> OwnerTableViewWeak);
	static const float ScrollBarPadding;

	/** Called to register/extend the row context menu. */
	virtual void PopulateContextMenu(UToolMenu* ToolMenu);

protected:

	/**
	 * The FPropertyUpdatedWidgetBuilder which builds a Property Updated widget which can be used in place of the
	 * reset to default button
	 */
	TSharedPtr<FPropertyUpdatedWidgetBuilder> PropertyUpdatedWidgetBuilder;

	/** Retrieve all property nodes represented by this row, and it's children if Recursive specified. */
	TArray<TSharedPtr<FPropertyNode>> GetPropertyNodes(const bool bRecursive = false) const;
	
	/** Retrieve all property nodes represented by the given property handles. */
	TArray<TSharedPtr<FPropertyNode>> GetPropertyNodesFromHandles(const TConstArrayView<TSharedPtr<IPropertyHandle>>& InPropertyHandles) const;

	/**
	 * Retrieve all property handles represented by this row, and it's children if Recursive specified.
	 * @note: This only returns the visible property handles, not the "primary" one (although often these are the same),
	 * for example FVector will give you the X, Y and Z handles, not the FVector itself.
	 * For the "primary" property handle, use GetPrimaryPropertyHandle.
	 */
	virtual TArray<TSharedPtr<IPropertyHandle>> GetPropertyHandles(const bool bRecursive = false) const;

	/** Retrieve the primary property handle for this row, ie. an FVector should return the Struct itself, not it's X, Y, Z properties. */
	virtual TSharedPtr<IPropertyHandle> GetPrimaryPropertyHandle() const;

	/** Refreshes widget and associated list views. */
	virtual void ForceRefresh();
	
	/** Called when row favoriting is toggled */
	virtual void OnFavoriteRow(bool bInAddToFavorite);
	
	/** Is row favoriting allowed */
	virtual bool CanFavoriteRow(bool bInAddToFavorite);

private:
	void OnExpandAllClicked()
	{
		TSharedPtr<FDetailTreeNode> OwnerTreeNodePin = OwnerTreeNode.Pin();
		if( OwnerTreeNodePin.IsValid() )
		{
			const bool bRecursive = true;
			const bool bIsExpanded = true;
			OwnerTreeNodePin->GetDetailsViewSharedPtr()->SetNodeExpansionState( OwnerTreeNodePin.ToSharedRef(), bIsExpanded, bRecursive );
		}
	}

	void OnCollapseAllClicked()
	{
		TSharedPtr<FDetailTreeNode> OwnerTreeNodePin = OwnerTreeNode.Pin();
		if( OwnerTreeNodePin.IsValid() )
		{
			const bool bRecursive = true;
			const bool bIsExpanded = false;
			OwnerTreeNodePin->GetDetailsViewSharedPtr()->SetNodeExpansionState( OwnerTreeNodePin.ToSharedRef(), bIsExpanded, bRecursive );
		}
	}
	
protected:
	/**
	 * A weak pointer to the STableViewBase from which we can get information such as if the scrollbar is showing. 
	 */
	TWeakPtr<STableViewBase> OwnerTableViewWeak;
	TWeakPtr<FDetailTreeNode> OwnerTreeNode;
	
	/**
	 * The @code DetailsDisplayManager @endcode which provides an API to manage some of the characteristics of the
	 * details display
	 */
	TSharedPtr<FDetailsDisplayManager> DisplayManager;
};
