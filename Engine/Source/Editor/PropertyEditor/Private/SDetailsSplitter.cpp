// Copyright Epic Games, Inc. All Rights Reserved.
#include "SDetailsSplitter.h"

#include "AsyncDetailViewDiff.h"
#include "DetailTreeNode.h"
#include "Editor.h"
#include "IDetailsViewPrivate.h"
#include "PropertyNode.h"
#include "Styling/StyleColors.h"
#include "Framework/Application/SlateApplication.h"
#include "Serialization/ObjectWriter.h"

#define LOCTEXT_NAMESPACE "DetailsSplitter"

namespace DetailsSplitterHelpers
{
	const TArray<TWeakObjectPtr<UObject>>* GetObjects(const TSharedPtr<FDetailTreeNode>& TreeNode)
	{
		if (TSharedPtr<const IDetailsViewPrivate> DetailsView = TreeNode->GetDetailsViewSharedPtr())
		{
			return &DetailsView->GetSelectedObjects();
		}
		return nullptr;
	}

	// converts from a container helper like FScriptArrayHelper to a property like FArrayProperty
	template<typename HelperType>
	using TContainerPropertyType =
		std::conditional_t<std::is_same_v<HelperType, FScriptArrayHelper>, FArrayProperty,
		std::conditional_t<std::is_same_v<HelperType, FScriptMapHelper>, FMapProperty,
		std::conditional_t<std::is_same_v<HelperType, FScriptSetHelper>, FSetProperty,
		void>>>;

	template<typename ContainerPropertyType>
	bool TryGetSourceContainer(TSharedPtr<FDetailTreeNode> DetailsNode, FPropertyNode*& OutPropNode)
	{
		if ((OutPropNode = DetailsNode->GetPropertyNode().Get()) == nullptr)
		{
			return false;
		}
		if ((OutPropNode = OutPropNode->GetParentNode()) == nullptr)
		{
			return false;
		}
		if (CastField<ContainerPropertyType>(OutPropNode->GetProperty()) != nullptr)
		{
			return true;
		}
		return false;
	}

	template<typename ContainerPropertyType>
	bool TryGetDestinationContainer(TSharedPtr<FDetailTreeNode> DetailsNode, FPropertyNode*& OutPropNode, int32& OutInsertIndex)
	{
		if ((OutPropNode = DetailsNode->GetPropertyNode().Get()) == nullptr)
		{
			return false;
		}
		if (CastField<ContainerPropertyType>(OutPropNode->GetProperty()) != nullptr)
		{
			OutInsertIndex = 0;
			return true;
		}
		OutInsertIndex = OutPropNode->GetArrayIndex() + 1;
		while((OutPropNode = OutPropNode->GetParentNode()) != nullptr)
		{
			if (CastField<ContainerPropertyType>(OutPropNode->GetProperty()) != nullptr)
			{
				return true;
			}
			OutInsertIndex = OutPropNode->GetArrayIndex() + 1;
		}
		return false;
	}

	template<typename HelperType>
	bool TryGetSourceContainer(TSharedPtr<FDetailTreeNode> DetailsNode, FPropertyNode*& OutPropNode, TArray<TUniquePtr<HelperType>>& OutContainerHelper)
	{
		using ContainerPropertyType = TContainerPropertyType<HelperType>;
		if (TryGetSourceContainer<ContainerPropertyType>(DetailsNode, OutPropNode))
		{
			const FPropertySoftPath SoftPropertyPath(FPropertyNode::CreatePropertyPath(OutPropNode->AsShared()).Get());
			if (const TArray<TWeakObjectPtr<UObject>>* Objects = DetailsSplitterHelpers::GetObjects(DetailsNode))
			{
				for (const TWeakObjectPtr<UObject> WeakObject : *Objects)
				{
					if (const UObject* Object = WeakObject.Get())
					{
						const FResolvedProperty Resolved = SoftPropertyPath.Resolve(Object);
						const ContainerPropertyType* ContainerProperty = CastFieldChecked<ContainerPropertyType>(Resolved.Property);
						OutContainerHelper.Add(MakeUnique<HelperType>(ContainerProperty, ContainerProperty->template ContainerPtrToValuePtr<UObject*>(Resolved.Object)));
					}
				}
			}
			return true;
		}
		return false;
	}

	template<typename HelperType>
	bool TryGetDestinationContainer(TSharedPtr<FDetailTreeNode> DetailsNode, FPropertyNode*& OutPropNode, TArray<TUniquePtr<HelperType>>& OutContainerHelper, int32& OutInsertIndex)
	{
		using ContainerPropertyType = TContainerPropertyType<HelperType>;
		if (TryGetDestinationContainer<ContainerPropertyType>(DetailsNode, OutPropNode, OutInsertIndex))
		{
			const FPropertySoftPath SoftPropertyPath(FPropertyNode::CreatePropertyPath(OutPropNode->AsShared()).Get());
			if (const TArray<TWeakObjectPtr<UObject>>* Objects = DetailsSplitterHelpers::GetObjects(DetailsNode))
			{
				for (const TWeakObjectPtr<UObject> WeakObject : *Objects)
				{
					if (const UObject* Object = WeakObject.Get())
					{
						const FResolvedProperty Resolved = SoftPropertyPath.Resolve(Object);
						const ContainerPropertyType* ContainerProperty = CastFieldChecked<ContainerPropertyType>(Resolved.Property);
						OutContainerHelper.Add(MakeUnique<HelperType>(ContainerProperty, ContainerProperty->template ContainerPtrToValuePtr<UObject*>(Resolved.Object)));
					}
				}
			}
			return true;
		}
		return false;
	}

	void CopyPropertyValueForInsert(const TSharedPtr<FDetailTreeNode>& SourceDetailsNode, const TSharedPtr<FDetailTreeNode>& DestinationDetailsNode)
	{
		const TSharedPtr<IPropertyHandle> DestinationHandle = DestinationDetailsNode->CreatePropertyHandle();
		const TSharedPtr<IPropertyHandle> SourceHandle = SourceDetailsNode->CreatePropertyHandle();

		// Array
		TArray<TUniquePtr<FScriptArrayHelper>> SourceArrays;
		FPropertyNode* SourceArrayPropertyNode;
		if (TryGetSourceContainer(SourceDetailsNode, SourceArrayPropertyNode, SourceArrays))
		{
			int32 InsertIndex;
			TArray<TUniquePtr<FScriptArrayHelper>> DestinationArrays;
			FPropertyNode* DestinationArrayPropertyNode;
			if (ensure(TryGetDestinationContainer(DestinationDetailsNode, DestinationArrayPropertyNode, DestinationArrays, InsertIndex)))
			{
				ensure(SourceArrays.Num() == DestinationArrays.Num());
				
				GEditor->BeginTransaction(TEXT("DetailsSplitter"), FText::Format(LOCTEXT("InsertPropertyValueTransaction","Insert {0}"), SourceHandle->GetPropertyDisplayName()), nullptr);
				DestinationHandle->NotifyPreChange();
				for (int32 ArrayNum = 0; ArrayNum < SourceArrays.Num(); ++ ArrayNum)
				{
					DestinationArrays[ArrayNum]->InsertValues(InsertIndex, 1);
					const void* SourceData = SourceArrays[ArrayNum]->GetElementPtr(SourceDetailsNode->GetPropertyNode()->GetArrayIndex());
					void* DestinationData = DestinationArrays[ArrayNum]->GetElementPtr(InsertIndex);
					const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(DestinationArrayPropertyNode->GetProperty());
					const FProperty* ElementProperty = ArrayProperty->Inner;
					ElementProperty->CopySingleValue(DestinationData, SourceData);
				}
				
				DestinationHandle->NotifyPostChange(EPropertyChangeType::ArrayAdd);
				DestinationHandle->NotifyFinishedChangingProperties();
				GEditor->EndTransaction();
			}
			return;
		}
		
		// Set
		TArray<TUniquePtr<FScriptSetHelper>> SourceSets;
		FPropertyNode* SourceSetPropertyNode;
		if (TryGetSourceContainer(SourceDetailsNode, SourceSetPropertyNode, SourceSets))
		{
			int32 InsertIndex;
			TArray<TUniquePtr<FScriptSetHelper>> DestinationSets;
			FPropertyNode* DestinationPropertyNode;
			if (ensure(TryGetDestinationContainer(DestinationDetailsNode, DestinationPropertyNode, DestinationSets, InsertIndex)))
			{
				ensure(SourceSets.Num() == DestinationSets.Num());
				GEditor->BeginTransaction(TEXT("DetailsSplitter"), FText::Format(LOCTEXT("InsertPropertyValueTransaction","Insert {0}"), SourceHandle->GetPropertyDisplayName()), nullptr);
				DestinationHandle->NotifyPreChange();
				for (int32 SetNum = 0; SetNum < SourceSets.Num(); ++ SetNum)
				{
					const void* SourceData = SourceSets[SetNum]->FindNthElementPtr(SourceDetailsNode->GetPropertyNode()->GetArrayIndex());
					DestinationSets[SetNum]->AddElement(SourceData);
				}
				
				DestinationHandle->NotifyPostChange(EPropertyChangeType::ArrayAdd);
				DestinationHandle->NotifyFinishedChangingProperties();
				GEditor->EndTransaction();
			}
			return;
		}
		
		// Map
		TArray<TUniquePtr<FScriptMapHelper>> SourceMaps;
		FPropertyNode* SourceMapPropertyNode;
		if (TryGetSourceContainer(SourceDetailsNode, SourceMapPropertyNode, SourceMaps))
		{
			int32 InsertIndex;
			TArray<TUniquePtr<FScriptMapHelper>> DestinationMaps;
			FPropertyNode* DestinationPropertyNode;
			if (ensure(TryGetDestinationContainer(DestinationDetailsNode, DestinationPropertyNode, DestinationMaps, InsertIndex)))
			{
				ensure(SourceMaps.Num() == DestinationMaps.Num());
				GEditor->BeginTransaction(TEXT("DetailsSplitter"), FText::Format(LOCTEXT("InsertPropertyValueTransaction","Insert {0}"), SourceHandle->GetPropertyDisplayName()), nullptr);
				DestinationHandle->NotifyPreChange();
				for (int32 MapNum = 0; MapNum < SourceMaps.Num(); ++ MapNum)
				{
					const int32 Index = SourceMaps[MapNum]->FindInternalIndex(SourceDetailsNode->GetPropertyNode()->GetArrayIndex());
					const void* SourceKey = SourceMaps[MapNum]->GetKeyPtr(Index);
					const void* SourceVal = SourceMaps[MapNum]->GetValuePtr(Index);
					DestinationMaps[MapNum]->AddPair(SourceKey, SourceVal);
				}
				
				DestinationHandle->NotifyPostChange(EPropertyChangeType::ArrayAdd);
				DestinationHandle->NotifyFinishedChangingProperties();
				GEditor->EndTransaction();
			}
			return;
		}
	}

	void AssignPropertyValue(const TSharedPtr<IPropertyHandle>& SourceHandle, const TSharedPtr<IPropertyHandle>& DestinationHandle)
	{
		TArray<FString> SourceValues;
		SourceHandle->GetPerObjectValues(SourceValues);
		DestinationHandle->SetPerObjectValues(SourceValues);
	}

	void CopyPropertyValue(const TSharedPtr<FDetailTreeNode>& SourceDetailsNode, const TSharedPtr<FDetailTreeNode>& DestinationDetailsNode, ETreeDiffResult Diff)
	{
		switch(Diff)
		{
			// traditional copy
			case ETreeDiffResult::DifferentValues:
				break;

			// insert
			case ETreeDiffResult::MissingFromTree1:
			case ETreeDiffResult::MissingFromTree2:
				CopyPropertyValueForInsert(SourceDetailsNode, DestinationDetailsNode);
				return;
			
			// no difference
			case ETreeDiffResult::Invalid:
			case ETreeDiffResult::Identical:
			default:
				return;
		}


		const TSharedPtr<IPropertyHandle> SourceHandle = SourceDetailsNode->CreatePropertyHandle();
		const TSharedPtr<IPropertyHandle> DestinationHandle = DestinationDetailsNode->CreatePropertyHandle();
		if (!ensure(SourceHandle && DestinationHandle))
		{
			return;
		}
		
		if (!SourceHandle->GetProperty()->SameType(DestinationHandle->GetProperty()))
		{
			// convert types by assigning via text serialization
			AssignPropertyValue(SourceHandle, DestinationHandle);
			return;
		}

		TArray<void*> SourceData;
		TArray<void*> DestinationData;
		SourceHandle->AccessRawData(SourceData);
		DestinationHandle->AccessRawData(DestinationData);
		if (!ensure(SourceData.Num() == DestinationData.Num()))
		{
			return;
		}

		GEditor->BeginTransaction(TEXT("DetailsSplitter"), FText::Format(LOCTEXT("CopyPropertyValueTransaction","Copy {0}"), SourceHandle->GetPropertyDisplayName()), nullptr);
		DestinationHandle->NotifyPreChange();
		for (int32 I = 0; I < SourceData.Num(); ++I)
		{
			if (DestinationHandle->GetArrayIndex() != INDEX_NONE)
			{
				DestinationHandle->GetProperty()->CopySingleValue(DestinationData[I], SourceData[I]);
			}
			else
			{
				DestinationHandle->GetProperty()->CopyCompleteValue(DestinationData[I], SourceData[I]);
			}
		}
		DestinationHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		DestinationHandle->NotifyFinishedChangingProperties();
		GEditor->EndTransaction();
	}

	// note: DestinationDetailsNode is the node before the position in the tree you wish to insert
	bool CanCopyPropertyValueForInsert(const TSharedPtr<FDetailTreeNode>& SourceDetailsNode, const TSharedPtr<FDetailTreeNode>& DestinationDetailsNode)
	{
		FPropertyNode* SourceArrayPropertyNode;
		if (TryGetSourceContainer<FArrayProperty>(SourceDetailsNode, SourceArrayPropertyNode))
		{
			FPropertyNode* DestinationArrayPropertyNode;
			int32 InsertIndex;
			if (TryGetDestinationContainer<FArrayProperty>(DestinationDetailsNode, DestinationArrayPropertyNode, InsertIndex))
			{
				return true;
			}
			return false;
		}
		
		FPropertyNode* SourceSetPropertyNode;
		if (TryGetSourceContainer<FSetProperty>(SourceDetailsNode, SourceSetPropertyNode))
		{
			FPropertyNode* DestinationSetPropertyNode;
			int32 InsertIndex;
			if (TryGetDestinationContainer<FSetProperty>(DestinationDetailsNode, DestinationSetPropertyNode, InsertIndex))
			{
				return true;
			}
			return false;
		}
		
		FPropertyNode* SourceMapPropertyNode;
		if (TryGetSourceContainer<FMapProperty>(SourceDetailsNode, SourceMapPropertyNode))
		{
			FPropertyNode* DestinationMapPropertyNode;
			int32 InsertIndex;
			if (TryGetDestinationContainer<FMapProperty>(DestinationDetailsNode, DestinationMapPropertyNode, InsertIndex))
			{
				return true;
			}
			return false;
		}

		// you can only insert into containers
		return false;
	}

	bool CanAssignPropertyValue(const TSharedPtr<IPropertyHandle>& SourceHandle, const TSharedPtr<IPropertyHandle>& DestinationHandle)
	{
		TArray<FString> SourceValues;
		TArray<FString> OldDestinationValues;
		SourceHandle->GetPerObjectValues(SourceValues);
		DestinationHandle->GetPerObjectValues(OldDestinationValues);
		
		FPropertyAccess::Result Result = DestinationHandle->SetPerObjectValues(SourceValues, EPropertyValueSetFlags::NotTransactable);
		TArray<FString> ChangedDestinationValues;
		DestinationHandle->GetPerObjectValues(ChangedDestinationValues);

		// revert changes and query whether anything changed
		DestinationHandle->SetPerObjectValues(OldDestinationValues, EPropertyValueSetFlags::NotTransactable);
		
		return Result != FPropertyAccess::Fail && OldDestinationValues != ChangedDestinationValues;
	}

	bool CanCopyPropertyValue(const TSharedPtr<FDetailTreeNode>& SourceDetailsNode, const TSharedPtr<FDetailTreeNode>& DestinationDetailsNode, ETreeDiffResult Diff)
	{
		if (!SourceDetailsNode || !DestinationDetailsNode)
		{
			return false;
		}
		
		// in order to copy properties there needs to be the same number of objects in each panel
		const TArray<TWeakObjectPtr<UObject>>* SourceObjects = DetailsSplitterHelpers::GetObjects(SourceDetailsNode);
		const TArray<TWeakObjectPtr<UObject>>* DestinationObjects = DetailsSplitterHelpers::GetObjects(DestinationDetailsNode);
		if (!SourceObjects || !DestinationObjects || SourceObjects->Num() != DestinationObjects->Num())
		{
			return false;
		}
		
		switch (Diff)
		{
		// traditional copy
		case ETreeDiffResult::DifferentValues:
		{
			const TSharedPtr<IPropertyHandle> SourceHandle = SourceDetailsNode->CreatePropertyHandle();
			const TSharedPtr<IPropertyHandle> DestinationHandle = DestinationDetailsNode->CreatePropertyHandle();
			if (SourceHandle && DestinationHandle && SourceHandle->GetProperty() && DestinationHandle->GetProperty())
			{
				TArray<void*> SourceData;
				TArray<void*> DestinationData;
				SourceHandle->AccessRawData(SourceData);
				DestinationHandle->AccessRawData(DestinationData);
				if (SourceData == DestinationData && SourceHandle->GetProperty() == DestinationHandle->GetProperty())
                {
                	// disable copying a value to itself since it's a no-op
                	return false;
                }
				if (SourceHandle->GetProperty()->SameType(DestinationHandle->GetProperty()))
				{
					return true;
				}
				if (CanAssignPropertyValue(SourceHandle, DestinationHandle))
				{
					return true;
				}
			}
			return false;
		}

		// insert
		case ETreeDiffResult::MissingFromTree1:
		case ETreeDiffResult::MissingFromTree2:
			return CanCopyPropertyValueForInsert(SourceDetailsNode, DestinationDetailsNode);

		// no difference
		case ETreeDiffResult::Invalid:
		case ETreeDiffResult::Identical:
		default:
			return false;
		}
	}
	
}

void SDetailsSplitter::Construct(const FArguments& InArgs)
{
	Splitter = SNew(SSplitter).PhysicalSplitterHandleSize(5.f);
	
	GetRowHighlightColor = InArgs._RowHighlightColor.IsBound() ? InArgs._RowHighlightColor : FRowHighlightColor::CreateStatic(
		[](const TUniquePtr<FAsyncDetailViewDiff::DiffNodeType>&)
		{
			return FLinearColor(0.f, 1.f, 1.f, .7f);
		});

	GetShouldHighlightRow = InArgs._ShouldHighlightRow.IsBound() ? InArgs._ShouldHighlightRow : FShouldHighlightRow::CreateStatic(
		[](const TUniquePtr<FAsyncDetailViewDiff::DiffNodeType>& DiffNode)
		{
			return DiffNode->DiffResult != ETreeDiffResult::Identical;
		});
	
	for(const FSlot::FSlotArguments& SlotArgs : InArgs._Slots)
	{
		AddSlot(SlotArgs);
	}
	
	ChildSlot
	[
		Splitter.ToSharedRef()
	];
}

void SDetailsSplitter::AddSlot(const FSlot::FSlotArguments& SlotArgs, int32 Index)
{
	if (Index == INDEX_NONE)
	{
		Index = Panels.Num();
	}
	
	Splitter->AddSlot(Index)
		.Value(SlotArgs._Value)
	[
		SNew(SBox).Padding(15.f,0.f, 15.f,0.f)
		[
			SlotArgs._DetailsView ? SlotArgs._DetailsView.ToSharedRef() : SNullWidget::NullWidget
		]
	];
	Panels.Insert({
		SlotArgs._DetailsView,
		SlotArgs._IsReadonly,
		SlotArgs._DifferencesWithRightPanel,
		SlotArgs._ShouldIgnoreRow.IsBound() ? SlotArgs._ShouldIgnoreRow : FShouldIgnoreRow::CreateStatic([](const TWeakPtr<FDetailTreeNode>&){return false;})
	}, Index);
}

SDetailsSplitter::FPanel& SDetailsSplitter::GetPanel(int32 Index)
{
	return Panels[Index];
}

// find the inner-most object in the property path and shorten the path relative to that object
static void ShortenToPathFromLastObject(const UObject*& InOutObject, FPropertyPath& InOutPath)
{
	TArray<FPropertyInfo> PathFromSubObjectReversed;
	PathFromSubObjectReversed.Add(InOutPath.GetLeafMostProperty());
	InOutPath = *InOutPath.TrimPath(1);
	while (InOutPath.IsValid())
	{
		// Note that if we have perf issues, this could be made faster by writing a custom resolve function that stops
		// at the last FObjectProperty. that way we wouldn't need to resolve multiple times
		const FResolvedProperty Resolved = FPropertySoftPath(InOutPath).Resolve(InOutObject);
		if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Resolved.Property))
		{
			InOutObject = ObjectProperty->GetObjectPropertyValue(ObjectProperty->ContainerPtrToValuePtr<void>(Resolved.Object));
			break;
		}
		PathFromSubObjectReversed.Add(InOutPath.GetLeafMostProperty());
		InOutPath = *InOutPath.TrimPath(1);
	}

	InOutPath = {};
	while(!PathFromSubObjectReversed.IsEmpty())
	{
		InOutPath.AddProperty(PathFromSubObjectReversed.Pop());
	}
}

void SDetailsSplitter::HighlightFromMergeResults(const TMap<FString, TMap<FPropertySoftPath, ETreeDiffResult>>& CustomHighlights)
{
	GetRowHighlightColor = FRowHighlightColor::CreateLambda([CustomHighlights](const TUniquePtr<FAsyncDetailViewDiff::DiffNodeType>& DiffNode)
	{
		const TSharedPtr<FDetailTreeNode> DetailNode = DiffNode->ValueA.IsValid() ? DiffNode->ValueA.Pin() : DiffNode->ValueB.Pin();
		
		FPropertyPath Path = DetailNode->GetPropertyPath();
		const UObject* OwningObject = DetailNode->GetDetailsViewSharedPtr()->GetSelectedObjects()[0].Get();
		if (OwningObject && Path.IsValid())
		{
			FPropertyPath PathFromSubObject;
			ShortenToPathFromLastObject(OwningObject, Path);
			if (OwningObject)
			{
				if (const TMap<FPropertySoftPath, ETreeDiffResult>* Highlights = CustomHighlights.Find(OwningObject->GetPathName(OwningObject->GetPackage())))
				{
					if (const ETreeDiffResult* DiffResult = Highlights->Find(FPropertySoftPath(Path)))
					{
						switch(*DiffResult)
						{
						case ETreeDiffResult::MissingFromTree1: // fall through
						case ETreeDiffResult::MissingFromTree2: // fall through
						case ETreeDiffResult::DifferentValues:
							// color is intentionally using values greater than 1 so that it stays very saturated
							return FLinearColor(1.5f, 0.3f, 0.3f);
						
						default:; // ignore identical and invalid
						}
					}
				}
			}
		}
		
		return FLinearColor(0.f, 1.f, 1.f, .7f);
	});
}

void SDetailsSplitter::SetRowHighlightColorDelegate(const FRowHighlightColor& Delegate)
{
	GetRowHighlightColor = Delegate;
}

SDetailsSplitter::FSlot::FSlotArguments SDetailsSplitter::Slot()
{
	return FSlot::FSlotArguments(MakeUnique<FSlot>());
}

int32 SDetailsSplitter::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry,
                                const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId,
                                const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	int32 MaxLayerId = LayerId;
	MaxLayerId = Splitter->Paint(Args,AllottedGeometry,MyCullingRect,OutDrawElements,MaxLayerId,InWidgetStyle,bParentEnabled);
	++MaxLayerId;

	TMap<FSlateRect, FLinearColor> RowHighlights;
	for (int32 LeftIndex = 0; LeftIndex < Panels.Num(); ++LeftIndex)
	{
		const FPanel& LeftPanel = Panels[LeftIndex];
		if (!LeftPanel.DiffRight.IsSet())
		{
			continue;
		}
		
		if (const TSharedPtr<FAsyncDetailViewDiff> Diff = LeftPanel.DiffRight.Get())
		{
			FSlateRect PrevLeftPropertyRect;
			FSlateRect PrevRightPropertyRect;
			
			TSharedPtr<FDetailTreeNode> LastSeenRightDetailsNode;
			TSharedPtr<FDetailTreeNode> LastSeenLeftDetailsNode;
			
			Diff->ForEachRow([&](const TUniquePtr<FAsyncDetailViewDiff::DiffNodeType>& DiffNode, int32, int32)->ETreeTraverseControl
			{
				const bool bShouldHighlightRow = GetShouldHighlightRow.Execute(DiffNode);
				const FLinearColor Color = GetRowHighlightColor.Execute(DiffNode);
				
				FSlateRect LeftPropertyRect;
				if (const TSharedPtr<FDetailTreeNode> LeftDetailNode = DiffNode->ValueA.Pin())
				{
					LastSeenLeftDetailsNode = LeftDetailNode;
					// if the other tree doesn't have a matching node, treat this and all it's children as a single group
					const bool bIncludeChildren = !DiffNode->ValueB.IsValid();
					LeftPropertyRect = LeftPanel.DetailsView->GetPaintSpacePropertyBounds(LeftDetailNode.ToSharedRef(), bIncludeChildren);
				}
				if (!LeftPropertyRect.IsValid() && PrevLeftPropertyRect.IsValid())
				{
					LeftPropertyRect = PrevLeftPropertyRect;
					LeftPropertyRect.Top = LeftPropertyRect.Bottom;
				}
				if (LeftPropertyRect.IsValid() && bShouldHighlightRow)
				{
					if (!LeftPanel.ShouldIgnoreRow.Execute(DiffNode->ValueA))
					{
						RowHighlights.Add(LeftPropertyRect, Color);
					}
				}
		
				const int32 RightIndex = LeftIndex + 1;
				FSlateRect RightPropertyRect;
				if (Panels.IsValidIndex(RightIndex))
				{
					const FPanel& RightPanel = Panels[RightIndex];
					if (const TSharedPtr<FDetailTreeNode> RightDetailNode = DiffNode->ValueB.Pin())
					{
						LastSeenRightDetailsNode = RightDetailNode;
						// if the other tree doesn't have a matching node, treat this and all it's children as a single group
						const bool bIncludeChildren = !DiffNode->ValueA.IsValid();
						RightPropertyRect = RightPanel.DetailsView->GetPaintSpacePropertyBounds(RightDetailNode.ToSharedRef(), bIncludeChildren);
					}
					if (!RightPropertyRect.IsValid() && PrevRightPropertyRect.IsValid())
					{
						RightPropertyRect = PrevRightPropertyRect;
						RightPropertyRect.Top = RightPropertyRect.Bottom;
					}
					if (RightPropertyRect.IsValid() && bShouldHighlightRow)
					{
						if (!RightPanel.ShouldIgnoreRow.Execute(DiffNode->ValueB))
						{
							RowHighlights.Add(RightPropertyRect, Color);
						}
					}

					if (LeftPropertyRect.IsValid() && RightPropertyRect.IsValid() && bShouldHighlightRow)
					{
						if (!LeftPanel.ShouldIgnoreRow.Execute(DiffNode->ValueA) && !RightPanel.ShouldIgnoreRow.Execute(DiffNode->ValueB))
						{
							FLinearColor FillColor = Color.Desaturate(.3f) * FLinearColor(0.053f,0.053f,0.053f);
							FillColor.A = 0.43f;
			
							const FLinearColor OutlineColor = Color;
							PaintPropertyConnector(OutDrawElements, MaxLayerId, LeftPropertyRect, RightPropertyRect, FillColor, OutlineColor);
							++MaxLayerId;

							if (!RightPanel.IsReadonly.Get(true) && DetailsSplitterHelpers::CanCopyPropertyValue(LastSeenLeftDetailsNode, LastSeenRightDetailsNode, DiffNode->DiffResult))
							{
								PaintCopyPropertyButton(OutDrawElements, MaxLayerId, DiffNode, LeftPropertyRect, RightPropertyRect, EPropertyCopyDirection::CopyLeftToRight);
							}
							if (!LeftPanel.IsReadonly.Get(true) && DetailsSplitterHelpers::CanCopyPropertyValue(LastSeenRightDetailsNode, LastSeenLeftDetailsNode, DiffNode->DiffResult))
							{
								PaintCopyPropertyButton(OutDrawElements, MaxLayerId, DiffNode, LeftPropertyRect, RightPropertyRect, EPropertyCopyDirection::CopyRightToLeft);
							}
							++MaxLayerId;
						}
						
					}
				}
		
				PrevLeftPropertyRect = MoveTemp(LeftPropertyRect);
				PrevRightPropertyRect = MoveTemp(RightPropertyRect);

				// only traverse children if both trees have them
				if (!DiffNode->ValueA.IsValid() || !DiffNode->ValueB.IsValid())
				{
					return ETreeTraverseControl::SkipChildren;
				}
				return ETreeTraverseControl::Continue;
				
			});
		}
	}

	for (const auto& [PropertyRect, Color] : RowHighlights)
	{
		FLinearColor FillColor = Color.Desaturate(.3f) * FLinearColor(0.053f,0.053f,0.053f);
		FillColor.A = 0.43f;
		
		FPaintGeometry Geometry(
			PropertyRect.GetTopLeft() + FVector2D{0.f,2.f},
			PropertyRect.GetSize() - FVector2D{0.f,4.f},
			1.f
		);
		
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + 8, // draw in front of background but behind text, buttons, etc
			Geometry,
			FAppStyle::GetBrush("WhiteBrush"),
			ESlateDrawEffect::None,
			FillColor
		);
	}
	
	return MaxLayerId;
}

FReply SDetailsSplitter::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	HoveredCopyButton = {};
	const FVector2D MousePosition = MouseEvent.GetScreenSpacePosition();
	
	for (int32 LeftIndex = 0; LeftIndex < Panels.Num() - 1; ++LeftIndex)
	{
		const FPanel& LeftPanel = Panels[LeftIndex];
		if (!LeftPanel.DiffRight.IsSet())
		{
			continue;
		}

		
		const int32 RightIndex = LeftIndex + 1;
		const FPanel& RightPanel = Panels[RightIndex];
		if (LeftPanel.IsReadonly.Get(true) && RightPanel.IsReadonly.Get(true))
		{
			continue;
		}
		if (const TSharedPtr<FAsyncDetailViewDiff> Diff = LeftPanel.DiffRight.Get())
		{
			
			TSharedPtr<FDetailTreeNode> LastSeenRightDetailsNode;
			TSharedPtr<FDetailTreeNode> LastSeenLeftDetailsNode;
			Diff->ForEachRow([&LastSeenLeftDetailsNode, &LastSeenRightDetailsNode, &LeftPanel, &RightPanel, &MousePosition, &HoveredCopyButton = HoveredCopyButton]
			(const TUniquePtr<FAsyncDetailViewDiff::DiffNodeType>& DiffNode, int32, int32)->ETreeTraverseControl
			{
				
				FSlateRect LeftPropertyRect;
				if (const TSharedPtr<FDetailTreeNode> LeftDetailNode = DiffNode->ValueA.Pin())
				{
					LastSeenLeftDetailsNode = LeftDetailNode;
					const bool bIncludeChildren = !DiffNode->ValueB.IsValid();
					LeftPropertyRect = LeftPanel.DetailsView->GetTickSpacePropertyBounds(LeftDetailNode.ToSharedRef(), bIncludeChildren);
				}
		
				FSlateRect RightPropertyRect;
				if (const TSharedPtr<FDetailTreeNode> RightDetailNode = DiffNode->ValueB.Pin())
				{
					LastSeenRightDetailsNode = RightDetailNode;
					const bool bIncludeChildren = !DiffNode->ValueA.IsValid();
					RightPropertyRect = RightPanel.DetailsView->GetTickSpacePropertyBounds(RightDetailNode.ToSharedRef(), bIncludeChildren);
				}
				
				if (DiffNode->DiffResult == ETreeDiffResult::Identical)
				{
					return ETreeTraverseControl::Continue;
				}

				if (LeftPropertyRect.IsValid() && !RightPanel.IsReadonly.Get(true))
				{
					const FSlateRect CopyButtonZoneLeftToRight = FSlateRect(
						LeftPropertyRect.Right,
						LeftPropertyRect.Top,
						LeftPropertyRect.Right + 15.f,
						LeftPropertyRect.Bottom
					);

					if (CopyButtonZoneLeftToRight.ContainsPoint(MousePosition))
					{
						if (DetailsSplitterHelpers::CanCopyPropertyValue(LastSeenLeftDetailsNode, LastSeenRightDetailsNode, DiffNode->DiffResult))
						{
							HoveredCopyButton = {
								LastSeenLeftDetailsNode,
								LastSeenRightDetailsNode,
								DiffNode->DiffResult,
								EPropertyCopyDirection::CopyLeftToRight
							};
						}
						return ETreeTraverseControl::Break;
					}
				}
				
				if (RightPropertyRect.IsValid() && !LeftPanel.IsReadonly.Get(true))
				{
					const FSlateRect CopyButtonZoneRightToLeft = FSlateRect(
						RightPropertyRect.Left - 15.f,
						RightPropertyRect.Top,
						RightPropertyRect.Left,
						RightPropertyRect.Bottom
					);

					if (CopyButtonZoneRightToLeft.ContainsPoint(MousePosition))
					{
						if (DetailsSplitterHelpers::CanCopyPropertyValue(LastSeenRightDetailsNode, LastSeenLeftDetailsNode, DiffNode->DiffResult))
						{
							HoveredCopyButton = {
								LastSeenRightDetailsNode,
								LastSeenLeftDetailsNode,
								DiffNode->DiffResult,
								EPropertyCopyDirection::CopyRightToLeft
							};
						}
						return ETreeTraverseControl::Break;
					}
				}
				
				return ETreeTraverseControl::Continue;
			});

			if (HoveredCopyButton.CopyDirection != EPropertyCopyDirection::Copy_None)
			{
				break;
			}
		}
	}
	return SCompoundWidget::OnMouseMove(MyGeometry, MouseEvent);
}

void SDetailsSplitter::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	HoveredCopyButton = {};
	SCompoundWidget::OnMouseLeave(MouseEvent);
}

FReply SDetailsSplitter::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}
	
	if (HoveredCopyButton.CopyDirection == EPropertyCopyDirection::Copy_None)
	{
		return FReply::Unhandled();
	}

	DetailsSplitterHelpers::CopyPropertyValue(
		HoveredCopyButton.SourceDetailsNode.Pin(),
		HoveredCopyButton.DestinationDetailsNode.Pin(),
		HoveredCopyButton.DiffResult);
	return FReply::Handled();
}

void SDetailsSplitter::PaintPropertyConnector(FSlateWindowElementList& OutDrawElements, int32 LayerId, const FSlateRect& LeftPropertyRect,
	const FSlateRect& RightPropertyRect, const FLinearColor& FillColor, const FLinearColor& OutlineColor) const
{
	FVector2D TopLeft = LeftPropertyRect.GetTopRight();
	FVector2D BottomLeft = LeftPropertyRect.GetBottomRight();
	
	FVector2D TopRight = RightPropertyRect.GetTopLeft();
	FVector2D BottomRight = RightPropertyRect.GetBottomLeft();

	{
		constexpr float YPadding = 2.f;
		if (BottomLeft.Y - TopLeft.Y > YPadding * 2.f)
		{
			BottomLeft.Y -= YPadding;
			TopLeft.Y += YPadding;
		}
		if (BottomRight.Y - TopRight.Y > YPadding * 2.f)
		{
			BottomRight.Y -= YPadding;
			TopRight.Y += YPadding;
		}
	}
	
	TArray<FSlateVertex> FillVerts;
	TArray<SlateIndex> FillIndices;
	TArray<FVector2D> TopBoarderLine;
	TArray<FVector2D> BottomBoarderLine;

	auto AddVert = [&FillVerts, &FillIndices](const FVector2D& Position, const FColor& VertColor)
	{
		FillVerts.Add(FSlateVertex::Make<ESlateVertexRounding::Disabled>(
			FSlateRenderTransform(),
			FVector2f(Position),
			{0.f,0.f},
			VertColor
		));
		if (FillVerts.Num() >= 3)
		{
			FillIndices.Add(FillVerts.Num()-3);
			FillIndices.Add(FillVerts.Num()-2);
			FillIndices.Add(FillVerts.Num()-1);
		}
	};

	// interpolate between left and right corners and add vertices to the mesh
	constexpr float StepSize = 1.f / 30.f;
	constexpr float InterpBoarder = .3f; // make room for buttons

		FillVerts.Empty();
		FillIndices.Empty();
		TopBoarderLine.Empty();
		BottomBoarderLine.Empty();

	float Alpha = 0.f;
	while (true)
	{
		constexpr float Exp = 3.f;
		const float TopAlpha = FMath::Clamp(Alpha, 0.f, 1.f);
		const float BottomAlpha = FMath::Clamp(Alpha, 0.f, 1.f);
		const double TopX = FMath::Lerp(TopLeft.X, TopRight.X, TopAlpha);
		const double BottomX = FMath::Lerp(TopLeft.X, TopRight.X, BottomAlpha);
		
		constexpr float InterpolationRange = 1.f - 2.f * InterpBoarder;

		const float TopTransformedAlpha = FMath::Clamp((TopAlpha - InterpBoarder) / InterpolationRange, 0.f, 1.f);
		const float BottomTransformedAlpha = FMath::Clamp((BottomAlpha - InterpBoarder) / InterpolationRange, 0.f, 1.f);
		
		const double TopY = FMath::InterpEaseInOut(TopLeft.Y, TopRight.Y, TopTransformedAlpha, Exp);
		const double BottomY = FMath::InterpEaseInOut(BottomLeft.Y, BottomRight.Y, BottomTransformedAlpha, Exp);

		FLinearColor ColumnColor = FillColor;
		if (Alpha <= 0.5f)
		{
			ColumnColor.A = FMath::InterpEaseOut(FillColor.A, 1.f, Alpha * 2.f, 2.f);
		}
		else
		{
			ColumnColor.A = FMath::InterpEaseIn( 1.f,FillColor.A, (Alpha - 0.5f) * 2.f, 2.f);
		}
	
		TopBoarderLine.Emplace(TopX,TopY);
		FLinearColor TopColor = ColumnColor.Desaturate(0.5f);
		AddVert(TopBoarderLine.Last(), TopColor.ToFColorSRGB());
	
		BottomBoarderLine.Emplace(BottomX,BottomY);
		FLinearColor BottomColor = ColumnColor;
		AddVert(BottomBoarderLine.Last(), BottomColor.ToFColorSRGB());
	
		if (Alpha >= 1.f)
		{
			break;
		}
		Alpha = FMath::Clamp(Alpha + StepSize, 0.f, 1.f);
	}
	
	const FSlateResourceHandle ResourceHandle = FSlateApplication::Get().GetRenderer()->GetResourceHandle(*FAppStyle::GetBrush("WhiteBrush"));
	FSlateDrawElement::MakeCustomVerts(
		OutDrawElements,
		LayerId,
		ResourceHandle,
		FillVerts,
		FillIndices,
		nullptr,
		0,
		0,
		ESlateDrawEffect::None
	);

	FSlateDrawElement::MakeLines(
		OutDrawElements,
		LayerId,
		FPaintGeometry(),
		TopBoarderLine,
		ESlateDrawEffect::None,
		OutlineColor,
		true,
		0.5f
	);

	FSlateDrawElement::MakeLines(
		OutDrawElements,
		LayerId,
		FPaintGeometry(),
		BottomBoarderLine,
		ESlateDrawEffect::None,
		OutlineColor,
		true,
		0.5f
	);
}

void SDetailsSplitter::PaintCopyPropertyButton(FSlateWindowElementList& OutDrawElements, int32 LayerId, const TUniquePtr<FAsyncDetailViewDiff::DiffNodeType>& DiffNode,
	const FSlateRect& LeftPropertyRect, const FSlateRect& RightPropertyRect, EPropertyCopyDirection CopyDirection) const
{
	constexpr float ButtonScale = 15.f;
	FPaintGeometry Geometry;
	const FSlateBrush* Brush = nullptr;
	FLinearColor ButtonColor = FStyleColors::Foreground.GetSpecifiedColor();
	switch (CopyDirection)
	{
	case EPropertyCopyDirection::CopyLeftToRight:
		if (LeftPropertyRect.GetSize().Y < UE_SMALL_NUMBER)
		{
			// space is too small to fit a button
			return;
		}
		Geometry = FPaintGeometry(
			FVector2D(LeftPropertyRect.Right, LeftPropertyRect.GetCenter().Y - 0.5f * ButtonScale),
			FVector2D(ButtonScale),
			1.f
		);
		Brush = FAppStyle::GetBrush("BlueprintDif.CopyPropertyRight");
		
		if (HoveredCopyButton.CopyDirection == EPropertyCopyDirection::CopyLeftToRight)
		{
			if (HoveredCopyButton.SourceDetailsNode == DiffNode->ValueA)
			{
				ButtonColor = FStyleColors::ForegroundHover.GetSpecifiedColor();
			}
		}
		break;
	case EPropertyCopyDirection::CopyRightToLeft:
		if (RightPropertyRect.GetSize().Y < UE_SMALL_NUMBER)
		{
			// space is too small to fit a button
			return;
		}
		Geometry = FPaintGeometry(
			FVector2D(RightPropertyRect.Left - ButtonScale, RightPropertyRect.GetCenter().Y - 0.5f * ButtonScale),
			FVector2D(ButtonScale),
			1.f
		);
		Brush = FAppStyle::GetBrush("BlueprintDif.CopyPropertyLeft");
		
		if (HoveredCopyButton.CopyDirection == EPropertyCopyDirection::CopyRightToLeft)
		{
			if (HoveredCopyButton.SourceDetailsNode == DiffNode->ValueB)
			{
				ButtonColor = FStyleColors::ForegroundHover.GetSpecifiedColor();
			}
		}
		break;
	default:
		return;
	}
	
		
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId, // draw in front of background but behind text, buttons, etc
		Geometry,
		Brush,
		ESlateDrawEffect::None,
		ButtonColor
	);
}

#undef LOCTEXT_NAMESPACE
