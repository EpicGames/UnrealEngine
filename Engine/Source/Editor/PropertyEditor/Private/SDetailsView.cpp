// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDetailsView.h"

#include "CategoryPropertyNode.h"
#include "Settings/EditorStyleSettings.h"
#include "DetailCategoryBuilderImpl.h"
#include "DetailLayoutBuilderImpl.h"
#include "DetailsNameWidgetOverrideCustomization.h"
#include "DetailsViewGenericObjectFilter.h"
#include "DetailsViewPropertyGenerationUtilities.h"
#include "Editor.h"
#include "EditorMetadataOverrides.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "ObjectPropertyNode.h"
#include "PropertyEditorCommands.h"
#include "PropertyEditorHelpers.h"
#include "SDetailNameArea.h"
#include "Styling/StyleColors.h"
#include "Templates/UnrealTemplate.h"
#include "UserInterface/PropertyDetails/PropertyDetailsUtilities.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Images/SLayeredImage.h"

#define LOCTEXT_NAMESPACE "SDetailsView"

static TAutoConsoleVariable<bool> CVarDetailsPanelEnableCardLayout(
	TEXT("DetailsPanel.Style.EnableCardLayout"),
	false,
	TEXT("Specifies whether the card layout is in effect for the Details View."),
	ECVF_Default);

namespace UE::PropertyEditor
{
	bool GPanelResolveInstanceDataObjectsDefault = false;
	static FAutoConsoleVariableRef CVarDetailsPanelResolveInstanceDataObjectsDefault(
	TEXT("DetailsPanel.ResolveInstanceDataObjects.Default"),
		GPanelResolveInstanceDataObjectsDefault,
		TEXT("Default value for the ResolveInstanceDataObjects DetailsView init parameter if not specified.  Will only apply to newly created details panels"),
		ECVF_Default);
}

SDetailsView::~SDetailsView()
{
	const FRootPropertyNodeList& RootNodes = GetRootNodes();
	for(const TSharedPtr<FComplexPropertyNode>& RootNode : RootNodes)
	{
		SaveExpandedItems(RootNode.ToSharedRef());
	}

	FEditorDelegates::PostUndoRedo.Remove(PostUndoRedoDelegateHandle);
};

/**
 * Constructs the widget
 *
 * @param InArgs   Declaration from which to construct this widget.              
 */
void SDetailsView::Construct(const FArguments& InArgs, const FDetailsViewArgs& InDetailsViewArgs)
{
	DetailsViewArgs = InDetailsViewArgs;
	DetailsNameWidgetOverrideCustomization = DetailsViewArgs.DetailsNameWidgetOverrideCustomization;
	
	BindCommands();

	const FDetailsViewConfig* ViewConfig = GetConstViewConfig();
	if (ViewConfig != nullptr)
	{
		if (ViewConfig->ValueColumnWidth != 0 && ViewConfig->ValueColumnWidth != DetailsViewArgs.ColumnWidth)
		{
			DetailsViewArgs.ColumnWidth = ViewConfig->ValueColumnWidth;
		}

		if (DetailsViewArgs.bShowSectionSelector)
		{
			DetailsViewArgs.bShowSectionSelector = ViewConfig->bShowSections;
		}

		CurrentFilter.bShowFavoritesCategory = ViewConfig->bShowFavoritesCategory;
	}

	CurrentFilter.bShowLooseProperties = InDetailsViewArgs.bShowLooseProperties;
	CurrentFilter.bShowHeaders = InDetailsViewArgs.bShowHeaders;
	CurrentFilter.ShouldForceHideProperty = InDetailsViewArgs.ShouldForceHideProperty;

	if (DetailsViewArgs.InitialSections.Num() > 0)
	{
		CurrentFilter.VisibleSections = DetailsViewArgs.InitialSections;
	}
	if (DetailsViewArgs.InitialCategories.Num() > 0)
	{
		CurrentFilter.VisibleCategories = DetailsViewArgs.InitialCategories;
	}

	ColumnSizeData.SetValueColumnWidth(DetailsViewArgs.ColumnWidth);
	ColumnSizeData.SetRightColumnMinWidth(DetailsViewArgs.RightColumnMinWidth);

	SetObjectFilter(InDetailsViewArgs.ObjectFilter);
	SetClassViewerFilters(InDetailsViewArgs.ClassViewerFilters);

	bViewingClassDefaultObject = false;
	bIsRefreshing = false;

	PropertyUtilities = MakeShareable( new FPropertyDetailsUtilities( *this ) );
	PropertyGenerationUtilities = MakeShareable( new FDetailsViewPropertyGenerationUtilities(*this) );

	PostUndoRedoDelegateHandle = FEditorDelegates::PostUndoRedo.AddSP(this, &SDetailsView::OnPostUndoRedo);

	// We want the scrollbar to always be visible when objects are selected, but not when there is no selection - however:
	//  - We can't use AlwaysShowScrollbar for this, as this will also show the scrollbar when nothing is selected
	//  - We can't use the Visibility construction parameter, as it gets translated into user visibility and can hide the scrollbar even when objects are selected
	// We instead have to explicitly set the visibility after the scrollbar has been constructed to get the exact behavior we want
	const TSharedRef<SScrollBar> ExternalScrollbar = InDetailsViewArgs.ExternalScrollbar ? InDetailsViewArgs.ExternalScrollbar.ToSharedRef() : SNew(SScrollBar);
	ExternalScrollbar->SetVisibility( TAttribute<EVisibility>( this, &SDetailsView::GetScrollBarVisibility ) );

	FMenuBuilder DetailViewOptions(true, InternalCommands, InDetailsViewArgs.OptionsExtender);
	const bool bDisplayShowSectionHeadingText = InDetailsViewArgs.OptionsExtender.IsValid() && (InDetailsViewArgs.OptionsExtender->NumExtensions() > 0);
	DetailViewOptions.BeginSection("DetailsViewShowOptions", (bDisplayShowSectionHeadingText ? LOCTEXT("ShowSectionHeading", "Show") : FText::GetEmpty()));

	if (DetailsViewArgs.bShowModifiedPropertiesOption)
	{
		DetailViewOptions.AddMenuEntry( 
			LOCTEXT("ShowOnlyModified", "Show Only Modified Properties"),
			LOCTEXT("ShowOnlyModified_ToolTip", "Displays only properties which have been changed from their default"),
			FSlateIcon(),
			FUIAction( 
				FExecuteAction::CreateSP( this, &SDetailsView::OnShowOnlyModifiedClicked ),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP( this, &SDetailsView::IsShowOnlyModifiedChecked )
			),
			NAME_None,
			EUserInterfaceActionType::Check 
		);
	}

	if (DetailsViewArgs.bShowCustomFilterOption)
	{
		TAttribute<FText> CustomFilterLabelDelegate;
		CustomFilterLabelDelegate.BindRaw(this, &SDetailsView::GetCustomFilterLabel);
		DetailViewOptions.AddMenuEntry(
			CustomFilterLabelDelegate,
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SDetailsView::OnCustomFilterClicked),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SDetailsView::IsCustomFilterChecked)
			),
			NAME_None,
			EUserInterfaceActionType::Check
		);
	}

	if (DetailsViewArgs.bShowDifferingPropertiesOption)
	{
		DetailViewOptions.AddMenuEntry(
			LOCTEXT("ShowOnlyDiffering", "Show Only Differing Properties"),
			LOCTEXT("ShowOnlyDiffering_ToolTip", "Displays only properties in this instance which have been changed or added from the instance being compared"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SDetailsView::OnShowOnlyAllowedClicked),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SDetailsView::IsShowOnlyAllowedChecked)
			),
			NAME_None,
			EUserInterfaceActionType::Check
		);
	}

	if (DetailsViewArgs.bShowKeyablePropertiesOption)
	{
		DetailViewOptions.AddMenuEntry(
			LOCTEXT("ShowOnlyKeyable", "Show Only Keyable Properties"),
			LOCTEXT("ShowOnlyKeyable_ToolTip", "Displays only properties which are keyable"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SDetailsView::OnShowKeyableClicked),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SDetailsView::IsShowKeyableChecked)
			),
			NAME_None,
			EUserInterfaceActionType::Check
		);
	}

	if (DetailsViewArgs.bShowAnimatedPropertiesOption)
	{
		DetailViewOptions.AddMenuEntry(
			LOCTEXT("ShowAnimated", "Show Only Animated Properties"),
			LOCTEXT("ShowAnimated_ToolTip", "Displays only properties which are animated (have tracks)"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SDetailsView::OnShowAnimatedClicked),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SDetailsView::IsShowAnimatedChecked)
			),
			NAME_None,
			EUserInterfaceActionType::Check
		);
	}

	DetailViewOptions.AddMenuEntry(
		LOCTEXT("ShowAllAdvanced", "Show All Advanced Details"),
		LOCTEXT("ShowAllAdvanced_ToolTip", "Shows all advanced detail sections in each category"),
		FSlateIcon(),
		FUIAction( 
		FExecuteAction::CreateSP( this, &SDetailsView::OnShowAllAdvancedClicked ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &SDetailsView::IsShowAllAdvancedChecked )
			),
		NAME_None,
		EUserInterfaceActionType::Check
	);

	if (DetailsViewArgs.bShowHiddenPropertiesWhilePlayingOption)
	{
		DetailViewOptions.AddMenuEntry(
			LOCTEXT("ShowHiddenPropertiesWhilePlaying", "Show Hidden Properties while Playing"),
			LOCTEXT("ShowHiddenPropertiesWhilePlaying_ToolTip", "When Playing or Simulating, shows all properties (even non-visible and non-editable properties), if the object belongs to a simulating world.  This is useful for debugging."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SDetailsView::OnShowHiddenPropertiesWhilePlayingClicked),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SDetailsView::IsShowHiddenPropertiesWhilePlayingChecked)
			),
			NAME_None,
			EUserInterfaceActionType::Check
		);
	}

	DetailViewOptions.AddMenuEntry(
		LOCTEXT("ShowAllChildrenIfCategoryMatches", "Show Child On Category Match"),
		LOCTEXT("ShowAllChildrenIfCategoryMatches_ToolTip", "Shows children if their category matches the search criteria"),
		FSlateIcon(),
		FUIAction( 
			FExecuteAction::CreateSP( this, &SDetailsView::OnShowAllChildrenIfCategoryMatchesClicked ),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP( this, &SDetailsView::IsShowAllChildrenIfCategoryMatchesChecked )
		),
		NAME_None,
		EUserInterfaceActionType::Check
	);

	DetailViewOptions.AddMenuEntry(
		LOCTEXT("ShowSections", "Show Sections"),
		LOCTEXT("ShowSections_ToolTip", "Shows the sections list."),
		FSlateIcon(),
		FUIAction( 
			FExecuteAction::CreateSP(this, &SDetailsView::OnShowSectionsClicked),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SDetailsView::IsShowSectionsChecked)
		),
		NAME_None,
		EUserInterfaceActionType::Check
	);
	
	DetailViewOptions.AddMenuEntry(
		LOCTEXT("ShowHelperSections", "Show Helper Sections"),
		LOCTEXT("ShowHelperSectionsToolTip", "Specifies whether helper sections like Favorites, Modified are shown in the filter bar"),
		FSlateIcon(),
		FUIAction( 
			FExecuteAction::CreateSP(this, &SDetailsView::OnShowHelperSectionsClicked),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SDetailsView::IsShowHelperSectionsChecked)
		),
		NAME_None,
		EUserInterfaceActionType::Check
	);

	DetailViewOptions.EndSection();
	DetailViewOptions.BeginSection("DetailsViewCategories");

	// Commands have been registered earlier
	DetailViewOptions.AddMenuEntry(FPropertyEditorCommands::Get().CollapseAll);
	DetailViewOptions.AddMenuEntry(FPropertyEditorCommands::Get().ExpandAll);

	TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);
		
	DetailViewOptions.EndSection();

	// Create the name area which does not change when selection changes
	SAssignNew(NameArea, SDetailNameArea, &SelectedObjects)
		// the name area is only for actors
		.Visibility(this, &SDetailsView::GetActorNameAreaVisibility)
		.OnLockButtonClicked(this, &SDetailsView::OnLockButtonClicked)
		.IsLocked(this, &SDetailsView::IsLocked)
		.ShowLockButton(DetailsViewArgs.bLockable)
		.ShowObjectLabel(DetailsViewArgs.bShowObjectLabel)
		// only show the selection tip if we're not selecting objects
		.SelectionTip(!DetailsViewArgs.bHideSelectionTip);

	if( !DetailsViewArgs.bCustomNameAreaLocation )
	{
		VerticalBox->AddSlot()
		.AutoHeight()
		.Padding(0.0f, 0.0f, 0.0f, 4.0f)
		[
			NameArea.ToSharedRef()
		];
	}

	TSharedRef<SHorizontalBox> FilterRowHBox = 
		SNew( SHorizontalBox )
		+SHorizontalBox::Slot()
		.Padding(6.f)
		.FillWidth(1.0f)
		[
			// Create the search box
			SAssignNew(SearchBox, SSearchBox)
			.HintText(LOCTEXT("SearchDetailsHint", "Search"))
			.OnTextChanged(this, &SDetailsView::OnFilterTextChanged)
			.OnTextCommitted(this, &SDetailsView::OnFilterTextCommitted)
			.AddMetaData<FTagMetaData>(TEXT("Details.Search"))
		];
	
	{
		FilterRowHBox->AddSlot()
			.Padding(0.0f)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				// Create the property matrix button
				SNew(SButton)
				.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
				.OnClicked(this, &SDetailsView::OnOpenRawPropertyEditorClicked)
				.IsEnabled(this, &SDetailsView::CanOpenRawPropertyEditor)
				.Visibility(this, &SDetailsView::CanShowRawPropertyEditorButton, DetailsViewArgs.bShowPropertyMatrixButton)
				.ToolTipText(LOCTEXT("RawPropertyEditorButtonLabel", "Edit Selection in Property Matrix"))
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("DetailsView.EditRawProperties"))
				]
			];
	} 

	if (DetailsViewArgs.bAllowFavoriteSystem)
	{
		FilterRowHBox->AddSlot()
			.Padding(0.0f)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				// Create the toggle favorites button
				SNew(SButton)
				.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
				.OnClicked(this, &SDetailsView::OnToggleFavoritesClicked)
				.ToolTipText(LOCTEXT("ToggleFavorites", "Toggle Favorites"))
				.AddMetaData<FTagMetaData>(TEXT("Details.ToggleFavorites"))
				[
					SNew(SImage)
					.ColorAndOpacity(this, &SDetailsView::GetToggleFavoritesColor)
					.Image(FAppStyle::Get().GetBrush("Icons.Star"))
				]
			];
	}

	if (DetailsViewArgs.bShowOptions)
	{
		TSharedPtr<SLayeredImage> FilterImage = SNew(SLayeredImage)
		 .Image(FAppStyle::Get().GetBrush("DetailsView.ViewOptions"))
		 .ColorAndOpacity(FSlateColor::UseForeground());

		// Badge the filter icon if there are filters active
		FilterImage->AddLayer(TAttribute<const FSlateBrush*>(this, &SDetailsView::GetViewOptionsBadgeIcon));
		
		FilterRowHBox->AddSlot()
			.Padding(0.0f)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew( SComboButton )
				.HasDownArrow(false)
				.ContentPadding(0.0f)
				.ForegroundColor( FSlateColor::UseForeground() )
				.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
				.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ViewOptions")))
				.MenuContent()
				[
					DetailViewOptions.MakeWidget()
				]
				.ButtonContent()
				[
					FilterImage.ToSharedRef()
				]
			];
	}

	TSharedRef<SVerticalBox> FilterRowVBox = SNew(SVerticalBox)
		.Visibility( this, &SDetailsView::GetFilterBoxVisibility )
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			FilterRowHBox
		];

	FilterRowVBox->AddSlot()
		.Padding(8.0f, 2.0f, 8.0f, 7.0f)
		.AutoHeight()
		[
			SAssignNew(SectionSelectorBox, SWrapBox)
			.UseAllottedSize(true)
			.InnerSlotPadding(FVector2D(4.0f,4.0f))
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("SectionView")))
		];

	RebuildSectionSelector();

	FilterRow = FilterRowVBox;

	if( !DetailsViewArgs.bCustomFilterAreaLocation )
	{
		VerticalBox->AddSlot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			[
				FilterRow.ToSharedRef()
			]
		];
	}

	VerticalBox->AddSlot()
	.FillHeight(1.0f)
	.Padding(0.0f)
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SBox)
				.Visibility(DetailsViewArgs.ScrollbarAlignment == HAlign_Left ? EVisibility::Visible : EVisibility::Collapsed)
				.WidthOverride(16.0f)
				[
					ExternalScrollbar
				]
			]
			+SHorizontalBox::Slot()
			[
	           SNew(SBorder)
				.Padding_Lambda([this]
				{
					if (DisplayManager.IsValid())
					{
						return DisplayManager->GetTablePadding();
					}
					static const FMargin Margin{0};
					return Margin;
				})
				.BorderImage_Lambda([this]
				{
					static const FSlateBrush* Background = FAppStyle::GetBrush("DetailsView.GridLine");
					static const FSlateBrush* Panel = FAppStyle::GetBrush("Brushes.Panel");
					
					return RootPropertyNodes.Num() == 0 ? Panel : Background;
				})
				.Visibility(this, &SDetailsView::GetTreeVisibility)
				[ ConstructTreeView(ExternalScrollbar) ]
			]
		]
		+ SOverlay::Slot()
		.HAlign(DetailsViewArgs.ScrollbarAlignment)
		[
			SNew(SBox)
			.Visibility(DetailsViewArgs.ScrollbarAlignment == HAlign_Left ? EVisibility::Collapsed : EVisibility::Visible)
			.WidthOverride(16.0f)
			[
				ExternalScrollbar
			]
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.Padding(2.0f, 24.0f, 2.0f, 2.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("AllItemsFiltered", "All results have been filtered.\nTry changing your active filters above."))
			.AutoWrapText(true)
			.Justification(ETextJustify::Center)
			.Visibility_Lambda([this]() { return ((this->GetFilterBoxVisibility() == EVisibility::Visible) && !this->CurrentFilter.IsEmptyFilter() && RootTreeNodes.Num() == 0) ? EVisibility::HitTestInvisible : EVisibility::Collapsed; })
		]
		+ SOverlay::Slot()
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Searching.SearchActiveBorder"))
			.Visibility_Lambda([this]() { return (this->GetFilterBoxVisibility() == EVisibility::Visible) && this->HasActiveSearch() ? EVisibility::HitTestInvisible : EVisibility::Collapsed; })
		]
		+ SOverlay::Slot()
		[
			SAssignNew(ExternalOverlayContainer, SBox)
			.Visibility(DisplayManager.IsValid() && DisplayManager->SupportsDetailTreeOverlay() ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed)
			.Content()
			[
				DisplayManager.IsValid() && DisplayManager->SupportsDetailTreeOverlay() ? DisplayManager->ConstructDetailTreeOverlay() : SNullWidget::NullWidget
			]
		]
	];

	ChildSlot
	[
		VerticalBox
	];
}

TSharedRef<SDetailTree> SDetailsView::ConstructTreeView(const TSharedRef<SScrollBar>& ScrollBar )
{
	check( !DetailTree.IsValid() || DetailTree.IsUnique() );

	return
		SAssignNew(DetailTree, SDetailTree)
		.Visibility(this, &SDetailsView::GetTreeVisibility)
		.TreeItemsSource(&RootTreeNodes)
		.OnGetChildren(this, &SDetailsView::OnGetChildrenForDetailTree)
		.OnSetExpansionRecursive(this, &SDetailsView::SetNodeExpansionStateRecursive)
		.OnGenerateRow(this, &SDetailsView::OnGenerateRowForDetailTree)
		.OnRowReleased(this, &SDetailsView::OnRowReleasedForDetailTree)
		.OnExpansionChanged(this, &SDetailsView::OnItemExpansionChanged)
		.SelectionMode(ESelectionMode::None)
		.HandleDirectionalNavigation(false)
		.AllowOverscroll(DetailsViewArgs.bShowScrollBar ? EAllowOverscroll::Yes : EAllowOverscroll::No)
		.ExternalScrollbar(ScrollBar);
}

bool SDetailsView::CanOpenRawPropertyEditor() const
{
	return SelectedObjects.Num() > 0 && IsPropertyEditingEnabled();
}

EVisibility SDetailsView::CanShowRawPropertyEditorButton(const bool bAllowedByDetailsViewArgs) const
{
	auto SelectionContainsNonActors = [this]() -> bool
	{
		for (const TWeakObjectPtr<UObject> Object : SelectedObjects)
		{
			if (Object.IsValid() && !Object->IsA<AActor>() && !Object->IsA<UActorComponent>())
			{
				return true;
			}
		}
		return false;
	};

	const FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	const bool bShowPropertyMatrixOverride = PropertyModule.GetCanUsePropertyMatrix();
	return (DetailsViewArgs.bShowPropertyMatrixButton && bShowPropertyMatrixOverride 
		&& (PropertyModule.CVarAllowPropertyMatrixForNonActors.GetValueOnAnyThread() || !SelectionContainsNonActors())) ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SDetailsView::OnOpenRawPropertyEditorClicked()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>( "PropertyEditor" );
	PropertyEditorModule.CreatePropertyEditorToolkit(TSharedPtr<IToolkitHost>(), SelectedObjects );

	return FReply::Handled();
}

EVisibility SDetailsView::GetActorNameAreaVisibility() const
{
	const bool bVisible = DetailsViewArgs.NameAreaSettings != FDetailsViewArgs::HideNameArea && !bViewingClassDefaultObject;
	return bVisible ? EVisibility::Visible : EVisibility::Collapsed; 
}

void SDetailsView::ForceRefresh()
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("SDetailsView::ForceRefresh");

	ClearPendingRefreshTimer();

	TArray<UObject*> NewObjectList;
	NewObjectList.Reserve(UnfilteredSelectedObjects.Num());
	TArray<TWeakObjectPtr<UObject>> ValidSelectedObjects;
	ValidSelectedObjects.Reserve(UnfilteredSelectedObjects.Num());

	for (const TWeakObjectPtr<UObject>& Object : UnfilteredSelectedObjects)
	{
		if (Object.IsValid())
		{
			ValidSelectedObjects.Add(Object);
			NewObjectList.Add(Object.Get());
		}
	}

	UnfilteredSelectedObjects = MoveTemp(ValidSelectedObjects);

	SetObjectArrayPrivate(NewObjectList);
}

void SDetailsView::MoveScrollOffset(int32 DeltaOffset)
{
	DetailTree->AddScrollOffset((float)DeltaOffset);
}

void SDetailsView::SetObjects(const TArray<UObject*>& InObjects, bool bForceRefresh/* = false*/, bool bOverrideLock/* = false*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("SDetailsView::SetObjects");
	if (!IsLocked() || bOverrideLock)
	{
		if( bForceRefresh || ShouldSetNewObjects(InObjects) )
		{
			// Keep source object list around to reapply the object filter when it changes or force refresh.
			UnfilteredSelectedObjects.Empty(InObjects.Num());
			for (UObject* InObject : InObjects)
			{
				if (InObject)
				{
					UnfilteredSelectedObjects.Add(InObject);
				}
			}

			SetObjectArrayPrivate(InObjects);
		}
	}
}

void SDetailsView::SetObjects(const TArray<TWeakObjectPtr<UObject>>& InObjects, bool bForceRefresh/* = false*/, bool bOverrideLock/* = false*/)
{
	TArray<UObject*> SourceObjects;
	SourceObjects.Reserve(InObjects.Num());
	for (const TWeakObjectPtr<UObject>& Object : InObjects)
	{
		if (Object.IsValid())
		{
			SourceObjects.Add(Object.Get());
		}
	}

	SetObjects(SourceObjects, bForceRefresh, bOverrideLock);
}

void SDetailsView::SetObject(UObject* InObject, bool bForceRefresh)
{
	TArray<UObject*> SourceObjects;
	SourceObjects.Reserve(1);

	SourceObjects.Add(InObject);

	SetObjects(SourceObjects, bForceRefresh);
}

void SDetailsView::RemoveInvalidObjects()
{
	ForceRefresh();
}

void SDetailsView::SetObjectPackageOverrides(const TMap<TWeakObjectPtr<UObject>, TWeakObjectPtr<UPackage>>& InMapping)
{
	for(TSharedPtr<FComplexPropertyNode>& ComplexRootNode : RootPropertyNodes)
	{
		FObjectPropertyNode* RootNode = ComplexRootNode->AsObjectNode();
		if(RootNode)
		{
			RootNode->SetObjectPackageOverrides(InMapping);
		}
	}
}

void SDetailsView::SetRootObjectCustomizationInstance(TSharedPtr<IDetailRootObjectCustomization> InRootObjectCustomization)
{
	RootObjectCustomization = InRootObjectCustomization;
	RerunCurrentFilter();
}

void SDetailsView::SetObjectFilter(TSharedPtr<FDetailsViewObjectFilter> InFilter)
{
	ObjectFilter = InFilter;

	if (!ObjectFilter.IsValid())
	{
		ObjectFilter = MakeShared<FDetailsViewDefaultObjectFilter>(!!DetailsViewArgs.bAllowMultipleTopLevelObjects);
	}

	RefreshDisplayManager();

	// hook up the OnDetailsNeedsUpdate to the details panel refresh function so that when the display data
	// is updated, the details panel will reflect it
	DisplayManager->OnDetailsNeedsUpdate.BindSP(this, &SDetailsView::ForceRefresh);
}

void SDetailsView::SetClassViewerFilters(const TArray<TSharedRef<class IClassViewerFilter>>& InFilters)
{
	ClassViewerFilters = InFilters;
}

bool SDetailsView::ShouldSetNewObjects(const TArray<UObject*>& InObjects) const
{
	bool bShouldSetObjects = false;

	const bool bHadBSPBrushSelected = SelectedActorInfo.bHaveBSPBrush;
	if( bHadBSPBrushSelected == true )
	{
		// If a BSP brush was selected we need to refresh because surface could have been selected and the object set not updated
		bShouldSetObjects = true;
	}
	else if( InObjects.Num() != GetNumObjects() )
	{
		// If the object arrays differ in size then at least one object is different so we must reset
		bShouldSetObjects = true;
	}
	else if(InObjects.Num() == 0)
	{
		// User is likely resetting details panel
		bShouldSetObjects = true;
	}
	else
	{
		// Check to see if the objects passed in are different. If not we do not need to set anything
		TSet<UObject*> NewObjects;
		NewObjects.Append(InObjects);

		if(RootPropertyNodes.Num() > 1)
		{
			// For multiple top level node support, if the single object in each node is not found in the new object set
			// then we need to refresh
			for(int32 RootNodeIndex = 0; RootNodeIndex < RootPropertyNodes.Num(); ++RootNodeIndex)
			{
				FObjectPropertyNode* RootPropertyNode = RootPropertyNodes[RootNodeIndex]->AsObjectNode();
				
				if(RootPropertyNode && RootPropertyNode->GetNumObjects() > 0)
				{
					if(!NewObjects.Contains(RootPropertyNode->GetUObject(0)))
					{
						bShouldSetObjects = true;
						break;
					}
				}
				else
				{
					bShouldSetObjects = true;
					break;
				}
			}
		}
		else
		{
			FObjectPropertyNode* RootPropertyNode = RootPropertyNodes[0]->AsObjectNode();
			if( RootPropertyNode )
			{
				for(TPropObjectIterator Itor(RootPropertyNode->ObjectIterator()); Itor; ++Itor)
				{
					TWeakObjectPtr<UObject> Object = *Itor;
					if(Object.IsValid() && !NewObjects.Contains(Object.Get()))
					{
						// An existing object is not in the list of new objects to set
						bShouldSetObjects = true;
						break;
					}
					else if(!Object.IsValid())
					{
						// An existing object is invalid
						bShouldSetObjects = true;
						break;
					}
				}
			}
			else
			{
				bShouldSetObjects = true;
			}
		}
	}
	
	if (!bShouldSetObjects && AssetSelectionUtils::IsAnySurfaceSelected(nullptr))
	{
		bShouldSetObjects = true;
	}

	return bShouldSetObjects;
}

int32 SDetailsView::GetNumObjects() const
{
	if(RootPropertyNodes.Num() > 1)
	{
		return RootPropertyNodes.Num();
	}
	else if( RootPropertyNodes.Num() > 0 && RootPropertyNodes[0]->AsObjectNode())
	{
		return RootPropertyNodes[0]->AsObjectNode()->GetNumObjects();
	}

	return 0;
}

void SDetailsView::SetObjectArrayPrivate(const TArray<UObject*>& InObjects)
{
	if (bIsRefreshing)
	{
		// SetObjectArrayPrivate can not be performed while in the middle of a previous refresh
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(SDetailsView::SetObjectArrayPrivate);

	TGuardValue<bool> RefreshGuard(bIsRefreshing, true);

	double StartTime = FPlatformTime::Seconds();

	const TArray<FDetailsViewObjectRoot> Roots = ObjectFilter->FilterObjects(InObjects);
	
	UpdateStyleKey();

	PreSetObject(Roots.Num());

	// Selected actors for building SelectedActorInfo
	TArray<AActor*> SelectedRawActors;

	bViewingClassDefaultObject = InObjects.Num() > 0 ? true : false;
	bool bOwnedByLockedLevel = false;

	check(RootPropertyNodes.Num() == Roots.Num());

	for(int32 RootIndex = 0; RootIndex < Roots.Num(); ++RootIndex)
	{
		const FDetailsViewObjectRoot& Root = Roots[RootIndex];

		FObjectPropertyNode* RootNode = RootPropertyNodes[RootIndex]->AsObjectNode();

		for(const TWeakObjectPtr<UObject> Object : Root.Objects)
		{
			if (Object.IsValid())
			{
				bViewingClassDefaultObject &= Object->HasAnyFlags(RF_ClassDefaultObject);

				RootNode->AddObject(Object.Get());

				SelectedObjects.Add(Object);
				if (AActor* Actor = Cast<AActor>(Object.Get()))
				{
					SelectedActors.Add(Actor);
					SelectedRawActors.Add(Actor);
				}
			}
		}
	}

	if(SelectedObjects.Num() == 0)
	{
		// Unlock the view automatically if we are viewing nothing
		bIsLocked = false;
	}

	// Selection changed, refresh the detail area
	NameArea->Refresh(SelectedObjects, DetailsViewArgs.NameAreaSettings);
	
	// When selection changes rebuild information about the selection
	SelectedActorInfo = AssetSelectionUtils::BuildSelectedActorInfo(SelectedRawActors);

	PostSetObject(Roots);

	// Set the title of the window based on the objects we are viewing
	// Or call the delegate for handling when the title changed
	FString Title;

	if(SelectedObjects.Num() == 0 )
	{
		Title = NSLOCTEXT("PropertyView", "NothingSelectedTitle", "Nothing selected").ToString();
	}
	else if(Roots.Num() == 1 && RootPropertyNodes[0]->AsObjectNode()->GetNumObjects() == 1)
	{
		// if the object is the default metaobject for a UClass, use the UClass's name instead
		UObject* Object = RootPropertyNodes[0]->AsObjectNode()->GetUObject(0);

		FString ObjectName;
		if ( Object && Object->GetClass()->GetDefaultObject() == Object )
		{
			ObjectName = Object->GetClass()->GetName();
		}
		else if( Object )
		{
			ObjectName = Object->GetName();
			// Is this an actor?  If so, it might have a friendly name to display
			const AActor* Actor = Cast<const  AActor >( Object );
			if( Actor != nullptr)
			{
				// Use the friendly label for this actor
				ObjectName = Actor->GetActorLabel();
			}
		}

		Title = ObjectName;
	}
	else if(Roots.Num() > 1)
	{
		Title = FText::Format(NSLOCTEXT("PropertyView", "MultipleToLevelObjectsSelectedFmt", "{0} selected"), Roots.Num()).ToString();
	}
	else
	{
		FObjectPropertyNode* RootPropertyNode = RootPropertyNodes[0]->AsObjectNode();
		Title = FText::Format( NSLOCTEXT("PropertyView", "MultipleSelected", "{0} ({1} selected)"), FText::FromString(RootPropertyNode->GetObjectBaseClass()->GetName()), RootPropertyNode->GetNumObjects() ).ToString();
	}

	OnObjectArrayChanged.ExecuteIfBound(Title, InObjects);

	RebuildSectionSelector();

	double ElapsedTime = FPlatformTime::Seconds() - StartTime;
}

void SDetailsView::ReplaceObjects(const TMap<UObject*, UObject*>& OldToNewObjectMap)
{
	TArray<UObject*> NewObjectList;
	NewObjectList.Reserve(UnfilteredSelectedObjects.Num());
	TArray<TWeakObjectPtr<UObject>> NewUnfilteredSelectedObjects;
	NewUnfilteredSelectedObjects.Reserve(UnfilteredSelectedObjects.Num());

	bool bNeedRefresh = false;
	for (const TWeakObjectPtr<UObject>& Object : UnfilteredSelectedObjects)
	{
		// We could be replacing an object that has already been garbage collected, so look up the object using the raw pointer.
		UObject* Replacement = OldToNewObjectMap.FindRef(Object.GetEvenIfUnreachable());
		if (Replacement)
		{
			NewObjectList.Add(Replacement);
			NewUnfilteredSelectedObjects.Add(Replacement);
			bNeedRefresh = true;
		}
		else if (Object.IsValid())
		{
			NewObjectList.Add(Object.Get());
			NewUnfilteredSelectedObjects.Add(Object);
		}
		else
		{
			bNeedRefresh = true;
		}
	}

	if (bNeedRefresh)
	{
		UnfilteredSelectedObjects = MoveTemp(NewUnfilteredSelectedObjects);
		SetObjectArrayPrivate(NewObjectList);
	}
}

void SDetailsView::RemoveDeletedObjects(const TArray<UObject*>& DeletedObjects)
{
	TArray<UObject*> NewObjectList;
	NewObjectList.Reserve(UnfilteredSelectedObjects.Num());
	TArray<TWeakObjectPtr<UObject>> NewUnfilteredSelectedObjects;
	NewUnfilteredSelectedObjects.Reserve(UnfilteredSelectedObjects.Num());

	for (const TWeakObjectPtr<UObject>& Object : UnfilteredSelectedObjects)
	{
		if (Object.IsValid() && !DeletedObjects.Contains(Object.Get()))
		{
			NewUnfilteredSelectedObjects.Add(Object);
			NewObjectList.Add(Object.Get());
		}
	}

	if (NewUnfilteredSelectedObjects.Num() != UnfilteredSelectedObjects.Num())
	{
		UnfilteredSelectedObjects = MoveTemp(NewUnfilteredSelectedObjects);
		SetObjectArrayPrivate(NewObjectList);
	}
}

/** Called before during SetObjectArray before we change the objects being observed */
void SDetailsView::PreSetObject(int32 InNewNumObjects)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SDetailsView::PreSetObject);

	// Save existing expanded items first
	for(TSharedPtr<FComplexPropertyNode>& RootNode : RootPropertyNodes)
	{
		SaveExpandedItems(RootNode.ToSharedRef());

		RootNodesPendingKill.Add(RootNode);
		FObjectPropertyNode* RootObjectNode = RootNode->AsObjectNode();
		RootObjectNode->RemoveAllObjects();
		RootObjectNode->ClearCachedReadAddresses(true);
		RootObjectNode->ClearObjectPackageOverrides();
	}

	for(const FDetailLayoutData& Layout : DetailLayouts)
	{
		FRootPropertyNodeList& ExternalRootPropertyNodes = Layout.DetailLayout->GetExternalRootPropertyNodes();
		for (TSharedPtr<FComplexPropertyNode>& ExternalRootNode : ExternalRootPropertyNodes)
		{
			if (ExternalRootNode.IsValid())
			{
				SaveExpandedItems(ExternalRootNode.ToSharedRef());

				ExternalRootNode->Disconnect();
			}
		}
	}

	RootPropertyNodes.Empty(InNewNumObjects);
	ExpandedDetailNodes.Clear();

	for (int32 NewRootIndex = 0; NewRootIndex < InNewNumObjects; ++NewRootIndex)
	{
		RootPropertyNodes.Add(MakeShareable(new FObjectPropertyNode));
	}

	SelectedActors.Empty();
	SelectedObjects.Empty();
}

/** Called at the end of SetObjectArray after we change the objects being observed */
void SDetailsView::PostSetObject(const TArray<FDetailsViewObjectRoot>& Roots)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SDetailsView::PostSetObject);

	ColorPropertyNode = nullptr;

	// Are we editing PIE objects?  If the bShowHiddenPropertiesWhilePlaying setting is enabled, we may want to
	// show all of the properties that would normally be hidden for objects that are part of the PIE world.
	bool bShowHiddenPropertiesWhilePlaying = false;
	if (IsShowHiddenPropertiesWhilePlayingChecked())
	{
		for( int32 RootNodeIndex = 0; RootNodeIndex < RootPropertyNodes.Num(); ++RootNodeIndex )
		{
			FObjectPropertyNode* RootPropertyNode = RootPropertyNodes[ RootNodeIndex ]->AsObjectNode();
			if( RootPropertyNode != nullptr )
			{
				const int32 ObjectCount = RootPropertyNode->GetNumObjects();
				for( int32 ObjectIndex = 0; ObjectIndex < ObjectCount; ++ObjectIndex )
				{
					UObject* Object = RootPropertyNode->GetUObject( ObjectIndex );
					if( Object->GetOutermost()->HasAnyPackageFlags( PKG_PlayInEditor ) )
					{
						bShowHiddenPropertiesWhilePlaying = true;
						break;
					}
				}
			}
		}
	}

	FPropertyNodeInitParams InitParams;
	InitParams.ParentNode = nullptr;
	InitParams.Property = nullptr;
	InitParams.ArrayOffset = 0;
	InitParams.ArrayIndex = INDEX_NONE;
	InitParams.bAllowChildren = true;
	if (DisplayManager && DisplayManager->OverrideCreateCategoryNodes().IsSet())
	{
		InitParams.bCreateCategoryNodes = DisplayManager->OverrideCreateCategoryNodes().GetValue();
	}
	InitParams.bForceHiddenPropertyVisibility = 
		FPropertySettings::Get().ShowHiddenProperties() || 
		bShowHiddenPropertiesWhilePlaying ||
		DetailsViewArgs.bForceHiddenPropertyVisibility;
	InitParams.bResolveInstanceDataObjects = DetailsViewArgs.bResolveInstanceDataObjects.Get(UE::PropertyEditor::GPanelResolveInstanceDataObjectsDefault);

	switch ( DetailsViewArgs.DefaultsOnlyVisibility )
	{
	case EEditDefaultsOnlyNodeVisibility::Hide:
		InitParams.bCreateDisableEditOnInstanceNodes = false;
		break;
	case EEditDefaultsOnlyNodeVisibility::Show:
		InitParams.bCreateDisableEditOnInstanceNodes = true;
		break;
	case EEditDefaultsOnlyNodeVisibility::Automatic:
		InitParams.bCreateDisableEditOnInstanceNodes = HasClassDefaultObject();
		break;
	default:
		checkNoEntry();
	}

	for( TSharedPtr<FComplexPropertyNode>& ComplexRootNode : RootPropertyNodes )
	{
		FObjectPropertyNode* RootPropertyNode = ComplexRootNode->AsObjectNode();

		RootPropertyNode->InitNode( InitParams );
	}

	UpdatePropertyMapsWithItemExpansions();
	UpdateFilteredDetails();
}

void SDetailsView::SetOnObjectArrayChanged(FOnObjectArrayChanged OnObjectArrayChangedDelegate)
{
	OnObjectArrayChanged = OnObjectArrayChangedDelegate;
}

void SDetailsView::OnPostUndoRedo()
{
	InvalidateCachedState();
}

void SDetailsView::InvalidateCachedState()
{
	for (const TSharedPtr<FComplexPropertyNode>& RootNode : RootPropertyNodes)
	{
		RootNode->InvalidateCachedState();
	}
}

bool SDetailsView::IsConnected() const
{
	return GetNumObjects() > 0;
}

const FSlateBrush* SDetailsView::OnGetLockButtonImageResource() const
{
	if (bIsLocked)
	{
		return FAppStyle::GetBrush(TEXT("PropertyWindow.Locked"));
	}
	else
	{
		return FAppStyle::GetBrush(TEXT("PropertyWindow.Unlocked"));
	}
}

bool SDetailsView::IsShowHiddenPropertiesWhilePlayingChecked() const
{
	const FDetailsViewConfig* ViewConfig = GetConstViewConfig();
	if (ViewConfig != nullptr)
	{
		return ViewConfig->bShowHiddenPropertiesWhilePlaying;
	}

	return GetDefault<UEditorStyleSettings>()->bShowHiddenPropertiesWhilePlaying;
}

void SDetailsView::OnShowHiddenPropertiesWhilePlayingClicked()
{
	const bool bNewValue = !IsShowHiddenPropertiesWhilePlayingChecked();

	FDetailsViewConfig* ViewConfig = GetMutableViewConfig();
	if (ViewConfig != nullptr)
	{
		ViewConfig->bShowHiddenPropertiesWhilePlaying = bNewValue;
		SaveViewConfig();
	}

	GConfig->SetBool(TEXT("/Script/UnrealEd.EditorStyleSettings"), TEXT("bShowHiddenPropertiesWhilePlaying"), bNewValue, GEditorPerProjectIni);

	// Force a refresh of the whole details panel, as the entire set of visible properties may be different
	ForceRefresh();
}

void SDetailsView::OnShowSectionsClicked()
{
	DetailsViewArgs.bShowSectionSelector = !DetailsViewArgs.bShowSectionSelector;

	FDetailsViewConfig* ViewConfig = GetMutableViewConfig();
	if (ViewConfig != nullptr)
	{
		ViewConfig->bShowSections = DetailsViewArgs.bShowSectionSelector;
		SaveViewConfig();
	}

	if (!DetailsViewArgs.bShowSectionSelector)
	{
		CurrentFilter.VisibleSections.Reset();
	}

	RebuildSectionSelector();
	UpdateFilteredDetails();
}

bool SDetailsView::IsShowHelperSectionsChecked() const
{
	const FDetailsViewConfig* ViewConfig = GetConstViewConfig();
	return ViewConfig && ViewConfig->bShowHelperSections;
}

void SDetailsView::OnShowHelperSectionsClicked()
{
	if (FDetailsViewConfig* ViewConfig = GetMutableViewConfig())
	{
		ViewConfig->bShowHelperSections = !ViewConfig->bShowHelperSections;

		SaveViewConfig();
		RebuildSectionSelector();
		UpdateFilteredDetails();
	}
}

FSlateColor SDetailsView::GetToggleFavoritesColor() const
{
	if (DetailsViewArgs.bAllowFavoriteSystem && CurrentFilter.bShowFavoritesCategory)
	{
		return FSlateColor(EStyleColor::AccentBlue);
	}
	else
	{
		return FSlateColor(EStyleColor::Foreground);
	}
}

FReply SDetailsView::OnToggleFavoritesClicked()
{
	CurrentFilter.bShowFavoritesCategory = !CurrentFilter.bShowFavoritesCategory;

	FDetailsViewConfig* ViewConfig = GetMutableViewConfig();
	if (ViewConfig != nullptr)
	{
		ViewConfig->bShowFavoritesCategory = CurrentFilter.bShowFavoritesCategory;
		SaveViewConfig();
	}

	RerunCurrentFilter();

	return FReply::Handled();
}

static bool IsInEditorMetadataList(FName ListKey, FStringView Value, const TArray<TSharedPtr<FComplexPropertyNode>>& RootPropertyNodes)
{
	if (Value.IsEmpty())
	{
		return false;
	}

	UEditorMetadataOverrides* EditorMetadata = GEditor->GetEditorSubsystem<UEditorMetadataOverrides>();
	if (!EditorMetadata)
	{
		return false;
	}

	const FString ValueString(Value);

	for (const TSharedPtr<FComplexPropertyNode>& RootPropertyNode : RootPropertyNodes)
	{
		const UStruct* BaseStruct = RootPropertyNode->GetBaseStructure();
		if (BaseStruct != nullptr)
		{
			TArray<FString> ValueList;
			if (EditorMetadata->GetArrayMetadata(BaseStruct, ListKey, ValueList))
			{
				if (ValueList.Contains(ValueString))
				{
					return true;
				}
			}
		}
	}

	return false;
}

static void AddToEditorMetadataList(FName ListKey, FStringView Value, const TArray<TSharedPtr<FComplexPropertyNode>>& RootPropertyNodes)
{
	if (Value.IsEmpty())
	{
		return;
	}

	UEditorMetadataOverrides* MetadataOverrides = GEditor->GetEditorSubsystem<UEditorMetadataOverrides>();
	if (!MetadataOverrides)
	{
		return;
	}

	const FString ValueString(Value);

	for (const TSharedPtr<FComplexPropertyNode>& RootPropertyNode : RootPropertyNodes)
	{
		const UStruct* BaseStruct = RootPropertyNode->GetBaseStructure();
		if (BaseStruct != nullptr)
		{

			TArray<FString> ValueList;
			if (MetadataOverrides->GetArrayMetadata(BaseStruct, ListKey, ValueList))
			{
				ValueList.AddUnique(ValueString);
				MetadataOverrides->SetArrayMetadata(BaseStruct, ListKey, ValueList);
			}
			else
			{
				ValueList.Add(ValueString);
				MetadataOverrides->SetArrayMetadata(BaseStruct, ListKey, ValueList);
			}
		}
	}
}

static void RemoveFromEditorMetadataList(FName ListKey, FStringView Value, const TArray<TSharedPtr<FComplexPropertyNode>>& RootPropertyNodes)
{
	if (Value.IsEmpty())
	{
		return;
	}

	UEditorMetadataOverrides* MetadataOverrides = GEditor->GetEditorSubsystem<UEditorMetadataOverrides>();
	if (!MetadataOverrides)
	{
		return;
	}

	const FString ValueString(Value);

	for (const TSharedPtr<FComplexPropertyNode>& RootPropertyNode : RootPropertyNodes)
	{
		const UStruct* BaseStruct = RootPropertyNode->GetBaseStructure();
		if (BaseStruct != nullptr)
		{
			TArray<FString> ValueList;
			if (MetadataOverrides->GetArrayMetadata(BaseStruct, ListKey, ValueList))
			{
				ValueList.Remove(ValueString);
				MetadataOverrides->SetArrayMetadata(BaseStruct, ListKey, ValueList);
			}
		}
	}
}

bool SDetailsView::IsGroupFavorite(FStringView GroupPath) const
{
	static const FName FavoriteGroupsName("FavoriteGroups");
	return IsInEditorMetadataList(FavoriteGroupsName, GroupPath, RootPropertyNodes);
}

void SDetailsView::SetGroupFavorite(FStringView GroupPath, bool IsFavorite)
{
	static const FName FavoriteGroupsName("FavoriteGroups");
	if (IsFavorite)
	{
		AddToEditorMetadataList(FavoriteGroupsName, GroupPath, RootPropertyNodes);
	}
	else
	{
		RemoveFromEditorMetadataList(FavoriteGroupsName, GroupPath, RootPropertyNodes);
	}
}

bool SDetailsView::IsCustomBuilderFavorite(FStringView Path) const
{
	static const FName FavoriteCustomBuildersName("FavoriteCustomBuilders");
	return IsInEditorMetadataList(FavoriteCustomBuildersName, Path, RootPropertyNodes);
}

void SDetailsView::SetCustomBuilderFavorite(FStringView Path, bool IsFavorite)
{
	static const FName FavoriteCustomBuildersName("FavoriteCustomBuilders");
	if (IsFavorite)
	{
		AddToEditorMetadataList(FavoriteCustomBuildersName, Path, RootPropertyNodes);
	}
	else
	{
		RemoveFromEditorMetadataList(FavoriteCustomBuildersName, Path, RootPropertyNodes);
	}
}

void SDetailsView::RebuildSectionSelector()
{
	SectionSelectorBox->ClearChildren();
	SectionSelectorBox->SetVisibility(EVisibility::Collapsed);

	if (!DetailsViewArgs.bShowSectionSelector)
	{
		return;
	}

	const TMap<FName, SDetailsView::FSection> AllSections = GetAllSections();
	if (AllSections.IsEmpty())
	{
		// we've selected something that has no sections - rather than show just "All", hide the box and clear visible sections
		CurrentFilter.VisibleSections.Reset();
		return;
	}

	auto CreateSection = [this](FName SectionName, FText SectionDisplayName) 
		-> TSharedRef<SWidget>
	{
		return SNew(SBox)
			.Padding(FMargin(0.0f))
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "DetailsView.SectionButton")
				.OnCheckStateChanged(this, &SDetailsView::OnSectionCheckedChanged, SectionName)
				.IsChecked(this, &SDetailsView::IsSectionChecked, SectionName)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "SmallText")
					.Text(SectionDisplayName)
				]
			];
	};

	struct FOrderedSection
	{
		FName Name;
		int32 Order = 0;
	};

	TArray<FOrderedSection> SortedKeys;
	SortedKeys.Reserve(AllSections.Num());

	for (const TPair<FName, SDetailsView::FSection>& SectionPair : AllSections)
	{
		SortedKeys.Emplace(FOrderedSection{ SectionPair.Key, SectionPair.Value.Order });
	}

	SortedKeys.Sort([](const FOrderedSection& A, const FOrderedSection& B)
		{
			static const FName General("General");
			static const FName All("All");

			// General first, All last, rest ordered then alphabetical
			if (A.Name.IsEqual(General) || B.Name.IsEqual(All))
			{
				return true;
			}
			if (A.Name.IsEqual(All) || B.Name.IsEqual(General))
			{
				return false;
			}
			if (A.Order != B.Order)
			{
				return A.Order < B.Order;
			}
			return A.Name.LexicalLess(B.Name);
		});

	for (const FOrderedSection& Key : SortedKeys)
	{
		SectionSelectorBox->AddSlot()
		[
			CreateSection(Key.Name, AllSections[Key.Name].DisplayName)
		];
	}

	SectionSelectorBox->AddSlot()
	[
		CreateSection(FName("All"), NSLOCTEXT("UObjectSection", "All", "All"))
	];

	const bool bAddHelperSections = IsShowHelperSectionsChecked();

	if (bAddHelperSections
		&& DetailsViewArgs.bShowOptions 
		&& DetailsViewArgs.bAllowFavoriteSystem)
	{
		SectionSelectorBox->AddSlot()
		[
			CreateSection(FName("Favorites"), NSLOCTEXT("UObjectSection", "Favorites", "Favorites"))
		];
	}

	if (bAddHelperSections
		&& DetailsViewArgs.bShowOptions 
		&& DetailsViewArgs.bShowModifiedPropertiesOption)
	{
		SectionSelectorBox->AddSlot()
		[
			CreateSection(FName("Modified"), NSLOCTEXT("UObjectSection", "Modified", "Modified"))
		];
	}

	SectionSelectorBox->SetVisibility(EVisibility::Visible);

	CurrentFilter.VisibleSections.Reset();

	for (const TSharedPtr<FComplexPropertyNode>& RootPropertyNode : RootPropertyNodes)
	{
		const UStruct* RootBaseStruct = RootPropertyNode->GetBaseStructure();
		if (const FDetailsViewConfig* ViewConfig = GetConstViewConfig())
		{
			const FDetailsSectionSelection* SectionSelection = ViewConfig->SelectedSections.Find(RootBaseStruct->GetFName());
			if (SectionSelection != nullptr)
			{
				CurrentFilter.VisibleSections = SectionSelection->SectionNames;
			}
		}
	}
}

void SDetailsView::OnSectionCheckedChanged(ECheckBoxState State, FName SectionName)
{
	const bool IsControlDown = FSlateApplication::Get().GetModifierKeys().IsControlDown();

	if (State == ECheckBoxState::Unchecked)
	{
		if (IsControlDown)
		{
			CurrentFilter.VisibleSections.Remove(SectionName);
		}
		else
		{
			CurrentFilter.VisibleSections.Reset();

			if (SectionName != "All")
			{
				CurrentFilter.VisibleSections.Add(SectionName);
			}
		}
	}
	else if (State == ECheckBoxState::Checked)
	{
		if (!IsControlDown)
		{
			CurrentFilter.VisibleSections.Reset();
		}

		if (SectionName != "All")
		{
			CurrentFilter.VisibleSections.Add(SectionName);
		}
	}

	for (const TSharedPtr<FComplexPropertyNode>& RootPropertyNode : RootPropertyNodes)
	{
		const UStruct* RootBaseStruct = RootPropertyNode->GetBaseStructure();
		FDetailsViewConfig* ViewConfig = GetMutableViewConfig();
		if (ViewConfig != nullptr)
		{
			FDetailsSectionSelection& SectionSelection = ViewConfig->SelectedSections.FindOrAdd(RootBaseStruct->GetFName());
			SectionSelection.SectionNames = CurrentFilter.VisibleSections;

			SaveViewConfig();
		}
	}

	UpdateFilteredDetails();
}

ECheckBoxState SDetailsView::IsSectionChecked(FName Section) const
{
	if (CurrentFilter.VisibleSections.IsEmpty() && Section == "All")
	{
		return ECheckBoxState::Checked;
	}

	for (FName VisibleSection : CurrentFilter.VisibleSections)
	{
		if (Section == VisibleSection)
		{
			return ECheckBoxState::Checked;
		}
	}

	return ECheckBoxState::Unchecked;
}

TMap<FName, SDetailsView::FSection> SDetailsView::GetAllSections() const
{
	static const FName PropertyEditor("PropertyEditor");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);

	TMap<FName, SDetailsView::FSection> AllSections;

	// Only include a category if it has children in the tree.
	// Permission-list filtering removes items from the tree entirely (not just visually), so a category
	// with no allowed properties will have no children. We use bInIgnoreVisibility=true so that
	// search/section filtering (which hides but doesn't remove items) doesn't affect this check.
	TArray<FName> LayoutCategories;
	auto AddCategoryIfNotEmpty = [&LayoutCategories](const TSharedRef<IDetailTreeNode>& CategoryNode)
	{
		TArray<TSharedRef<IDetailTreeNode>> Children;
		const bool bInIgnoreVisibility = true;
		CategoryNode->GetChildren(Children, bInIgnoreVisibility);
		if (!Children.IsEmpty())
		{
			LayoutCategories.Add(CategoryNode->GetNodeName());
		}
	};

	// for every category, check every base struct and find the associated section
	// if one exists, add it to the set of valid section names
	// we fetch the list of categories from the layouts since AllTreeNodes gets filtered
	for (const FDetailLayoutData& Layout : DetailLayouts)
	{
		for (const TSharedRef<FDetailTreeNode>& TreeNode : Layout.DetailLayout->GetAllRootTreeNodes())
		{
			if (TreeNode->GetNodeType() == EDetailNodeType::Category)
			{
				AddCategoryIfNotEmpty(TreeNode);
			}
			else if (TreeNode->GetNodeType() == EDetailNodeType::Object)
			{
				TArray<TSharedRef<IDetailTreeNode>> RootObjectChildren;
				TreeNode->GetChildren(RootObjectChildren);

				for (const TSharedRef<IDetailTreeNode>& RootObjectChild : RootObjectChildren)
				{
					if (RootObjectChild->GetNodeType() == EDetailNodeType::Category)
					{
						AddCategoryIfNotEmpty(RootObjectChild);
					}
				}
			}
		}
	}

	for (const TSharedPtr<FComplexPropertyNode>& RootPropertyNode : RootPropertyNodes)
	{
		const UStruct* RootBaseStruct = RootPropertyNode->GetBaseStructure();

		for (const FName& Category : LayoutCategories)
		{
			TArray<TSharedPtr<FPropertySection>> SectionsForCategory = PropertyModule.FindSectionsForCategory(RootBaseStruct, Category);
			for (const TSharedPtr<FPropertySection>& Section : SectionsForCategory)
			{
				AllSections.Add(Section->GetName(), SDetailsView::FSection{ Section->GetDisplayName(), Section->GetOrder() });
			}
		}
	}

	return MoveTemp(AllSections);
}

bool SDetailsView::ResetToDefaultSection()
{
	if (RootPropertyNodes.IsEmpty())
	{
		return false;
	}

	bool bResetViewConfig = false;
	CurrentFilter.VisibleSections.Reset();
	for (const TSharedPtr<FComplexPropertyNode>& RootPropertyNode : RootPropertyNodes)
	{
		if (const UStruct* RootBaseStruct = RootPropertyNode->GetBaseStructure())
		{
			if (FDetailsViewConfig* MutableViewConfig = GetMutableViewConfig())
			{
				FDetailsSectionSelection& SectionSelection = MutableViewConfig->SelectedSections.FindOrAdd(RootBaseStruct->GetFName());
				SectionSelection.SectionNames = CurrentFilter.VisibleSections;
				SaveViewConfig();
				bResetViewConfig |= true;
			}
		}
	}

	return bResetViewConfig;
}

const FSlateBrush* SDetailsView::GetViewOptionsBadgeIcon() const
{
	// Badge the icon if any view option that narrows down the results is checked
	bool bHasBadge = (DetailsViewArgs.bShowModifiedPropertiesOption && IsShowOnlyModifiedChecked() )
					|| (DetailsViewArgs.bShowDifferingPropertiesOption && IsShowOnlyAllowedChecked() )
					|| (DetailsViewArgs.bShowKeyablePropertiesOption && IsShowKeyableChecked() )
					|| (DetailsViewArgs.bShowAnimatedPropertiesOption && IsShowAnimatedChecked() );

	return bHasBadge ? FAppStyle::Get().GetBrush("Icons.BadgeModified") : nullptr;
}

bool SDetailsView::IsDefaultStyle() const
{
	return StyleKeySP.IsValid() && *StyleKeySP.Get() == GetPrimaryDetailsViewStyleKey();
}

TSharedPtr<FDetailsNameWidgetOverrideCustomization> SDetailsView::GetDetailsNameWidgetOverrideCustomization()
{
	return DetailsNameWidgetOverrideCustomization;
}

void SDetailsView::UpdateStyleKey()
{
	const bool IsDefault = FDetailsViewStyleKeys::IsDefault(DisplayManager->GetDetailsViewStyleKey());

	if (IsDefault && DetailsViewArgs.StyleKey.IsValid()) 
	{
		StyleKeySP = DetailsViewArgs.StyleKey;
	}
	else if (!IsDefault)
	{
		FDetailsViewStyleKey Key = DisplayManager->GetDetailsViewStyleKey(); 
		StyleKeySP = MakeShared<FDetailsViewStyleKey>(Key);
	}
	else
	{
		// convert the const shared pointer value into a non-const the pointer can hold
		StyleKeySP = MakeShared<FDetailsViewStyleKey>(GetPrimaryDetailsViewStyleKey());
	} 
}

const FDetailsViewStyleKey& SDetailsView::GetStyleKey()
{
	const FDetailsViewStyleKey& PrimaryKey = GetPrimaryDetailsViewStyleKey();
	return StyleKeySP.IsValid() ? *StyleKeySP.Get() : PrimaryKey;
}

void SDetailsView::RefreshDisplayManager()
{
	TSharedPtr<FDetailsDisplayManager> OldDisplayManager = DisplayManager;
	if (ObjectFilter.IsValid())
	{
		DisplayManager = ObjectFilter->GetDisplayManager();
	}
	if (!DisplayManager.IsValid())
	{
		DisplayManager = MakeShared<FDetailsDisplayManager>();
	}
	if (OldDisplayManager != DisplayManager && ExternalOverlayContainer.IsValid())
	{
		if (DisplayManager->SupportsDetailTreeOverlay())
		{
			ExternalOverlayContainer->SetVisibility(EVisibility::SelfHitTestInvisible);
			ExternalOverlayContainer->SetContent(DisplayManager->ConstructDetailTreeOverlay());
		}
		else
		{
			ExternalOverlayContainer->SetVisibility(EVisibility::Collapsed);
			ExternalOverlayContainer->SetContent(SNullWidget::NullWidget);
		}
	}
}

TSharedPtr<FDetailsDisplayManager> SDetailsView::GetDisplayManager()
{
	RefreshDisplayManager();
	return DisplayManager;
}

const FDetailsViewStyleKey& SDetailsView::GetPrimaryDetailsViewStyleKey()
{
	if (CVarDetailsPanelEnableCardLayout.GetValueOnAnyThread())
	{
		return FDetailsViewStyleKeys::Card();
	}
	return FDetailsViewStyleKeys::Classic();
}

#undef LOCTEXT_NAMESPACE
