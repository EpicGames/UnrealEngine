// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetSelection.h"
#include "IDetailsView.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "PropertyNode.h"
#include "SDetailsViewBase.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SScrollBar.h"

class AActor;
class IDetailRootObjectCustomization;
class FDetailsViewObjectFilter;

class SDetailsView : public SDetailsViewBase
{
	friend class FPropertyDetailsUtilities;
public:

	SLATE_BEGIN_ARGS(SDetailsView){}
	SLATE_END_ARGS()

	virtual ~SDetailsView();

	/** Causes the details view to be refreshed (new widgets generated) with the current set of objects */
	virtual void ForceRefresh() override;

	/** Invalidates cached state such as the "revert to default" arrow and edit conditions, without rebuilding the entire panel. */
	virtual void InvalidateCachedState() override;

	/** Move the scrolling offset (by item), but do not refresh the tree*/
	void MoveScrollOffset(int32 DeltaOffset) override;

	/**
	 * Constructs the property view widgets                   
	 */
	void Construct(const FArguments& InArgs, const FDetailsViewArgs& InDetailsViewArgs);

	/** IDetailsView interface */
	virtual void SetObjects(const TArray<UObject*>& InObjects, bool bForceRefresh = false, bool bOverrideLock = false) override;
	virtual void SetObjects(const TArray<TWeakObjectPtr<UObject>>& InObjects, bool bForceRefresh = false, bool bOverrideLock = false) override;
	virtual void SetObject(UObject* InObject, bool bForceRefresh = false) override;
	virtual bool IsGroupFavorite(FStringView GroupPath) const override;
	virtual void SetGroupFavorite(FStringView GroupPath, bool IsFavorite) override;
	virtual bool IsCustomBuilderFavorite(FStringView Path) const override;
	virtual void SetCustomBuilderFavorite(FStringView Path, bool IsFavorite) override;

	virtual void RemoveInvalidObjects() override;
	virtual void SetObjectPackageOverrides(const TMap<TWeakObjectPtr<UObject>, TWeakObjectPtr<UPackage>>& InMapping) override;
	virtual void SetRootObjectCustomizationInstance(TSharedPtr<IDetailRootObjectCustomization> InRootObjectCustomization) override;
	virtual void SetObjectFilter(TSharedPtr<FDetailsViewObjectFilter> InFilter) override;
	virtual void SetClassViewerFilters(const TArray<TSharedRef<class IClassViewerFilter>>& InFilters) override;

	/**
	 * Replaces objects being observed by the view with new objects
	 *
	 * @param OldToNewObjectMap	Mapping from objects to replace to their replacement
	 */
	void ReplaceObjects(const TMap<UObject*, UObject*>& OldToNewObjectMap);

	/**
	 * Removes objects from the view because they are about to be deleted
	 *
	 * @param DeletedObjects	The objects to delete
	 */
	void RemoveDeletedObjects(const TArray<UObject*>& DeletedObjects);

	/** Sets the callback for when the property view changes */
	virtual void SetOnObjectArrayChanged(FOnObjectArrayChanged OnObjectArrayChangedDelegate) override;

	/** @return	Returns list of selected objects we're inspecting */
	virtual const TArray<TWeakObjectPtr<UObject>>& GetSelectedObjects() const override
	{
		return SelectedObjects;
	} 

	/** @return	Returns list of selected actors we're inspecting */
	virtual const TArray<TWeakObjectPtr<AActor>>& GetSelectedActors() const override
	{
		return SelectedActors;
	}

	/** @return Returns information about the selected set of actors */
	virtual const FSelectedActorInfo& GetSelectedActorInfo() const override
	{
		return SelectedActorInfo;
	}

	virtual bool HasClassDefaultObject() const override
	{
		return bViewingClassDefaultObject;
	}

	virtual bool IsConnected() const override;

	virtual FRootPropertyNodeList& GetRootNodes() override
	{
		return RootPropertyNodes;
	}

	virtual bool DontUpdateValueWhileEditing() const override
	{ 
		return false; 
	}

	virtual bool ContainsMultipleTopLevelObjects() const override
	{
		return DetailsViewArgs.bAllowMultipleTopLevelObjects && GetNumObjects() > 1;
	}

	virtual TSharedPtr<IDetailRootObjectCustomization> GetRootObjectCustomization() const override
	{
		return RootObjectCustomization;
	}
	
	/**
	 * Gets the Shared pointer which points to the @code const FDetailsViewStyleKey& @endcode which is used for
	 * @code SDetailsView @endcode instances for which no other @code FDetailsViewStyleKey @endcode has been provided,
	 * for example through @code FDetailsViewArgs @endcode or an @code FDetailsViewObjectFilter @endcode.
	 *
	 * @return the shared pointer for the Primary details style key 
	 */
	static PROPERTYEDITOR_API const FDetailsViewStyleKey& GetPrimaryDetailsViewStyleKey();

	/**
	* Returns a bool indicating whether the given @code FDetailsViewStyleKey @endcode is the default Details View Style 
	*/
	virtual bool IsDefaultStyle() const override;

	/**
	 * Resets the selected sections to the default 'All' section.
	 *
	 * @return whether the operation succeeded.
	 */
	virtual bool ResetToDefaultSection() override;

private:
	/** Minimal struct to hold section information. Note that the Name is the key, so is not present here. */
	struct FSection
	{
		FText DisplayName;
		int32 Order = 0;
	};

private:
	/**
	 * returns a pointer to a Customization that can be applied to all name widgets, if one exists
	 */
	virtual TSharedPtr<FDetailsNameWidgetOverrideCustomization> GetDetailsNameWidgetOverrideCustomization() override;

	void SetObjectArrayPrivate(const TArray<UObject*>& InObjects);

	TSharedRef<SDetailTree> ConstructTreeView(const TSharedRef<SScrollBar>& ScrollBar );

	/**
	 * Returns whether or not new objects need to be set. If the new objects being set are identical to the objects 
	 * already in the details panel, nothing needs to be set
	 *
	 * @param InObjects The potential new objects to set
	 * @return true if the new objects should be set
	 */
	bool ShouldSetNewObjects(const TArray<UObject*>& InObjects) const;

	/**
	 * Returns the number of objects being edited by this details panel.
	 */
	int32 GetNumObjects() const;

	/** Called before during SetObjectArray before we change the objects being observed */
	void PreSetObject(int32 InNewNumObjects);

	/** Called at the end of SetObjectArray after we change the objects being observed */
	void PostSetObject(const TArray<FDetailsViewObjectRoot>& Roots);
	
	/** Called to get the visibility of the actor name area */
	EVisibility GetActorNameAreaVisibility() const;

	/** Returns the name of the image used for the icon on the locked button */
	const FSlateBrush* OnGetLockButtonImageResource() const;

	/** Whether the property matrix button should be enabled */
	bool CanOpenRawPropertyEditor() const;

	/** Whether the property matrix button should be shown */
	EVisibility CanShowRawPropertyEditorButton(const bool bAllowedByDetailsViewArgs) const;

	/**
	 * Called to open the raw property editor (property matrix)                                                              
	 */
	FReply OnOpenRawPropertyEditorClicked();

	/** @return Returns true if show hidden properties while playing is checked */
	bool IsShowHiddenPropertiesWhilePlayingChecked() const;

	/** Called when show hidden properties while playing is clicked */
	void OnShowHiddenPropertiesWhilePlayingClicked();

	/** @return true if Show Sections is checked. */
	bool IsShowSectionsChecked() const { return DetailsViewArgs.bShowSectionSelector; }

	/** Called when Show Sections is clicked */
	void OnShowSectionsClicked();
	
	/** @return true if Show Helper Sections is checked. */
	bool IsShowHelperSectionsChecked() const;
	
	/** Called when Show Helper Sections is clicked */
	void OnShowHelperSectionsClicked();

	/** Get the color of the toggle favorites button. */
	FSlateColor GetToggleFavoritesColor() const;

	/** Called when the toggle favorites button is clicked. */
	FReply OnToggleFavoritesClicked();

	/** Called after an undo or redo operation occurs in the editor. */
	void OnPostUndoRedo();

	/** Get all sections, keyed by name for the objects currently selected in the view. */
	TMap<FName, FSection> GetAllSections() const;

	/** Rebuild the section selector widget after a selection has been changed. */
	void RebuildSectionSelector();

	/** Refilter the details view after the user has selected a new section. */
	void OnSectionCheckedChanged(ECheckBoxState State, FName NewSelection);

	/** Get the currently selected section. */
	ECheckBoxState IsSectionChecked(FName Section) const;

	/** Get the badge for the view options icon (if there are any options active */
	const FSlateBrush* GetViewOptionsBadgeIcon() const;

	/**
	* Updates @code FDetailsViewStyleKey& StyleKey @endcode for the current @code SDetailsView @endcode state
	*/
	virtual void UpdateStyleKey() override;

	/**
	* Gets the @code FDetailsViewStyleKey& @endcode which provides a Key to the current style for a Details View
	*/
	virtual const FDetailsViewStyleKey& GetStyleKey() override;

	/**
	 * Returns a @code TSharedPtr @endcode to the @code FDetailsDisplayManager @endcode
	 */
	virtual TSharedPtr<FDetailsDisplayManager> GetDisplayManager() override;

private:

	/**
	 * Refreshes the class member @code TSharedPtr<FDetailsDisplayManager> DisplayManager @endcode. This is required
	 * anytime the data in the display manager may have grown stale.
	 */
	void RefreshDisplayManager();

	/**
	* The @code FDetailsViewStyle& @endcode shared pointer which provides style
	* information for this details view.
	*/
	TSharedPtr<FDetailsViewStyleKey> StyleKeySP;
	
	/** A Customization that can be applied to all name widgets */
	TSharedPtr<FDetailsNameWidgetOverrideCustomization> DetailsNameWidgetOverrideCustomization;;
	
	/** The filter for objects viewed by this details panel */
	TSharedPtr<FDetailsViewObjectFilter> ObjectFilter;

	/** Information about the current set of selected actors */
	FSelectedActorInfo SelectedActorInfo;

	/** Set of selected objects for this detail view that were passed in through SetObjects (before the object filter is applied). */
	TArray<TWeakObjectPtr<UObject>> UnfilteredSelectedObjects;

	/** Final set of selected objects for this detail view after applying the object filter. It may be different from the set passed in through SetObjects. */
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;

	/** 
	 * Selected actors for this detail view.  Note that this is not necessarily the same editor selected actor set.  If this detail view is locked
	 * It will only contain actors from when it was locked 
	 */
	TArray< TWeakObjectPtr<AActor> > SelectedActors;
	/** The root property nodes of the property tree for a specific set of UObjects */
	TArray<TSharedPtr<FComplexPropertyNode>> RootPropertyNodes;
	/** Callback to send when the property view changes */
	FOnObjectArrayChanged OnObjectArrayChanged;
	/** Customization instance used when there are multiple top level objects in this view */
	TSharedPtr<IDetailRootObjectCustomization> RootObjectCustomization;
	/** True if at least one viewed object is a CDO (blueprint editing) */
	bool bViewingClassDefaultObject;
	/** Delegate handle for unregistering from the PostUndoRedo event. */
	FDelegateHandle PostUndoRedoDelegateHandle;
	/** The section selector widget to show if DetailsViewArgs.bShowSectionSelector is true. */
	TSharedPtr<class SWrapBox> SectionSelectorBox;
	/** True when currently in the middle of a refreshing the object array */
	bool bIsRefreshing;
};
