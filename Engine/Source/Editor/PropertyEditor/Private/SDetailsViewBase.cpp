// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDetailsViewBase.h"

#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "Containers/StringFwd.h"
#include "Engine/Engine.h"
#include "GameFramework/Actor.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SSearchBox.h"
#include "Framework/Application/SlateApplication.h"
#include "DetailCategoryBuilderImpl.h"
#include "DetailLayoutBuilderImpl.h"
#include "DetailLayoutHelpers.h"
#include "Editor.h"
#include "IDetailCustomization.h"
#include "LevelEditorActions.h"
#include "ObjectPropertyNode.h"
#include "PropertyEditorCommands.h"
#include "PropertyPermissionList.h"
#include "ScopedTransaction.h"
#include "SDetailNameArea.h"
#include "ThumbnailRendering/ThumbnailManager.h"

static FString DetailPropertyExpansion = TEXT("DetailPropertyExpansion");
static FString TransientPerViewDetailPropertyExpansion = TEXT("TransientPerViewDetailPropertyExpansion");
static FString PersistentPerViewDetailPropertyExpansion = TEXT("PersistentPerViewDetailPropertyExpansion");

static FString DetailCustomWidgetExpansion = TEXT("DetailCustomWidgetExpansion");
static FString TransientPerViewDetailCustomWidgetExpansion = TEXT("TransientPerViewDetailCustomWidgetExpansion");
static FString PersistentPerViewDetailCustomWidgetExpansion = TEXT("PersistentPerViewDetailCustomWidgetExpansion");

void SDetailsViewBase::ResetTransientPerViewDetailExpansion()
{
	GConfig->RemoveSectionFromBranch(*TransientPerViewDetailPropertyExpansion, *GEditorPerProjectIni);
	GConfig->RemoveSectionFromBranch(*TransientPerViewDetailCustomWidgetExpansion, *GEditorPerProjectIni);
	GConfig->RemoveSectionFromBranch(*FDetailCategoryImpl::TransientPerViewDetailCategories, *GEditorPerProjectIni);
}

void SDetailsViewBase::ResetPersistentPerViewDetailExpansion()
{
	GConfig->RemoveSectionFromBranch(*PersistentPerViewDetailPropertyExpansion, *GEditorPerProjectIni);
	GConfig->RemoveSectionFromBranch(*PersistentPerViewDetailCustomWidgetExpansion, *GEditorPerProjectIni);
	GConfig->RemoveSectionFromBranch(*FDetailCategoryImpl::PersistentPerViewDetailCategories, *GEditorPerProjectIni);
}

static FAutoConsoleCommand ResetTransientPerViewDetailExpansionCommand(
	TEXT("PropertyEditor.ResetTransientPerViewDetailExpansion"),
	TEXT("Removes all transient per view detail expansions. These or automatically purged without read on editor startup to avoid incremental file bloat."),
	FConsoleCommandDelegate::CreateStatic(&SDetailsViewBase::ResetTransientPerViewDetailExpansion)
);

static FAutoConsoleCommand ResetPersistentPerViewDetailExpansionCommand(
	TEXT("PropertyEditor.ResetPersistentPerViewDetailExpansion"),
	TEXT("Removes all persistent per view detail expansions."),
	FConsoleCommandDelegate::CreateStatic(&SDetailsViewBase::ResetPersistentPerViewDetailExpansion)
);

SDetailsViewBase::SDetailsViewBase() :
	  NumVisibleTopLevelObjectNodes(0)
	, bIsLocked(false)
	, bHasOpenColorPicker(false)
	, bDisableCustomDetailLayouts(false)
	, bPendingCleanupTimerSet(false)
	, bRunningDeferredActions(false)
{
	UDetailsConfig::Initialize();
	UDetailsConfig::Get()->LoadEditorConfig();

	const FDetailsViewConfig* ViewConfig = GetConstViewConfig();
	if (ViewConfig != nullptr)
	{
		CurrentFilter.bShowAllAdvanced = ViewConfig->bShowAllAdvanced;
		CurrentFilter.bShowAllChildrenIfCategoryMatches = ViewConfig->bShowAllChildrenIfCategoryMatches;
		CurrentFilter.bShowOnlyAnimated = ViewConfig->bShowOnlyAnimated;
		CurrentFilter.bShowOnlyKeyable = ViewConfig->bShowOnlyKeyable;
		CurrentFilter.bShowOnlyModified = ViewConfig->bShowOnlyModified;
	}

	// RequestForceRefresh is deferred until next tick to avoid the editor locking up for several minutes in one frame when refreshing multiple times
	PropertyPermissionListChangedDelegate = FPropertyEditorPermissionList::Get().PermissionListUpdatedDelegate.AddLambda([this](TSoftObjectPtr<const UStruct> Struct, FName Owner) { RequestForceRefresh(); });
	PropertyPermissionListEnabledDelegate = FPropertyEditorPermissionList::Get().PermissionListEnabledDelegate.AddRaw(this, &SDetailsViewBase::RequestForceRefresh);

	OnCultureChangedDelegate = FInternationalization::Get().OnCultureChanged().AddRaw(this, &SDetailsViewBase::RequestForceRefresh);
}

SDetailsViewBase::~SDetailsViewBase()
{
	FPropertyEditorPermissionList::Get().PermissionListUpdatedDelegate.Remove(PropertyPermissionListChangedDelegate);
	FPropertyEditorPermissionList::Get().PermissionListEnabledDelegate.Remove(PropertyPermissionListEnabledDelegate);

	if (FInternationalization::IsAvailable())
	{
		FInternationalization::Get().OnCultureChanged().Remove(OnCultureChangedDelegate);
	}
}

void SDetailsViewBase::ClearSearch()
{
	CurrentFilter.FilterStrings.Empty();
	SearchBox->SetText(FText::GetEmpty());
	RerunCurrentFilter();
}

void SDetailsViewBase::SetSearchText(const FText& InSearch)
{
	SearchBox->SetText(InSearch);
}

void SDetailsViewBase::OnGetChildrenForDetailTree(TSharedRef<FDetailTreeNode> InTreeNode, TArray< TSharedRef<FDetailTreeNode> >& OutChildren)
{
	InTreeNode->GetChildren(OutChildren);
}

TSharedRef<ITableRow> SDetailsViewBase::OnGenerateRowForDetailTree(TSharedRef<FDetailTreeNode> InTreeNode, const TSharedRef<STableViewBase>& OwnerTable)
{
	return InTreeNode->GenerateWidgetForTableView(OwnerTable, DetailsViewArgs.bAllowFavoriteSystem);
}

void SDetailsViewBase::ClearKeyboardFocusIfWithin(const TSharedRef<SWidget>& Widget) const
{
	// search upwards from the current keyboard-focused widget to see if it's contained in our row
	TSharedPtr<SWidget> CurrentWidget = FSlateApplication::Get().GetKeyboardFocusedWidget();
	while (CurrentWidget.IsValid())
	{
		if (CurrentWidget == Widget)
		{
			// if so, clear focus so that any pending value changes are committed
			FSlateApplication::Get().ClearKeyboardFocus();
			return;
		}
		CurrentWidget = CurrentWidget->GetParentWidget();
	}
}

FDetailFilter SDetailsViewBase::CreateDetailFilter()
{
	FDetailFilter Filter = CurrentFilter;
	
	// When "Favorites" section is active, show favorited items
	Filter.bShowFavoritesCategory |= CurrentFilter.VisibleSections.Contains(TEXT("Favorites"));

	// When "Modified" section is active, show all items with modified properties
	if (CurrentFilter.VisibleSections.Contains(TEXT("Modified")))
	{
		Filter.VisibleSections.Empty();
		Filter.bShowOnlyModified |= true;
	}

	return Filter;
}

void SDetailsViewBase::BindCommands()
{
	if (InternalCommands.IsValid())
	{
		return;
	}

	// Register commands here otherwise they are destroyed...
	FPropertyEditorCommands::Register();
	const FPropertyEditorCommands& PropertyEditorCommands = FPropertyEditorCommands::Get();

	InternalCommands = MakeShared<FUICommandList>();
	InternalCommands->MapAction(
		PropertyEditorCommands.CollapseAll, 
		FExecuteAction::CreateSP(this, &SDetailsViewBase::SetRootExpansionStates, /** Expand */false, /** Recurse */false)
	);

	InternalCommands->MapAction(
		PropertyEditorCommands.ExpandAll, 
		FExecuteAction::CreateSP(this, &SDetailsViewBase::SetRootExpansionStates, /** Expand */true, /** Recurse */false)
	);
}

void SDetailsViewBase::OnRowReleasedForDetailTree(const TSharedRef<ITableRow>& TableRow)
{
	ClearKeyboardFocusIfWithin(TableRow->AsWidget());
}

void SDetailsViewBase::SetRootExpansionStates(const bool bExpand, const bool bRecurse)
{
	if(ContainsMultipleTopLevelObjects())
	{
		FDetailNodeList Children;
		for(auto Iter = RootTreeNodes.CreateIterator(); Iter; ++Iter)
		{
			Children.Reset();
			(*Iter)->GetChildren(Children);

			for(TSharedRef<FDetailTreeNode>& Child : Children)
			{
				SetNodeExpansionState(Child, bExpand, bRecurse);
			}
		}
	}
	else
	{
		for(auto Iter = RootTreeNodes.CreateIterator(); Iter; ++Iter)
		{
			SetNodeExpansionState(*Iter, bExpand, bRecurse);
		}
	}
}

void SDetailsViewBase::SetNodeExpansionState(TSharedRef<FDetailTreeNode> InTreeNode, bool bExpand, bool bRecursive)
{
	TArray< TSharedRef<FDetailTreeNode> > Children;
	InTreeNode->GetChildren(Children);

	RequestItemExpanded(InTreeNode, bExpand);
	constexpr bool bShouldSaveState = true;
	InTreeNode->OnItemExpansionChanged(bExpand, bShouldSaveState);

	if (bRecursive && !Children.IsEmpty())
	{
		TArray<FString> TypesToIgnoreForRecursiveExpansion;
		GConfig->GetArray(TEXT("DetailPropertyExpansion"), TEXT("TypesToIgnoreForRecursiveExpansion"), TypesToIgnoreForRecursiveExpansion, GEditorPerProjectIni);

		for (const TSharedRef<FDetailTreeNode>& Child : Children)
		{
			if (bExpand)
			{
				const FProperty* ChildProperty = Child->GetPropertyNode().IsValid() ? Child->GetPropertyNode()->GetProperty() : nullptr;
				if (ChildProperty != nullptr)
				{
					if (TypesToIgnoreForRecursiveExpansion.Contains(ChildProperty->GetClass()->GetName()))
					{
						continue;
					}
				}
			}


			SetNodeExpansionState(Child, bExpand, bRecursive);
		}
	}
}

void SDetailsViewBase::SetNodeExpansionStateRecursive(TSharedRef<FDetailTreeNode> InTreeNode, bool bIsItemExpanded)
{
	SetNodeExpansionState(InTreeNode, bIsItemExpanded, true);
}

void SDetailsViewBase::OnItemExpansionChanged(TSharedRef<FDetailTreeNode> InTreeNode, bool bIsItemExpanded)
{
	SetNodeExpansionState(InTreeNode, bIsItemExpanded, false);
}

FReply SDetailsViewBase::OnLockButtonClicked()
{
	bIsLocked = !bIsLocked;
	return FReply::Handled();
}

void SDetailsViewBase::HideFilterArea(bool bHide)
{
	DetailsViewArgs.bAllowSearch = !bHide;
}

static void GetPropertiesInOrderDisplayedRecursive(const TArray< TSharedRef<FDetailTreeNode> >& TreeNodes, TArray< FPropertyPath > &OutLeaves)
{
	for( const TSharedRef<FDetailTreeNode>& TreeNode : TreeNodes )
	{
		const bool bIsPropertyRoot = TreeNode->IsLeaf() ||
			(	TreeNode->GetPropertyNode() && 
				TreeNode->GetPropertyNode()->GetProperty() );
		if( bIsPropertyRoot )
		{
			FPropertyPath Path = TreeNode->GetPropertyPath();
			// Some leaf nodes are not associated with properties, specifically the collision presets.
			// @todo doc: investigate what we can do about this, result is that for these fields
			// we can't highlight the property in the diff tool.
			if( Path.GetNumProperties() != 0 )
			{
				OutLeaves.Push(Path);
			}
		}
		else
		{
			TArray< TSharedRef<FDetailTreeNode> > Children;
			TreeNode->GetChildren(Children);
			GetPropertiesInOrderDisplayedRecursive(Children, OutLeaves);
		}
	}
}
TArray< FPropertyPath > SDetailsViewBase::GetPropertiesInOrderDisplayed() const
{
	TArray< FPropertyPath > Ret;
	GetPropertiesInOrderDisplayedRecursive(RootTreeNodes, Ret);
	return Ret;
}

static void GetPropertyRowRowNumbersRecursive(const TArray<TSharedRef<FDetailTreeNode>>& TreeNodes, int32& CurrentRowNumber, const TSharedPtr<SDetailTree>& DetailTree, TArray<TPair<int32, FPropertyPath>>& OutResults)
{
	for (const TSharedRef<FDetailTreeNode>& TreeNode : TreeNodes)
	{
		// pre-order traversal (add parents to list before their children)
		FPropertyPath Path = TreeNode->GetPropertyPath();

		if(Path.IsValid())
		{
			OutResults.Add(TPair<int32, FPropertyPath>(CurrentRowNumber, Path));
		}
		++CurrentRowNumber;

		if (!TreeNode->IsLeaf() && DetailTree->IsItemExpanded(TreeNode))
		{
			TArray<TSharedRef<FDetailTreeNode>> Children;
			TreeNode->GetChildren(Children);
			GetPropertyRowRowNumbersRecursive(Children, CurrentRowNumber, DetailTree, OutResults);
		}
	}
}

TArray<TPair<int32, FPropertyPath>> SDetailsViewBase::GetPropertyRowNumbers() const
{
	TArray<TPair<int32, FPropertyPath>> Results;
	int32 OverallIdx = 0;
	GetPropertyRowRowNumbersRecursive(RootTreeNodes, OverallIdx, DetailTree, Results);
	return Results;
}

static int32 CountRowsRecursive(const TArray<TSharedRef<FDetailTreeNode>>& TreeNodes, const TSharedPtr<SDetailTree>& DetailTree)
{
	int32 Count = 0;
	for (const TSharedRef<FDetailTreeNode>& TreeNode : TreeNodes)
	{
		++Count;
		
		if (!TreeNode->IsLeaf() && DetailTree->IsItemExpanded(TreeNode))
		{
			TArray<TSharedRef<FDetailTreeNode>> Children;
			TreeNode->GetChildren(Children);
			Count += CountRowsRecursive(Children, DetailTree);
		}
	}
	return Count;
}

int32 SDetailsViewBase::CountRows() const
{
	return CountRowsRecursive(RootTreeNodes, DetailTree);
}

// construct a map for fast FPropertyPath.ToString() -> FDetailTreeNode lookup
static void GetPropertyPathToTreeNodes(const TArray<TSharedRef<FDetailTreeNode>>& RootTreeNodes, TMap<FString, TSharedRef<FDetailTreeNode>> &OutMap)
{
	for (const TSharedRef<FDetailTreeNode>& TreeNode : RootTreeNodes)
	{
		FPropertyPath Path = TreeNode->GetPropertyPath();
		if (Path.IsValid())
		{
			OutMap.Add(Path.ToString(), TreeNode);
		}

		// Need to check children even if we're a leaf, because all DetailItemNodes are leaves, even if they may have sub-children
		TArray< TSharedRef<FDetailTreeNode> > Children;
		TreeNode->GetChildren(Children);
		GetPropertyPathToTreeNodes(Children, OutMap);
	}
}

// Find an FDetailTreeNode in the tree that most closely matches the provided property.
// If there isn't an exact match, the provided property will be trimmed until we find a match
static TSharedPtr<FDetailTreeNode> FindBestFitTreeNodeFromProperty(const TArray<TSharedRef<FDetailTreeNode>>& RootTreeNodes, FPropertyPath Property)
{
	TMap<FString, TSharedRef<FDetailTreeNode>> PropertyPathToTreeNodes;
	GetPropertyPathToTreeNodes(RootTreeNodes, PropertyPathToTreeNodes);
	
	TSharedRef< FDetailTreeNode >* TreeNode = PropertyPathToTreeNodes.Find(Property.ToString());

	// if we couldn't find the exact property, trim the path to get the next best option
	while (!TreeNode && Property.GetNumProperties() > 1)
	{
		Property = *Property.TrimPath(1);
		TreeNode = PropertyPathToTreeNodes.Find(Property.ToString());
	}
	return TreeNode? *TreeNode : TSharedPtr<FDetailTreeNode>();
}

void SDetailsViewBase::HighlightProperty(const FPropertyPath& Property)
{
	TSharedPtr<FDetailTreeNode> PrevHighlightedNodePtr = CurrentlyHighlightedNode.Pin();
	if (PrevHighlightedNodePtr.IsValid())
	{
		PrevHighlightedNodePtr->SetIsHighlighted(false);
	}
	
	if (!Property.IsValid())
	{
		CurrentlyHighlightedNode = nullptr;
		return;
	}

	const TSharedPtr< FDetailTreeNode > TreeNode = FindBestFitTreeNodeFromProperty(RootTreeNodes, Property);
	if (TreeNode.IsValid())
	{
		// highlight the found node
		TreeNode->SetIsHighlighted(true);

		// make sure all ancestors are expanded so we can see the found node
		TSharedPtr< FDetailTreeNode > Ancestor = TreeNode;
		while(Ancestor->GetParentNode().IsValid())
		{
			Ancestor = Ancestor->GetParentNode().Pin();
			DetailTree->SetItemExpansion(Ancestor.ToSharedRef(), true);
		}
		
		// scroll to the found node
		DetailTree->RequestScrollIntoView(TreeNode.ToSharedRef());
	}
	
	CurrentlyHighlightedNode = TreeNode;
}

void SDetailsViewBase::ScrollPropertyIntoView(const FPropertyPath& Property, const bool bExpandProperty)
{
	if (!Property.IsValid())
	{
		return;
	}

	const TSharedPtr<FDetailTreeNode> TreeNode = FindBestFitTreeNodeFromProperty(RootTreeNodes, Property);
	if (TreeNode.IsValid())
	{
		// make sure all ancestors are expanded so we can see the found node
		TSharedPtr<FDetailTreeNode> Ancestor = TreeNode;
		while(Ancestor->GetParentNode().IsValid())
		{
			Ancestor = Ancestor->GetParentNode().Pin();
			DetailTree->SetItemExpansion(Ancestor.ToSharedRef(), true);
		}

		if (bExpandProperty)
		{
			DetailTree->SetItemExpansion(TreeNode.ToSharedRef(), true);
		}
		
		// scroll to the found node
		DetailTree->RequestScrollIntoView(TreeNode.ToSharedRef());
	}
}

static void ExpandPaintSpacePropertyBoundsRecursive(const TArray<TSharedRef<FDetailTreeNode>>& TreeNodes, TSharedPtr<SDetailTree> DetailTree, FSlateRect& InOutRect)
{
	for(const TSharedRef<FDetailTreeNode>& TreeNode : TreeNodes)
	{
		if (const TSharedPtr<ITableRow> Row = DetailTree->WidgetFromItem(TreeNode))
		{
			if (const TSharedPtr<SWidget> Widget = Row->AsWidget())
			{
				FSlateRect ChildRect = Widget->GetPaintSpaceGeometry().GetLayoutBoundingRect();
				InOutRect = InOutRect.IsValid() ? InOutRect.Expand(ChildRect) : ChildRect;
			}
		}
		
		if (!TreeNode->IsLeaf() && DetailTree->IsItemExpanded(TreeNode))
		{
			TArray<TSharedRef<FDetailTreeNode>> Children;
			TreeNode->GetChildren(Children);
			ExpandPaintSpacePropertyBoundsRecursive(Children, DetailTree, InOutRect);
		}
	}
}

FSlateRect SDetailsViewBase::GetPaintSpacePropertyBounds(const TSharedRef<FDetailTreeNode>& InDetailTreeNode, bool bIncludeChildren) const
{
	const TSharedRef<FDetailTreeNode> DetailTreeNode = StaticCastSharedRef<FDetailTreeNode>(InDetailTreeNode);
	FSlateRect Result{-1,-1,-1,-1};
	if (const TSharedPtr<ITableRow> Row = DetailTree->WidgetFromItem(DetailTreeNode))
	{
		if (const TSharedPtr<SWidget> Widget = Row->AsWidget())
		{
			Result = Widget->GetPaintSpaceGeometry().GetLayoutBoundingRect();
		}
	}
	if (bIncludeChildren && !DetailTreeNode->IsLeaf() && DetailTree->IsItemExpanded(DetailTreeNode))
	{
		TArray<TSharedRef<FDetailTreeNode>> Children;
		DetailTreeNode->GetChildren(Children);
		ExpandPaintSpacePropertyBoundsRecursive(Children, DetailTree, Result);
	}
	return Result;
}

static void ExpandTickSpacePropertyBoundsRecursive(const TArray<TSharedRef<FDetailTreeNode>>& TreeNodes, TSharedPtr<SDetailTree> DetailTree, FSlateRect& InOutRect)
{
	for(const TSharedRef<FDetailTreeNode>& TreeNode : TreeNodes)
	{
		if (const TSharedPtr<ITableRow> Row = DetailTree->WidgetFromItem(TreeNode))
		{
			if (const TSharedPtr<SWidget> Widget = Row->AsWidget())
			{
				FSlateRect ChildRect = Widget->GetTickSpaceGeometry().GetLayoutBoundingRect();
				InOutRect = InOutRect.IsValid() ? InOutRect.Expand(ChildRect) : ChildRect;
			}
		}
		
		if (!TreeNode->IsLeaf() && DetailTree->IsItemExpanded(TreeNode))
		{
			TArray<TSharedRef<FDetailTreeNode>> Children;
			TreeNode->GetChildren(Children);
			ExpandTickSpacePropertyBoundsRecursive(Children, DetailTree, InOutRect);
		}
	}
}

FSlateRect SDetailsViewBase::GetTickSpacePropertyBounds(const TSharedRef<FDetailTreeNode>& InDetailTreeNode, bool bIncludeChildren) const
{
	const TSharedRef<FDetailTreeNode> DetailTreeNode = StaticCastSharedRef<FDetailTreeNode>(InDetailTreeNode);
	FSlateRect Result{-1,-1,-1,-1};
	if (const TSharedPtr<ITableRow> Row = DetailTree->WidgetFromItem(DetailTreeNode))
	{
		if (const TSharedPtr<SWidget> Widget = Row->AsWidget())
		{
			Result = Widget->GetTickSpaceGeometry().GetLayoutBoundingRect();
		}
	}
	if (bIncludeChildren && !DetailTreeNode->IsLeaf() && DetailTree->IsItemExpanded(DetailTreeNode))
	{
		TArray<TSharedRef<FDetailTreeNode>> Children;
		DetailTreeNode->GetChildren(Children);
		ExpandTickSpacePropertyBoundsRecursive(Children, DetailTree, Result);
	}
	return Result;
}

bool SDetailsViewBase::IsAncestorCollapsed(const TSharedRef<IDetailTreeNode>& Node) const
{
	// in order for a node to be expanded, all it's parents have to be expanded
	bool bIsCollapsed = false;
	TSharedPtr<FDetailTreeNode> Ancestor = StaticCastSharedRef<FDetailTreeNode>(Node)->GetParentNode().Pin();
	while (!bIsCollapsed && Ancestor)
	{
		bIsCollapsed = !Ancestor->ShouldShowOnlyChildren() && !DetailTree->IsItemExpanded(Ancestor.ToSharedRef());
		Ancestor = Ancestor->GetParentNode().Pin();
	}
	return bIsCollapsed;
}

void SDetailsViewBase::ShowAllAdvancedProperties()
{
	CurrentFilter.bShowAllAdvanced = true;
}

void SDetailsViewBase::SetOnDisplayedPropertiesChanged(FOnDisplayedPropertiesChanged InOnDisplayedPropertiesChangedDelegate)
{
	OnDisplayedPropertiesChangedDelegate = InOnDisplayedPropertiesChangedDelegate;
}

void SDetailsViewBase::GetHeadNodes(TArray<TWeakPtr<FDetailTreeNode>>& OutNodes)
{
	for (TSharedRef<FDetailTreeNode>& Node : RootTreeNodes)
	{
		OutNodes.Add(Node.ToWeakPtr());
	}
}

float SDetailsViewBase::GetRightColumnMinWidth() const
{
	return ColumnSizeData.GetRightColumnMinWidth().Get(0);
}

void SDetailsViewBase::SetRightColumnMinWidth(float InMinWidth)
{
	ColumnSizeData.SetRightColumnMinWidth(InMinWidth);
}

void SDetailsViewBase::RerunCurrentFilter()
{
	UpdateFilteredDetails();
}

EVisibility SDetailsViewBase::GetTreeVisibility() const
{
	for(const FDetailLayoutData& Data : DetailLayouts)
	{
		if(Data.DetailLayout.IsValid() && Data.DetailLayout->HasDetails())
		{
			return EVisibility::Visible;
		}
	}

	return DetailTree->IsPendingRefresh() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SDetailsViewBase::GetScrollBarVisibility() const
{
	const bool bHasAnythingToShow = RootTreeNodes.Num() > 0;
	const bool bIsScrollbarNeeded = DisplayManager.IsValid() ? DisplayManager->GetIsScrollBarNeeded() : true;
	const bool bShowScrollBar = DetailsViewArgs.bShowScrollBar && bHasAnythingToShow && bIsScrollbarNeeded;
	return bShowScrollBar ? EVisibility::Visible : EVisibility::Collapsed;
}

/** Returns the image used for the icon on the filter button */
const FSlateBrush* SDetailsViewBase::OnGetFilterButtonImageResource() const
{
	if (HasActiveSearch())
	{
		return FAppStyle::GetBrush(TEXT("PropertyWindow.FilterCancel"));
	}
	else
	{
		return FAppStyle::GetBrush(TEXT("PropertyWindow.FilterSearch"));
	}
}

void SDetailsViewBase::EnqueueDeferredAction(FSimpleDelegate& DeferredAction)
{
	DeferredActions.Add(DeferredAction);
}

namespace UE::PropertyEditor::Private
{
/** Set the color for the property node */
void SDetailsViewBase_SetColor(FLinearColor NewColor, TWeakPtr<IPropertyHandle> WeakPropertyHandle)
{
	if (TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin())
	{
		const FProperty* NodeProperty = PropertyHandle->GetProperty();
		check(NodeProperty);

		const UScriptStruct* Struct = CastField<FStructProperty>(NodeProperty)->Struct;
		if (Struct->GetFName() == NAME_Color)
		{
			const bool bSRGB = true;
			FColor NewFColor = NewColor.ToFColor(bSRGB);
			ensure(PropertyHandle->SetValueFromFormattedString(NewFColor.ToString(), EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
		}
		else
		{
			check(Struct->GetFName() == NAME_LinearColor);
			ensure(PropertyHandle->SetValueFromFormattedString(NewColor.ToString(), EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
		}
	}
}
}//namespace

void SDetailsViewBase::CreateColorPickerWindow(const TSharedRef< FPropertyEditor >& PropertyEditor, bool bUseAlpha)
{
	const FProperty* Property = PropertyEditor->GetProperty();
	check(Property);

	FReadAddressList ReadAddresses;
	PropertyEditor->GetPropertyNode()->GetReadAddress(false, ReadAddresses, false);

	TOptional<FLinearColor> DefaultColor;
	bool bClampValue = false;
	if (ReadAddresses.Num())
	{
		for (int32 ColorIndex = 0; ColorIndex < ReadAddresses.Num(); ++ColorIndex)
		{
			const uint8* Addr = ReadAddresses.GetAddress(ColorIndex);
			if (Addr)
			{
				if (CastField<FStructProperty>(Property)->Struct->GetFName() == NAME_Color)
				{
					DefaultColor = *reinterpret_cast<const FColor*>(Addr);
					bClampValue = true;
				}
				else
				{
					check(CastField<FStructProperty>(Property)->Struct->GetFName() == NAME_LinearColor);
					DefaultColor = *reinterpret_cast<const FLinearColor*>(Addr);
				}
			}
		}
	}

	if (DefaultColor.IsSet())
	{
		bHasOpenColorPicker = true;
		ColorPropertyNode = PropertyEditor->GetPropertyNode();

		TWeakPtr<IPropertyHandle> WeakPropertyHandle = PropertyEditor->GetPropertyHandle();
		FColorPickerArgs PickerArgs = FColorPickerArgs(DefaultColor.GetValue(), FOnLinearColorValueChanged::CreateStatic(&UE::PropertyEditor::Private::SDetailsViewBase_SetColor, WeakPropertyHandle));
		PickerArgs.ParentWidget = AsShared();
		PickerArgs.bUseAlpha = bUseAlpha;
		PickerArgs.bClampValue = bClampValue;
		PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
		PickerArgs.OnColorPickerWindowClosed = FOnWindowClosed::CreateSP(this, &SDetailsViewBase::OnColorPickerWindowClosed);
		PickerArgs.OptionalOwningDetailsView = AsShared();
		OpenColorPicker(PickerArgs);
	}
}

void SDetailsViewBase::UpdatePropertyMaps()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SDetailsView::UpdatePropertyMaps);

	RootTreeNodes.Empty();

	for(FDetailLayoutData& LayoutData : DetailLayouts)
	{
		// Allow customizations to perform cleanup as the delete occurs later on
		for (TSharedPtr<IDetailCustomization>& DetailCustomization : LayoutData.CustomizationClassInstances)
		{
			if (DetailCustomization.IsValid())
			{
				DetailCustomization->PendingDelete();
			}
		}

		// All the current customization instances need to be deleted when it is safe
		CustomizationClassInstancesPendingDelete.Append(LayoutData.CustomizationClassInstances);

		// All the current detail layouts need to be deleted when it is safe
		DetailLayoutsPendingDelete.Add(LayoutData.DetailLayout);
	}

	if (DetailLayouts.Num() > 0)
	{
		// Set timer to free memory even when widget is not visible or ticking
		SetPendingCleanupTimer();
	}

	FRootPropertyNodeList& RootPropertyNodes = GetRootNodes();
	
	DetailLayouts.Empty(RootPropertyNodes.Num());

	// There should be one detail layout for each root node
	DetailLayouts.AddDefaulted(RootPropertyNodes.Num());

	for(int32 RootNodeIndex = 0; RootNodeIndex < RootPropertyNodes.Num(); ++RootNodeIndex)
	{
		FDetailLayoutData& LayoutData = DetailLayouts[RootNodeIndex];
		UpdateSinglePropertyMap(RootPropertyNodes[RootNodeIndex], LayoutData, false);
	}
}

void SDetailsViewBase::GetExpansionState(UStruct* BaseStruct, FStringPrefixTree& OutPrefixTree) const
{
	//while a valid class, and we're either the same as the base class (for multiple actors being selected and base class is AActor) OR we're not down to AActor yet)
	TArray<FString> DetailPropertyExpansionStrings;
	for (UStruct* Struct = BaseStruct; Struct && ((BaseStruct == Struct) || (Struct != AActor::StaticClass())); Struct = Struct->GetSuperStruct())
	{
		GConfig->GetSingleLineArray(*GetDetailPropertyExpansionSection(), *GetStructExpansionConfigKey(BaseStruct), DetailPropertyExpansionStrings, GEditorPerProjectIni);
	}

	OutPrefixTree.InsertAll(DetailPropertyExpansionStrings);
}

void SDetailsViewBase::UpdatePropertyMapsWithItemExpansions()
{
	// Restore expanded items is called twice. The first time handles the existing property nodes,
	// and second time after handling property customization as it might add new items that need to be expanded too.

	// Collect the expansion states here, since the first call to RestoreAllExpandedItems() might overwrite the values and that may prevent some customization expansions to be saved.
	TMap<UStruct*, FStringPrefixTree> ExpansionStates;
	for (TSharedPtr<FComplexPropertyNode>& RootPropertyNode : GetRootNodes())
	{
		check(RootPropertyNode.IsValid());
		UStruct* BaseStruct = GetExpansionStruct(RootPropertyNode.ToSharedRef());
		FStringPrefixTree& PrefixTree = ExpansionStates.Add(BaseStruct);
		GetExpansionState(BaseStruct, PrefixTree);
	}

	for (const FDetailLayoutData& LayoutData : DetailLayouts)
	{
		FRootPropertyNodeList& ExternalRootPropertyNodes = LayoutData.DetailLayout->GetExternalRootPropertyNodes();
		for (TSharedPtr<FComplexPropertyNode>& ExternalRootPropertyNode : ExternalRootPropertyNodes)
		{
			check(ExternalRootPropertyNode.IsValid());
			UStruct* BaseStruct = GetExpansionStruct(ExternalRootPropertyNode.ToSharedRef());
			FStringPrefixTree& PrefixTree = ExpansionStates.Add(BaseStruct);
			GetExpansionState(BaseStruct, PrefixTree);
		}
	}

	// Restore existing nodes
	RestoreAllExpandedItems(&ExpansionStates);

	UpdatePropertyMaps();

	// Restore nodes generated by property customization
	RestoreAllExpandedItems(&ExpansionStates);
}

void SDetailsViewBase::UpdateSinglePropertyMap(TSharedPtr<FComplexPropertyNode> InRootPropertyNode, FDetailLayoutData& LayoutData, bool bIsExternal)
{
	// Reset everything
	LayoutData.ClassToPropertyMap.Empty();

	TSharedPtr<FDetailLayoutBuilderImpl> DetailLayout = MakeShareable(new FDetailLayoutBuilderImpl(InRootPropertyNode, LayoutData.ClassToPropertyMap, PropertyUtilities.ToSharedRef(), PropertyGenerationUtilities.ToSharedRef(), SharedThis(this), bIsExternal));
	LayoutData.DetailLayout = DetailLayout;

	TSharedPtr<FComplexPropertyNode> RootPropertyNode = InRootPropertyNode;
	check(RootPropertyNode.IsValid());

	const bool bEnableFavoriteSystem = IsEngineExitRequested() ? false : DetailsViewArgs.bAllowFavoriteSystem;

	DetailLayoutHelpers::FUpdatePropertyMapArgs Args;

	Args.LayoutData = &LayoutData;
	Args.InstancedPropertyTypeToDetailLayoutMap = &InstancedTypeToLayoutMap;
	Args.bEnableFavoriteSystem = bEnableFavoriteSystem;
	Args.bUpdateFavoriteSystemOnly = false;
	DetailLayoutHelpers::UpdateSinglePropertyMapRecursive(*RootPropertyNode, NAME_None, RootPropertyNode.Get(), Args);

	DetailLayout->AddEmptyCategoryIfNeeded(RootPropertyNode);
	
	CustomUpdatePropertyMap(LayoutData.DetailLayout);

	// Ask for custom detail layouts, unless disabled. One reason for disabling custom layouts is that the custom layouts
	// inhibit our ability to find a single property's tree node. This is problematic for the diff and merge tools, that need
	// to display and highlight each changed property for the user. We could allow 'known good' customizations here if 
	// we can make them work with the diff/merge tools.
	if (!bDisableCustomDetailLayouts)
	{
		DetailLayoutHelpers::QueryCustomDetailLayout(LayoutData, InstancedClassToDetailLayoutMap, GenericLayoutDelegate);
	}

	if (bEnableFavoriteSystem && InRootPropertyNode->GetInstancesNum() > 0)
	{
		static const FName FavoritesCategoryName("Favorites");
		FDetailCategoryImpl& FavoritesCategory = LayoutData.DetailLayout->DefaultCategory(FavoritesCategoryName);
		FavoritesCategory.SetSortOrder(0);
		FavoritesCategory.SetCategoryAsSpecialFavorite();

		if (FavoritesCategory.GetNumCustomizations() == 0)
		{
			FavoritesCategory.AddCustomRow(NSLOCTEXT("DetailLayoutHelpers", "Favorites", "Favorites"))
				.WholeRowContent()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(NSLOCTEXT("DetailLayoutHelpers", "AddToFavoritesDescription", "Right-click on a property to add it to your Favorites."))
					.TextStyle(FAppStyle::Get(), "HintText")
				];
		}
	}

	LayoutData.DetailLayout->GenerateDetailLayout();
}

void SDetailsViewBase::OnColorPickerWindowClosed(const TSharedRef<SWindow>& Window)
{
	// A color picker window is no longer open
	bHasOpenColorPicker = false;
	ColorPropertyNode.Reset();
}

void SDetailsViewBase::SetIsPropertyVisibleDelegate(FIsPropertyVisible InIsPropertyVisible)
{
	IsPropertyVisibleDelegate = InIsPropertyVisible;
}

void SDetailsViewBase::SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly InIsPropertyReadOnly)
{
	IsPropertyReadOnlyDelegate = InIsPropertyReadOnly;
}

void SDetailsViewBase::SetIsCustomRowVisibleDelegate(FIsCustomRowVisible InIsCustomRowVisible)
{
	IsCustomRowVisibleDelegate = InIsCustomRowVisible;
}

void SDetailsViewBase::SetIsCustomRowReadOnlyDelegate(FIsCustomRowReadOnly InIsCustomRowReadOnly)
{
	IsCustomRowReadOnlyDelegate = InIsCustomRowReadOnly;
}

void SDetailsViewBase::SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled IsPropertyEditingEnabled)
{
	IsPropertyEditingEnabledDelegate = IsPropertyEditingEnabled;
}

bool SDetailsViewBase::IsPropertyEditingEnabled() const
{
	// If the delegate is not bound assume property editing is enabled, otherwise ask the delegate
	return !IsPropertyEditingEnabledDelegate.IsBound() || IsPropertyEditingEnabledDelegate.Execute();
}

void SDetailsViewBase::SetKeyframeHandler( TSharedPtr<class IDetailKeyframeHandler> InKeyframeHandler )
{
	// if we don't have a keyframe handler and a valid handler is set, add width to the right column
	// if we do have a keyframe handler and an invalid handler is set, remove width
	float ExtraWidth = 0;
	if (!KeyframeHandler.IsValid() && InKeyframeHandler.IsValid())
	{
		ExtraWidth = 22;
	}
	else if (KeyframeHandler.IsValid() && !InKeyframeHandler.IsValid())
	{
		ExtraWidth = -22;
	}
	
	const float NewWidth = ColumnSizeData.GetRightColumnMinWidth().Get(0) + ExtraWidth;
	ColumnSizeData.SetRightColumnMinWidth(NewWidth);

	KeyframeHandler = InKeyframeHandler;
	RefreshTree();
}

void SDetailsViewBase::SetExtensionHandler(TSharedPtr<class IDetailPropertyExtensionHandler> InExtensionHandler)
{
	ExtensionHandler = InExtensionHandler;
}

void SDetailsViewBase::SetChildrenCustomizationHandler(TSharedPtr<IDetailPropertyChildrenCustomizationHandler> InChildrenHandler)
{
	ChildrenCustomizationHandler = InChildrenHandler;
}

void SDetailsViewBase::SetGenericLayoutDetailsDelegate(FOnGetDetailCustomizationInstance OnGetGenericDetails)
{
	GenericLayoutDelegate = OnGetGenericDetails;
}

void SDetailsViewBase::RefreshRootObjectVisibility()
{
	RerunCurrentFilter();
}

TSharedPtr<FAssetThumbnailPool> SDetailsViewBase::GetThumbnailPool() const
{
	return UThumbnailManager::Get().GetSharedThumbnailPool();
}

const TArray<TSharedRef<class IClassViewerFilter>>& SDetailsViewBase::GetClassViewerFilters() const
{
	return ClassViewerFilters;
}

void SDetailsViewBase::NotifyFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	OnFinishedChangingPropertiesDelegate.Broadcast(PropertyChangedEvent);
}

void SDetailsViewBase::RequestItemExpanded(TSharedRef<FDetailTreeNode> TreeNode, bool bExpand)
{
	FilteredNodesRequestingExpansionState.Add(TreeNode, bExpand);
}

void SDetailsViewBase::RefreshTree()
{
	if (OnDisplayedPropertiesChangedDelegate.IsBound())
	{
		OnDisplayedPropertiesChangedDelegate.Execute();
	}

	DetailTree->RequestTreeRefresh();
}

void SDetailsViewBase::RequestForceRefresh()
{
	SetPendingRefreshTimer();
}

void SDetailsViewBase::SaveCustomExpansionState(const FString& NodePath, bool bIsExpanded)
{
	if (bIsExpanded)
	{
		ExpandedDetailNodes.Insert(NodePath);
	}
	else
	{
		ExpandedDetailNodes.Remove(NodePath);
	}
}

bool SDetailsViewBase::GetCustomSavedExpansionState(const FString& NodePath) const
{
	return ExpandedDetailNodes.Contains(NodePath);
}

FName SDetailsViewBase::GetViewIdentifier() const
{
	return DetailsViewArgs.ViewIdentifier;
}

FDetailsViewArgs::EExpansionPersistenceMethod SDetailsViewBase::GetExpansionPersistenceMethod() const
{
	return DetailsViewArgs.ExpansionPersistenceMethod;
}

bool SDetailsViewBase::IsPropertyVisible( const FPropertyAndParent& PropertyAndParent ) const
{
	return IsPropertyVisibleDelegate.IsBound() ? IsPropertyVisibleDelegate.Execute(PropertyAndParent) : true;
}

bool SDetailsViewBase::IsPropertyReadOnly( const FPropertyAndParent& PropertyAndParent ) const
{
	return IsPropertyReadOnlyDelegate.IsBound() ? IsPropertyReadOnlyDelegate.Execute(PropertyAndParent) : false;
}

bool SDetailsViewBase::IsCustomRowVisible(FName InRowName, FName InParentName) const
{
	return IsCustomRowVisibleDelegate.IsBound() ? IsCustomRowVisibleDelegate.Execute(InRowName, InParentName) : true;
}

bool SDetailsViewBase::IsCustomRowReadOnly(FName InRowName, FName InParentName) const
{
	return IsCustomRowReadOnlyDelegate.IsBound() ? IsCustomRowReadOnlyDelegate.Execute(InRowName, InParentName) : false;
}

TSharedPtr<IPropertyUtilities> SDetailsViewBase::GetPropertyUtilities()
{
	return PropertyUtilities;
}

const FDetailsViewConfig* SDetailsViewBase::GetConstViewConfig() const 
{
	if (DetailsViewArgs.ViewIdentifier.IsNone())
	{
		return nullptr;
	}

	return UDetailsConfig::Get()->Views.Find(DetailsViewArgs.ViewIdentifier);
}

FDetailsViewConfig* SDetailsViewBase::GetMutableViewConfig()
{ 
	if (DetailsViewArgs.ViewIdentifier.IsNone())
	{
		return nullptr;
	}

	return &UDetailsConfig::Get()->Views.FindOrAdd(DetailsViewArgs.ViewIdentifier);
}

void SDetailsViewBase::SaveViewConfig()
{
	UDetailsConfig::Get()->SaveEditorConfig();
}

void SDetailsViewBase::OnShowOnlyModifiedClicked()
{
	CurrentFilter.bShowOnlyModified = !CurrentFilter.bShowOnlyModified;

	FDetailsViewConfig* ViewConfig = GetMutableViewConfig();
	if (ViewConfig != nullptr)
	{
		ViewConfig->bShowOnlyModified = CurrentFilter.bShowOnlyModified;
		SaveViewConfig();
	}

	UpdateFilteredDetails();
}

void SDetailsViewBase::OnCustomFilterClicked()
{
	if (CustomFilterDelegate.IsBound())
	{
		CustomFilterDelegate.Execute();
		bCustomFilterActive = !bCustomFilterActive;
	}
}

void SDetailsViewBase::OnShowAllAdvancedClicked()
{
	CurrentFilter.bShowAllAdvanced = !CurrentFilter.bShowAllAdvanced;

	FDetailsViewConfig* ViewConfig = GetMutableViewConfig();
	if (ViewConfig != nullptr)
	{
		ViewConfig->bShowAllAdvanced = CurrentFilter.bShowAllAdvanced;
		SaveViewConfig();
	}

	UpdateFilteredDetails();
}

void SDetailsViewBase::OnShowOnlyAllowedClicked()
{
	CurrentFilter.bShowOnlyAllowed = !CurrentFilter.bShowOnlyAllowed;

	UpdateFilteredDetails();
}

void SDetailsViewBase::OnShowAllChildrenIfCategoryMatchesClicked()
{
	CurrentFilter.bShowAllChildrenIfCategoryMatches = !CurrentFilter.bShowAllChildrenIfCategoryMatches;

	FDetailsViewConfig* ViewConfig = GetMutableViewConfig();
	if (ViewConfig != nullptr)
	{
		ViewConfig->bShowAllChildrenIfCategoryMatches = CurrentFilter.bShowAllChildrenIfCategoryMatches;
		SaveViewConfig();
	}

	UpdateFilteredDetails();
}

void SDetailsViewBase::OnShowKeyableClicked()
{
	CurrentFilter.bShowOnlyKeyable = !CurrentFilter.bShowOnlyKeyable;

	FDetailsViewConfig* ViewConfig = GetMutableViewConfig();
	if (ViewConfig != nullptr)
	{
		ViewConfig->bShowOnlyKeyable = CurrentFilter.bShowOnlyKeyable;
		SaveViewConfig();
	}
	UpdateFilteredDetails();
}

void SDetailsViewBase::OnShowAnimatedClicked()
{
	CurrentFilter.bShowOnlyAnimated = !CurrentFilter.bShowOnlyAnimated;

	FDetailsViewConfig* ViewConfig = GetMutableViewConfig();
	if (ViewConfig != nullptr)
	{
		ViewConfig->bShowOnlyAnimated = CurrentFilter.bShowOnlyAnimated;
		SaveViewConfig();
	}

	UpdateFilteredDetails();
}

/** Called when the filter text changes.  This filters specific property nodes out of view */
void SDetailsViewBase::OnFilterTextChanged(const FText& InFilterText)
{
	FilterView(InFilterText.ToString());
}

void SDetailsViewBase::OnFilterTextCommitted(const FText& InSearchText, ETextCommit::Type InCommitType)
{
	if (InCommitType == ETextCommit::OnCleared)
	{
		SearchBox->SetText(FText::GetEmpty());
		OnFilterTextChanged(FText::GetEmpty());
		FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Cleared);
	}
}

TSharedPtr<SWidget> SDetailsViewBase::GetNameAreaWidget()
{
	return DetailsViewArgs.bCustomNameAreaLocation ? NameArea : nullptr;
}

void SDetailsViewBase::SetNameAreaCustomContent( TSharedRef<SWidget>& InCustomContent )
{
	NameArea->SetCustomContent(InCustomContent);	
}

TSharedPtr<SWidget> SDetailsViewBase::GetFilterAreaWidget()
{
	return DetailsViewArgs.bCustomFilterAreaLocation ? FilterRow : nullptr;
}

TSharedPtr<FUICommandList> SDetailsViewBase::GetHostCommandList() const
{
	return DetailsViewArgs.HostCommandList;
}

TSharedPtr<FTabManager> SDetailsViewBase::GetHostTabManager() const
{
	return DetailsViewArgs.HostTabManager;
}

void SDetailsViewBase::SetHostTabManager(TSharedPtr<FTabManager> InTabManager)
{
	DetailsViewArgs.HostTabManager = InTabManager;
}

/** 
 * Hides or shows properties based on the passed in filter text
 * 
 * @param InFilter The filter text
 */
void SDetailsViewBase::FilterView(const FString& InFilter)
{
	bool bHadActiveFilter = CurrentFilter.FilterStrings.Num() > 0;

	TArray<FString> CurrentFilterStrings;

	FString ParseString = InFilter;
	// Remove whitespace from the front and back of the string
	ParseString.TrimStartAndEndInline();
	ParseString.ParseIntoArray(CurrentFilterStrings, TEXT(" "), true);

	CurrentFilter.FilterStrings = CurrentFilterStrings;

	if (!bHadActiveFilter && CurrentFilter.FilterStrings.Num() > 0)
	{
		SavePreSearchExpandedItems();
	}

	UpdateFilteredDetails();
	
	if (bHadActiveFilter && CurrentFilter.FilterStrings.Num() == 0)
	{
		RestorePreSearchExpandedItems();
	}
}

EVisibility SDetailsViewBase::GetFilterBoxVisibility() const
{
	EVisibility Result = EVisibility::Collapsed;
	// Visible if we allow search and we have anything to search otherwise collapsed so it doesn't take up room	
	if (DetailsViewArgs.bAllowSearch && IsConnected())
	{
		if (RootTreeNodes.Num() > 0 || HasActiveSearch() || !CurrentFilter.IsEmptyFilter())
		{
			Result = EVisibility::Visible;
		}
	}

	return Result;
}

bool SDetailsViewBase::SupportsKeyboardFocus() const
{
	return DetailsViewArgs.bSearchInitialKeyFocus && SearchBox->SupportsKeyboardFocus() && GetFilterBoxVisibility() == EVisibility::Visible;
}

FReply SDetailsViewBase::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	FReply Reply = FReply::Handled();

	if (InFocusEvent.GetCause() != EFocusCause::Cleared)
	{
		Reply.SetUserFocus(SearchBox.ToSharedRef(), InFocusEvent.GetCause());
	}

	return Reply;
}

void SDetailsViewBase::SetPendingCleanupTimer()
{
	if (!bPendingCleanupTimerSet)
	{
		if (GEditor && GEditor->IsTimerManagerValid())
		{
			bPendingCleanupTimerSet = true;
			GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &SDetailsViewBase::HandlePendingCleanupTimer));
		}
	}
}

void SDetailsViewBase::HandlePendingCleanupTimer()
{
	bPendingCleanupTimerSet = false;

	HandlePendingCleanup();

	for (auto It = FilteredNodesRequestingExpansionState.CreateIterator(); It; ++It)
	{
		if (!It->Key.IsValid())
		{
			It.RemoveCurrent();
		}
	}
}

void SDetailsViewBase::HandlePendingCleanup()
{
	for (int32 i = 0; i < CustomizationClassInstancesPendingDelete.Num(); ++i)
	{
		ensure(CustomizationClassInstancesPendingDelete[i].IsUnique());
	}

	// Release any pending kill nodes.
	for ( TSharedPtr<FComplexPropertyNode>& PendingKillNode : RootNodesPendingKill )
	{
		if ( PendingKillNode.IsValid() )
		{
			PendingKillNode->Disconnect();
			PendingKillNode.Reset();
		}
	}
	RootNodesPendingKill.Empty();

	// Empty all the customization instances that need to be deleted
	CustomizationClassInstancesPendingDelete.Empty();

	// Empty all the detail layouts that need to be deleted
	DetailLayoutsPendingDelete.Empty();
}

void SDetailsViewBase::SetPendingRefreshTimer()
{
	if (!PendingRefreshTimerHandle.IsValid())
	{
		if (GEditor && GEditor->IsTimerManagerValid())
		{
			PendingRefreshTimerHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &SDetailsViewBase::HandlePendingRefreshTimer));
		}
	}
}

void SDetailsViewBase::ClearPendingRefreshTimer()
{
	if (PendingRefreshTimerHandle.IsValid())
	{
		if (GEditor && GEditor->IsTimerManagerValid())
		{
			GEditor->GetTimerManager()->ClearTimer(PendingRefreshTimerHandle);
		}
		PendingRefreshTimerHandle.Invalidate();
	}
}

void SDetailsViewBase::HandlePendingRefreshTimer()
{
	PendingRefreshTimerHandle.Invalidate();
	ForceRefresh();
}

/** Ticks the property view.  This function performs a data consistency check */
void SDetailsViewBase::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SDetailsViewBase::Tick);

	HandlePendingCleanup();

	FDetailsViewConfig* ViewConfig = GetMutableViewConfig();
	if (ViewConfig != nullptr && 
		ViewConfig->ValueColumnWidth != ColumnSizeData.GetValueColumnWidth().Get(0))
	{
		ViewConfig->ValueColumnWidth = ColumnSizeData.GetValueColumnWidth().Get(0);
		SaveViewConfig();
	}

	FRootPropertyNodeList& RootPropertyNodes = GetRootNodes();

	bool bHadDeferredActions = DeferredActions.Num() > 0;
	bool bDidPurgeObjects = false;
	auto PreProcessRootNode = [bHadDeferredActions, &bDidPurgeObjects, this](TSharedPtr<FComplexPropertyNode> RootPropertyNode)
	{
		check(RootPropertyNode.IsValid());

		// Purge any objects that are marked pending kill from the object list
		if (FObjectPropertyNode* ObjectRoot = RootPropertyNode->AsObjectNode())
		{
			bDidPurgeObjects |= ObjectRoot->PurgeKilledObjects();
		}

		if (bHadDeferredActions)
		{
			// Any deferred actions are likely to cause the node tree to be at least partially rebuilt
			// Save the expansion state of existing nodes so we can expand them later
			SaveExpandedItems(RootPropertyNode.ToSharedRef());
		}
	};

	for (TSharedPtr<FComplexPropertyNode>& RootPropertyNode : RootPropertyNodes)
	{
		PreProcessRootNode(RootPropertyNode);
	}

	for (FDetailLayoutData& LayoutData : DetailLayouts)
	{
		FRootPropertyNodeList& ExternalRootPropertyNodes = LayoutData.DetailLayout->GetExternalRootPropertyNodes();
		for (TSharedPtr<FComplexPropertyNode>& ExternalRootPropertyNode : ExternalRootPropertyNodes)
		{
			PreProcessRootNode(ExternalRootPropertyNode);
		}
	}

	if (bHadDeferredActions)
	{
		// Note: Extra scope so that RestoreAllExpandedItems actually restores state, as it does nothing when bRunningDeferredActions is true
		{
			bRunningDeferredActions = true;
			ON_SCOPE_EXIT{ bRunningDeferredActions = false; };

			TArray<FSimpleDelegate> DeferredActionsCopy;

			do
			{
				// Execute any deferred actions
				DeferredActionsCopy = MoveTemp(DeferredActions);
				DeferredActions.Reset();

				// Execute any deferred actions
				for (const FSimpleDelegate& DeferredAction : DeferredActionsCopy)
				{
					DeferredAction.ExecuteIfBound();
				}
			} while (DeferredActions.Num() > 0);
		}

		RestoreAllExpandedItems();
	}

	TSharedPtr<FComplexPropertyNode> LastRootPendingKill;
	if (RootNodesPendingKill.Num() > 0 )
	{
		LastRootPendingKill = RootNodesPendingKill.Last();
	}

	bool bValidateExternalNodes = true;

	int32 FoundIndex = RootPropertyNodes.Find(LastRootPendingKill);
	bool bUpdateFilteredDetails = false;
	if (FoundIndex != INDEX_NONE || bDidPurgeObjects)
	{ 
		// Reacquire the root property nodes.  It may have been changed by the deferred actions if something like a blueprint editor forcefully resets a details panel during a posteditchange
		ForceRefresh();

		// All objects are being reset, no need to validate external nodes
		bValidateExternalNodes = false;
	}
	else
	{
		if (CustomValidatePropertyNodesFunction.IsBound())
		{
			bool bIsValid = CustomValidatePropertyNodesFunction.Execute(RootPropertyNodes);
		}
		else // standard validation behavior
		{
			for (const TSharedPtr<FComplexPropertyNode>& RootPropertyNode : RootPropertyNodes)
			{
				EPropertyDataValidationResult Result = RootPropertyNode->EnsureDataIsValid();
				if (Result == EPropertyDataValidationResult::PropertiesChanged || Result == EPropertyDataValidationResult::EditInlineNewValueChanged)
				{
					UpdatePropertyMaps();
					bUpdateFilteredDetails = true;
				}
				else if (Result == EPropertyDataValidationResult::ArraySizeChanged)
				{
					bUpdateFilteredDetails = true;
				}
				else if (Result == EPropertyDataValidationResult::ChildrenRebuilt)
				{
					RootPropertyNode->MarkChildrenAsRebuilt();
				
					bUpdateFilteredDetails = true;
				}
				else if (Result == EPropertyDataValidationResult::ObjectInvalid)
				{
					bValidateExternalNodes = false;

					ForceRefresh();
					break;
				}
			}
		}
	}

	if (bValidateExternalNodes)
	{
		for (const FDetailLayoutData& LayoutData : DetailLayouts)
		{
			for (const TSharedPtr<FComplexPropertyNode>& PropertyNode : LayoutData.DetailLayout->GetExternalRootPropertyNodes())
			{
				EPropertyDataValidationResult Result = PropertyNode->EnsureDataIsValid();
				if (Result == EPropertyDataValidationResult::PropertiesChanged || Result == EPropertyDataValidationResult::EditInlineNewValueChanged)
				{
					// Note this will invalidate all the external root nodes so there is no need to continue
					LayoutData.DetailLayout->ClearExternalRootPropertyNodes();

					UpdatePropertyMaps();
					bUpdateFilteredDetails = true;

					break;
				}
				else if (Result == EPropertyDataValidationResult::ArraySizeChanged || Result == EPropertyDataValidationResult::ChildrenRebuilt)
				{
					bUpdateFilteredDetails = true;
				}
			}
		}
	}

	if (bUpdateFilteredDetails)
	{
		UpdateFilteredDetails();
	}

	for (FDetailLayoutData& LayoutData : DetailLayouts)
	{
		if (LayoutData.DetailLayout.IsValid())
		{
			LayoutData.DetailLayout->Tick(InDeltaTime);
		}
	}

	if (!ColorPropertyNode.IsValid() && bHasOpenColorPicker)
	{
		// Destroy the color picker window if the color property node has become invalid
		DestroyColorPicker();
		bHasOpenColorPicker = false;
	}

	if (FilteredNodesRequestingExpansionState.Num() > 0)
	{
		// change expansion state on the nodes that request it
		for (TMap<TWeakPtr<FDetailTreeNode>, bool>::TConstIterator It(FilteredNodesRequestingExpansionState); It; ++It)
		{
			TSharedPtr<FDetailTreeNode> DetailTreeNode = It.Key().Pin();
			if (DetailTreeNode.IsValid())
			{
				DetailTree->SetItemExpansion(DetailTreeNode.ToSharedRef(), It.Value());
			}
		}

		FilteredNodesRequestingExpansionState.Empty();
	}
}

FReply SDetailsViewBase::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (InternalCommands.IsValid() && InternalCommands->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return IDetailsViewPrivate::OnKeyDown(InGeometry, InKeyEvent);
}

SDetailsViewBase::FAnimatingNodeCollection::FAnimatingNodeCollection(
	const TConstArrayView<TSharedPtr<FPropertyPath>>& InNodePaths,
	const FGuid& InBatchId)
	: NodePaths(InNodePaths)
	, BatchId((InBatchId))
	, CreationTime(FPlatformTime::Seconds())
{
	if (!InBatchId.IsValid())
	{
		FGuid CombinedBatchId;
		for (const TSharedPtr<FPropertyPath>& NodePath : NodePaths)
		{
			if (NodePath.IsValid())
			{
				CombinedBatchId = FGuid::Combine(CombinedBatchId, GetIdForNodePath(NodePath));
			}
		}

		BatchId = CombinedBatchId;
	}
}

FGuid SDetailsViewBase::FAnimatingNodeCollection::GetIdForNodePath(const TSharedPtr<FPropertyPath>& InNodePath)
{
	return InNodePath.IsValid() ? FGuid::NewDeterministicGuid(InNodePath->ToString()) : FGuid();
}

bool SDetailsViewBase::FAnimatingNodeCollection::IsValid() const
{
	return Algo::AllOf(NodePaths,
		[](const TSharedPtr<FPropertyPath>& InNodePath)
		{
			return InNodePath.IsValid();
		});
}

void SDetailsViewBase::FAnimatingNodeCollection::ResetElapsedSeconds()
{
	CreationTime = FPlatformTime::Seconds();
}

float SDetailsViewBase::FAnimatingNodeCollection::GetElapsedSeconds() const
{
	return FPlatformTime::Seconds() - CreationTime;
}

/**
* Recursively gets expanded items for a node
*
* @param InPropertyNode			The node to get expanded items from
* @param OutExpandedItems	List of expanded items that were found
*/
static void GetExpandedItems(TSharedPtr<FPropertyNode> InPropertyNode, FStringPrefixTree& OutExpandedItems)
{
	if (InPropertyNode->HasNodeFlags(EPropertyNodeFlags::Expanded))
	{
		const bool bWithArrayIndex = true;
		FString Path;
		Path.Empty(128);
		InPropertyNode->GetQualifiedName(Path, bWithArrayIndex);

		OutExpandedItems.Insert(Path);
	}

	for (int32 ChildIndex = 0; ChildIndex < InPropertyNode->GetNumChildNodes(); ++ChildIndex)
	{
		GetExpandedItems(InPropertyNode->GetChildNode(ChildIndex), OutExpandedItems);
	}
}

/**
* Recursively sets expanded items for a node
*
* @param InNode			The node to set expanded items on
* @param OutExpandedItems	List of expanded items to set
*/
static void SetExpandedItems(TSharedPtr<FPropertyNode> InPropertyNode, const FStringPrefixTree& InExpandedItems, bool bCollapseRest)
{
	const bool bWithArrayIndex = true;
	FString Path;
	Path.Empty(128);
	InPropertyNode->GetQualifiedName(Path, bWithArrayIndex);

	if (InExpandedItems.Contains(Path))
	{
		InPropertyNode->SetNodeFlags(EPropertyNodeFlags::Expanded, true);
	}
	else if (bCollapseRest)
	{
		InPropertyNode->SetNodeFlags(EPropertyNodeFlags::Expanded, false);
	}

	if (InExpandedItems.AnyStartsWith(Path))
	{
		for (int32 NodeIndex = 0; NodeIndex < InPropertyNode->GetNumChildNodes(); ++NodeIndex)
		{
			SetExpandedItems(InPropertyNode->GetChildNode(NodeIndex), InExpandedItems, bCollapseRest);
		}
	}
}

const FString& SDetailsViewBase::GetDetailPropertyExpansionSection() const
{
	if (DetailsViewArgs.ExpansionPersistenceMethod == FDetailsViewArgs::EExpansionPersistenceMethod::TransientPerViewIdentifier)
	{
		return TransientPerViewDetailPropertyExpansion;
	}
	else if (DetailsViewArgs.ExpansionPersistenceMethod == FDetailsViewArgs::EExpansionPersistenceMethod::PersistentPerViewIdentifier)
	{
		return PersistentPerViewDetailPropertyExpansion;
	}

	return DetailPropertyExpansion;
}

const FString& SDetailsViewBase::GetDetailCustomWidgetExpansionSection() const
{
	if (DetailsViewArgs.ExpansionPersistenceMethod == FDetailsViewArgs::EExpansionPersistenceMethod::TransientPerViewIdentifier)
	{
		return TransientPerViewDetailCustomWidgetExpansion;
	}
	else if (DetailsViewArgs.ExpansionPersistenceMethod == FDetailsViewArgs::EExpansionPersistenceMethod::PersistentPerViewIdentifier)
	{
		return PersistentPerViewDetailCustomWidgetExpansion;
	}

	return DetailCustomWidgetExpansion;
}

FString SDetailsViewBase::GetStructExpansionConfigKey(UStruct* InStruct) const
{
	if (DetailsViewArgs.ExpansionPersistenceMethod == FDetailsViewArgs::EExpansionPersistenceMethod::TransientPerViewIdentifier
		|| DetailsViewArgs.ExpansionPersistenceMethod == FDetailsViewArgs::EExpansionPersistenceMethod::PersistentPerViewIdentifier)
	{
		if (ensure(!DetailsViewArgs.ViewIdentifier.IsNone()))
		{
			TStringBuilder<128> ViewIdentifierString;
			ViewIdentifierString.Append(DetailsViewArgs.ViewIdentifier.ToString());
			ViewIdentifierString.Append(".");
			ViewIdentifierString.Append(InStruct->GetName());
			return ViewIdentifierString.ToString();
		}
	}

	return InStruct->GetName();
}

void SDetailsViewBase::SavePreSearchExpandedItems()
{
	PreSearchExpandedItems.Clear();

	FRootPropertyNodeList& RootPropertyNodes = GetRootNodes();

	for (TSharedPtr<FComplexPropertyNode>& RootPropertyNode : RootPropertyNodes)
	{
		GetExpandedItems(RootPropertyNode.ToSharedRef(), PreSearchExpandedItems);
	}

	for (FDetailLayoutData& LayoutData : DetailLayouts)
	{
		FRootPropertyNodeList& ExternalRootPropertyNodes = LayoutData.DetailLayout->GetExternalRootPropertyNodes();
		for (TSharedPtr<FComplexPropertyNode>& ExternalRootPropertyNode : ExternalRootPropertyNodes)
		{
			GetExpandedItems(ExternalRootPropertyNode.ToSharedRef(), PreSearchExpandedItems);
		}
	}

	PreSearchExpandedCategories.Clear();

	for (const TSharedRef<FDetailTreeNode>& RootNode : RootTreeNodes)
	{
		if (RootNode->GetNodeType() == EDetailNodeType::Category)
		{
			FDetailCategoryImpl& Category = (FDetailCategoryImpl&) RootNode.Get();
			if (Category.ShouldBeExpanded())
			{
				PreSearchExpandedCategories.Insert(Category.GetCategoryPathName());
			}
		}
	}
}

void SDetailsViewBase::RestorePreSearchExpandedItems()
{
	FRootPropertyNodeList& RootPropertyNodes = GetRootNodes();

	for (TSharedPtr<FComplexPropertyNode>& RootPropertyNode : RootPropertyNodes)
	{
		SetExpandedItems(RootPropertyNode, PreSearchExpandedItems, true);
	}

	for (FDetailLayoutData& LayoutData : DetailLayouts)
	{
		FRootPropertyNodeList& ExternalRootPropertyNodes = LayoutData.DetailLayout->GetExternalRootPropertyNodes();
		for (TSharedPtr<FComplexPropertyNode>& ExternalRootPropertyNode : ExternalRootPropertyNodes)
		{
			SetExpandedItems(ExternalRootPropertyNode, PreSearchExpandedItems, true);
		}
	}

	PreSearchExpandedItems.Clear();

	for (const TSharedRef<FDetailTreeNode>& RootNode : RootTreeNodes)
	{
		if (RootNode->GetNodeType() == EDetailNodeType::Category)
		{
			FDetailCategoryImpl& Category = (FDetailCategoryImpl&) RootNode.Get();
			
			bool bShouldBeExpanded = PreSearchExpandedCategories.Contains(Category.GetCategoryPathName());
			RequestItemExpanded(RootNode, bShouldBeExpanded);
		}
	}

	PreSearchExpandedCategories.Clear();
}

UStruct* SDetailsViewBase::GetExpansionStruct(const TSharedRef<FPropertyNode>& StartNode)
{
	FComplexPropertyNode* ComplexParent = StartNode->FindComplexParent();
	UStruct* BaseStructure = ComplexParent->GetBaseStructure();

	FString FuncName;
	if (BaseStructure && BaseStructure->GetStringMetaDataHierarchical("GetExpansionStructFunc", &FuncName))
	{
		if (const FObjectPropertyNode* ObjectNode = ComplexParent->AsObjectNode())
		{
			const UClass* BaseClass = Cast<UClass>(BaseStructure);
			const int32 InstancesNum = ObjectNode->GetInstancesNum();

			if (InstancesNum > 0 && BaseClass)
			{
				if (UFunction* Function = BaseClass->FindFunctionByName(FName(FuncName)))
				{
					if (Function->HasAnyFunctionFlags(FUNC_Static))
					{
						struct FFunctionParams
						{
							TArray<UObject*> Objects;
							const UStruct* Struct;
						} Params;

						Params.Objects.Reserve(InstancesNum);
						for (int32 i = 0; i < InstancesNum; ++i)
						{
							if (UObject* Object = ObjectNode->GetInstanceAsUObject(i).Get())
							{
								Params.Objects.Add(Object);
							}
						}

						BaseStructure->ProcessEvent(Function, &Params);

						if (Params.Struct)
						{
							return const_cast<UStruct*>(Params.Struct);
						}
					}
				}
			}
		}
	}

	return BaseStructure;
}

void SDetailsViewBase::SaveExpandedItems(TSharedRef<FPropertyNode> StartNode)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SDetailsView::SaveExpandedItems);

	if (bRunningDeferredActions)
	{
		// Deferred actions can manipulate the tree, eg. Add Item
		// However, there's no reason for us to save expanded items during deferred actions, 
		// because the expansion state was already stored at the beginning and will be restored afterwards.
		return;
	}

	UStruct* BestBaseStruct = GetExpansionStruct(StartNode);

	FStringPrefixTree ExpandedPropertyItemSet;
	GetExpandedItems(StartNode, ExpandedPropertyItemSet);

	TArray<FString> ExpandedPropertyItems = ExpandedPropertyItemSet.GetAllEntries();

	// Handle spaces in expanded node names by wrapping them in quotes
	for (FString& String : ExpandedPropertyItems)
	{
		String.InsertAt(0, '"');
		String.AppendChar('"');
	}

	//while a valid class, and we're either the same as the base class (for multiple actors being selected and base class is AActor) OR we're not down to AActor yet)
	for (UStruct* Struct = BestBaseStruct; Struct && ((BestBaseStruct == Struct) || (Struct != AActor::StaticClass())); Struct = Struct->GetSuperStruct())
	{
		if (StartNode->GetNumChildNodes() > 0)
		{
			bool bShouldSave = ExpandedPropertyItems.Num() > 0;
			if (!bShouldSave)
			{
				TArray<FString> DummyExpandedPropertyItems;
				GConfig->GetSingleLineArray(*GetDetailPropertyExpansionSection(), *GetStructExpansionConfigKey(Struct), DummyExpandedPropertyItems, GEditorPerProjectIni);
				bShouldSave = DummyExpandedPropertyItems.Num() > 0;
			}

			if (bShouldSave)
			{
				GConfig->SetSingleLineArray(*GetDetailPropertyExpansionSection(), *GetStructExpansionConfigKey(Struct), ExpandedPropertyItems, GEditorPerProjectIni);
			}
		}
	}

	if (DetailLayouts.Num() > 0 && BestBaseStruct)
	{
		TArray<FString> ExpandedCustomItems = ExpandedDetailNodes.GetAllEntries();

		// Expanded custom items may have spaces but SetSingleLineArray doesn't support spaces (treats it as another element in the array)
		// Append a ',' after each element instead
		TStringBuilder<256> ExpandedCustomItemsBuilder;
		ExpandedCustomItemsBuilder.Join(ExpandedCustomItems, TEXT(","));

		// if there are no expanded custom items currently, save if there used to be items
		bool bShouldSave = ExpandedCustomItemsBuilder.Len() > 0;
		if (!bShouldSave)
		{
			FString DummyExpandedCustomItemsString;
			GConfig->GetString(*GetDetailCustomWidgetExpansionSection(), *GetStructExpansionConfigKey(BestBaseStruct), DummyExpandedCustomItemsString, GEditorPerProjectIni);
			bShouldSave = !DummyExpandedCustomItemsString.IsEmpty();
		}

		if (bShouldSave)
		{
			GConfig->SetString(*GetDetailCustomWidgetExpansionSection(), *GetStructExpansionConfigKey(BestBaseStruct), *ExpandedCustomItemsBuilder, GEditorPerProjectIni);
		}
	}
}

void SDetailsViewBase::RestoreAllExpandedItems(TMap<UStruct*, FStringPrefixTree>* OptionalExpansionStates)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SDetailsViewBase::RestoreAllExpandedItems);

	if (bRunningDeferredActions)
	{
		// Deferred actions can manipulate the tree, eg. Add Item
		// However, there's no reason for us to restore expanded items during deferred actions, 
		// because the expansion state was already stored at the beginning and will be restored afterwards.
		return;
	}

	for (TSharedPtr<FComplexPropertyNode>& RootPropertyNode : GetRootNodes())
	{
		check(RootPropertyNode.IsValid());
		RestoreExpandedItems(RootPropertyNode.ToSharedRef(), OptionalExpansionStates);
	}

	for (FDetailLayoutData& LayoutData : DetailLayouts)
	{
		FRootPropertyNodeList& ExternalRootPropertyNodes = LayoutData.DetailLayout->GetExternalRootPropertyNodes();
		for (TSharedPtr<FComplexPropertyNode>& ExternalRootPropertyNode : ExternalRootPropertyNodes)
		{
			check(ExternalRootPropertyNode.IsValid());
			RestoreExpandedItems(ExternalRootPropertyNode.ToSharedRef(), OptionalExpansionStates);
		}
	}
}

void SDetailsViewBase::RestoreExpandedItems(TSharedRef<FPropertyNode> StartNode)
{
	RestoreExpandedItems(StartNode, nullptr);
}

void SDetailsViewBase::RestoreExpandedItems(TSharedRef<FPropertyNode> StartNode, TMap<UStruct*, FStringPrefixTree>* OptionalExpansionStates)
{
	if (bRunningDeferredActions)
	{
		// Deferred actions can manipulate the tree, eg. Add Item
		// However, there's no reason for us to restore expanded items during deferred actions,
		// because the expansion state was already stored at the beginning and will be restored afterwards.
		return;
	}

	UStruct* BestBaseStruct = GetExpansionStruct(StartNode);

	if (OptionalExpansionStates)
	{
		const FStringPrefixTree* PrefixTree = OptionalExpansionStates->Find(BestBaseStruct);
		if (PrefixTree)
		{
			SetExpandedItems(StartNode, *PrefixTree, false);
		}
	}
	else
	{
		FStringPrefixTree PrefixTree;
		GetExpansionState(BestBaseStruct, PrefixTree);
		SetExpandedItems(StartNode, PrefixTree, false);
	}

	if (BestBaseStruct)
	{
		FString ExpandedCustomItems;
		GConfig->GetString(*GetDetailCustomWidgetExpansionSection(), *GetStructExpansionConfigKey(BestBaseStruct), ExpandedCustomItems, GEditorPerProjectIni);

		TArray<FString> ExpandedCustomItemsArray;
		ExpandedCustomItems.ParseIntoArray(ExpandedCustomItemsArray, TEXT(","), true);

		ExpandedDetailNodes.InsertAll(ExpandedCustomItemsArray);
	}
}

void SDetailsViewBase::MarkNodeAnimating(TSharedPtr<FPropertyNode> InNode, float InAnimationDuration, TOptional<FGuid> InAnimationBatchId)
{
	if (InNode.IsValid() && InAnimationDuration > 0.0f)
	{
		TSharedPtr<FPropertyPath> NodePropertyPath = FPropertyNode::CreatePropertyPath(InNode.ToSharedRef());
		
		// Try finding existing, ignore Batch Id
		FAnimatingNodeCollection* AnimatingNode = GetAnimatingNodeCollectionForPropertyNode(InNode);

		// If this node is part of a batch animation, let it play out (don't reset it)
		if (AnimatingNode)
		{
			if (AnimatingNode->NodePaths.Num() > 1
				|| (InAnimationBatchId.IsSet() && AnimatingNode->BatchId == InAnimationBatchId))
			{
				return;
			}
		}

		// If not found, but batch id found, add it to that batch
		if (!AnimatingNode && InAnimationBatchId.IsSet())
		{
			AnimatingNode = CurrentlyAnimatingNodeCollections.FindByPredicate(
				[InAnimationBatchId](const FAnimatingNodeCollection& AnimatingNode)
				{
					return AnimatingNode.BatchId == InAnimationBatchId;
				});
			
			if (AnimatingNode != nullptr)
			{
				AnimatingNode->NodePaths.Add(NodePropertyPath);

				// Timer Handle already in progress, so no need to clear/set below
				return;
			}
		}

		// Otherwise add it
		if (AnimatingNode == nullptr)
		{
			AnimatingNode = &CurrentlyAnimatingNodeCollections.Add_GetRef(FAnimatingNodeCollection({ NodePropertyPath }));
			if (InAnimationBatchId.IsSet())
			{
				AnimatingNode->BatchId = InAnimationBatchId.GetValue();
			}
		}
		// Otherwise it already exists, so reset it's animated state
		else
		{
			AnimatingNode->ResetElapsedSeconds();
			GEditor->GetTimerManager()->ClearTimer(AnimatingNode->NodeTimer);
		}

		GEditor->GetTimerManager()->SetTimer(AnimatingNode->NodeTimer, FTimerDelegate::CreateSP(this, &SDetailsViewBase::HandleNodeAnimationComplete, AnimatingNode), InAnimationDuration, false);
	}
}

bool SDetailsViewBase::IsNodeAnimating(TSharedPtr<FPropertyNode> InNode)
{
	if (!InNode.IsValid())
	{
		return false;
	}

	return GetAnimatingNodeCollectionForPropertyNode(InNode) != nullptr;
}

float SDetailsViewBase::GetNodeAnimationTime(TSharedPtr<FPropertyNode> InNode)
{
	if (const FAnimatingNodeCollection* AnimatingNodeCollection = GetAnimatingNodeCollectionForPropertyNode(InNode))
	{
		return AnimatingNodeCollection->GetElapsedSeconds();
	}

	return 0.0f;
}

void SDetailsViewBase::HandleNodeAnimationComplete(FAnimatingNodeCollection* InAnimatedNode)
{
	if (InAnimatedNode)
	{
		GEditor->GetTimerManager()->ClearTimer(InAnimatedNode->NodeTimer);
		CurrentlyAnimatingNodeCollections.RemoveAll([InAnimatedNode](FAnimatingNodeCollection& AnimatedNode)
		{
			return AnimatedNode == *InAnimatedNode;		
		});
	}
}

SDetailsViewBase::FAnimatingNodeCollection* SDetailsViewBase::GetAnimatingNodeCollectionForPropertyNode(const TSharedPtr<FPropertyNode>& InNode)
{
	if (!InNode.IsValid())
	{
		return nullptr;
	}

	TSharedRef<FPropertyPath> InNodePath = FPropertyNode::CreatePropertyPath(InNode.ToSharedRef());
	return CurrentlyAnimatingNodeCollections.FindByPredicate(
		[InNodePath](const FAnimatingNodeCollection& AnimatingNode)
		{
			return Algo::AnyOf(
				AnimatingNode.NodePaths,
				[InNodePath](const TSharedPtr<FPropertyPath>& NodePath)
				{
					return FPropertyPath::AreEqual(NodePath.ToSharedRef(), InNodePath);
				});
		});
}

void SDetailsViewBase::FilterRootNode(const TSharedPtr<FComplexPropertyNode>& RootNode)
{
	if (RootNode.IsValid())
	{
		SaveExpandedItems(RootNode.ToSharedRef());

		RootNode->FilterNodes(CurrentFilter.FilterStrings);
		RootNode->ProcessSeenFlags(true);

		RestoreExpandedItems(RootNode.ToSharedRef());
	}
}

void SDetailsViewBase::UpdateFilteredDetails()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SDetailsViewBase::UpdateFilteredDetails);

	RootTreeNodes.Reset();
	
	const FDetailFilter Filter = CreateDetailFilter();

	FDetailNodeList InitialRootNodeList;
	NumVisibleTopLevelObjectNodes = 0;
	
	const FRootPropertyNodeList& RootPropertyNodes = GetRootNodes();
	for (int32 RootNodeIndex = 0; RootNodeIndex < RootPropertyNodes.Num(); ++RootNodeIndex)
	{
		if (RootPropertyNodes[RootNodeIndex].IsValid())
		{
			FilterRootNode(RootPropertyNodes[RootNodeIndex]);

			const TSharedPtr<FDetailLayoutBuilderImpl>& DetailLayout = DetailLayouts[RootNodeIndex].DetailLayout;
			if (DetailLayout.IsValid())
			{
				for (const TSharedPtr<FComplexPropertyNode>& ExternalRootNode : DetailLayout->GetExternalRootPropertyNodes())
				{
					FilterRootNode(ExternalRootNode);
				}

				DetailLayout->FilterDetailLayout(Filter);

				const FDetailNodeList& LayoutRoots = DetailLayout->GetFilteredRootTreeNodes();
				if (LayoutRoots.Num() > 0)
				{
					// A top level object nodes has a non-filtered away root so add one to the total number we have
					++NumVisibleTopLevelObjectNodes;

					InitialRootNodeList.Append(LayoutRoots);
				}
			}
		}
	}

	// for multiple top level object we need to do a secondary pass on top level object nodes after we have determined if there is any nodes visible at all.  If there are then we ask the details panel if it wants to show childen
	for (const TSharedRef<FDetailTreeNode>& RootNode : InitialRootNodeList)
	{
		if (RootNode->ShouldShowOnlyChildren())
		{
			RootNode->GetChildren(RootTreeNodes);
		}
		else
		{
			RootTreeNodes.Add(RootNode);
		}
	}

	RefreshTree();
}

void SDetailsViewBase::RegisterInstancedCustomPropertyLayout(UStruct* Class, FOnGetDetailCustomizationInstance DetailLayoutDelegate)
{
	check( Class );

	FDetailLayoutCallback Callback;
	Callback.DetailLayoutDelegate = DetailLayoutDelegate;
	// @todo: DetailsView: Fix me: this specifies the order in which detail layouts should be queried
	Callback.Order = InstancedClassToDetailLayoutMap.Num();

	InstancedClassToDetailLayoutMap.Add( Class, Callback );	
}

void SDetailsViewBase::RegisterInstancedCustomPropertyTypeLayout(FName PropertyTypeName, FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate, TSharedPtr<IPropertyTypeIdentifier> Identifier /*= nullptr*/)
{
	FPropertyTypeLayoutCallback Callback;
	Callback.PropertyTypeLayoutDelegate = PropertyTypeLayoutDelegate;
	Callback.PropertyTypeIdentifier = Identifier;

	FPropertyTypeLayoutCallbackList* LayoutCallbacks = InstancedTypeToLayoutMap.Find(PropertyTypeName);
	if (LayoutCallbacks)
	{
		LayoutCallbacks->Add(Callback);
	}
	else
	{
		FPropertyTypeLayoutCallbackList NewLayoutCallbacks;
		NewLayoutCallbacks.Add(Callback);
		InstancedTypeToLayoutMap.Add(PropertyTypeName, NewLayoutCallbacks);
	}
}

void SDetailsViewBase::UnregisterInstancedCustomPropertyLayout(UStruct* Class)
{
	check( Class );

	InstancedClassToDetailLayoutMap.Remove( Class );	
}

void SDetailsViewBase::UnregisterInstancedCustomPropertyTypeLayout(FName PropertyTypeName, TSharedPtr<IPropertyTypeIdentifier> Identifier /*= nullptr*/)
{
	FPropertyTypeLayoutCallbackList* LayoutCallbacks = InstancedTypeToLayoutMap.Find(PropertyTypeName);

	if (LayoutCallbacks)
	{
		LayoutCallbacks->Remove(Identifier);
	}
}

TSharedPtr<FDetailsDisplayManager> SDetailsViewBase::GetDisplayManager()
{
	if (!DisplayManager.IsValid())
	{
		DisplayManager = MakeShared<FDetailsDisplayManager>();
	}
	return DisplayManager;
}

FVector2D SDetailsViewBase::GetScrollDistance()
{
	if (!DetailTree.IsValid())
	{
		return FVector2D::ZeroVector;
	}

	return DetailTree->GetScrollDistance();

}

FVector2D SDetailsViewBase::GetScrollDistanceRemaining()
{
	if (!DetailTree.IsValid())
	{
		return FVector2D::ZeroVector;
	}

	return DetailTree->GetScrollDistanceRemaining();

}

TSharedRef<SWidget> SDetailsViewBase::GetScrollWidget()
{
	return SharedThis(this);
}
