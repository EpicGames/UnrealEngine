// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetThumbnail.h"
#include "DetailColumnSizeData.h"
#include "DetailFilter.h"
#include "DetailsViewConfig.h"
#include "DetailTreeNode.h"
#include "Framework/Commands/UICommandList.h"
#include "IDetailsView.h"
#include "IDetailsViewPrivate.h"
#include "Input/Reply.h"
#include "IPropertyUtilities.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "PropertyEditorModule.h"
#include "PropertyNode.h"
#include "PropertyPath.h"
#include "PropertyRowGenerator.h"
#include "StringPrefixTree.h"
#include "Widgets/SWindow.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

class FAssetThumbnailPool;
class FDetailCategoryImpl;
class FDetailLayoutBuilderImpl;
class FDetailTreeNode;
class FNotifyHook;
class FPropertyEditor;
class FUICommandList;
class IClassViewerFilter;
class IDetailCustomization;
class IDetailKeyframeHandler;
class IDetailPropertyExtensionHandler;
class IDetailPropertyChildrenCustomizationHandler;
class IPropertyGenerationUtilities;
class IPropertyUtilities;
class SDetailNameArea;
class SSearchBox;
struct FDetailsViewObjectRoot;

typedef STreeView< TSharedRef<FDetailTreeNode> > SDetailTree;

class SDetailsViewBase : public IDetailsViewPrivate
{
public:

	/** Global method to remove saved transient detail expansions */
	static void ResetTransientPerViewDetailExpansion();

	/** Global method to remove saved persistent detail expansions */
	static void ResetPersistentPerViewDetailExpansion();

public:
	SDetailsViewBase();
	virtual ~SDetailsViewBase();

	/**
	* @return true of the details view can be updated from editor selection
	*/
	virtual bool IsUpdatable() const override
	{
		return DetailsViewArgs.bUpdatesFromSelection;
	}

	virtual bool HasActiveSearch() const override
	{
		return CurrentFilter.FilterStrings.Num() > 0;
	}
	
	virtual void ClearSearch() override;
	virtual void SetSearchText(const FText& InSearch) override;

	virtual int32 GetNumVisibleTopLevelObjects() const override
	{
		return NumVisibleTopLevelObjectNodes;
	}

	/** @return The identifier for this details view, or NAME_None is this view is anonymous */
	virtual FName GetIdentifier() const override
	{
		return DetailsViewArgs.ViewIdentifier;
	}

	/**
	 * Sets the visible state of the filter box/property grid area
	 */
	virtual void HideFilterArea(bool bHide) override;

	/** 
	 * IDetailsView interface 
	 */
	virtual TArray< FPropertyPath > GetPropertiesInOrderDisplayed() const override;
	virtual TArray<TPair<int32, FPropertyPath>> GetPropertyRowNumbers() const override;
	virtual int32 CountRows() const override;
	virtual void HighlightProperty(const FPropertyPath& Property) override;
	virtual void ScrollPropertyIntoView(const FPropertyPath& Property, bool bExpandProperty = false) override;
	virtual FSlateRect GetPaintSpacePropertyBounds(const TSharedRef<FDetailTreeNode>& InDetailTreeNode, bool bIncludeChildren = true) const override;
	virtual FSlateRect GetTickSpacePropertyBounds(const TSharedRef<FDetailTreeNode>& InDetailTreeNode, bool bIncludeChildren = true) const override;
	virtual bool IsAncestorCollapsed(const TSharedRef<IDetailTreeNode>& Node) const override;
	virtual void ShowAllAdvancedProperties() override;
	virtual void SetOnDisplayedPropertiesChanged(FOnDisplayedPropertiesChanged InOnDisplayedPropertiesChangedDelegate) override;
	virtual FOnDisplayedPropertiesChanged& GetOnDisplayedPropertiesChanged() override { return OnDisplayedPropertiesChangedDelegate; }
	virtual void SetDisableCustomDetailLayouts( bool bInDisableCustomDetailLayouts ) override { bDisableCustomDetailLayouts = bInDisableCustomDetailLayouts; }
	virtual void SetIsPropertyVisibleDelegate(FIsPropertyVisible InIsPropertyVisible) override;
	virtual FIsPropertyVisible& GetIsPropertyVisibleDelegate() override { return IsPropertyVisibleDelegate; }
	virtual void SetIsCustomRowVisibleDelegate(FIsCustomRowVisible InIsCustomRowVisible) override;
	virtual FIsCustomRowVisible& GetIsCustomRowVisibleDelegate() override { return IsCustomRowVisibleDelegate; }
	virtual void SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly InIsPropertyReadOnly) override;
	virtual FIsCustomRowReadOnly& GetIsCustomRowReadOnlyDelegate() override { return IsCustomRowReadOnlyDelegate; }
	virtual void SetIsCustomRowReadOnlyDelegate(FIsCustomRowReadOnly InIsCustomRowReadOnly) override;
	virtual FIsPropertyReadOnly& GetIsPropertyReadOnlyDelegate() override { return IsPropertyReadOnlyDelegate; }
	virtual void SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled IsPropertyEditingEnabled) override;
	virtual FIsPropertyEditingEnabled& GetIsPropertyEditingEnabledDelegate() override { return IsPropertyEditingEnabledDelegate; }
	virtual bool IsPropertyEditingEnabled() const override;
	virtual void SetKeyframeHandler(TSharedPtr<IDetailKeyframeHandler> InKeyframeHandler) override;
	virtual TSharedPtr<IDetailKeyframeHandler> GetKeyframeHandler() const override { return KeyframeHandler; }
	virtual void SetExtensionHandler(TSharedPtr<IDetailPropertyExtensionHandler> InExtensionHandler) override;
	virtual TSharedPtr<IDetailPropertyExtensionHandler> GetExtensionHandler() const override { return ExtensionHandler; }
	virtual void SetChildrenCustomizationHandler(TSharedPtr<IDetailPropertyChildrenCustomizationHandler> InChildrenHandler) override;
	virtual TSharedPtr<IDetailPropertyChildrenCustomizationHandler> GetChildrenCustomizationHandler() const override { return ChildrenCustomizationHandler; }
	virtual bool IsPropertyVisible(const struct FPropertyAndParent& PropertyAndParent) const override;
	virtual bool IsPropertyReadOnly(const struct FPropertyAndParent& PropertyAndParent) const override;
	virtual bool IsCustomRowVisible(FName InRowName, FName InParentName) const override;
	virtual bool IsCustomRowReadOnly(FName InRowName, FName InParentName) const override;
	virtual void SetGenericLayoutDetailsDelegate(FOnGetDetailCustomizationInstance OnGetGenericDetails) override;
	virtual FOnGetDetailCustomizationInstance& GetGenericLayoutDetailsDelegate() override { return GenericLayoutDelegate; }
	virtual bool IsLocked() const override { return bIsLocked; }
	virtual void RefreshRootObjectVisibility() override;
	virtual FOnFinishedChangingProperties& OnFinishedChangingProperties() const override { return OnFinishedChangingPropertiesDelegate; }
	virtual void RegisterInstancedCustomPropertyLayout(UStruct* Class, FOnGetDetailCustomizationInstance DetailLayoutDelegate) override;
	virtual void RegisterInstancedCustomPropertyTypeLayout(FName PropertyTypeName, FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate, TSharedPtr<IPropertyTypeIdentifier> Identifier = nullptr) override;
	virtual void UnregisterInstancedCustomPropertyLayout(UStruct* Class) override;
	virtual void UnregisterInstancedCustomPropertyTypeLayout(FName PropertyTypeName, TSharedPtr<IPropertyTypeIdentifier> Identifier = nullptr) override;
	virtual void SetCustomValidatePropertyNodesFunction(FOnValidatePropertyRowGeneratorNodes InCustomValidatePropertyNodesFunction) override
	{
		CustomValidatePropertyNodesFunction = MoveTemp(InCustomValidatePropertyNodesFunction);
	}
	virtual float GetRightColumnMinWidth() const override;
	virtual void SetRightColumnMinWidth(float InMinWidth) override;
	
	/** IDetailsViewPrivate interface */
	virtual void RerunCurrentFilter() override;
	void SetNodeExpansionState(TSharedRef<FDetailTreeNode> InTreeNode, bool bExpand, bool bRecursive) override;
	void SaveCustomExpansionState(const FString& NodePath, bool bIsExpanded) override;
	bool GetCustomSavedExpansionState(const FString& NodePath) const override;
	virtual FName GetViewIdentifier() const override;
	virtual FDetailsViewArgs::EExpansionPersistenceMethod GetExpansionPersistenceMethod() const override;
	virtual void NotifyFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) override;
	void RefreshTree() override;
	virtual void RequestForceRefresh() override;
	TSharedPtr<FAssetThumbnailPool> GetThumbnailPool() const override;
	virtual const TArray<TSharedRef<IClassViewerFilter>>& GetClassViewerFilters() const override;
	TSharedPtr<IPropertyUtilities> GetPropertyUtilities() override;
	void CreateColorPickerWindow(const TSharedRef<FPropertyEditor>& PropertyEditor, bool bUseAlpha) override;
	virtual void UpdateSinglePropertyMap(TSharedPtr<FComplexPropertyNode> InRootPropertyNode, FDetailLayoutData& LayoutData, bool bIsExternal) override;
	virtual FNotifyHook* GetNotifyHook() const override { return DetailsViewArgs.NotifyHook; }
	virtual const FCustomPropertyTypeLayoutMap& GetCustomPropertyTypeLayoutMap() const { return InstancedTypeToLayoutMap; }
	virtual void SaveExpandedItems( TSharedRef<FPropertyNode> StartNode ) override;
	virtual void RestoreExpandedItems(TSharedRef<FPropertyNode> StartNode) override;
	virtual void RestoreExpandedItems(TSharedRef<FPropertyNode> StartNode, TMap<UStruct*, FStringPrefixTree>* OptionalExpansionStates);
	virtual void MarkNodeAnimating(TSharedPtr<FPropertyNode> InNode, float InAnimationDuration, TOptional<FGuid> InAnimationBatchId) override;
	virtual bool IsNodeAnimating(TSharedPtr<FPropertyNode> InNode) override;
	virtual float GetNodeAnimationTime(TSharedPtr<FPropertyNode> InNode) override;
	virtual FDetailColumnSizeData& GetColumnSizeData() override { return ColumnSizeData; }
	virtual bool IsFavoritingEnabled() const override { return DetailsViewArgs.bAllowFavoriteSystem; }
	virtual bool IsConnected() const = 0;
	virtual FRootPropertyNodeList& GetRootNodes() = 0;
	virtual void GetHeadNodes(TArray<TWeakPtr<FDetailTreeNode>>&) override;

	/**
	 * Called when the open color picker window associated with this details view is closed
	 */
	void OnColorPickerWindowClosed(const TSharedRef<SWindow>& Window);

	/**
	 * Requests that an item in the tree be expanded or collapsed
	 *
	 * @param TreeNode	The tree node to expand
	 * @param bExpand	true if the item should be expanded, false otherwise
	 */
	void RequestItemExpanded(TSharedRef<FDetailTreeNode> TreeNode, bool bExpand) override;

	/**
	 * Sets the expansion state all root nodes and optionally all of their children
	 *
	 * @param bExpand			The new expansion state
	 * @param bRecurse			Whether or not to apply the expansion change to any children
	 */
	virtual void SetRootExpansionStates(const bool bExpand, const bool bRecurse) override;

	/**
	 * Adds an action to execute next tick
	 */
	virtual void EnqueueDeferredAction(FSimpleDelegate& DeferredAction) override;

	/** Restore all expanded items in root nodes and external root nodes. */
	void RestoreAllExpandedItems(TMap<UStruct*, FStringPrefixTree>* OptionalExpansionStates = nullptr);

	/**
	 * Returns a @code TSharedPtr @endcode to the @code FDetailsDisplayManager @endcode for this
	 * details view
	 */
	virtual TSharedPtr<FDetailsDisplayManager> GetDisplayManager() override;

	//~ Begin SWidget
	virtual bool SupportsKeyboardFocus() const override;
	virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget

	// IScrollableWidget interface
	virtual FVector2D GetScrollDistance() override;
	virtual FVector2D GetScrollDistanceRemaining() override;
	virtual TSharedRef<SWidget> GetScrollWidget() override;
	// End of IScrollableWidget interface

	/**
	 * Resets the selected sections to the default 'All' section.
	 * Base class returns false as sections selection is not application to all detail views.
	 * 
	 * @return whether the operation succeeded.
	 */
	virtual bool ResetToDefaultSection() override { return false; }

protected:
	/**
	 * Called when a color property is changed from a color picker
	 */
	void SetColorPropertyFromColorPicker(FLinearColor NewColor);

	/** Contains one or more nodes being animated by a single timer. */
	struct FAnimatingNodeCollection
	{
		FAnimatingNodeCollection()
			: BatchId(FGuid::NewGuid())
			, CreationTime(FPlatformTime::Seconds())
		{
		}

		/** Creates a collection for the given node(s). If a BatchId is not specified, it's calculated based on the provided nodes. */
		explicit FAnimatingNodeCollection(const TConstArrayView<TSharedPtr<FPropertyPath>>& InNodePaths, const FGuid& InBatchId = FGuid());

		TArray<TSharedPtr<FPropertyPath>> NodePaths;

		/** Required id for tracking and removal (Property Nodes aren't reliable), which also allow multiple nodes to be part of the same animation. */
		FGuid BatchId;

		FTimerHandle NodeTimer;

		/** Utility function that provides a deterministic Id for the given NodePath. */
		static FGuid GetIdForNodePath(const TSharedPtr<FPropertyPath>& InNodePath);

		/** Get validity of this AnimatingNode, determined by it's node path. */
		bool IsValid() const;

		bool operator==(const FAnimatingNodeCollection& Other) const
		{
			if (NodePaths.Num() != Other.NodePaths.Num())
			{
				return false;
			}

			if (BatchId.IsValid())
			{
				return BatchId == Other.BatchId;
			}

			for (int32 Idx = 0; Idx < NodePaths.Num(); ++Idx)
			{
				if (!NodePaths[Idx].IsValid()
					|| !Other.NodePaths[Idx].IsValid())
				{
					continue;
				}

				if (!FPropertyPath::AreEqual(NodePaths[Idx].ToSharedRef(), Other.NodePaths[Idx].ToSharedRef()))
				{
					return false;
				}
			}
			
			return true;
		}

		bool operator!=(const FAnimatingNodeCollection& Other) const
		{
			return !(*this == Other);
		}

		/** Effectively resets the stored animation time. */
		void ResetElapsedSeconds();

		float GetElapsedSeconds() const;

	private:
        /** The time at which this FAnimatingNodeCollection was created, useful for resuming animation on widget reconstruction. */
        double CreationTime = 0.0f;
	};

	/** Return string key for the given struct in config .ini for this details view's properties */
	const FString& GetDetailPropertyExpansionSection() const;

	/** Return string key for the given struct in config .ini for this details view's custom widgets */
	const FString& GetDetailCustomWidgetExpansionSection() const;

	/** Return string key for the given struct in config .ini for this details view */
	FString GetStructExpansionConfigKey(UStruct* InStruct) const;

	/** Called when a node collection animation completes */
	void HandleNodeAnimationComplete(FAnimatingNodeCollection* InAnimatedNode);

	/** Returns the owning FAnimatingNodeCollection for the provided PropertyNode, if any. */
	FAnimatingNodeCollection* GetAnimatingNodeCollectionForPropertyNode(const TSharedPtr<FPropertyNode>& InNode);

	/** Updates the property map for access when customizing the details view.  Generates default layout for properties */
	void UpdatePropertyMaps();

	virtual void CustomUpdatePropertyMap(TSharedPtr<FDetailLayoutBuilderImpl>& InDetailLayout) {}

	void GetExpansionState(UStruct* BaseStruct, FStringPrefixTree& OutPrefixTree) const;
	UStruct* GetExpansionStruct(const TSharedRef<FPropertyNode>& StartNode);

	/** Updates the property map while keeping the expansion state of all items */
	void UpdatePropertyMapsWithItemExpansions();

	/** Called to get the visibility of the tree view */
	EVisibility GetTreeVisibility() const;

	/** Called to get the visibility of the scrollbar based on options, needs to be dynamic to avoid layout changing on expansion */
	EVisibility GetScrollBarVisibility() const;

	/** Returns the name of the image used for the icon on the filter button */
	const FSlateBrush* OnGetFilterButtonImageResource() const;

	/** Called when the locked button is clicked */
	FReply OnLockButtonClicked();

	/**
	 * Called to recursively expand/collapse the children of the given item
	 *
	 * @param InTreeNode		The node that was expanded or collapsed
	 * @param bIsItemExpanded	True if the item is expanded, false if it is collapsed
	 */
	void SetNodeExpansionStateRecursive( TSharedRef<FDetailTreeNode> InTreeNode, bool bIsItemExpanded );

	/**
	 * Called when an item is expanded or collapsed in the detail tree
	 *
	 * @param InTreeNode		The node that was expanded or collapsed
	 * @param bIsItemExpanded	True if the item is expanded, false if it is collapsed
	 */
	void OnItemExpansionChanged( TSharedRef<FDetailTreeNode> InTreeNode, bool bIsItemExpanded );

	/** 
	 * Function called through a delegate on the TreeView to request children of a tree node 
	 * 
	 * @param InTreeNode		The tree node to get children from
	 * @param OutChildren		The list of children of InTreeNode that should be visible 
	 */
	void OnGetChildrenForDetailTree( TSharedRef<FDetailTreeNode> InTreeNode, TArray< TSharedRef<FDetailTreeNode> >& OutChildren );

	/**
	 * Returns an SWidget used as the visual representation of a node in the treeview.                     
	 */
	TSharedRef<ITableRow> OnGenerateRowForDetailTree( TSharedRef<FDetailTreeNode> InTreeNode, const TSharedRef<STableViewBase>& OwnerTable );

	/** Clear focus when the row is scrolled out of view. */
	void OnRowReleasedForDetailTree(const TSharedRef<ITableRow>& TableRow);

	/** @return true if show only modified is checked */
	bool IsShowOnlyModifiedChecked() const { return CurrentFilter.bShowOnlyModified; }

	/** @return true if custom filter is checked */
	bool IsCustomFilterChecked() const { return bCustomFilterActive; }

	/** @return true if show all advanced is checked */
	bool IsShowAllAdvancedChecked() const { return CurrentFilter.bShowAllAdvanced; }

	/** @return true if show only differing is checked */
	bool IsShowOnlyAllowedChecked() const { return CurrentFilter.bShowOnlyAllowed; }

	/** @return true if show all advanced is checked */
	bool IsShowAllChildrenIfCategoryMatchesChecked() const { return CurrentFilter.bShowAllChildrenIfCategoryMatches; }
	
	/** @return true if show keyable is checked */
	bool IsShowKeyableChecked() const { return CurrentFilter.bShowOnlyKeyable; }
	
	/** @return true if show animated is checked */
	bool IsShowAnimatedChecked() const { return CurrentFilter.bShowOnlyAnimated; }

	/** Called when show only modified is clicked */
	void OnShowOnlyModifiedClicked();

	/** Called when custom filter is clicked */
	void OnCustomFilterClicked();

	/** Called when show all advanced is clicked */
	void OnShowAllAdvancedClicked();

	/** Called when show only differing is clicked */
	void OnShowOnlyAllowedClicked();

	/** Called when show all children if category matches is clicked */
	void OnShowAllChildrenIfCategoryMatchesClicked();

	/** Called when show keyable is clicked */
	void OnShowKeyableClicked();
	
	/** Called when show animated is clicked */
	void OnShowAnimatedClicked();

	/** Apply the CurrentFilter to the given root node. */
	void FilterRootNode(const TSharedPtr<FComplexPropertyNode>& RootNode);

	/** Updates the details with the CurrentFilter */
	void UpdateFilteredDetails();

	/** Called when the filter text changes.  This filters specific property nodes out of view */
	void OnFilterTextChanged(const FText& InFilterText);

	/** Called when the filter text is cleared. Resets the filters */
	void OnFilterTextCommitted(const FText& InSearchText, ETextCommit::Type InCommitType);

	/** Called when the list of currently differing properties changes */
	virtual void UpdatePropertyAllowList(const TSet<FPropertyPath>& InAllowedProperties) override { CurrentFilter.PropertyAllowList = InAllowedProperties; }

	virtual TSharedPtr<SWidget> GetNameAreaWidget() override;
	virtual void SetNameAreaCustomContent(TSharedRef<SWidget>& InCustomContent) override;
	virtual TSharedPtr<SWidget> GetFilterAreaWidget() override;
	virtual TSharedPtr<FUICommandList> GetHostCommandList() const override;
	virtual TSharedPtr<FTabManager> GetHostTabManager() const override;
	virtual void SetHostTabManager(TSharedPtr<FTabManager> InTabManager) override;

	/** 
	 * Hides or shows properties based on the passed in filter text
	 * 
	 * @param InFilterText	The filter text
	 */
	void FilterView( const FString& InFilterText );

	/** Called to get the visibility of the filter box */
	EVisibility GetFilterBoxVisibility() const;


	virtual void SetCustomFilterDelegate(FSimpleDelegate InOverride) override
	{
		CustomFilterDelegate = InOverride;
	}

	virtual void SetCustomFilterLabel(FText InText) override
	{
		CustomFilterLabel = InText;
	}

	FText GetCustomFilterLabel() const
	{
		return CustomFilterLabel;
	}

	/** Free memory that is pending delete next editor tick even when widget not visible or ticking */
	void SetPendingCleanupTimer();
	/** Free memory that is pending delete during editor tick */
	void HandlePendingCleanupTimer();
	/** Free memory that is pending delete */
	void HandlePendingCleanup();

	/** Set timer to force refresh if one not already set */
	void SetPendingRefreshTimer();
	/** Clear timer to force refresh if set */
	void ClearPendingRefreshTimer();
	/** Force refresh during editor tick */
	void HandlePendingRefreshTimer();

	void SavePreSearchExpandedItems();
	void RestorePreSearchExpandedItems();

	/** 
	 * Get a mutable version of the view config for setting values.
	 * @returns		The view config for this view. 
	 * @note		If DetailsViewArgs.ViewIdentifier is not set, it is not possible to store settings for this view.
	 */
	struct FDetailsViewConfig* GetMutableViewConfig();

	/**
	 * Get a const version of the view config for getting values.
	 * @returns		The view config for this view. 
	 * @note		If DetailsViewArgs.ViewIdentifier is not set, it is not possible to retrieve settings for this view.
	 */
	const FDetailsViewConfig* GetConstViewConfig() const;
	void SaveViewConfig();

	/** Clear the keyboard focus if it's inside the given widget. */
	void ClearKeyboardFocusIfWithin(const TSharedRef<SWidget>& Widget) const;
	
	/** Creates details filter before filtering nodes */
	FDetailFilter CreateDetailFilter();
	
	void BindCommands();

protected:
	/** The user defined args for the details view */
	FDetailsViewArgs DetailsViewArgs;
	/** A mapping of classes to detail layout delegates, called when querying for custom detail layouts in this instance of the details view only*/
	FCustomDetailLayoutMap InstancedClassToDetailLayoutMap;
	/** A mapping of type names to detail layout delegates, called when querying for custom detail layouts in this instance of the details view only */
	FCustomPropertyTypeLayoutMap InstancedTypeToLayoutMap;
	/** The current detail layout based on objects in this details panel.  There is one layout for each top level object node.*/
	TArray<FDetailLayoutData> DetailLayouts;
	/** Row for searching and view options */
	TSharedPtr<SWidget> FilterRow;
	/** Search box */
	TSharedPtr<SSearchBox> SearchBox;
	/** Customization instances that need to be destroyed when safe to do so */
	TArray< TSharedPtr<IDetailCustomization> > CustomizationClassInstancesPendingDelete;
	/** Detail layouts that need to be destroyed when safe to do so */
	TArray< TSharedPtr<FDetailLayoutBuilderImpl> > DetailLayoutsPendingDelete;
	/** Map of nodes that are requesting an automatic expansion/collapse due to being filtered */
	TMap< TWeakPtr<FDetailTreeNode>, bool > FilteredNodesRequestingExpansionState;
	/** Tree view */
	TSharedPtr<SDetailTree> DetailTree;
	/** Root tree nodes visible in the tree */
	FDetailNodeList RootTreeNodes;
	/** A container for external overlays provided by display managers. */
	TSharedPtr<SBox> ExternalOverlayContainer;
	/** Delegate executed to determine if a property should be visible */
	FIsPropertyVisible IsPropertyVisibleDelegate;
	/** Delegate executed to determine if a custom row should be visible. */
	FIsCustomRowVisible IsCustomRowVisibleDelegate;
	/** Delegate executed to determine if a property should be read-only */
	FIsPropertyReadOnly IsPropertyReadOnlyDelegate;
	/** Delegate executed to determine if a custom row should be read-only. */
	FIsCustomRowReadOnly IsCustomRowReadOnlyDelegate;
	/** Delegate called to see if a property editing is enabled */
	FIsPropertyEditingEnabled IsPropertyEditingEnabledDelegate;
	/** Delegate called when the details panel finishes editing a property (after post edit change is called) */
	mutable FOnFinishedChangingProperties OnFinishedChangingPropertiesDelegate;
	/** Container for passing around column size data to rows in the tree (each row has a splitter which can affect the column size)*/
	FDetailColumnSizeData ColumnSizeData;
	/** The property node that the color picker is currently editing. */
	TWeakPtr<FPropertyNode> ColorPropertyNode;
	/** Settings for this view */
	TSharedPtr<IPropertyUtilities> PropertyUtilities;
	/** Gets internal utilities for generating property layouts. */
	TSharedPtr<IPropertyGenerationUtilities> PropertyGenerationUtilities;
	/** The name area which is not recreated when selection changes */
	TSharedPtr<SDetailNameArea> NameArea;
	/** The current filter */
	FDetailFilter CurrentFilter;
	/** Delegate called to get generic details not specific to an object being viewed */
	FOnGetDetailCustomizationInstance GenericLayoutDelegate;
	/** Actions that should be executed next tick */
	TArray<FSimpleDelegate> DeferredActions;
	/** Root tree nodes that needs to be destroyed when safe */
	FRootPropertyNodeList RootNodesPendingKill;
	/** The handler for the keyframe UI, determines if the key framing UI should be displayed. */
	TSharedPtr<IDetailKeyframeHandler> KeyframeHandler;
	/** Property extension handler returns additional UI to apply after the customization is applied to the property. */
	TSharedPtr<IDetailPropertyExtensionHandler> ExtensionHandler;
	/** Property children customization handler allows to customize property's children. */
	TSharedPtr<IDetailPropertyChildrenCustomizationHandler> ChildrenCustomizationHandler;
	/** The tree node that is currently highlighted, may be none. */
	TWeakPtr<FDetailTreeNode> CurrentlyHighlightedNode;
	/** The list of nodes whose widgets should be animating. */
	TArray<FAnimatingNodeCollection> CurrentlyAnimatingNodeCollections;

	/**
	 * The @code DetailsDisplayManager @endcode which provides an API to manage some of the characteristics of the
	 * details display
	 */
	TSharedPtr<FDetailsDisplayManager> DisplayManager;

	/** Current set of expanded detail nodes (by path) that should be saved when the details panel closes */
	FStringPrefixTree ExpandedDetailNodes;
	FStringPrefixTree PreSearchExpandedItems;
	FStringPrefixTree PreSearchExpandedCategories;

	/** Executed when the tree is refreshed */
	FOnDisplayedPropertiesChanged OnDisplayedPropertiesChangedDelegate;

	int32 NumVisibleTopLevelObjectNodes;

	/** Used to refresh the tree when the allow list filter changes */
	FDelegateHandle PropertyPermissionListChangedDelegate;
	FDelegateHandle PropertyPermissionListEnabledDelegate;

	/** Used to refresh the tree when the language changes */
	FDelegateHandle OnCultureChangedDelegate;

	/** Delegate for overriding the show modified filter */
	FSimpleDelegate CustomFilterDelegate;
	FText CustomFilterLabel;

	/** True if this property view is currently locked (I.E The objects being observed are not changed automatically due to user selection)*/
	bool bIsLocked : 1;

	/** Whether or not this instance of the details view opened a color picker and it is not closed yet */
	bool bHasOpenColorPicker : 1;

	/** True if we want to skip generation of custom layouts for displayed object */
	bool bDisableCustomDetailLayouts : 1;

	/** When overriding show modified, you can't use the filter state to determine what is overridden anymore. Use this variable instead. */
	bool bCustomFilterActive : 1;

	/** Timer has already been set to be run next tick */
	bool bPendingCleanupTimerSet : 1;

	/** Are we currently running deferred actions? */
	bool bRunningDeferredActions : 1;

	/** Handle to the pending refresh request, if any */
	FTimerHandle PendingRefreshTimerHandle;

	mutable TSharedPtr<FEditConditionParser> EditConditionParser;
	
	/** Optional custom filter(s) to be applied when selecting values for class properties */
	TArray<TSharedRef<IClassViewerFilter>> ClassViewerFilters;

	/** The EnsureDataIsValid function can be skipped with this member, if set. Useful if your implementation doesn't require this kind of validation each Tick. */
	FOnValidateDetailsViewPropertyNodes CustomValidatePropertyNodesFunction;
	
	/** Internal details commands */
	TSharedPtr<FUICommandList> InternalCommands;
};
