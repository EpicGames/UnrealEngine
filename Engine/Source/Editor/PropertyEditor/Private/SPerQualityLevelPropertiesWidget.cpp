// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPerQualityLevelPropertiesWidget.h"
#include "Layout/Margin.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/CoreStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "PlatformInfo.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBox.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Images/SImage.h"
#include "Scalability.h"
#include "PerQualityLevelProperties.h"

void SOverridePropertiesWidget::Construct(const typename SOverridePropertiesWidget::FArguments& InArgs)
{
	this->OnGenerateWidget = InArgs._OnGenerateWidget;
	this->OnAddEntry = InArgs._OnAddEntry;
	this->OnRemoveEntry = InArgs._OnRemoveEntry;
	this->EntryNames = InArgs._EntryNames;
}


FReply SOverridePropertiesWidget::RemoveEntry(FName OverrideName)
{
	if (OnRemoveEntry.IsBound() && OnRemoveEntry.Execute(OverrideName))
	{
		ConstructChildren();
		Invalidate(EInvalidateWidget::LayoutAndVolatility);
	}
	return FReply::Handled();
}

TSharedRef<SWidget> SOverridePropertiesWidget::MakeOverrideWidget(FName InName, FText InDisplayText, const TArray<FName>& InEntries, FMenuBuilder& InAddMenuBuilder)
{
	TSharedPtr<SHorizontalBox> HorizontalBox;

	TSharedRef<SWidget> Widget =
		SNew(SBox)
		.ToolTipText((InName == NAME_None) ?
			NSLOCTEXT("SPerQualityLevelPropertiesWidget", "DefaultQualityLevelDesc", "This property can have per quality level overrides.\nThis is the default value used when no override has been set for a specific quality level.") :
			FText::Format(NSLOCTEXT("SPerQualityLevelPropertiesWidget", "QualityLevelDesc", "Override for {0}"), InDisplayText))
		.Padding(FMargin(0.0f, 2.0f, 4.0f, 2.0f))
		.MinDesiredWidth(50.0f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(HorizontalBox, SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(0.0f, 0.0f, 2.0f, 2.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(InDisplayText)
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				OnGenerateWidget.Execute(InName)
			]
		];

	if (InName != NAME_None)
	{
		HorizontalBox->AddSlot()
			.AutoWidth()
			.Padding(2.0f, 0.0f, 0.0f, 2.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(2.0f)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.OnClicked(this, &SOverridePropertiesWidget::RemoveEntry, InName)
				.ToolTipText(FText::Format(NSLOCTEXT("SOverridePropertiesWidget", "RemoveOverrideFor", "Remove Override for {0}"), InDisplayText))
				.ForegroundColor(FSlateColor::UseForeground())
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Delete"))
				]
			];
	}
	else
	{
		HorizontalBox->AddSlot()
			.AutoWidth()
			.Padding(2.0f, 0.0f, 0.0f, 2.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SComboButton)
				.Visibility_Lambda([this]() { return bAddedMenuItem ? EVisibility::Visible : EVisibility::Hidden; })
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ContentPadding(2.0f)
				.ForegroundColor(FSlateColor::UseForeground())
				.HasDownArrow(false)
				.ButtonContent()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
				]
				.MenuContent()
				[
					InAddMenuBuilder.MakeWidget()
				]
				.ToolTipText(NSLOCTEXT("SOverridePropertiesWidget", "AddOverrideToolTip", "Add an override for a specific quality level"))
			];
	}

	return Widget;
}

void SOverridePropertiesWidget::AddEntryToMenu(const FName& EntryName, const FTextFormat Format, FMenuBuilder& AddEntryMenuBuilder)
{
	const FText MenuText = FText::Format(FText::FromString(TEXT("{0}")), FText::AsCultureInvariant(EntryName.ToString()));
	const FText MenuTooltipText = FText::Format(Format, FText::AsCultureInvariant(EntryName.ToString()));
	AddEntryMenuBuilder.AddMenuEntry(
		MenuText,
		MenuTooltipText,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "OverridePropertiesWidget.AddEntry"),
		FUIAction(FExecuteAction::CreateSP(this, &SOverridePropertiesWidget::AddEntry, EntryName))
	);
}

void SOverridePropertiesWidget::AddEntry(FName EntryName)
{
	if (OnAddEntry.IsBound() && OnAddEntry.Execute(EntryName))
	{
		ConstructChildren();
		Invalidate(EInvalidateWidget::LayoutAndVolatility);
	}
}

void SPerQualityLevelPropertiesWidget::Construct(const typename SOverridePropertiesWidget::FArguments& InArgs)
{
	ToolTip = FString(TEXT("Add an override for a specific quality level"));

	SOverridePropertiesWidget::Construct(InArgs);

	ConstructChildren();
}


void SPerQualityLevelPropertiesWidget::ConstructChildren()
{
	TSharedPtr<SWrapBox> WrapBox;

	TArray<FName> Overrides = EntryNames.Get();
	LastEntryNames = Overrides.Num();

	ChildSlot
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Fill)
		[
			SAssignNew(WrapBox, SWrapBox)
			.UseAllottedSize(true)
		];

	if (OnGenerateWidget.IsBound())
	{
		// Build quality level menu
		bAddedMenuItem = false;
		FMenuBuilder AddMenuBuilder(true, nullptr, nullptr, true);

		const FName Section(TEXT("PlatformGroupSection"));
		AddMenuBuilder.BeginSection(Section, FText::FromString(TEXT("Quality Levels")));

		int32 QualityLevelCount = 0;
		while (QualityLevelCount < static_cast<int32>(EPerQualityLevels::Num))
		{
			FName QualityName = QualityLevelProperty::QualityLevelToFName(QualityLevelCount++);
			const FTextFormat Format = NSLOCTEXT("SPerQualityLevelPropertiesWidget", "AddOverrideGroupFor", "Add Override specifically for {0}");
			AddEntryToMenu(QualityName, Format, AddMenuBuilder);
			bAddedMenuItem = true;
			
		}

		AddMenuBuilder.EndSection();

		// Default control
		WrapBox->AddSlot()
			[
				MakeOverrideWidget(NAME_None, NSLOCTEXT("SPerQualityLevelPropertiesWidget", "DefaultQuality", "Default"), Overrides, AddMenuBuilder)
			];

		for (FName Override : Overrides)
		{
			WrapBox->AddSlot()
				[
					MakeOverrideWidget(Override, FText::AsCultureInvariant(Override.ToString()), Overrides, AddMenuBuilder)
				];
		}
	}
	else
	{
		WrapBox->AddSlot()
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("SPerQualityLevelPropertiesWidget", "OnGenerateWidgetWarning", "No OnGenerateWidget() Provided"))
			.ColorAndOpacity(FLinearColor::Red)
			];
	}
}