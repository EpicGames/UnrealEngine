// Copyright Epic Games, Inc. All Rights Reserved.


#include "SPropertyTreeViewImpl.h"
#include "PropertyNode.h"
#include "ObjectPropertyNode.h"
#include "EngineGlobals.h"
#include "GameFramework/Actor.h"
#include "Engine/Engine.h"
#include "Styling/AppStyle.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "EditConditionParser.h"

#include "CategoryPropertyNode.h"
#include "UserInterface/PropertyTree/PropertyTreeConstants.h"

#include "UserInterface/PropertyEditor/SPropertyEditorTableRow.h"
#include "UserInterface/PropertyTree/SPropertyTreeCategoryRow.h"
#include "ScopedTransaction.h"
#include "PropertyEditorHelpers.h"
#include "PropertyPermissionList.h"
#include "Misc/ConfigCacheIni.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SSearchBox.h"


class FPropertyUtilitiesTreeView : public IPropertyUtilities
{
public:

	FPropertyUtilitiesTreeView( SPropertyTreeViewImpl& InView )
		: View( InView )
	{
		EditConditionParser = MakeShared<FEditConditionParser>();
	}

	virtual class FNotifyHook* GetNotifyHook() const override
	{
		return View.GetNotifyHook();
	}

	virtual bool AreFavoritesEnabled() const override
	{
		return View.AreFavoritesEnabled();
	}

	virtual void ToggleFavorite( const TSharedRef< class FPropertyEditor >& PropertyEditor ) const override
	{
		View.ToggleFavorite( PropertyEditor );
	}

	virtual void CreateColorPickerWindow( const TSharedRef< class FPropertyEditor >& PropertyEditor, bool bUseAlpha ) const override
	{
		View.CreateColorPickerWindow( PropertyEditor, bUseAlpha );
	}

	virtual void EnqueueDeferredAction( FSimpleDelegate DeferredAction ) override
	{
		View.EnqueueDeferredAction( DeferredAction );
	}

	virtual void ForceRefresh() override
	{
		RequestRefresh();
	}

	virtual void RequestRefresh() override
	{
		View.RequestRefresh();
	}

	virtual void RequestForceRefresh() override
	{
		// RequestRefresh is already a deferred ForceRefresh
		RequestRefresh();
	}

	virtual bool IsPropertyEditingEnabled() const override
	{
		return true;
	}

	virtual TSharedPtr<class FAssetThumbnailPool> GetThumbnailPool() const override
	{
		return NULL;
	}

	virtual const TArray<TSharedRef<class IClassViewerFilter>>& GetClassViewerFilters() const override
	{
		// not implemented
		static TArray<TSharedRef<class IClassViewerFilter>> NotImplemented;
		return NotImplemented;
	}

	virtual void NotifyStartedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) override {}
	virtual void NotifyFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) override {}

	virtual bool DontUpdateValueWhileEditing() const override
	{
		return false;
	}

	const TArray<TWeakObjectPtr<UObject>>& GetSelectedObjects() const override
	{
		static TArray<TWeakObjectPtr<UObject>> NotSupported;
		return NotSupported;
	}

	virtual bool HasClassDefaultObject() const override
	{
		return false;
	}

	virtual TSharedPtr<FEditConditionParser> GetEditConditionParser() const
	{
		return EditConditionParser;
	}

private:
	TSharedPtr<FEditConditionParser> EditConditionParser;
	SPropertyTreeViewImpl& View;
};


SPropertyTreeViewImpl::SPropertyTreeViewImpl()
	: RootPath( FPropertyPath::CreateEmpty() )
{

}


/**
 * Constructs the widget
 *
 * @param InArgs   Declaration from which to construct this widget.              
 */
void SPropertyTreeViewImpl::Construct(const FArguments& InArgs)
{
	bLockable = InArgs._IsLockable;
	bHasActiveFilter = false;
	bIsLocked = false;
	bAllowSearch = InArgs._AllowSearch;
	bFavoritesAllowed =  InArgs._AllowFavorites;
	bShowTopLevelPropertyNodes = InArgs._ShowTopLevelNodes;
	NotifyHook = InArgs._NotifyHook;
	bForceHiddenPropertyVisibility = InArgs._HiddenPropertyVis;
	InitialNameColumnWidth = InArgs._NameColumnWidth;
	bNodeTreeExternallyManaged = false;
	OnPropertySelectionChanged = InArgs._OnPropertySelectionChanged;
	OnPropertyMiddleClicked = InArgs._OnPropertyMiddleClicked;
	ConstructExternalColumnHeaders = InArgs._ConstructExternalColumnHeaders;
	ConstructExternalColumnCell = InArgs._ConstructExternalColumnCell;

	if( !GConfig->GetBool(TEXT("PropertyWindow"), TEXT("ShowFavoritesWindow"), bFavoritesEnabled, GEditorPerProjectIni) )
	{
		bFavoritesEnabled = false;
	}

	bFavoritesEnabled = bFavoritesEnabled && bFavoritesAllowed;

	// Create the root property now
	RootPropertyNode = MakeShared<FObjectPropertyNode>();

	PropertySettings = MakeShared<FPropertyUtilitiesTreeView>(*this);

	ConstructPropertyTree();
	
	FPropertyEditorPermissionList::Get().PermissionListUpdatedDelegate.AddSP(this, &SPropertyTreeViewImpl::OnPermissionListUpdated);
	FPropertyEditorPermissionList::Get().PermissionListEnabledDelegate.AddSP(this, &SPropertyTreeViewImpl::RequestRefresh);

	FInternationalization::Get().OnCultureChanged().AddSP(this, &SPropertyTreeViewImpl::RequestRefresh);
}

void SPropertyTreeViewImpl::OnPermissionListUpdated(TSoftObjectPtr<const UStruct>, FName)
{
	RequestRefresh();
}

/** Reconstructs the entire property tree widgets */
void SPropertyTreeViewImpl::ConstructPropertyTree()
{
	const FString OldFilterText = CurrentFilterText;
	CurrentFilterText.Empty();

	FavoritesTree.Reset();
	PropertyTree.Reset();
	FilterTextBox.Reset();

	// Don't pad area around the search bar if we aren't showing anything in that area
	float PaddingBeforeFilter = ( bAllowSearch || bFavoritesAllowed || bLockable ) ? 5.0f : 0.0f;
	float PaddingAfterFilter = ( bAllowSearch || bFavoritesAllowed || bLockable ) ? 10.0f : 0.0f;

	ESelectionMode::Type SelectionMode = ESelectionMode::None;
	if ( OnPropertySelectionChanged.IsBound() )
	{
		SelectionMode = ESelectionMode::Single;
	}

	this->ChildSlot
	[
		SNew( SVerticalBox )
		+SVerticalBox::Slot()
		.AutoHeight()
		.VAlign( VAlign_Fill )
		.Padding( PaddingBeforeFilter )
		[
			SNew( SHorizontalBox )
			+ SHorizontalBox::Slot()
			.HAlign( HAlign_Fill )
			.FillWidth(1.0f)
			.Padding(0,0,3,0)
			[
				SAssignNew( FilterTextBox, SSearchBox )
				.Visibility( bAllowSearch ? EVisibility::Visible : EVisibility::Collapsed )
				.OnTextChanged( this, &SPropertyTreeViewImpl::OnFilterTextChanged  )
			]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
			[
				SNew( SButton )
				.Visibility( bFavoritesAllowed ? EVisibility::Visible : EVisibility::Collapsed )
				.OnClicked( this, &SPropertyTreeViewImpl::OnToggleFavoritesClicked )
				.ContentPadding(1.0f)
				.ButtonStyle( FAppStyle::Get(), "NoBorder" )
				[
					SNew( SImage )
						.Image( this, &SPropertyTreeViewImpl::OnGetFavoriteButtonImageResource )
				]
			]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
			[
				SNew( SButton )
				.Visibility( bLockable ? EVisibility::Visible : EVisibility::Collapsed )
				.OnClicked( this, &SPropertyTreeViewImpl::OnLockButtonClicked )
				.ContentPadding(1.0f)
				.ButtonStyle( FAppStyle::Get(), "NoBorder" )
				[
					SNew( SImage )
					.Image( this, &SPropertyTreeViewImpl::OnGetLockButtonImageResource )
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.VAlign( VAlign_Top )
		.MaxHeight(200.0f)
		[

			SAssignNew( FavoritesTree, SPropertyTree )
				.Visibility( this, &SPropertyTreeViewImpl::OnGetFavoritesVisibility )
				.TreeItemsSource( &TopLevelFavorites )
				.OnGetChildren( this, &SPropertyTreeViewImpl::OnGetChildFavoritesForPropertyNode )
				.OnGenerateRow( this, &SPropertyTreeViewImpl::OnGenerateRowForPropertyTree )
				.SelectionMode( ESelectionMode::None )
				.HeaderRow
				(
					SNew(SHeaderRow)
					+SHeaderRow::Column(PropertyTreeConstants::ColumnId_Name)
					.DefaultLabel(PropertyTreeConstants::ColumnText_Name)
					.FillWidth(200)
					+SHeaderRow::Column(PropertyTreeConstants::ColumnId_Property)
					.DefaultLabel(PropertyTreeConstants::ColumnText_Property)
					.FillWidth(800)
				)
		]
		+SVerticalBox::Slot()
			.VAlign( VAlign_Fill )
			.FillHeight(1.0f)
			.Padding( 0.0f, PaddingAfterFilter, 0.0f, 0.0f )
		[
			SAssignNew( PropertyTree, SPropertyTree )
				.TreeItemsSource( &TopLevelPropertyNodes )
				.OnGetChildren( this, &SPropertyTreeViewImpl::OnGetChildrenForPropertyNode )
				.OnGenerateRow( this, &SPropertyTreeViewImpl::OnGenerateRowForPropertyTree )
				.OnSelectionChanged( this, &SPropertyTreeViewImpl::OnSelectionChanged )
				.SelectionMode( SelectionMode )
				.HeaderRow
				(
					SAssignNew(ColumnHeaderRow, SHeaderRow)
					+SHeaderRow::Column(PropertyTreeConstants::ColumnId_Name)
					.FillWidth(InitialNameColumnWidth)
					[
						SNew(SBorder)
						.Padding(3.0f)
						.BorderImage( FAppStyle::GetBrush("NoBorder") )
						[
							SNew(STextBlock)
							.Text( NSLOCTEXT("PropertyEditor", "NameColumn", "Name") )
						]
					]
					+SHeaderRow::Column(PropertyTreeConstants::ColumnId_Property)
					.FillWidth(1.0f)
					[
						SNew(SBorder)
						.Padding(3.0f)
						.BorderImage( FAppStyle::GetBrush("NoBorder") )
						[
							SNew(STextBlock)
							.Text( NSLOCTEXT("PropertyEditor", "PropertyColumn", "Value") )
						]
					]
				)
		]
	];

	// If we had an old filter, restore it.
	if(!OldFilterText.IsEmpty())
	{
		SetFilterText(FText::FromString(OldFilterText));
	}

	ConstructExternalColumnHeaders.ExecuteIfBound( ColumnHeaderRow.ToSharedRef() );
}

FReply SPropertyTreeViewImpl::OnToggleFavoritesClicked()
{
	check( RootPropertyNode.IsValid() );
	
	// Toggle favorites
	bFavoritesEnabled = !bFavoritesEnabled;

	//save off state of the filter window
	GConfig->SetBool(TEXT("PropertyWindow"), TEXT("ShowFavoritesWindow"), bFavoritesEnabled, GEditorPerProjectIni);

	return FReply::Handled();
}

FReply SPropertyTreeViewImpl::OnLockButtonClicked()
{
	bIsLocked = !bIsLocked;
	return FReply::Handled();
}

void SPropertyTreeViewImpl::SetFilterText( const FText& InFilterText )
{
	FilterTextBox->SetText(InFilterText);
}

/** Called when the filter text changes.  This filters specific property nodes out of view */
void SPropertyTreeViewImpl::OnFilterTextChanged( const FText& InFilterText )
{
	const bool bFilterCleared = InFilterText.ToString().Len() == 0 && CurrentFilterText.Len() > 0;
	const bool bFilterJustActivated = CurrentFilterText.Len() == 0 && InFilterText.ToString().Len() > 0;

	CurrentFilterText = InFilterText.ToString();

	if( bFilterJustActivated )
	{
		// Store off the expanded items when starting a new filter
		// We will restore them after the filter is cleared
		PreFilterExpansionSet.Empty();
		PropertyTree->GetExpandedItems( PreFilterExpansionSet );
	}	
		
	FilterView( CurrentFilterText );

	if( bFilterCleared )
	{
		// Clear the current expanded state
		PropertyTree->ClearExpandedItems();
		
		// Restore previously expanded items
		for( TSet< TSharedPtr<FPropertyNode> >::TConstIterator It( PreFilterExpansionSet ); It; ++It )
		{
			PropertyTree->SetItemExpansion( *It, true );
		}
	}
}

/** Called when the favorites tree requests its visibility state */
EVisibility SPropertyTreeViewImpl::OnGetFavoritesVisibility() const
{
	if( bFavoritesEnabled )
	{
		return EVisibility::Visible;
	}
	
	// If favorites are not enabled the tree should not be visible and no space should be taken up for it
	return EVisibility::Collapsed;
}

/** Returns the image used for the icon on the filter button */ 
const FSlateBrush* SPropertyTreeViewImpl::OnGetFilterButtonImageResource() const
{
	if( bHasActiveFilter )
	{
		return FAppStyle::GetBrush(TEXT("PropertyWindow.FilterCancel"));
	}
	else
	{
		return FAppStyle::GetBrush(TEXT("PropertyWindow.FilterSearch"));
	}
}

/** Returns the image used for the icon on the favorites button */
const FSlateBrush* SPropertyTreeViewImpl::OnGetFavoriteButtonImageResource() const
{
	if( bFavoritesEnabled )
	{
		return FAppStyle::GetBrush(TEXT("Icons.Star"));
	}
	else
	{
		return FAppStyle::GetBrush(TEXT("PropertyWindow.Favorites_Disabled"));
	}
}

/** Returns the image used for the icon on the lock button */
const FSlateBrush* SPropertyTreeViewImpl::OnGetLockButtonImageResource() const
{
	if( bIsLocked )
	{
		return FAppStyle::GetBrush(TEXT("PropertyWindow.Locked"));
	}
	else
	{
		return FAppStyle::GetBrush(TEXT("PropertyWindow.Unlocked"));
	}
}


/**
 * Helper function to recursively set an items expanded state
 *
 * @param InPropertyNode	The property node to possibly expand
 * @param InPropertyTree	The tree containing nodes to expand
 * @param InExpandedItems	The list of property node names that should be expanded.
 */
static void SetExpandedItems( const TSharedPtr<FPropertyNode>& InPropertyNode, TSharedRef<SPropertyTree>& InPropertyTree, const TArray<FString>& InExpandedItems )
{
	// Expand this property window if the current item's name exists in the list of expanded items.
	const bool bWithArrayIndex = true;

	FString Path;
	Path.Empty(128);
	InPropertyNode->GetQualifiedName(Path, bWithArrayIndex);

	for (int32 i = 0; i < InExpandedItems.Num(); ++i)
	{
		if ( InExpandedItems[i] == Path )
		{
			InPropertyNode->SetNodeFlags( EPropertyNodeFlags::Expanded, true );
			InPropertyTree->SetItemExpansion( InPropertyNode, true );
			break;
		}
	}

	for( int32 NodeIndex = 0; NodeIndex < InPropertyNode->GetNumChildNodes(); ++NodeIndex )
	{
		SetExpandedItems( InPropertyNode->GetChildNode( NodeIndex ), InPropertyTree, InExpandedItems );
	}
}

/**
 * Saves expansion state of the property tree
 */
void SPropertyTreeViewImpl::SaveExpandedItems()
{
	if( RootPropertyNode->GetNumChildNodes() > 0 )
	{
		TSet< TSharedPtr<FPropertyNode> > ExpandedNodes;
		PropertyTree->GetExpandedItems(ExpandedNodes);

		TArray<FString> ExpandedItemNames;

		for( TSet< TSharedPtr<FPropertyNode> >::TConstIterator It(ExpandedNodes); It; ++It )
		{
			TSharedPtr<FPropertyNode> PropertyNode = *It;

			//don't save the root, it gets expanded by default
			if (PropertyNode->GetParentNode())
			{
				const bool bWithArrayIndex = true;
				FString Path;
				Path.Empty(128);
				PropertyNode->GetQualifiedName(Path, bWithArrayIndex);

				new( ExpandedItemNames )FString( Path );
			}
		}


		UClass* BestBaseClass = RootPropertyNode->GetObjectBaseClass();
		//while a valid class, and we're either the same as the base class (for multiple actors being selected and base class is AActor) OR we're not down to AActor yet)
		for( UClass* Class = BestBaseClass; Class && ((BestBaseClass == Class) || (Class!=AActor::StaticClass())); Class = Class->GetSuperClass() )
		{
			FString ExpansionName = Class->GetName();
//			@todo Slate Property window
// 			if (HasFlags(EPropertyWindowFlags::Favorites))
// 			{
// 				ExpansionName += TEXT("Favorites");
// 			}

			GConfig->SetSingleLineArray(TEXT("PropertyWindowExpansion"), *ExpansionName, ExpandedItemNames, GEditorPerProjectIni);
		}
	}
}

void SPropertyTreeViewImpl::SaveColumnWidths()
{
	const TIndirectArray<SHeaderRow::FColumn>& Columns = ColumnHeaderRow->GetColumns();
	for (int32 Idx = 0; Idx < Columns.Num(); ++Idx)
	{
		const SHeaderRow::FColumn& Column = Columns[Idx];
		const float Width = Column.GetWidth();
		GConfig->SetFloat(TEXT("PropertyWindowWidths"), *Column.ColumnId.ToString(), Width, GEditorPerProjectIni);
	}
}

void SPropertyTreeViewImpl::ExpandAllNodes()
{
	for (int32 Idx = 0; Idx < TopLevelPropertyNodes.Num(); ++Idx)
	{
		RequestItemExpanded(TopLevelPropertyNodes[Idx], true, false);
	}
}


void SPropertyTreeViewImpl::RestoreExpandedItems()
{
	TArray<FString> ExpandedItems;
	UClass* BestBaseClass = RootPropertyNode->GetObjectBaseClass();
	//while a valid class, and we're either the same as the base class (for multiple actors being selected and base class is AActor) OR we're not down to AActor yet)
	for( UClass* Class = BestBaseClass; Class && ((BestBaseClass == Class) || (Class!=AActor::StaticClass())); Class = Class->GetSuperClass() )
	{
		FString ExpansionName = Class->GetName();
//		@todo Slate Property window
// 		if (HasFlags(EPropertyWindowFlags::Favorites))
// 		{
// 			ExpansionName += TEXT("Favorites");
// 		}

		GConfig->GetSingleLineArray(TEXT("PropertyWindowExpansion"), *ExpansionName, ExpandedItems, GEditorPerProjectIni);
		
		TSharedRef<SPropertyTree> PropertyTreeRef = PropertyTree.ToSharedRef();
		SetExpandedItems( RootPropertyNode, PropertyTreeRef, ExpandedItems );

	}
}

void SPropertyTreeViewImpl::RestoreColumnWidths()
{
	const TIndirectArray<SHeaderRow::FColumn>& Columns = ColumnHeaderRow->GetColumns();
	for (int32 Idx = 0; Idx < Columns.Num(); ++Idx)
	{
		const SHeaderRow::FColumn& Column = Columns[Idx];
		float Width = 1.0f;
		if ( GConfig->GetFloat(TEXT("PropertyWindowWidths"), *Column.ColumnId.ToString(), Width, GEditorPerProjectIni) )
		{
			ColumnHeaderRow->SetColumnWidth( Column.ColumnId, Width );
		}
	}
}

void SPropertyTreeViewImpl::EnqueueDeferredAction( FSimpleDelegate& DeferredAction )
{
	DeferredActions.Add( DeferredAction );
}

void SPropertyTreeViewImpl::SetFromExistingTree( TSharedPtr<FObjectPropertyNode> RootNode, TSharedPtr<FPropertyNode> PropertyToView )
{
	RootPropertyNode = RootNode;

	TSharedPtr<FPropertyNode> ParentPropertyNode = PropertyToView->GetParentNodeSharedPtr();
	if( ParentPropertyNode.IsValid() && ParentPropertyNode->GetProperty() && ParentPropertyNode->GetProperty()->IsA( FArrayProperty::StaticClass() ) )
	{
		// Force arrays to display so that deletion,insertion and removal work correctly.
		UpdateTopLevelPropertyNodes( ParentPropertyNode );
		const bool bExpand = true;
		bool bRecrsiveExpand = false;

		// Expand the array being viewed
		RequestItemExpanded( ParentPropertyNode, bExpand, bRecrsiveExpand );

		// Expand the array element being viewed
		bRecrsiveExpand = true;
		RequestItemExpanded( PropertyToView, bExpand, bRecrsiveExpand );
	}
	else
	{
		// Force arrays to display so that deletion,insertion and removal work correctly.
		UpdateTopLevelPropertyNodes( PropertyToView );
		// Expand the property being viewed
		RequestItemExpanded( PropertyToView, true, true );
	}

	bNodeTreeExternallyManaged = true;
	RequestRefresh();
}

/** Updates the top level property nodes.  The root nodes for the treeview. */
void SPropertyTreeViewImpl::UpdateTopLevelPropertyNodes( TSharedPtr<FPropertyNode> FirstVisibleNode )
{
	TopLevelPropertyNodes.Empty();

	if ( FirstVisibleNode.IsValid() )
	{
		FObjectPropertyNode* ObjNode = FirstVisibleNode->AsObjectNode();
		if( ObjNode || !bShowTopLevelPropertyNodes )
		{
			// Currently object property nodes do not provide any useful information other than being a container for its children.  We do not draw anything for them.
			// When we encounter object property nodes, add their children instead of adding them to the tree.
			OnGetChildrenForPropertyNode( FirstVisibleNode, TopLevelPropertyNodes );
		}
		else if ( bShowTopLevelPropertyNodes )
		{
			TopLevelPropertyNodes.Add( FirstVisibleNode );
		}
	}
}

/** 
 * Recursively marks nodes which should be favorite starting from the root
 */
void SPropertyTreeViewImpl::MarkFavorites()
{
	check( RootPropertyNode.IsValid() );

	TopLevelFavorites.Empty();
	MarkFavoritesInternal( RootPropertyNode, false );
	RootPropertyNode->ProcessSeenFlagsForFavorites();
}

/** 
 * Recursively marks nodes which should be favorite
 *
 * @param InPropertyNode	The property node to start marking favorites from
 * @param bAnyParentIsFavorite	true of any parent of InPropertyNode is already marked as a favorite
 */
void SPropertyTreeViewImpl::MarkFavoritesInternal( TSharedPtr<FPropertyNode> InPropertyNode, bool bAnyParentIsFavorite )
{
	FString Path;
	Path.Empty(256);

	bool bShouldBeFavorite = false;
	
	//get the fully qualified name of this node
	const bool bWithArrayIndex = false;
	InPropertyNode->GetQualifiedName(Path, bWithArrayIndex);

	//See if this should be marked as a favorite
		
	if( FavoritesList.Find( Path ) )
	{
		bShouldBeFavorite = true;
	}

	InPropertyNode->SetNodeFlags(EPropertyNodeFlags::IsFavorite, bShouldBeFavorite);

	if( bShouldBeFavorite && !bAnyParentIsFavorite )
	{
		TopLevelFavorites.Add( InPropertyNode );
	}
	
	//recurse for all children
	for( int32 x = 0 ; x < InPropertyNode->GetNumChildNodes(); ++x )
	{
		TSharedPtr<FPropertyNode> ChildTreeNode = InPropertyNode->GetChildNode(x);
		check( ChildTreeNode.IsValid() );
		MarkFavoritesInternal( ChildTreeNode, bShouldBeFavorite | bAnyParentIsFavorite );
	}
}

bool SPropertyTreeViewImpl::IsPropertyNodeVisible(TSharedPtr<FPropertyNode> InPropertyNode)
{
	// Object nodes always mark themselves as visible (but are never actually shown)
	if(InPropertyNode->AsObjectNode())
	{
		return true;
	}

	// The node is marked not visible due to filtering etc
	if(!InPropertyNode->IsVisible())
	{
		return false;
	}
	
	// Category nodes are visible if they have any visible children
	if(InPropertyNode->AsCategoryNode())
	{
		bool bHasVisibleChild = false;

		// If we have a permission list in use, make sure the category has at least one child visible otherwise hide it
		if(FPropertyEditorPermissionList::Get().IsEnabled())
		{
			for( int32 GrandChildIndex = 0; GrandChildIndex < InPropertyNode->GetNumChildNodes(); ++GrandChildIndex )
			{
				const TSharedPtr<FPropertyNode> GrandChildNode = InPropertyNode->GetChildNode( GrandChildIndex );

				if(IsPropertyNodeVisible(GrandChildNode) && GrandChildNode->IsVisible())
				{
					bHasVisibleChild = true;
					break;
				}
			}
		}
		// If we don't have a permission list, we can trivially check the child node count to see if there are any children
		else
		{
			bHasVisibleChild = InPropertyNode->GetNumChildNodes() > 0;
		}

		return bHasVisibleChild;
	}

	// Regular property nodes are visible if they pass the IsPropertyVisible delegate and any permission lists if valid
	bool bPropertyVisible = true;
	
	const FProperty* Property = InPropertyNode->GetProperty();
	if(Property != NULL)
	{
		// If we have a permission list in use, make sure this property is in the list for the most common base class and the class of all objects we are viewing
		if(FPropertyEditorPermissionList::Get().IsEnabled())
		{
			bPropertyVisible &= FPropertyEditorPermissionList::Get().DoesPropertyPassFilter(FDetailTreeNode::GetPropertyNodeBaseStructure(InPropertyNode->GetParentNode()), Property->GetFName());

			for (int32 ObjectIndex = 0; ObjectIndex < RootPropertyNode->GetNumObjects(); ++ObjectIndex)
			{
				UObject* ObjectToCheck = RootPropertyNode->GetUObject(ObjectIndex);

				const UClass* ObjClass = Cast<UClass>(ObjectToCheck);
				if (ObjClass == nullptr)
				{
					ObjClass = ObjectToCheck->GetClass();
				}

				if(ObjClass)
				{
					bPropertyVisible &= FPropertyEditorPermissionList::Get().DoesPropertyPassFilter(ObjClass, Property->GetFName());
				}
			}
		}

		if(IsPropertyVisible.IsBound())
		{
			const FPropertyAndParent PropertyAndParent(InPropertyNode.ToSharedRef());
			bPropertyVisible &= IsPropertyVisible.Execute(PropertyAndParent);
		}
	}

	return bPropertyVisible;
}

void SPropertyTreeViewImpl::OnGetChildrenForPropertyNode( TSharedPtr<FPropertyNode> InPropertyNode, TArray< TSharedPtr<FPropertyNode> >& OutChildren )
{
	if( CurrentFilterText.Len() > 0 ) 
	{
		if( InPropertyNode->HasNodeFlags( EPropertyNodeFlags::IsSeenDueToChildFiltering ) )
		{
			// The node should be expanded because its children are in the filter
			RequestItemExpanded( InPropertyNode, true );
		}
		else if( InPropertyNode->HasNodeFlags( EPropertyNodeFlags::AutoExpanded ) )
		{
			// This property node has no children in the filter and was previously auto expanded
			// So collapse it now
			InPropertyNode->SetNodeFlags( EPropertyNodeFlags::AutoExpanded, false );
			RequestItemExpanded( InPropertyNode, false );
		}
	}
	else
	{
		// Check and see if the node wants to be expanded and we haven't already expanded this node before
		if( InPropertyNode->HasNodeFlags(EPropertyNodeFlags::Expanded) != 0 && InPropertyNode->HasNodeFlags(EPropertyNodeFlags::HasEverBeenExpanded) == 0 ) 
		{
			RequestItemExpanded( InPropertyNode, true );
		}

		// No nodes are auto expanded when we have no filter 
		InPropertyNode->SetNodeFlags( EPropertyNodeFlags::AutoExpanded, false );
	}

	// If we are getting children for this node then its been expanded
	InPropertyNode->SetNodeFlags(EPropertyNodeFlags::HasEverBeenExpanded, true);

	for( int32 ChildIndex = 0; ChildIndex < InPropertyNode->GetNumChildNodes(); ++ChildIndex )
	{
		TSharedPtr<FPropertyNode> ChildNode = InPropertyNode->GetChildNode( ChildIndex );

		const bool bPropertyVisible = IsPropertyNodeVisible(ChildNode);
		
		if(bPropertyVisible)
		{
			const FObjectPropertyNode* ObjNode = ChildNode->AsObjectNode();

			if( ObjNode )
			{
				// Currently object property nodes do not provide any useful information other than being a container for its children.  We do not draw anything for them.
				// When we encounter object property nodes, add their children instead of adding them to the tree.
				OnGetChildrenForPropertyNode( ChildNode, OutChildren );
			}
			else
			{
				OutChildren.Add(ChildNode);
			}
		}
	}
}


void SPropertyTreeViewImpl::OnGetChildFavoritesForPropertyNode( TSharedPtr<FPropertyNode> InPropertyNode, TArray< TSharedPtr<FPropertyNode> >& OutChildren )
{
	for( int32 ChildIndex = 0; ChildIndex < InPropertyNode->GetNumChildNodes(); ++ChildIndex )
	{
		TSharedPtr<FPropertyNode> ChildNode = InPropertyNode->GetChildNode( ChildIndex );
		FObjectPropertyNode* ObjNode = ChildNode->AsObjectNode();
		FCategoryPropertyNode* CatNode = ChildNode->AsCategoryNode();

		bool bIsChildOfFavorite = ChildNode->IsChildOfFavorite();
		if( ObjNode || (CatNode && !bIsChildOfFavorite) )
		{

			// Currently object property nodes do not provide any useful information other than being a container for its children.  We do not draw anything for them.
			// When we encounter object property nodes, add their children instead of adding them to the tree.
			OnGetChildFavoritesForPropertyNode( ChildNode, OutChildren );
		}
		else
		{
			if( ChildNode->HasNodeFlags( EPropertyNodeFlags::IsFavorite ) || 
				ChildNode->HasNodeFlags( EPropertyNodeFlags::IsSeenDueToChildFavorite ) ||
				bIsChildOfFavorite )
			{
				OutChildren.Add( ChildNode );
			}

		}
	}
}

void SPropertyTreeViewImpl::RequestRefresh()
{
	PropertyTree->RequestTreeRefresh();
	FavoritesTree->RequestTreeRefresh();
}

void SPropertyTreeViewImpl::SetObjectArray( const TArray<UObject*>& InObjects )
{
	check( RootPropertyNode.IsValid() );

	PreSetObject();

	TArray<UObject*> SetObjects;
	SetObjects.Reserve(InObjects.Num());

	bool bOwnedByLockedLevel = false;
	for(UObject* Object : InObjects)
	{
		RootPropertyNode->AddObject(Object);
	}

	// @todo Slate Property Window
	//SetFlags(EPropertyWindowFlags::ReadOnly, bOwnedByLockedLevel);

	PostSetObject();

	// Set the title of the window based on the objects we are viewing
	if( !RootPropertyNode->GetObjectBaseClass() )
	{
		Title = NSLOCTEXT("PropertyView", "NothingSelectedTitle", "Nothing selected").ToString();
	}
	else if( RootPropertyNode->GetNumObjects() == 1 )
	{
		// if the object is the default metaobject for a UClass, use the UClass's name instead
		const UObject* Object = RootPropertyNode->ObjectConstIterator()->Get();
		FString ObjectName = Object->GetName();
		if ( Object->GetClass()->GetDefaultObject() == Object )
		{
			ObjectName = Object->GetClass()->GetName();
		}
		else
		{
			// Is this an actor?  If so, it might have a friendly name to display
			const AActor* Actor = Cast<const  AActor >( Object );
			if( Actor != NULL && !Object->IsTemplate() )
			{
				// Use the friendly label for this actor
				ObjectName = Actor->GetActorLabel();
			}
		}

		Title = ObjectName;
	}
	else
	{
		Title = FText::Format( NSLOCTEXT("PropertyView", "MultipleSelectedFmt", "{0} ({1} selected)"), FText::FromString(RootPropertyNode->GetObjectBaseClass()->GetName()), RootPropertyNode->GetNumObjects() ).ToString();
	}

	OnObjectArrayChanged.ExecuteIfBound(Title, SetObjects);
}

TSharedRef< FPropertyPath > SPropertyTreeViewImpl::GetRootPath() const
{
	return RootPath;
}

void SPropertyTreeViewImpl::SetRootPath( const TSharedPtr< FPropertyPath >& Path )
{
	if ( Path.IsValid() )
	{
		RootPath = Path.ToSharedRef();
	}
	else
	{
		RootPath = FPropertyPath::CreateEmpty();
	}

	ConstructPropertyTree();
	UpdateTopLevelPropertyNodes( FPropertyNode::FindPropertyNodeByPath( RootPath, RootPropertyNode.ToSharedRef() ) );

	// Restore expansion state of items in the tree
	RestoreExpandedItems();

	// Restore the widths of columns
	RestoreColumnWidths();
}

void SPropertyTreeViewImpl::ReplaceObjects( const TMap<UObject*, UObject*>& OldToNewObjectMap )
{

	TArray<UObject*> NewObjectList;
	bool bObjectsReplaced = false;

	TArray< FObjectPropertyNode* > ObjectNodes;
	PropertyEditorHelpers::CollectObjectNodes( RootPropertyNode, ObjectNodes );

	for( int32 ObjectNodeIndex = 0; ObjectNodeIndex < ObjectNodes.Num(); ++ObjectNodeIndex )
	{
		FObjectPropertyNode* CurrentNode = ObjectNodes[ObjectNodeIndex];

		// Scan all objects and look for objects which need to be replaced
		for ( TPropObjectIterator Itor( CurrentNode->ObjectIterator() ); Itor; ++Itor )
		{
			UObject* Replacement = OldToNewObjectMap.FindRef( Itor->Get() );
			if( Replacement )
			{
				bObjectsReplaced = true;
				if( CurrentNode == RootPropertyNode.Get() )
				{
					// Note: only root objects count for the new object list. Sub-Objects (i.e components count as needing to be replaced but they don't belong in the top level object list
					NewObjectList.Add( Replacement );
				}
			}
			else if( CurrentNode == RootPropertyNode.Get() )
			{
				// Note: only root objects count for the new object list. Sub-Objects (i.e components count as needing to be replaced but they don't belong in the top level object list
				NewObjectList.Add( Itor->Get() );
			}
		}
	}

	if( bObjectsReplaced )
	{
		SetObjectArray( NewObjectList );
	}
}

void SPropertyTreeViewImpl::RemoveDeletedObjects( const TArray<UObject*>& DeletedObjects )
{
	TArray<UObject*> NewObjectList;
	bool bObjectsRemoved = false;

	// Scan all objects and look for objects which need to be replaced
	for ( TPropObjectIterator Itor( RootPropertyNode->ObjectIterator() ); Itor; ++Itor )
	{
		if( DeletedObjects.Contains( Itor->Get() ) )
		{
			// An object we had needs to be removed
			bObjectsRemoved = true;
		}
		else
		{
			// If the deleted object list does not contain the current object, its ok to keep it in the list
			NewObjectList.Add( Itor->Get() );
		}
	}

	// if any objects were replaced update the observed objects
	if( bObjectsRemoved )
	{
		SetObjectArray( NewObjectList );
	}
}

/**
 * Removes actors from the property nodes object array which are no longer available
 * 
 * @param ValidActors	The list of actors which are still valid
 */
void SPropertyTreeViewImpl::RemoveInvalidActors( const TSet<AActor*>& ValidActors )
{
	TArray<UObject*> ResetArray;

	bool bAllFound = true;
	for ( TPropObjectIterator Itor( RootPropertyNode->ObjectIterator() ); Itor; ++Itor )
	{
		AActor* Actor = Cast<AActor>( Itor->Get() );

		bool bFound = ValidActors.Contains( Actor );

		// If the selected actor no longer exists, remove it from the property window.
		if( bFound )
		{
			ResetArray.Add(Actor);
		}
		else
		{
			bAllFound = false;
		}
	}

	if ( !bAllFound ) 
	{
		SetObjectArray( ResetArray );
	}
}


/** Called before during SetObjectArray before we change the objects being observed */
void SPropertyTreeViewImpl::PreSetObject()
{
	check( RootPropertyNode.IsValid() );

	// Save all expanded items before setting new objects 
	SaveExpandedItems();

	// Save all the column widths before setting new objects
	SaveColumnWidths();

	RootPropertyNode->RemoveAllObjects();
}

/** Called at the end of SetObjectArray after we change the objects being observed */
void SPropertyTreeViewImpl::PostSetObject()
{
	check( RootPropertyNode.IsValid() );
	check( !bNodeTreeExternallyManaged );

	DestroyColorPicker();

	// Reconstruct the property tree so we don't have a tree filled with data we are about to destroy
	ConstructPropertyTree();

	FPropertyNodeInitParams InitParams;
	InitParams.ParentNode = NULL;
	InitParams.Property = NULL;
	InitParams.ArrayOffset = 0;
	InitParams.ArrayIndex = INDEX_NONE;
	InitParams.bAllowChildren = true;
	InitParams.bForceHiddenPropertyVisibility = bForceHiddenPropertyVisibility;

	RootPropertyNode->InitNode( InitParams );

	RootPropertyNode->ProcessSeenFlags(true);
	UpdateTopLevelPropertyNodes( FPropertyNode::FindPropertyNodeByPath( RootPath, RootPropertyNode.ToSharedRef() ) );		

	LoadFavorites();

	// Restore expansion state of items in the tree
	RestoreExpandedItems();

	// Restore the widths of columns
	RestoreColumnWidths();

	RequestRefresh();
}

/** 
 * Hides or shows properties based on the passed in filter text
 * 
 * @param InFilterText	The filter text
 */
void SPropertyTreeViewImpl::FilterView( const FString& InFilterText )
{
	TArray<FString> FilterStrings;

	FString ParseString = InFilterText;
	// Remove whitespace from the front and back of the string
	ParseString.TrimStartAndEndInline();
	ParseString.ParseIntoArray(FilterStrings, TEXT(" "), true);

	RootPropertyNode->FilterNodes( FilterStrings );
	RootPropertyNode->ProcessSeenFlags(true);

	bHasActiveFilter = FilterStrings.Num() > 0;

	if( !bNodeTreeExternallyManaged )
	{
		UpdateTopLevelPropertyNodes( FPropertyNode::FindPropertyNodeByPath( RootPath, RootPropertyNode.ToSharedRef() ) );		
	}
	

	RequestRefresh();
}

/** Ticks the property view.  This function performs a data consistency check */
void SPropertyTreeViewImpl::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	check( RootPropertyNode.IsValid() );

	// Purge any objects that are marked pending kill from the object list
	RootPropertyNode->PurgeKilledObjects();

	for( int32 ActionIndex = 0; ActionIndex < DeferredActions.Num(); ++ActionIndex )
	{
		DeferredActions[ActionIndex].ExecuteIfBound();
	}
	DeferredActions.Empty();

	EPropertyDataValidationResult Result = RootPropertyNode->EnsureDataIsValid();
	if( Result == EPropertyDataValidationResult::PropertiesChanged || Result == EPropertyDataValidationResult::ArraySizeChanged || Result == EPropertyDataValidationResult::EditInlineNewValueChanged )
	{
		// Make sure our new property windows are properly filtered.  
		FilterView( CurrentFilterText );
	}
	else if( Result == EPropertyDataValidationResult::ObjectInvalid && !bNodeTreeExternallyManaged )
	{
		TArray<UObject*> ResetArray;
		for ( TPropObjectIterator Itor( RootPropertyNode->ObjectIterator() ) ; Itor ; ++Itor )
		{
			TWeakObjectPtr<UObject> Object = *Itor;

			if( Object.IsValid() )
			{
				ResetArray.Add( Object.Get() );
			}
		}

		SetObjectArray(ResetArray);
	}
	

	if( FilteredNodesRequestingExpansionState.Num() > 0 )
	{
		// change expansion state on the nodes that request it
		for( TMap<TSharedPtr<FPropertyNode>, bool >::TConstIterator It(FilteredNodesRequestingExpansionState); It; ++It )
		{
			PropertyTree->SetItemExpansion( It.Key(), It.Value() );
			It.Key()->SetNodeFlags( EPropertyNodeFlags::Expanded, It.Value() );
		}

		FilteredNodesRequestingExpansionState.Empty();
	}
}

/** 
 * Creates a property editor (the visual portion of a PropertyNode), for a specific property node
 *
 * @param InPropertyNode	The property node to create the visual for
 */
TSharedRef<ITableRow> SPropertyTreeViewImpl::CreatePropertyEditor( TSharedPtr<FPropertyNode> InPropertyNode, const TSharedPtr<STableViewBase>& OwnerTable )
{
	FCategoryPropertyNode* CategoryNode = InPropertyNode->AsCategoryNode();

	if (CategoryNode != nullptr)
	{
		// This is a category node; it does not need columns.
		// Just use a simple setup.
		return SNew( SPropertyTreeCategoryRow, OwnerTable.ToSharedRef() )
			.DisplayName( CategoryNode->GetDisplayName() );
	}
	else
	{
		TSharedRef< IPropertyUtilities > PropertyUtilities = PropertySettings.ToSharedRef();
		TSharedRef< FPropertyEditor > PropertyEditor = FPropertyEditor::Create( InPropertyNode.ToSharedRef(), PropertyUtilities );
		return SNew( SPropertyEditorTableRow, PropertyEditor, PropertyUtilities, OwnerTable.ToSharedRef() )
			.OnMiddleClicked( OnPropertyMiddleClicked )
			.ConstructExternalColumnCell( ConstructExternalColumnCell );
	}
}

/**
 * Returns an SWidget used as the visual representation of a node in the property treeview.                     
 */
TSharedRef<ITableRow> SPropertyTreeViewImpl::OnGenerateRowForPropertyTree( TSharedPtr<FPropertyNode> InPropertyNode, const TSharedRef<STableViewBase>& OwnerTable )
{
	// Generate a row that represents a property
	return CreatePropertyEditor( InPropertyNode, OwnerTable );
}

void SPropertyTreeViewImpl::OnSelectionChanged( TSharedPtr<FPropertyNode> InPropertyNode, ESelectInfo::Type SelectInfo )	
{
	if(InPropertyNode.IsValid())
	{
		OnPropertySelectionChanged.ExecuteIfBound( InPropertyNode->GetProperty() );
	}
}

/** 
 * Marks or unmarks a property node as a favorite.  
 *
 * @param InPropertyNode	The node to toggle favorite on
 */
void SPropertyTreeViewImpl::ToggleFavorite( const TSharedRef< class FPropertyEditor >& PropertyEditor )
{
	const TSharedRef< FPropertyNode > PropertyNode = PropertyEditor->GetPropertyNode();
	FString NodeName;
	PropertyNode->GetQualifiedName(NodeName, false);

	if( PropertyNode->HasNodeFlags( EPropertyNodeFlags::IsFavorite ) )
	{
		// Remove the favorite from the list so it will be toggled off in MarkFavorites
		FavoritesList.Remove( NodeName );
	}
	else
	{
		// Add the favorite to the list so it will be toggled on in MarkFavorites
		FavoritesList.Add( NodeName );
	}

	// Save new favorites to INI so they can be restored later
	SaveFavorites();

	// Mark all favorites so we know what to display
	MarkFavorites();

	// Refresh the display
	FavoritesTree->RequestTreeRefresh();
}

/** 
 * Loads favorites from INI
 */
void SPropertyTreeViewImpl::LoadFavorites()
{
	FavoritesList.Empty();

	if( RootPropertyNode.IsValid() )
	{
		UClass* BestClass = RootPropertyNode->GetObjectBaseClass();
		if( BestClass != NULL )
		{
			FString ContextName = BestClass->GetName() + TEXT("Favorites");

			TArray<FString> OutFavoritesList;
			GConfig->GetSingleLineArray(TEXT("PropertyWindow"), *ContextName, OutFavoritesList, GEditorPerProjectIni);
		
			for(int32 i=0; i<OutFavoritesList.Num(); i++)
			{
				// skip numerics.  They were indices that we do not use
				if( !OutFavoritesList[i].IsNumeric() )
				{
					FavoritesList.Add( OutFavoritesList[i] );
				}
			}
		}

		MarkFavorites();
	}
}

/**
 * Saves favorites to INI                   
 */
void SPropertyTreeViewImpl::SaveFavorites()
{
	if( RootPropertyNode.IsValid() )
	{
		UClass* BestClass = RootPropertyNode->GetObjectBaseClass();
		if (BestClass)
		{
			FString ContextName = BestClass->GetName() + TEXT("Favorites");
			TArray<FString> FavoritesArray;

			for( TSet<FString>::TConstIterator It(FavoritesList); It; ++It )
			{
				FavoritesArray.Add( *It );
			}

			GConfig->SetSingleLineArray(TEXT("PropertyWindow"), *ContextName, FavoritesArray, GEditorPerProjectIni);
		}
	}
}

namespace UE::PropertyEditor::Private
{
/** Set the color for the property node */
void CreateColorPickerWindow_SetColor(FLinearColor NewColor, TWeakPtr<IPropertyHandle> WeakPropertyHandle)
{
	if (TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin())
	{
		FProperty* NodeProperty = PropertyHandle->GetProperty();
		check(NodeProperty);

		const UScriptStruct* Struct = CastField<FStructProperty>(NodeProperty)->Struct;
		if (Struct->GetFName() == NAME_Color)
		{
			const bool bSRGB = true;
			FColor NewFColor = NewColor.ToFColor(bSRGB);
			ensure(PropertyHandle->SetValueFromFormattedString(NewFColor.ToString(), EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
		}
		else
		{
			check(Struct->GetFName() == NAME_LinearColor);
			ensure(PropertyHandle->SetValueFromFormattedString(NewColor.ToString(), EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
		}
	}
}
}//namespace

void SPropertyTreeViewImpl::CreateColorPickerWindow(const TSharedRef< class FPropertyEditor >& PropertyEditor, bool bUseAlpha)
{
	const FProperty* Property = PropertyEditor->GetProperty();
	check(Property);

	FReadAddressList ReadAddresses;
	PropertyEditor->GetPropertyNode()->GetReadAddress(false, ReadAddresses, false);

	// Use the first address for the initial color
	TOptional<FLinearColor> DefaultColor;
	bool bClampValue = false;
	if (ReadAddresses.Num())
	{
		const uint8* Addr = ReadAddresses.GetAddress(0);
		if (Addr)
		{
			if (CastField<FStructProperty>(Property)->Struct->GetFName() == NAME_Color)
			{
				DefaultColor = *reinterpret_cast<const FColor*>(Addr);
				bClampValue = true;
			}
			else
			{
				check(CastField<FStructProperty>(Property)->Struct->GetFName() == NAME_LinearColor);
				DefaultColor = *reinterpret_cast<const FLinearColor*>(Addr);
			}
		}
	}

	if (DefaultColor.IsSet())
	{
		TWeakPtr<IPropertyHandle> WeakPropertyHandle = PropertyEditor->GetPropertyHandle();
		FColorPickerArgs PickerArgs = FColorPickerArgs(DefaultColor.GetValue(), FOnLinearColorValueChanged::CreateStatic(&UE::PropertyEditor::Private::CreateColorPickerWindow_SetColor, WeakPropertyHandle));
		PickerArgs.ParentWidget = AsShared();
		PickerArgs.bUseAlpha = bUseAlpha;
		PickerArgs.bClampValue = bClampValue;
		PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));

		OpenColorPicker(PickerArgs);
	}
}

void SPropertyTreeViewImpl::SetOnObjectArrayChanged(FOnObjectArrayChanged OnObjectArrayChangedDelegate)
{
	OnObjectArrayChanged = OnObjectArrayChangedDelegate;
}

void SPropertyTreeViewImpl::SetIsPropertyVisible(FIsPropertyVisible IsPropertyVisibleDelegate)
{
	IsPropertyVisible = IsPropertyVisibleDelegate;

	if( RootPropertyNode.IsValid() )
	{
		TArray<UObject*> Objects;
		for( int32 ObjIndex = 0; ObjIndex < RootPropertyNode->GetNumObjects(); ++ObjIndex )
		{
			Objects.Add( RootPropertyNode->GetUObject( ObjIndex ) );
		}

		// Refresh the entire tree
		SetObjectArray( Objects );			
	}
}

void SPropertyTreeViewImpl::RequestItemExpanded( TSharedPtr<FPropertyNode> PropertyNode, bool bExpand, bool bRecursiveExpansion )
{
	// Don't change expansion state if its already in that state
	if( PropertyTree->IsItemExpanded(PropertyNode) != bExpand )
	{
		PropertyNode->SetNodeFlags( EPropertyNodeFlags::AutoExpanded, true );
		FilteredNodesRequestingExpansionState.Add( PropertyNode, bExpand );
	}

	if (bRecursiveExpansion)
	{
		check(PropertyNode.IsValid());
		int32 NumChildren = PropertyNode->GetNumChildNodes();
		for (int32 Index = 0; Index < NumChildren; ++Index)
		{
			TSharedPtr<FPropertyNode> ChildNode = PropertyNode->GetChildNode(Index);
			if (ChildNode.IsValid())
			{
				RequestItemExpanded(ChildNode, bExpand, bRecursiveExpansion);
			}
		}
	}
}

bool SPropertyTreeViewImpl::IsPropertySelected( const FString& InName, const int32 InArrayIndex)
{
	return IsPropertyOrChildrenSelected(InName, InArrayIndex, false);
}

bool SPropertyTreeViewImpl::IsPropertyOrChildrenSelected( const FString& InName, const int32 InArrayIndex, const bool CheckChildren )
{
	// Safety check, no items are selected so return immediately.
	if(PropertyTree->GetSelectedItems().Num() == 0)
	{
		return false;
	}

	TSharedPtr<FPropertyNode> PropNode = PropertyTree->GetSelectedItems()[0];

	do
	{
		bool bMatch = true;

		FProperty *Prop = PropNode->GetProperty();
		int32 Index = PropNode->GetArrayIndex();
		if( Prop )
		{
			FString Name = Prop->GetName();
			if( Index >= 0 )
			{
				FPropertyNode* ParentPropNode = PropNode->GetParentNode();
				if( ParentPropNode )
				{
					FProperty* ParentProp = ParentPropNode->GetProperty();
					if( ParentProp )
					{
						Name = ParentProp->GetName();
					}
				}
			}
			if( Name != InName )
			{
				bMatch = false;
			}
		}
		else
		{
			bMatch = false;
		}

		if( Index != InArrayIndex )
		{
			bMatch = false;
		}

		if( bMatch == true )
		{
			return true;
		}

		PropNode = PropNode->GetParentNodeSharedPtr();
	}
	while( CheckChildren && ( PropNode.IsValid() ) );

	return false;
}

