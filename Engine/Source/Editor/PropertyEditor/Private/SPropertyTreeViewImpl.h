// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "PropertyPath.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "IPropertyUtilities.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "PropertyEditorDelegates.h"

class AActor;
class FNotifyHook;
class FObjectPropertyNode;
class FPropertyNode;

typedef STreeView< TSharedPtr<FPropertyNode> > SPropertyTree;

class SPropertyTreeViewImpl : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SPropertyTreeViewImpl )
		: _IsLockable(true)
		, _HiddenPropertyVis( false )
		, _AllowFavorites(true)
		, _AllowSearch( true )
		, _ShowTopLevelNodes( true )
		, _NotifyHook( NULL )
		, _NameColumnWidth( 0.20f )
		, _OnPropertySelectionChanged()
		, _OnPropertyMiddleClicked()
		, _ConstructExternalColumnHeaders()
		, _ConstructExternalColumnCell()
		{}

		SLATE_ARGUMENT( bool, IsLockable )
		SLATE_ARGUMENT( bool, HiddenPropertyVis );
		SLATE_ARGUMENT( bool, AllowFavorites )
		SLATE_ARGUMENT( bool, AllowSearch )
		SLATE_ARGUMENT( bool, ShowTopLevelNodes )
		SLATE_ARGUMENT( FNotifyHook*, NotifyHook )
		SLATE_ARGUMENT( float, NameColumnWidth )
		SLATE_EVENT( FOnPropertySelectionChanged, OnPropertySelectionChanged )
		SLATE_EVENT( FOnPropertyClicked, OnPropertyMiddleClicked )
		SLATE_EVENT( FConstructExternalColumnHeaders, ConstructExternalColumnHeaders )
		SLATE_EVENT( FConstructExternalColumnCell, ConstructExternalColumnCell )
	SLATE_END_ARGS()

	SPropertyTreeViewImpl();

	/**
	 * Constructs the property view widgets                   
	 */
	void Construct(const FArguments& InArgs);

	/** Tells the property treeview to refresh its elements during the next tick */
	virtual void RequestRefresh();

	/**
	 * Sets the objects that this property view will observe
	 *
	 * @param InObjects	The list of objects to observe
	 */
	virtual void SetObjectArray( const TArray<UObject*>& InObjects );

	virtual TSharedRef< FPropertyPath > GetRootPath() const;

	virtual void SetRootPath( const TSharedPtr< FPropertyPath >& Path );

	/**
	 * Replaces objects being observed by the view with new objects
	 *
	 * @param OldToNewObjectMap	Mapping from objects to replace to their replacement
	 */
	virtual void ReplaceObjects( const TMap<UObject*, UObject*>& OldToNewObjectMap );

	/**
	 * Removes objects from the view because they are about to be deleted
	 *
	 * @param DeletedObjects	The objects to delete
	 */
	virtual void RemoveDeletedObjects( const TArray<UObject*>& DeletedObjects );

	/**
     * Removes actors from the property nodes object array which are no longer available
	 * 
	 * @param ValidActors	The list of actors which are still valid
	 */
	virtual void RemoveInvalidActors( const TSet<AActor*>& ValidActors );

	/** 
	 * Marks or unmarks a property node as a favorite.  
	 *
	 * @param InPropertyNode	The node to toggle favorite on
	 */
	virtual void ToggleFavorite( const TSharedRef< class FPropertyEditor >& PropertyEditor );

	/**
	 * Returns true if favorites are enabled for this property view
	 */
	virtual bool AreFavoritesEnabled() const { return bFavoritesEnabled; }
	
	/** 
	 * Creates the color picker window for this property view.
	 *
	 * @param Node				The slate property node to edit.
	 * @param bUseAlpha			Whether or not alpha is supported
	 */
	virtual void CreateColorPickerWindow( const TSharedRef< class FPropertyEditor >& PropertyEditor, bool bUseAlpha );

	/**
	 * Returns true if the property window is locked and cant have its observed objects changed 
	 */
	virtual bool IsLocked() const { return bIsLocked; }

	/** Returns the notify hook to use when properties change */
	virtual FNotifyHook* GetNotifyHook() const { return NotifyHook; }

	/** Allows setting a custom delegate for deciding property visibility */
	virtual void SetIsPropertyVisible( FIsPropertyVisible IsPropertyVisibleDelegate );

	/** Sets the callback for when the property view changes */
	virtual void SetOnObjectArrayChanged(FOnObjectArrayChanged OnObjectArrayChangedDelegate);

	/** Saves the expansion state of the property tree */
	virtual void SaveExpandedItems();

	/** Saves the widths of the columns in the property view */
	virtual void SaveColumnWidths();

	/** Gets the title for this property view based on objects being observed */
	virtual FString GetTitle() const { return Title; }

	/** Expands all top level nodes */
	virtual void ExpandAllNodes();

	/** Enqueues an action to be executed next tick */
	void EnqueueDeferredAction( FSimpleDelegate& DeferredAction );

	/**
	 * Sets the property tree to display from a node tree that always exists. 
	 * This puts the view into a mode where it does not refresh or rebuild the tree because it is managed externally
	 *
	 * @param RootNode			The root of the existing property tree
	 * @param PropertyToView	The root of the displayed tree
	 */
	void SetFromExistingTree( TSharedPtr<FObjectPropertyNode> RootNode, TSharedPtr<FPropertyNode> PropertyToView );
	
	/**
	 * Sets the filter text to the given value.
	 */
	virtual void SetFilterText( const FText& InFilterText );

	/** Called when the filter text changes.  This filters specific property nodes out of view */
	void OnFilterTextChanged( const FText& InFilterText );

	/** 
	 * Checks if the property or a child property is selected.
	 *
	 * @param InName			The name of the property.
	 * @param InArrayIndex		The array index of the property.
	 *
	 * @return true if the Property is selected.
	 */
	virtual bool IsPropertySelected( const FString& InName, const int32 InArrayIndex = INDEX_NONE);

	/** 
	 * Checks if the property or a child property is selected.
	 *
	 * @param InName			The name of the property.
	 * @param InArrayIndex		The array index of the property.
	 * @param CheckChildren		true if children should be checked.
	 *
	 * @return true if the Property or a child is selected.
	 */
	virtual bool IsPropertyOrChildrenSelected( const FString& InName, const int32 InArrayIndex = INDEX_NONE, const bool CheckChildren = true );

private:

	/** Delegate called when the property editor permission list is updated */
	void OnPermissionListUpdated(TSoftObjectPtr<const UStruct>, FName);

	/** Restores the expansion state in the tree for a specific object set */
	void RestoreExpandedItems();

	/** Restores the widths of the columns in the property view */
	void RestoreColumnWidths();

	/** Updates the top level property nodes.  The root nodes for the treeview. */
	void UpdateTopLevelPropertyNodes( TSharedPtr<FPropertyNode> FirstVisibleNode );

	/** Called before during SetObjectArray before we change the objects being observed */
	void PreSetObject();

	/** Called at the end of SetObjectArray after we change the objects being observed */
	void PostSetObject();

	// SWidget interface
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	// End of SWidget interface

	/** 
	 * Function called through a delegate on the property TreeView to request children of a property node 
	 * 
	 * @param InPropertyNode	The property node to get children from
	 * @param OutChildren		The list of children of InPropertyNode that should be visible 
	 */
	void OnGetChildrenForPropertyNode( TSharedPtr<FPropertyNode> InPropertyNode, TArray< TSharedPtr<FPropertyNode> >& OutChildren );

	/** Check if a Property Node should be shown in the property tree if we have any external IsPropertyVisible delegates or permission lists */
	bool IsPropertyNodeVisible(TSharedPtr<FPropertyNode> InPropertyNode);

	/** 
	 * Function called through a delegate on the favorites TreeView to request children of a property node  
	 * 
	 * @param InPropertyNode	The property node to get children from
	 * @param OutChildren		The list of children of InPropertyNode that should be visible 
	 */
	void OnGetChildFavoritesForPropertyNode( TSharedPtr<FPropertyNode> InPropertyNode, TArray< TSharedPtr<FPropertyNode> >& OutChildren );

	/** Called when the toggle favorites button is clicked */
	FReply OnToggleFavoritesClicked();

	/** Called when the locked button is clicked */
	FReply OnLockButtonClicked();

	/** 
	 * Hides or shows properties based on the passed in filter text
	 * 
	 * @param InFilterText	The filter text
	 */
	void FilterView( const FString& InFilterText );

	/** Called when the favorites tree requests its visibility state */
	EVisibility OnGetFavoritesVisibility() const;

	/** Returns the name of the image used for the icon on the filter button */ 
	const FSlateBrush* OnGetFilterButtonImageResource() const;
	
	/** Returns the name of the image used for the icon on the favorites button */
	const FSlateBrush* OnGetFavoriteButtonImageResource() const;

	/** Returns the name of the image used for the icon on the locked button */
	const FSlateBrush* OnGetLockButtonImageResource() const;

	/** Reconstructs the entire property tree widgets */
	void ConstructPropertyTree();

	/** 
	 * Recursively marks nodes which should be favorite starting from the root
	 */
	void MarkFavorites();

	/** 
	 * Recursively marks nodes which should be favorite
	 *
	 * @param InPropertyNode	The property node to start marking favorites from
	 * @param bAnyParentIsFavorite	true of any parent of InPropertyNode is already marked as a favorite
	 */
	void MarkFavoritesInternal( TSharedPtr<FPropertyNode> InPropertyNode, bool bAnyParentIsFavorite );

	/** 
	 * Creates a property editor (the visual portion of a PropertyNode), for a specific property node
	 *
	 * @param InPropertyNode	The property node to create the visual for
	 */
	TSharedRef<ITableRow> CreatePropertyEditor( TSharedPtr<FPropertyNode> InPropertyNode, const TSharedPtr<STableViewBase>& OwnerTable );

	/**
	 * Returns an SWidget used as the visual representation of a node in the property treeview.                     
	 */
	TSharedRef<ITableRow> OnGenerateRowForPropertyTree( TSharedPtr<FPropertyNode> InPropertyNode, const TSharedRef<STableViewBase>& OwnerTable );

	/** Callback when property selection changes. */
	void OnSelectionChanged( TSharedPtr<FPropertyNode> InPropertyNode, ESelectInfo::Type SelectInfo );

	/** 
	 * Loads favorites from INI
	 */
	void LoadFavorites();

	/**
	 * Saves favorites to INI                   
	 */
	void SaveFavorites();

	/**
	 * Requests that a property node's expansion state be changed due to being filtered
	 *
	 * @param PropertyNode	The property node to change expansion state on
	 * @param bExpand		true to expand the node, false to collapse it
	 */
	void RequestItemExpanded( TSharedPtr<FPropertyNode> PropertyNode, bool bExpand, bool bRecursiveExpansion = false );
	
private:
	/** List of properties which are favorites */
	TSet<FString> FavoritesList;
	/** Stored set of expanded nodes before a filter was set */
	TSet< TSharedPtr<FPropertyNode> > PreFilterExpansionSet;
	/** Map of nodes that are requesting an automatic expansion/collapse due to being filtered */
	TMap< TSharedPtr<FPropertyNode>, bool > FilteredNodesRequestingExpansionState;
	/** Top level property nodes which are visible in the TreeView. These are always category nodes belonging to the RootPropertyNode */
	TArray< TSharedPtr<FPropertyNode> > TopLevelPropertyNodes;
	/** Top level favorite property nodes. These are root nodes of the favorite tree view */
	TArray< TSharedPtr<FPropertyNode> > TopLevelFavorites;
	/** Actions that should be executed next tick */
	TArray<FSimpleDelegate> DeferredActions;
	/** The root property node of the property tree for a specific set of UObjects */
	TSharedPtr<FObjectPropertyNode> RootPropertyNode;
	/** Our property treeview */
	TSharedPtr<SPropertyTree> PropertyTree;
	/** Our property favorites treeview */
	TSharedPtr<SPropertyTree> FavoritesTree;
	/** The filter text box */
	TSharedPtr<SSearchBox> FilterTextBox;
	/** The header row for the primary trees columns */
	TSharedPtr< SHeaderRow > ColumnHeaderRow;
	/** Settings for this property view */
	TSharedPtr<class IPropertyUtilities> PropertySettings;
	/** The current filter text */
	FString CurrentFilterText;
	/** The title of the property view (for a window title) */
	FString Title;
	/** Notify hook to call when properties are changed */
	FNotifyHook* NotifyHook;
	/** True if there is an active filter (text in the filter box) */
	bool bHasActiveFilter;
	/** True if this property view can be locked */
	bool bLockable;
	/** True if this property view is currently locked (I.E The objects being observed are not changed automatically due to user selection)*/
	bool bIsLocked;
	/** True if this property view allows favorites to visible and modified */
	bool bFavoritesAllowed;
	/** True if favorites are currently enabled */
	bool bFavoritesEnabled;
	/** True if we allow searching */
	bool bAllowSearch;
	/** True if the property view shows all properties regardless of their flags */
	bool bForceHiddenPropertyVisibility;
	/** Whether or not this tree view manages and creates property nodes or whether the nodes are externally managed */
	bool bNodeTreeExternallyManaged;
	/** Whether or not this tree should ever display top level property nodes as categories */
	bool bShowTopLevelPropertyNodes;
	/** Callback to send when the property view changes */
	FOnObjectArrayChanged OnObjectArrayChanged;
	/** Callback when property selection changes. */
	FOnPropertySelectionChanged OnPropertySelectionChanged;
	/**	Callback when a property is clicked with the middle mouse button by the user  */
	FOnPropertyClicked OnPropertyMiddleClicked;
	/** Callback to see if a property is visible */
	FIsPropertyVisible IsPropertyVisible;
	/** The initial width of the name column */
	float InitialNameColumnWidth;
	/** The path to the property node that should be treated as the root */
	TSharedRef< class FPropertyPath > RootPath;
	/** Called to construct any extra column headers external code wants to expose */
	FConstructExternalColumnHeaders ConstructExternalColumnHeaders;
	/** Called to construct any the cell contents for columns created by external code*/
	FConstructExternalColumnCell ConstructExternalColumnCell;
};
