// Copyright Epic Games, Inc. All Rights Reserved.

#include "SResetToDefaultMenu.h"

#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"

class SWidget;
struct FGeometry;

void SResetToDefaultMenu::AddProperty( TSharedRef<IPropertyHandle> InProperty )
{
	// Only add properties which are valid for reading/writing
	if( InProperty->IsValidHandle() )
	{
		Properties.Add( InProperty );
	}
}

void SResetToDefaultMenu::Construct( const FArguments& InArgs )
{
	VisibilityWhenDefault = InArgs._VisibilityWhenDefault;
	DiffersFromDefault = InArgs._DiffersFromDefault;
	OnResetToDefault = InArgs._OnResetToDefault;
	OnGetResetToDefaultText = InArgs._OnGetResetToDefaultText;

	ChildSlot
	[
		SNew(SComboButton)
		.ToolTipText(NSLOCTEXT( "PropertyEditor", "ResetToDefault", "Reset to Default" ))
		.HasDownArrow(false)
		.ButtonStyle( FAppStyle::Get(), "NoBorder" )
		.ContentPadding(0.0f)
		.Visibility( this, &SResetToDefaultMenu::GetResetToDefaultVisibility )
		.OnGetMenuContent( this, &SResetToDefaultMenu::OnGenerateResetToDefaultMenuContent )
		.ButtonContent()
		[
			SNew(SImage)
			.Image( FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault") )
		]
	];
}

void SResetToDefaultMenu::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	// Cache the reset to default visibility
	bShouldBeVisible = DiffersFromDefault.Get();
	for ( int32 PropIndex = 0; PropIndex < Properties.Num(); ++PropIndex )
	{
		TSharedRef<IPropertyHandle> Property = Properties[PropIndex];
		if( Property->DiffersFromDefault() )
		{
			bShouldBeVisible = true;
			// If one property should show reset to default, the menu must be visible
			break;
		}
	}
}


EVisibility SResetToDefaultMenu::GetResetToDefaultVisibility() const
{
	return bShouldBeVisible ? EVisibility::Visible : VisibilityWhenDefault;
}

TSharedRef<SWidget> SResetToDefaultMenu::OnGenerateResetToDefaultMenuContent()
{
	const bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder( bCloseAfterSelection, NULL );

	MenuBuilder.BeginSection("PropertyEditorResetToDefault", NSLOCTEXT( "PropertyEditor", "ResetToDefault", "Reset to Default" ) );
	{
		if(Properties.Num() > 0)
		{
			for ( int32 PropIndex = 0; PropIndex < Properties.Num(); ++PropIndex )
			{
				TSharedRef<IPropertyHandle> PropertyHandle = Properties[PropIndex];
	
				// Only display the reset to default option if the value actually differs from default
				if( PropertyHandle->IsValidHandle() && PropertyHandle->DiffersFromDefault() && !PropertyHandle->IsEditConst() )
				{
					MenuBuilder.AddMenuEntry
					( 
						PropertyHandle->GetResetToDefaultLabel(),
						NSLOCTEXT("PropertyEditor", "ResetToDefault_ToolTip", "Resets the value to its default"),
						FSlateIcon(),
						FUIAction( FExecuteAction::CreateSP( PropertyHandle, &IPropertyHandle::ResetToDefault ) )
					);
				}
			}
		}
		else
		{
			FText ResetDescription = NSLOCTEXT("PropertyEditor", "ResetToDefault_DefaultLabel", "Reset to default");
			if(OnGetResetToDefaultText.IsBound())
			{
				ResetDescription = FText::Format(NSLOCTEXT("PropertyEditor", "ResetToDefault_Label", "Reset to default: {0}"), OnGetResetToDefaultText.Execute());
			}

			MenuBuilder.AddMenuEntry
			( 
				ResetDescription,
				NSLOCTEXT("PropertyEditor", "ResetToDefault_ToolTip", "Resets the value to its default"),
				FSlateIcon(),
				FUIAction( FExecuteAction::CreateSP( this, &SResetToDefaultMenu::ResetToDefault ) )
			);
		}
	}
	MenuBuilder.EndSection();

	if(Properties.Num() > 0)
	{
		MenuBuilder.AddMenuEntry
		(
			NSLOCTEXT("PropertyEditor", "ResetAllToDefault", "Reset All"),
			NSLOCTEXT("PropertyEditor", "ResetAllToDefault_ToolTip", "Resets all the values to default"),
			FSlateIcon(),
			FUIAction( FExecuteAction::CreateSP( this, &SResetToDefaultMenu::ResetAllToDefault ) )
		);
	}

	return MenuBuilder.MakeWidget();

}

void SResetToDefaultMenu::ResetToDefault()
{
	OnResetToDefault.ExecuteIfBound();
}

void SResetToDefaultMenu::ResetAllToDefault()
{
	// Create a single transaction for resetting all rather than one for each reset
	FScopedTransaction Transaction( NSLOCTEXT("PropertyEditor", "ResetAllToDefault_Transaction", "Reset All to Default") );

	for( int32 PropIndex = 0; PropIndex < Properties.Num(); ++PropIndex )
	{
		TSharedRef<IPropertyHandle> PropertyHandle = Properties[PropIndex];

		// Only display the reset to default option if the value actually differs from default
		if( PropertyHandle->IsValidHandle() && PropertyHandle->DiffersFromDefault() && !PropertyHandle->IsEditConst() )
		{
			PropertyHandle->ResetToDefault();
		}
	}
}
