// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSingleProperty.h"

#include "AssetThumbnail.h"
#include "DetailPropertyRow.h"
#include "IPropertyUtilities.h"
#include "Modules/ModuleManager.h"
#include "ObjectPropertyNode.h"
#include "PropertyEditorHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyNode.h"
#include "SResetToDefaultPropertyEditor.h"
#include "SStandaloneCustomizedValueWidget.h"
#include "Engine/Engine.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "UObject/UnrealType.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Text/STextBlock.h"
#include "StructurePropertyNode.h"


class FSinglePropertyUtilities : public IPropertyUtilities
{
public:

	FSinglePropertyUtilities( const TWeakPtr< SSingleProperty >& InView, bool bInShouldDisplayThumbnail )
		: View( InView )
		, bShouldHideAssetThumbnail(bInShouldDisplayThumbnail)
	{
	}

	virtual class FNotifyHook* GetNotifyHook() const override
	{
		TSharedPtr< SSingleProperty > PinnedView = View.Pin();
		return PinnedView->GetNotifyHook();
	}

	virtual void CreateColorPickerWindow( const TSharedRef< class FPropertyEditor >& PropertyEditor, bool bUseAlpha ) const override
	{
		TSharedPtr< SSingleProperty > PinnedView = View.Pin();

		if ( PinnedView.IsValid() )
		{
			PinnedView->CreateColorPickerWindow( PropertyEditor, bUseAlpha );
		}
	}

	virtual void EnqueueDeferredAction( FSimpleDelegate DeferredAction ) override
	{
		 // not implemented
	}

	virtual bool AreFavoritesEnabled() const override
	{
		// not implemented
		return false;
	}

	virtual void ToggleFavorite( const TSharedRef< class FPropertyEditor >& PropertyEditor ) const override
	{
		// not implemented
	}

	virtual bool IsPropertyEditingEnabled() const override
	{
		return true;
	}

	virtual void ForceRefresh() override {}

	virtual void RequestRefresh() override {}

	virtual void RequestForceRefresh() override {}

	virtual TSharedPtr<class FAssetThumbnailPool> GetThumbnailPool() const override
	{
		return bShouldHideAssetThumbnail ? nullptr : UThumbnailManager::Get().GetSharedThumbnailPool();
	}

	virtual void NotifyFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) override
	{}

	virtual void NotifyStartedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) override
	{}

	virtual bool DontUpdateValueWhileEditing() const override
	{
		return false;
	}

	virtual const TArray<TSharedRef<class IClassViewerFilter>>& GetClassViewerFilters() const override
	{
		// not implemented
		static TArray<TSharedRef<class IClassViewerFilter>> NotImplemented;
		return NotImplemented;
	}

	const TArray<TWeakObjectPtr<UObject>>& GetSelectedObjects() const override
	{
		static TArray<TWeakObjectPtr<UObject>> Empty;
		return Empty;
	}

	virtual bool HasClassDefaultObject() const override
	{
		return false;
	}

private:
	TWeakPtr< SSingleProperty > View;
	bool bShouldHideAssetThumbnail = false;
};

void SSingleProperty::Construct( const FArguments& InArgs )
{
	PropertyName = InArgs._PropertyName;
	NameOverride = InArgs._NameOverride;
	NamePlacement = InArgs._NamePlacement;
	NotifyHook = InArgs._NotifyHook;
	PropertyFont = InArgs._PropertyFont;
	bShouldHideResetToDefault = InArgs._bShouldHideResetToDefault;

	PropertyUtilities = MakeShareable( new FSinglePropertyUtilities( SharedThis( this ), InArgs._bShouldHideAssetThumbnail ) );

	if (InArgs._Object != nullptr)
	{
		SetObject( InArgs._Object );
	}
	else if(InArgs._StructData.IsValid())
	{
		SetStruct(InArgs._StructData);
	}
}

void SSingleProperty::SetObject( UObject* InObject )
{
	if( !RootPropertyNode.IsValid() || RootPropertyNode->GetPropertyType() != FComplexPropertyNode::EPT_Object)
	{
		RootPropertyNode = MakeShareable( new FObjectPropertyNode );
	}

	FObjectPropertyNode* RootObjectPropertyNode = static_cast<FObjectPropertyNode*>(RootPropertyNode.Get());

	RootObjectPropertyNode->RemoveAllObjects();
	ValueNode.Reset();

	if( InObject )
	{
		RootObjectPropertyNode->AddObject( InObject );
	}

	if( !GeneratePropertyCustomization() )
	{
		// invalid or missing property
		RootObjectPropertyNode->RemoveAllObjects();
		RootPropertyNode.Reset();
	}
}

void SSingleProperty::SetStruct(const TSharedPtr<IStructureDataProvider>& InStruct)
{
	if (!RootPropertyNode.IsValid() || RootPropertyNode->GetPropertyType() != FComplexPropertyNode::EPT_StandaloneStructure)
	{
		RootPropertyNode = MakeShareable(new FStructurePropertyNode);
		RootPropertyNode->SetNodeFlags(EPropertyNodeFlags::RequiresValidation, true);
	}

	FStructurePropertyNode* RootStructPropertyNode = (FStructurePropertyNode*)RootPropertyNode.Get();

	RootStructPropertyNode->RemoveStructure();
	ValueNode.Reset();

	if (InStruct)
	{
		RootStructPropertyNode->SetStructure(InStruct);
	}

	if( !GeneratePropertyCustomization() )
	{
		// invalid or missing property
		RootStructPropertyNode->RemoveStructure();
		RootPropertyNode.Reset();
	}
}

bool SSingleProperty::GeneratePropertyCustomization()
{
	DestroyColorPicker();

	FPropertyNodeInitParams InitParams;
	InitParams.ParentNode = NULL;
	InitParams.Property = NULL;
	InitParams.ArrayOffset = 0;
	InitParams.ArrayIndex = INDEX_NONE;
	// we'll generate the children
	InitParams.bAllowChildren = false;
	InitParams.bForceHiddenPropertyVisibility = false;

	RootPropertyNode->InitNode( InitParams );

	ValueNode = RootPropertyNode->GenerateSingleChild( PropertyName );

	bool bIsAcceptableProperty = false;
	FProperty* Property = nullptr; 
	// valid criteria for standalone properties 
	if( ValueNode.IsValid() )
	{
		//TODO MaterialLayers: Remove below commenting
		bIsAcceptableProperty = true;
		Property = ValueNode->GetProperty();
		check(Property);
		// not an array property (dynamic or static)
		//bIsAcceptableProperty &= !( Property->IsA( FArrayProperty::StaticClass() ) || (Property->ArrayDim > 1 && ValueNode->GetArrayIndex() == INDEX_NONE) );
		// not a struct property unless its a built in type like a vector
		//bIsAcceptableProperty &= ( !Property->IsA( FStructProperty::StaticClass() ) || PropertyEditorHelpers::IsBuiltInStructProperty( Property ) );
	}

	if( bIsAcceptableProperty )
	{
		ValueNode->RebuildChildren();

		TSharedRef< FPropertyEditor > PropertyEditor = FPropertyEditor::Create( ValueNode.ToSharedRef(), TSharedPtr< IPropertyUtilities >( PropertyUtilities ).ToSharedRef() );
		ValueNode->SetDisplayNameOverride( NameOverride );
		
		PropertyHandle = PropertyEditorHelpers::GetPropertyHandle(ValueNode.ToSharedRef(), NotifyHook, PropertyUtilities);

		TSharedPtr<SHorizontalBox> HorizontalBox;

		ChildSlot
		[
			SAssignNew( HorizontalBox, SHorizontalBox )
		];

		if( NamePlacement != EPropertyNamePlacement::Hidden )
		{
			HorizontalBox->AddSlot()
			.Padding(4.0f, 0.0f)
			.AutoWidth()
			.VAlign( VAlign_Center )
			[
				SNew( SPropertyNameWidget, PropertyEditor )
			];
		}

		// For structs and other properties with a customized header
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		if (const FPropertyTypeLayoutCallback LayoutCallback =
				PropertyEditorModule.GetPropertyTypeCustomization(
					Property, *PropertyHandle, FCustomPropertyTypeLayoutMap()
				); LayoutCallback.IsValid()
			)
		{
			TSharedRef<IPropertyTypeCustomization> CustomizationInstance = LayoutCallback.GetCustomizationInstance();
			
			HorizontalBox->AddSlot()
			.Padding( 4.0f, 0.0f)
			.FillWidth(1.0f)
			.VAlign( VAlign_Center )
			[
				SNew( SStandaloneCustomizedValueWidget, CustomizationInstance, PropertyHandle.ToSharedRef())
			];
		}
		else // For properties without customization
		{

			HorizontalBox->AddSlot()
			.Padding( 4.0f, 0.0f)
			.FillWidth(1.0f)
			.VAlign( VAlign_Center )
			[
				SNew( SPropertyValueWidget, PropertyEditor, PropertyUtilities.ToSharedRef() )
			];			
		}

		if (!PropertyEditor->GetPropertyHandle()->HasMetaData(TEXT("NoResetToDefault")) && !bShouldHideResetToDefault)
		{
			HorizontalBox->AddSlot()
			.Padding( 2.0f )
			.AutoWidth()
			.VAlign( VAlign_Center )
			[
				SNew( SResetToDefaultPropertyEditor,  PropertyEditor->GetPropertyHandle() )
			];
		}
	}
	else
	{
		if (ValueNode.IsValid())
		{
			// Still create a PropertyHandle, though it may not be fully initialized as the ValueNode will not have had children rebuilt
			PropertyHandle = PropertyEditorHelpers::GetPropertyHandle(ValueNode.ToSharedRef(), NotifyHook, PropertyUtilities);
		}
		
		ChildSlot
		[
			SNew(STextBlock)
			.Font(PropertyFont)
			.Text(NSLOCTEXT("PropertyEditor", "SinglePropertyInvalidType", "Cannot Edit Inline"))
			.ToolTipText(NSLOCTEXT("PropertyEditor", "SinglePropertyInvalidType_Tooltip", "Properties of this type cannot be edited inline; edit it elsewhere"))
		];

		ValueNode.Reset();
	}

	return bIsAcceptableProperty;
}


void SSingleProperty::SetOnPropertyValueChanged( const FSimpleDelegate& InOnPropertyValueChanged )
{
	if( HasValidProperty() )
	{
		ValueNode->OnPropertyValueChanged().Add( InOnPropertyValueChanged );
	}
}

void SSingleProperty::ReplaceObjects( const TMap<UObject*, UObject*>& OldToNewObjectMap )
{
	if( HasValidProperty() && RootPropertyNode->GetPropertyType() == FComplexPropertyNode::EPT_Object )
	{
		TArray<UObject*> NewObjectList;
		bool bObjectsReplaced = false;

		FObjectPropertyNode* RootObjectPropertyNode = static_cast<FObjectPropertyNode*>(RootPropertyNode.Get());

		// Scan all objects and look for objects which need to be replaced
		for ( TPropObjectIterator Itor(RootObjectPropertyNode->ObjectIterator() ); Itor; ++Itor )
		{
			UObject* Replacement = OldToNewObjectMap.FindRef( Itor->Get(true) );
			if( Replacement )
			{
				bObjectsReplaced = true;
				NewObjectList.Add( Replacement );
			}
			else
			{
				NewObjectList.Add( Itor->Get() );
			}
		}

		// if any objects were replaced update the observed objects
		if( bObjectsReplaced )
		{
			SetObject( NewObjectList[0] );
		}
	}
}

void SSingleProperty::RemoveDeletedObjects( const TArray<UObject*>& DeletedObjects )
{

	if( HasValidProperty() && RootPropertyNode->GetPropertyType() == FComplexPropertyNode::EPT_Object )
	{
		FObjectPropertyNode* RootObjectPropertyNode = static_cast<FObjectPropertyNode*>(RootPropertyNode.Get());

		// Scan all objects and look for objects which need to be replaced
		for ( TPropObjectIterator Itor(RootObjectPropertyNode->ObjectIterator() ); Itor; ++Itor )
		{
			if( DeletedObjects.Contains( Itor->Get() ) )
			{
				SetObject( NULL );
				break;
			}
		}
	}
}

void SSingleProperty::CreateColorPickerWindow( const TSharedRef< class FPropertyEditor >& PropertyEditor, bool bUseAlpha )
{
	if( HasValidProperty() )
	{
		TSharedRef<FPropertyNode> Node = PropertyEditor->GetPropertyNode();
		check( &Node.Get() == ValueNode.Get() );
		FProperty* Property = Node->GetProperty();
		check(Property);

		FReadAddressList ReadAddresses;
		Node->GetReadAddress( false, ReadAddresses, false );

		// Use the first address for the initial color
		TOptional<FLinearColor> DefaultColor;
		bool bClampValue = false;
		if( ReadAddresses.Num() ) 
		{
			const uint8* Addr = ReadAddresses.GetAddress(0);
			if( Addr )
			{
				if( CastField<FStructProperty>(Property)->Struct->GetFName() == NAME_Color )
				{
					DefaultColor = *reinterpret_cast<const FColor*>(Addr);
					bClampValue = true;
				}
				else
				{
					check( CastField<FStructProperty>(Property)->Struct->GetFName() == NAME_LinearColor );
					DefaultColor = *reinterpret_cast<const FLinearColor*>(Addr);
				}
			}
		}

		if (DefaultColor.IsSet())
		{
			FColorPickerArgs PickerArgs = FColorPickerArgs(DefaultColor.GetValue(), FOnLinearColorValueChanged::CreateSP(this, &SSingleProperty::SetColorPropertyFromColorPicker));
			PickerArgs.ParentWidget = AsShared();
			PickerArgs.bUseAlpha = bUseAlpha;
			PickerArgs.bClampValue = bClampValue;
			PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));

			OpenColorPicker(PickerArgs);
		}
	}
}

void SSingleProperty::SetColorPropertyFromColorPicker(FLinearColor NewColor)
{
	if( HasValidProperty() )
	{
		FProperty* NodeProperty = ValueNode->GetProperty();
		check(NodeProperty);
		check(GetPropertyHandle());

		if (CastField<FStructProperty>(NodeProperty)->Struct->GetFName() == NAME_Color)
		{
			const bool bSRGB = true;
			FColor NewFColor = NewColor.ToFColor(bSRGB);
			ensure(GetPropertyHandle()->SetValueFromFormattedString(NewFColor.ToString(), EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
		}
		else
		{
			ensure(GetPropertyHandle()->SetValueFromFormattedString(NewColor.ToString(), EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
		}
	}
}
