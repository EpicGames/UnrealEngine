// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/AppStyle.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "ISinglePropertyView.h"

class FNotifyHook;
class FObjectPropertyNode;
class FPropertyNode;
class FSinglePropertyUtilities;
class FStructurePropertyNode;

class SSingleProperty : public ISinglePropertyView
{
public:
	SLATE_BEGIN_ARGS( SSingleProperty )
		: _Object(NULL)
		, _StructData(NULL)
		, _NotifyHook( NULL )
		, _PropertyFont( FAppStyle::GetFontStyle( PropertyEditorConstants::PropertyFontStyle ) ) 
		, _NamePlacement( EPropertyNamePlacement::Left )
		, _NameOverride()
		, _bShouldHideAssetThumbnail(false)
		, _bShouldHideResetToDefault(false)
	{}

		SLATE_ARGUMENT( UObject*, Object )
		SLATE_ARGUMENT(TSharedPtr<IStructureDataProvider>, StructData)
		SLATE_ARGUMENT( FName, PropertyName )
		SLATE_ARGUMENT( FNotifyHook*, NotifyHook )
		SLATE_ARGUMENT( FSlateFontInfo, PropertyFont )
		SLATE_ARGUMENT( EPropertyNamePlacement::Type, NamePlacement )
		SLATE_ARGUMENT( FText, NameOverride )
		SLATE_ARGUMENT( bool, bShouldHideAssetThumbnail )
		SLATE_ARGUMENT( bool, bShouldHideResetToDefault )
	SLATE_END_ARGS()	

	void Construct( const FArguments& InArgs );

	/** ISinglePropertyView interface */
	virtual bool HasValidProperty() const override { return (RootPropertyNode.IsValid() || RootPropertyNode) && ValueNode.IsValid(); }
	virtual void SetObject( UObject* InObject ) override;
	virtual void SetStruct( const TSharedPtr<IStructureDataProvider>& InStruct ) override;
	virtual void SetOnPropertyValueChanged( const FSimpleDelegate& InOnPropertyValueChanged ) override;

	/**
	 * Replaces objects being observed by the view with new objects
	 *
	 * @param OldToNewObjectMap	Mapping from objects to replace to their replacement
	 */
	void ReplaceObjects( const TMap<UObject*, UObject*>& OldToNewObjectMap );

	/**
	 * Removes objects from the view because they are about to be deleted
	 *
	 * @param DeletedObjects	The objects to delete
	 */
	void RemoveDeletedObjects( const TArray<UObject*>& DeletedObjects );

	/** Creates a color picker window for a property node */
	void CreateColorPickerWindow( const TSharedRef< class FPropertyEditor >& PropertyEditor, bool bUseAlpha );

	/** @return The notify hook used by the property */
	FNotifyHook* GetNotifyHook() const { return NotifyHook; }

	virtual TSharedPtr<IPropertyHandle> GetPropertyHandle() const override { return PropertyHandle;  }
private:
	/**
	 * Sets the color if this is a color property
	 *
	 * @param NewColor The color to set
	 */
	void SetColorPropertyFromColorPicker(FLinearColor NewColor);

	/**
	 * Generates the SingleProperty customization
	 *
	 * @return true if valid property and widget has been generated
	 */
	bool GeneratePropertyCustomization();

private:
	/** The root property node for the value node (contains the root object */
	TSharedPtr<FComplexPropertyNode> RootPropertyNode;
	/** The node for the property being edited */
	TSharedPtr<FPropertyNode> ValueNode;
	/** Property utilities for handling common functionality of property editors */
	TSharedPtr<class FSinglePropertyUtilities> PropertyUtilities;
	/** Name override to display instead of the property name */
	FText NameOverride;
	/** Font to use */
	FSlateFontInfo PropertyFont;
	/** Notify hook to use when editing values */
	FNotifyHook* NotifyHook;
	/** Name of the property */
	FName PropertyName;
	/** Location of the name in the view */
	EPropertyNamePlacement::Type NamePlacement;
	TSharedPtr<class IPropertyHandle> PropertyHandle;
	/** Whether the 'reset to default' button should be hidden */
	bool bShouldHideResetToDefault;
};
