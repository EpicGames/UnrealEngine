// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStandaloneCustomizedValueWidget.h"

#include "DetailCategoryBuilderImpl.h"
#include "IPropertyTypeCustomization.h"
	
void SStandaloneCustomizedValueWidget::Construct( const FArguments& InArgs,
	TSharedPtr<IPropertyTypeCustomization> InCustomizationInterface, TSharedRef<IPropertyHandle> InPropertyHandle)
{
	ParentCategory = InArgs._ParentCategory;
	
	CustomizationInterface = InCustomizationInterface;
	PropertyHandle = InPropertyHandle;
	CustomPropertyWidget = MakeShared<FDetailWidgetRow>();

	CustomizationInterface->CustomizeHeader(InPropertyHandle, *CustomPropertyWidget, *this);

	ChildSlot
	[
		CustomPropertyWidget->ValueWidget.Widget
	];
}

TSharedPtr<FAssetThumbnailPool> SStandaloneCustomizedValueWidget::GetThumbnailPool() const
{
	TSharedPtr<FDetailCategoryImpl> ParentCategoryPinned = ParentCategory.Pin();
	return ParentCategoryPinned.IsValid() ? ParentCategoryPinned->GetParentLayout().GetThumbnailPool() : nullptr;
}
