// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStructureDetailsView.h"

#include "AssetSelection.h"
#include "DetailCategoryBuilderImpl.h"
#include "DetailsViewPropertyGenerationUtilities.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "PropertyEditorCommands.h"
#include "StructurePropertyNode.h"
#include "UserInterface/PropertyDetails/PropertyDetailsUtilities.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"

#define LOCTEXT_NAMESPACE "SStructureDetailsView"

SStructureDetailsView::~SStructureDetailsView()
{
	auto RootNodeLocal = GetRootNode();

	if (RootNodeLocal.IsValid())
	{
		SaveExpandedItems(RootNodeLocal.ToSharedRef());
	}
}

UStruct* SStructureDetailsView::GetBaseScriptStruct() const
{
	const UStruct* Struct = StructProvider.IsValid() ? StructProvider->GetBaseStructure() : nullptr;
	return const_cast<UStruct*>(Struct);
}

void SStructureDetailsView::Construct(const FArguments& InArgs)
{
	DetailsViewArgs = InArgs._DetailsViewArgs;

	ColumnSizeData.SetValueColumnWidth(DetailsViewArgs.ColumnWidth);
	ColumnSizeData.SetRightColumnMinWidth(DetailsViewArgs.RightColumnMinWidth);

	CustomName = InArgs._CustomName;
	
	BindCommands();

	// Create the root property now
	// Only one root node in a structure details view
	RootNodes.Empty(1);
	RootNodes.Add(MakeShareable(new FStructurePropertyNode));
		
	PropertyUtilities = MakeShareable( new FPropertyDetailsUtilities( *this ) );
	PropertyGenerationUtilities = MakeShareable(new FDetailsViewPropertyGenerationUtilities(*this));
	
	TSharedRef<SScrollBar> ExternalScrollbar = SNew(SScrollBar);

	// See note in SDetailsView for why visibility is set after construction
	ExternalScrollbar->SetVisibility( TAttribute<EVisibility>(this, &SStructureDetailsView::GetScrollBarVisibility) );

	FMenuBuilder DetailViewOptions(true, InternalCommands);

	FUIAction ShowOnlyModifiedAction( 
		FExecuteAction::CreateSP(this, &SStructureDetailsView::OnShowOnlyModifiedClicked),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SStructureDetailsView::IsShowOnlyModifiedChecked)
	);

	if (DetailsViewArgs.bShowModifiedPropertiesOption)
	{
		DetailViewOptions.AddMenuEntry( 
			LOCTEXT("ShowOnlyModified", "Show Only Modified Properties"),
			LOCTEXT("ShowOnlyModified_ToolTip", "Displays only properties which have been changed from their default"),
			FSlateIcon(),
			ShowOnlyModifiedAction,
			NAME_None,
			EUserInterfaceActionType::Check
		);
	}
	if (DetailsViewArgs.bShowKeyablePropertiesOption)
	{
		DetailViewOptions.AddMenuEntry(
			LOCTEXT("ShowOnlyKeyable", "Show Only Keyable Properties"),
			LOCTEXT("ShowOnlyKeyable_ToolTip", "Displays only properties which are keyable"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SStructureDetailsView::OnShowKeyableClicked),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SStructureDetailsView::IsShowKeyableChecked)
			),
			NAME_None,
			EUserInterfaceActionType::Check
		);
	}
	if (DetailsViewArgs.bShowAnimatedPropertiesOption)
	{
		DetailViewOptions.AddMenuEntry(
			LOCTEXT("ShowAnimated", "Show Only Animated Properties"),
			LOCTEXT("ShowAnimated_ToolTip", "Displays only properties which are animated (have tracks)"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SStructureDetailsView::OnShowAnimatedClicked),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SStructureDetailsView::IsShowAnimatedChecked)
			),
			NAME_None,
			EUserInterfaceActionType::Check
		);
	}
	FUIAction ShowAllAdvancedAction( 
		FExecuteAction::CreateSP(this, &SStructureDetailsView::OnShowAllAdvancedClicked),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SStructureDetailsView::IsShowAllAdvancedChecked)
	);

	DetailViewOptions.AddMenuEntry(
		LOCTEXT("ShowAllAdvanced", "Show All Advanced Details"),
		LOCTEXT("ShowAllAdvanced_ToolTip", "Shows all advanced detail sections in each category"),
		FSlateIcon(),
		ShowAllAdvancedAction,
		NAME_None,
		EUserInterfaceActionType::Check
		);

	// Commands have been registered earlier
	DetailViewOptions.AddMenuEntry(FPropertyEditorCommands::Get().CollapseAll);
	DetailViewOptions.AddMenuEntry(FPropertyEditorCommands::Get().ExpandAll);

	TSharedRef<SHorizontalBox> FilterBoxRow = SNew( SHorizontalBox )
		.Visibility(this, &SStructureDetailsView::GetFilterBoxVisibility)
		+SHorizontalBox::Slot()
		.FillWidth( 1 )
		.VAlign( VAlign_Center )
		.Padding( 6.0f )
		[
			// Create the search box
			SAssignNew( SearchBox, SSearchBox )
			.OnTextChanged(this, &SStructureDetailsView::OnFilterTextChanged)
		];

	if (DetailsViewArgs.bShowOptions)
	{
		const TSharedPtr<SLayeredImage> FilterImage = SNew(SLayeredImage)
			.Image(FAppStyle::Get().GetBrush("DetailsView.ViewOptions"))
			.ColorAndOpacity(FSlateColor::UseForeground());

		// Badge the filter icon if there are filters active
		FilterImage->AddLayer(TAttribute<const FSlateBrush*>(
			this, &SStructureDetailsView::GetViewOptionsBadgeIcon)
		);

		FilterBoxRow->AddSlot()
			.Padding(0.0f)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SComboButton)
				.HasDownArrow(false)
				.ContentPadding(0.0f)
				.ForegroundColor(FSlateColor::UseForeground())
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.AddMetaData<FTagMetaData>(FTagMetaData("ViewOptions"))
				.MenuContent()
				[
					DetailViewOptions.MakeWidget()
				]
				.ButtonContent()
				[
					FilterImage.ToSharedRef()
				]
			];
	}

	SAssignNew(DetailTree, SDetailTree)
		.Visibility(this, &SStructureDetailsView::GetTreeVisibility)
		.TreeItemsSource(&RootTreeNodes)
		.OnGetChildren(this, &SStructureDetailsView::OnGetChildrenForDetailTree)
		.OnSetExpansionRecursive(this, &SStructureDetailsView::SetNodeExpansionStateRecursive)
		.OnGenerateRow(this, &SStructureDetailsView::OnGenerateRowForDetailTree)
		.OnExpansionChanged(this, &SStructureDetailsView::OnItemExpansionChanged)
		.SelectionMode(ESelectionMode::None)
		.ExternalScrollbar(ExternalScrollbar);

	constexpr float ScrollbarWidth = 16.0f;

	ChildSlot
	[
		SNew( SBox )
		.Visibility(this, &SStructureDetailsView::GetPropertyEditingVisibility)
		[
			SNew( SVerticalBox )
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding( 2.0f )
			[
				FilterBoxRow
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0.0f)
			[
				SNew( SOverlay )
				+ SOverlay::Slot()
				.Padding(TAttribute<FMargin>::CreateLambda([ExternalScrollbar]()
				{
					if (ExternalScrollbar->GetVisibility().IsVisible())
					{
						return FMargin(0.0f, 0.0f, ScrollbarWidth, 0.0f);
					}

					return FMargin();
				}))
				[
					DetailTree.ToSharedRef()
				]
				+ SOverlay::Slot()
				.HAlign(HAlign_Right)
				[
					SNew( SBox )
					.WidthOverride( ScrollbarWidth )
					[
						ExternalScrollbar
					]
				]
			]
		]
	];
}

void SStructureDetailsView::SetStructureData(TSharedPtr<FStructOnScope> InStructData)
{
	SetStructureProvider(InStructData ? MakeShared<FStructOnScopeStructureDataProvider>(InStructData) : TSharedPtr<IStructureDataProvider>());
}

void SStructureDetailsView::SetStructureProvider(TSharedPtr<IStructureDataProvider> InStructProvider)
{

	TSharedPtr<FComplexPropertyNode> RootNode = GetRootNode();
	//PRE SET
	SaveExpandedItems(RootNode.ToSharedRef() );
	RootNode->AsStructureNode()->RemoveStructure();
	RootNodesPendingKill.Add(RootNode);

	RootNodes.Empty(1);
	ExpandedDetailNodes.Clear();

	RootNode = MakeShareable(new FStructurePropertyNode);
	RootNodes.Add(RootNode);

	//SET
	StructProvider = InStructProvider;
	RootNode->AsStructureNode()->SetStructure(StructProvider);
	if (!StructProvider.IsValid())
	{
		bIsLocked = false;
	}
	
	//POST SET
	TSharedPtr<SColorPicker> ExistingColorPicker = GetColorPicker();
	if (ExistingColorPicker.IsValid()
		&& (!ExistingColorPicker->GetOptionalOwningDetailsView().IsValid()
			|| ExistingColorPicker->GetOptionalOwningDetailsView().Get() == this))
	{
		DestroyColorPicker();
		bHasOpenColorPicker = false;
	}

	FPropertyNodeInitParams InitParams;
	InitParams.ParentNode = nullptr;
	InitParams.Property = nullptr;
	InitParams.ArrayOffset = 0;
	InitParams.ArrayIndex = INDEX_NONE;
	InitParams.bAllowChildren = true;
	InitParams.bForceHiddenPropertyVisibility = FPropertySettings::Get().ShowHiddenProperties() || DetailsViewArgs.bForceHiddenPropertyVisibility;
	InitParams.bCreateCategoryNodes = false;

	RootNode->InitNode(InitParams);
	RootNode->SetDisplayNameOverride(CustomName);

	UpdatePropertyMapsWithItemExpansions();
	UpdateFilteredDetails();
}

void SStructureDetailsView::SetCustomName(const FText& Text)
{
	CustomName = Text;
}

void SStructureDetailsView::ForceRefresh()
{
	ClearPendingRefreshTimer();

	SetStructureProvider(StructProvider);
}

void SStructureDetailsView::InvalidateCachedState()
{
	for (const TSharedPtr<FComplexPropertyNode>& RootNode : RootNodes)
	{
		RootNode->InvalidateCachedState();
	}
}

const TArray< TWeakObjectPtr<UObject> >& SStructureDetailsView::GetSelectedObjects() const
{
	static const TArray< TWeakObjectPtr<UObject> > DummyRef;
	return DummyRef;
}

const TArray< TWeakObjectPtr<AActor> >& SStructureDetailsView::GetSelectedActors() const
{
	static const TArray< TWeakObjectPtr<AActor> > DummyRef;
	return DummyRef;
}

const FSelectedActorInfo& SStructureDetailsView::GetSelectedActorInfo() const
{
	static const FSelectedActorInfo DummyRef;
	return DummyRef;
}

bool SStructureDetailsView::IsConnected() const
{
	const FStructurePropertyNode* RootNode = GetRootNode().IsValid() ? GetRootNode()->AsStructureNode() : nullptr;
	return StructProvider.IsValid() && StructProvider->IsValid() && RootNode && RootNode->HasValidStructData();
}

FRootPropertyNodeList& SStructureDetailsView::GetRootNodes()
{
	return RootNodes;
}

TSharedPtr<class FComplexPropertyNode> SStructureDetailsView::GetRootNode()
{
	return RootNodes[0];
}

const TSharedPtr<class FComplexPropertyNode> SStructureDetailsView::GetRootNode() const
{
	return RootNodes[0];
}

void SStructureDetailsView::CustomUpdatePropertyMap(TSharedPtr<FDetailLayoutBuilderImpl>& InDetailLayout)
{
	FName StructCategoryName = NAME_None;
	
	const UStruct* Struct = StructProvider.IsValid() ? StructProvider->GetBaseStructure() : nullptr;
	if (Struct)
	{
		TArray<FName> CategoryNames;
		InDetailLayout->GetCategoryNames(CategoryNames);

		int32 StructCategoryNameIndex = INDEX_NONE;
		CategoryNames.Find(Struct->GetFName(), StructCategoryNameIndex);
		StructCategoryName = StructCategoryNameIndex != INDEX_NONE ? CategoryNames[StructCategoryNameIndex] : NAME_None;
	}
	
	InDetailLayout->DefaultCategory(StructCategoryName).SetDisplayName(NAME_None, CustomName);
}

EVisibility SStructureDetailsView::GetPropertyEditingVisibility() const
{
	const FStructurePropertyNode* RootNode = GetRootNode().IsValid() ? GetRootNode()->AsStructureNode() : nullptr;
	return StructProvider.IsValid() && StructProvider->IsValid() && RootNode && RootNode->HasValidStructData() ? EVisibility::Visible : EVisibility::Collapsed;
}

const FSlateBrush* SStructureDetailsView::GetViewOptionsBadgeIcon() const
{
	// Badge the icon if any view option that narrows down the results is checked
	const bool bHasBadge =
		(DetailsViewArgs.bShowModifiedPropertiesOption && IsShowOnlyModifiedChecked()) ||
		(DetailsViewArgs.bShowDifferingPropertiesOption && IsShowOnlyAllowedChecked()) ||
		(DetailsViewArgs.bShowKeyablePropertiesOption && IsShowKeyableChecked()) ||
		(DetailsViewArgs.bShowAnimatedPropertiesOption && IsShowAnimatedChecked());

	return bHasBadge ? FAppStyle::Get().GetBrush("Icons.BadgeModified") : nullptr;
}

#undef LOCTEXT_NAMESPACE
