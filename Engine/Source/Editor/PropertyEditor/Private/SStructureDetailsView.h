// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "UObject/StructOnScope.h"
#include "PropertyNode.h"
#include "IDetailsView.h"
#include "SDetailsViewBase.h"
#include "IStructureDetailsView.h"

class AActor;
class FDetailLayoutBuilderImpl;
class IDetailRootObjectCustomization;
class IStructureDataProvider;

class SStructureDetailsView : public SDetailsViewBase, public IStructureDetailsView
{

public:

	SLATE_BEGIN_ARGS(SStructureDetailsView)
		: _DetailsViewArgs()
	{ }
		/** User defined arguments for the details view */
		SLATE_ARGUMENT(FDetailsViewArgs, DetailsViewArgs)

		/** Custom name for the root property node. */
		SLATE_ARGUMENT(FText, CustomName)
	SLATE_END_ARGS()

	/** Destructor. */
	~SStructureDetailsView();

	/** Constructs the property view widgets. */
	void Construct(const FArguments& InArgs);

	UStruct* GetBaseScriptStruct() const;

	virtual bool IsConnected() const override;

	virtual bool DontUpdateValueWhileEditing() const override
	{
		return true;
	}

	virtual bool ContainsMultipleTopLevelObjects() const override
	{
		return false;
	}

public:

	// IStructureDetailsView interface

	virtual IDetailsView* GetDetailsView() override
	{
		return this;
	}

	virtual TSharedPtr<SWidget> GetWidget() override
	{
		return AsShared();
	}

	virtual void SetStructureData(TSharedPtr<FStructOnScope> InStructData) override;

	virtual void SetStructureProvider(TSharedPtr<IStructureDataProvider> StructProvider) override;

	virtual FOnFinishedChangingProperties& GetOnFinishedChangingPropertiesDelegate() override
	{
		return OnFinishedChangingProperties();
	}

	TSharedPtr<const IStructureDataProvider> GetStructureProvider() const override { return StructProvider; }

public:
	/** Causes the details view to be refreshed (new widgets generated) with the current set of objects */
	virtual void ForceRefresh() override;
	/** Invalidates cached state such as the "revert to default" arrow and edit conditions, without rebuilding the entire panel. */
	virtual void InvalidateCachedState() override;
	virtual void MoveScrollOffset(int32 DeltaOffset) override {}

	// IDetailsView interface
	virtual const TArray< TWeakObjectPtr<UObject> >& GetSelectedObjects() const override;
	virtual const TArray< TWeakObjectPtr<AActor> >& GetSelectedActors() const override;
	virtual const struct FSelectedActorInfo& GetSelectedActorInfo() const override;

	virtual bool HasClassDefaultObject() const override
	{
		return false;
	}

	virtual void SetOnObjectArrayChanged(FOnObjectArrayChanged OnObjectArrayChangedDelegate) override {}
	virtual void SetObjects(const TArray<UObject*>& InObjects, bool bForceRefresh = false, bool bOverrideLock = false) override {}
	virtual void SetObjects(const TArray< TWeakObjectPtr< UObject > >& InObjects, bool bForceRefresh = false, bool bOverrideLock = false) override {}
	virtual void SetObject(UObject* InObject, bool bForceRefresh = false) override{}
	virtual void RemoveInvalidObjects() override {}
	virtual void SetObjectPackageOverrides(const TMap<TWeakObjectPtr<UObject>, TWeakObjectPtr<UPackage>>& InMapping) override {}
	virtual void SetRootObjectCustomizationInstance(TSharedPtr<IDetailRootObjectCustomization> InRootObjectCustomization) override {}
	virtual TSharedPtr<class IDetailRootObjectCustomization> GetRootObjectCustomization() const override { return nullptr; }
	virtual void SetObjectFilter(TSharedPtr<FDetailsViewObjectFilter> InFilter) override {}
	virtual void SetClassViewerFilters(const TArray<TSharedRef<class IClassViewerFilter>>& InFilters) override {}
	virtual bool IsGroupFavorite(FStringView GroupPath) const { return false; }
	virtual void SetGroupFavorite(FStringView GroupPath, bool IsFavorite) {}
	virtual bool IsCustomBuilderFavorite(FStringView Path) const { return false; }
	virtual void SetCustomBuilderFavorite(FStringView Path, bool IsFavorite) {}

	/* This is required by the base class but there is only ever one root node in a structure details view */
	virtual FRootPropertyNodeList& GetRootNodes() override;

	TSharedPtr<class FComplexPropertyNode> GetRootNode();
	const TSharedPtr<class FComplexPropertyNode> GetRootNode() const;

	virtual void SetCustomName(const FText& Text) override;

protected:

	virtual void CustomUpdatePropertyMap(TSharedPtr<FDetailLayoutBuilderImpl>& InDetailLayout) override;

	EVisibility GetPropertyEditingVisibility() const;

	/** Get the badge for the view options icon (if there are any options active */
	const FSlateBrush* GetViewOptionsBadgeIcon() const;

private:
	TSharedPtr<IStructureDataProvider> StructProvider;
	FRootPropertyNodeList RootNodes;
	FText CustomName;
};
