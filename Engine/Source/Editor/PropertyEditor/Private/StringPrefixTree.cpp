// Copyright Epic Games, Inc. All Rights Reserved.

#include "StringPrefixTree.h"

FStringPrefixTree::~FStringPrefixTree()
{
	Clear();
}

void FStringPrefixTree::Insert(FStringView ToAdd)
{
	if (ToAdd.IsEmpty())
	{
		Root.IsLeaf = true;
		return;
	}

	Root.Insert(ToAdd);
}

void FStringPrefixTree::InsertAll(const TArray<FString>& Array)
{
	for (const FString& Entry : Array)
	{
		Insert(Entry);
	}
}

void FStringPrefixTree::InsertAll(const TArray<FStringView>& Array)
{
	for (const FStringView& Entry : Array)
	{
		Insert(Entry);
	}
}

void FStringPrefixTree::Remove(FStringView ToRemove)
{
	if (ToRemove.IsEmpty())
	{
		Root.IsLeaf = false;
		return;
	}

	Root.Remove(ToRemove);
}

void FStringPrefixTree::Clear()
{
	Root.Children.Reset();
	Root.IsLeaf = false;
	Root.Prefix = FString();
}

bool FStringPrefixTree::Contains(FStringView ToFind) const
{
	if (ToFind.IsEmpty())
	{
		return Root.IsLeaf;
	}

	return Root.Contains(ToFind);
}

bool FStringPrefixTree::AnyStartsWith(FStringView Prefix) const
{
	if (Prefix.IsEmpty())
	{
		return Root.IsLeaf || Root.Children.Num() > 0;
	}

	return Root.AnyStartsWith(Prefix);
}

int32 FStringPrefixTree::Size() const
{
	return Root.Size();
}

int32 FStringPrefixTree::NumNodes() const
{
	return Root.NumNodes();
}

TArray<FString> FStringPrefixTree::GetAllEntries() const 
{
	TArray<FString> Entries;

	Root.GetAllEntries(FString(), Entries);

	return MoveTemp(Entries);
}

FString FStringPrefixTree::DumpToString() const
{
	TStringBuilder<512> Builder;
	TArray<bool, TInlineAllocator<32>> DrawLines;
	Root.DumpToString(Builder, DrawLines, false);
	return Builder.ToString();
}

FStringPrefixTree::FNode::~FNode()
{
	Children.Reset();
	Prefix.Reset();
	IsLeaf = false;
}

static int32 GetCommonPrefixLength(FStringView A, FStringView B)
{
	int32 CharIdx = 0;
	for (; CharIdx < A.Len() && CharIdx < B.Len(); ++CharIdx)
	{
		if (A[CharIdx] != B[CharIdx])
		{
			break;
		}
	}
	return CharIdx;
}

void FStringPrefixTree::FNode::Insert(FStringView ToAdd)
{
	bool bAdded = false;

	// find a child that shares a prefix
	for (FNode& Child : Children)
	{
		int32 CharIdx = GetCommonPrefixLength(Child.Prefix, ToAdd);
		if (CharIdx > 0)
		{
			// the paths share a common prefix, there can be only one such node
			bAdded = true;

			if (CharIdx == Child.Prefix.Len())
			{
				// the prefix is the whole child path, we don't need to split the child node, but rather just add the postfix as a child
				if (CharIdx == ToAdd.Len())
				{
					// the paths are identical, we don't need to add a node, just mark this as a leaf node
					Child.IsLeaf = true;
					break;
				}

				// eg. Child = Foo, ToAdd = Foo.Bar
				Child.Insert(ToAdd.RightChop(CharIdx));
			}

			// this is a partial prefix of the child
			// eg. Child = Foobar, ToAdd = Foo
			else if (CharIdx < Child.Prefix.Len())
			{
				// modify the child node such that we take the common prefix and insert a node above it
				// eg. Child = "Foobar", ToAdd = "Foo",
				// Child -> "Foo", Split -> "bar"
				
				FNode Split;
				Split.Prefix = Child.Prefix.RightChop(CharIdx);
				check(Split.Prefix.Len() > 0);

				Split.Children = MoveTemp(Child.Children);
				Split.IsLeaf = Child.IsLeaf;

				Child.Prefix.LeftInline(CharIdx);
				check(Child.Prefix.Len() > 0);

				Child.Children.Reset();
				Child.Children.Add(MoveTemp(Split));

				if (CharIdx < ToAdd.Len())
				{
					// this was a partial match that still has leftover chars, 
					// eg. Child = "Foo", ToAdd = "Foz"
					// Child -> "Fo", Split -> "o", Added -> "z"
					FNode& AddedNode = Child.Children.AddDefaulted_GetRef();
					AddedNode.Prefix = ToAdd.RightChop(CharIdx);
					check(AddedNode.Prefix.Len() > 0);

					AddedNode.IsLeaf = true;
				
					Child.IsLeaf = false;
				}
				else
				{
					// this was a full match, eg. Child = "Foobar", ToAdd = "Foo"
					// this is a leaf because this represents ToAdd now
					Child.IsLeaf = true;
				}
			}
				
			break;
		}
	}

	if (!bAdded)
	{
		FNode& NewChild = Children.AddDefaulted_GetRef();
		NewChild.Prefix = ToAdd;
		check(NewChild.Prefix.Len() > 0);

		NewChild.IsLeaf = true;
	}
}

/** 
	* Remove an entry. This can fragment the tree and should be avoided.
	*/
void FStringPrefixTree::FNode::Remove(FStringView ToRemove)
{
	for (int32 Idx = 0; Idx < Children.Num(); ++Idx)
	{
		FNode& Child = Children[Idx];

		if (ToRemove.StartsWith(Child.Prefix))
		{
			if (ToRemove.Len() == Child.Prefix.Len())
			{
				// matched the entire string
				Child.IsLeaf = false;
			}
			else
			{
				Child.Remove(ToRemove.RightChop(Child.Prefix.Len()));
			}

			if (Child.CanPrune())
			{
				Children.RemoveAt(Idx);
			}

			break;
		}
	}
}

bool FStringPrefixTree::FNode::Contains(FStringView ToFind) const
{
	for (const FNode& Child : Children)
	{
		if (ToFind.StartsWith(Child.Prefix))
		{
			if (Child.Prefix.Len() == ToFind.Len())
			{
				// matched the entire string
				return Child.IsLeaf;
			}

			return Child.Contains(ToFind.RightChop(Child.Prefix.Len()));
		}
	}

	return false;
}

bool FStringPrefixTree::FNode::AnyStartsWith(FStringView ToFind) const
{
	for (const FNode& Child : Children)
	{
		int32 CharIdx = GetCommonPrefixLength(Child.Prefix, ToFind);
		if (CharIdx > 0)
		{
			if (CharIdx <= Child.Prefix.Len())
			{
				// identical strings or child is longer than the prefix we're searching for
				return true;
			}
			
			return Child.AnyStartsWith(ToFind.RightChop(Child.Prefix.Len()));
		}

	}

	return false;
}

void FStringPrefixTree::FNode::GetAllEntries(const FString& CurrentPrefix, TArray<FString>& OutEntries) const 
{
	FString NewPrefix = CurrentPrefix + Prefix;

	if (IsLeaf)
	{
		OutEntries.Add(NewPrefix);
	}

	for (const FNode& Child : Children)
	{
		Child.GetAllEntries(NewPrefix, OutEntries);
	}
}

int32 FStringPrefixTree::FNode::Size() const 
{
	int32 Result = 0;
	for (const FNode& Child : Children)
	{
		// only leaf children count
		if (Child.IsLeaf)
		{
			++Result;
		}
		Result += Child.Size();
	}
	return Result;
}

int32 FStringPrefixTree::FNode::NumNodes() const
{
	int32 Result = 1;
	for (const FNode& Child : Children)
	{
		Result += Child.NumNodes();
	}
	return Result;
}

void FStringPrefixTree::FNode::DumpToString(FStringBuilderBase& Builder, TArray<bool, TInlineAllocator<32>>& DrawLines, bool bLastChild) const
{
	for (int32 Level = 0; Level < DrawLines.Num(); ++Level)
	{
		if (DrawLines[Level])
		{
			Builder += TEXT("|");
		}
		else
		{
			Builder += TEXT(" ");
		}
		
		if (Level != DrawLines.Num() - 1)
		{
			Builder += TEXT(" ");
		}
	}

	if (bLastChild)
	{
		DrawLines.Last() = false;
	}

	if (Prefix.IsEmpty())
	{
		Builder += TEXT("\"\"");
	}
	else
	{
		Builder += TEXT("- ");
		Builder += Prefix;
	}

	if (IsLeaf)
	{
		Builder += TEXT("*");
	}

	Builder += TEXT("\n");

	for (int32 Idx = 0; Idx < Children.Num(); ++Idx)
	{
		DrawLines.Push(true);
		Children[Idx].DumpToString(Builder, DrawLines, Idx == Children.Num() - 1);
		DrawLines.Pop();
	}
}

// we can prune any node that is not a leaf and does not contain any children which are leaves anymore
bool FStringPrefixTree::FNode::CanPrune() const 
{
	if (IsLeaf)
	{
		return false;
	}

	for (const FNode& Child : Children)
	{
		if (!Child.CanPrune())
		{
			return false;
		}
	}

	return true;
}
