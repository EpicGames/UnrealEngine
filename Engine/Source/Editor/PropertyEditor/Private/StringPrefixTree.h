// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Misc/StringBuilder.h"

/**
 * A string prefix tree is conceptually similar to a trie.
 * However, instead of adding a new nodes per single character, nodes encode a string prefix, and entries are built up by concatenating the path to the node.
 * For example, if we add the entries "ABC", "AC", and "D" to the tree, we will have:
 *   <root>
 *   |- "A"
 *   | |- "B"
 *   | | |- "C" *
 *   | |- "C" *
 *   |- "D" *
 * 
 * Where only the nodes marked with * are leaf nodes. 
 * An entry is considered to be in the list only if it is a leaf node.
 * This tree structure allows for long strings that may have long similar sections to be searched far faster than storing them in eg. a set.
 */
class FStringPrefixTree
{
public:

	FStringPrefixTree() = default;
	~FStringPrefixTree();

	/**
	 * Insert the given string into the tree.
	 */
	void Insert(FStringView ToAdd);

	/**
	 * Insert all the strings in the array.
	 */
	void InsertAll(const TArray<FString>& Array);

	/**
	 * Insert all the strings in the array.
	 */
	void InsertAll(const TArray<FStringView>& Array);

	/** 
	 * Remove an entry. This can fragment the tree and should be avoided.
	 */
	void Remove(FStringView ToRemove);

	/**
	 * Clear the tree.
	 */
	void Clear();

	/**
	 * Does the tree contain the given entry?
	 */
	bool Contains(FStringView Entry) const;

	/**
	 * Does any entry start with the given prefix.
	 * Eg. if the tree contains "Foobar", this would return true if given "Foo".
	 */
	bool AnyStartsWith(FStringView Prefix) const;

	/**
	 * Returns the number of leaf entries in the tree.
	 * Equivalent to the number of items added via Insert() or InsertAll().
	 */
	int32 Size() const;

	/**
	 * Dump all leaf nodes into an array.
	 */
	TArray<FString> GetAllEntries() const;

	/**
	 * For testing and debugging.
	 * The number of internal nodes. This is a different (usually larger) number than the number of entries.
	 */
	int32 NumNodes() const;
	
	/**
	 * For testing and debugging.
	 * Pretty prints the internal tree structure as a string.
	 */
	FString DumpToString() const;

private:

	struct FNode
	{
		FString Prefix;
		TArray<FNode> Children;
		bool IsLeaf = false;

		FNode() = default;
		~FNode();

		void Insert(FStringView ToAdd);
		void Remove(FStringView ToRemove);
		bool Contains(FStringView ToFind) const;
		bool AnyStartsWith(FStringView ToFind) const;
		int32 Size() const;

		int32 NumNodes() const;
		void GetAllEntries(const FString& CurrentPrefix, TArray<FString>& OutEntries) const;
		void DumpToString(FStringBuilderBase& Builder, TArray<bool, TInlineAllocator<32>>& DrawLines, bool bLastChild) const;

	private:
		// we can prune any node that is not a leaf and does not contain any children which are leaves anymore
		bool CanPrune() const;
	};

	FNode Root;
};