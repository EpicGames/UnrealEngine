// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "Misc/AutomationTest.h"
#include "StringPrefixTree.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStringPrefixTreeTests_Insert, "StringPrefixTree.Insert", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FStringPrefixTreeTests_Insert::RunTest(const FString& Parameters)
{
	FStringPrefixTree Tree;

	Tree.Insert(TEXT("Foobar"));

	TestEqual(TEXT("One"), Tree.Size(), 1);
	TestEqual(TEXT("One"), Tree.NumNodes(), 2);
	TestTrue(TEXT("One"), Tree.Contains(TEXT("Foobar")));


	Tree.Insert(TEXT("Foo"));

	TestEqual(TEXT("Two"), Tree.Size(), 2);
	TestEqual(TEXT("Two"), Tree.NumNodes(), 3);
	TestTrue(TEXT("Two"), Tree.Contains(TEXT("Foo")));
	TestTrue(TEXT("Two"), Tree.Contains(TEXT("Foobar")));
	TestFalse(TEXT("Two"), Tree.Contains(TEXT("Fo")));
	TestFalse(TEXT("Two"), Tree.Contains(TEXT("Foob")));

	// check that we don't add nodes if we add another identical entry
	Tree.Insert(TEXT("Foo"));
	TestEqual(TEXT("Two"), Tree.Size(), 2);
	TestEqual(TEXT("Two"), Tree.NumNodes(), 3);

	// check that we don't add nodes if we add another identical entry
	Tree.Insert(TEXT("Foobar"));
	TestEqual(TEXT("Two"), Tree.Size(), 2);
	TestEqual(TEXT("Two"), Tree.NumNodes(), 3);

	// check that the same holds if we build the tree in the opposite way
	// ie. Foo, Foobar
	Tree.Clear();

	TestEqual(TEXT("Cleared"), Tree.Size(), 0);

	Tree.Insert(TEXT("Foo"));

	TestEqual(TEXT("One"), Tree.Size(), 1);
	TestTrue(TEXT("One"), Tree.Contains(TEXT("Foo")));

	Tree.Insert(TEXT("Foobar"));

	TestEqual(TEXT("Two"), Tree.Size(), 2);
	TestTrue(TEXT("Two"), Tree.Contains(TEXT("Foo")));
	TestTrue(TEXT("Two"), Tree.Contains(TEXT("Foobar")));
	TestFalse(TEXT("Two"), Tree.Contains(TEXT("Fo")));
	TestFalse(TEXT("Two"), Tree.Contains(TEXT("Foob")));

	Tree.Insert(TEXT("Foobaz"));
	
	TestEqual(TEXT("Three"), Tree.Size(), 3);
	TestTrue(TEXT("Three"), Tree.Contains(TEXT("Foobar")));
	TestTrue(TEXT("Three"), Tree.Contains(TEXT("Foobaz")));
	TestFalse(TEXT("Three"), Tree.Contains(TEXT("Fooba")));

	Tree.Insert(TEXT("Foobar.Foo"));
	
	TestEqual(TEXT("Four"), Tree.Size(), 4);
	TestEqual(TEXT("Four"), Tree.NumNodes(), 6);
	TestTrue(TEXT("Four"), Tree.Contains(TEXT("Foobar.Foo")));
	TestTrue(TEXT("Four"), Tree.Contains(TEXT("Foobaz")));
	TestTrue(TEXT("Four"), Tree.Contains(TEXT("Foobar")));
	TestFalse(TEXT("Four"), Tree.Contains(TEXT("Foobar.")));

	Tree.Insert(TEXT("Foobar"));
	
	TestEqual(TEXT("Four"), Tree.Size(), 4);
	TestEqual(TEXT("Four"), Tree.NumNodes(), 6);

	Tree.Insert(TEXT("Bar"));
	Tree.Insert(TEXT("Bar.Foo"));

	TestEqual(TEXT("Six"), Tree.Size(), 6);
	TestTrue(TEXT("Six"), Tree.Contains(TEXT("Bar.Foo")));
	TestTrue(TEXT("Six"), Tree.Contains(TEXT("Bar")));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStringPrefixTreeTests_Fuzz, "StringPrefixTree.Fuzz", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FStringPrefixTreeTests_Fuzz::RunTest(const FString& Parameters)
{
	const int32 MAX_STRINGS = 1000;

	TArray<FString> Added;
	Added.Reserve(MAX_STRINGS);

	FStringPrefixTree Tree;

	FString Current;
	Current.Reserve(MAX_STRINGS);

	for (int32 Idx = 0; Idx < MAX_STRINGS; ++Idx)
	{
		TCHAR ToAdd = TEXT('A') + (TCHAR) FMath::RandRange(0, 25);
		
		int32 Where = FMath::RandRange(0, Current.Len());
		Current.InsertAt(Where, ToAdd);

		Tree.Insert(Current);
		Added.Add(Current);

		TestEqual(TEXT("Size"), Tree.Size(), Added.Num());
	}

	for (const FString& Str : Added)
	{
		TestTrue(TEXT("Contains"), Tree.Contains(Str));
	}
	
	TArray<FString> Actual = Tree.GetAllEntries();
	TestEqual(TEXT("GetAllEntries"), Actual.Num(), Tree.Size());

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStringPrefixTreeTests_Contains, "StringPrefixTree.Contains", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FStringPrefixTreeTests_Contains::RunTest(const FString& Parameters)
{
	FStringPrefixTree Tree;

	TestFalse(TEXT("Zero"), Tree.Contains(TEXT("")));

	Tree.Insert(TEXT("Foobar"));

	TestEqual(TEXT("One"), Tree.Size(), 1);

	auto ContainsMatch = [](const FStringPrefixTree& Tree, const FString& Value)
	{
		TArray<FString> Entries = Tree.GetAllEntries();
		return Tree.Contains(Value) == Entries.Contains(Value);
	};

	TestTrue(TEXT("One"), Tree.Contains(TEXT("Foobar")));

	Tree.Insert(TEXT("Foo"));

	TestEqual(TEXT("Two"), Tree.Size(), 2);
	TestTrue(TEXT("Two"), ContainsMatch(Tree, TEXT("Foo")));
	TestTrue(TEXT("Two"), ContainsMatch(Tree, TEXT("Foobar")));
	TestTrue(TEXT("Two"), ContainsMatch(Tree, TEXT("Fo")));
	TestTrue(TEXT("Two"), ContainsMatch(Tree, TEXT("Foob")));

	// check that the same holds if we build the tree in the opposite way
	// ie. Foo, Foobar
	Tree.Clear();

	TestEqual(TEXT("Cleared"), Tree.Size(), 0);

	Tree.Insert(TEXT("Foo"));

	TestEqual(TEXT("One"), Tree.Size(), 1);
	TestTrue(TEXT("One"), ContainsMatch(Tree, TEXT("Foo")));

	Tree.Insert(TEXT("Foobar"));

	TestEqual(TEXT("Two"), Tree.Size(), 2);
	TestTrue(TEXT("Two"), ContainsMatch(Tree, TEXT("Foo")));
	TestTrue(TEXT("Two"), ContainsMatch(Tree, TEXT("Foobar")));
	TestTrue(TEXT("Two"), ContainsMatch(Tree, TEXT("Fo")));
	TestTrue(TEXT("Two"), ContainsMatch(Tree, TEXT("Foob")));

	Tree.Insert(TEXT("Foobaz"));
	
	TestEqual(TEXT("Two"), Tree.Size(), 3);
	TestTrue(TEXT("Three"), ContainsMatch(Tree, TEXT("Foobar")));
	TestTrue(TEXT("Three"), ContainsMatch(Tree, TEXT("Foobaz")));

	Tree.Insert(TEXT("Foobar.Foo"));
	
	TestEqual(TEXT("Four"), Tree.Size(), 4);
	TestTrue(TEXT("Four"), ContainsMatch(Tree, TEXT("Foobar.Foo")));
	TestTrue(TEXT("Four"), ContainsMatch(Tree, TEXT("Foobaz")));
	TestTrue(TEXT("Four"), ContainsMatch(Tree, TEXT("Foobar")));
	TestTrue(TEXT("Four"), ContainsMatch(Tree, TEXT("Foobar.")));

	Tree.Insert(TEXT(""));

	TestTrue(TEXT("Five"), Tree.Contains(TEXT("")));
	TestTrue(TEXT("Five"), ContainsMatch(Tree, TEXT("")));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStringPrefixTreeTests_AnyStartsWith, "StringPrefixTree.AnyStartsWith", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FStringPrefixTreeTests_AnyStartsWith::RunTest(const FString& Parameters)
{
	FStringPrefixTree Tree;
	TestFalse(TEXT("Zero"), Tree.AnyStartsWith(TEXT("Foo")));
	TestFalse(TEXT("Zero"), Tree.AnyStartsWith(TEXT("")));

	Tree.Insert(TEXT("Foobar"));

	TestTrue(TEXT("One"), Tree.AnyStartsWith(TEXT("Foo")));
	TestTrue(TEXT("One"), Tree.AnyStartsWith(TEXT("Foobar")));
	TestTrue(TEXT("One"), Tree.AnyStartsWith(TEXT("F")));

	TestFalse(TEXT("One"), Tree.AnyStartsWith(TEXT("b")));
	TestFalse(TEXT("One"), Tree.AnyStartsWith(TEXT("bar")));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStringPrefixTreeTests_Remove, "StringPrefixTree.Remove", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FStringPrefixTreeTests_Remove::RunTest(const FString& Parameters)
{
	FStringPrefixTree Tree;

	TestFalse(TEXT("Zero"), Tree.Contains(TEXT("")));

	Tree.Insert(TEXT("Foobar"));
	Tree.Remove(TEXT("Foobar"));
	
	TestFalse(TEXT("Zero"), Tree.Contains(TEXT("Foobar")));
	TestEqual(TEXT("Zero"), Tree.NumNodes(), 1);
	TestEqual(TEXT("Zero"), Tree.Size(), 0);

	Tree.Insert(TEXT("Foo"));
	Tree.Remove(TEXT("Foo"));

	TestEqual(TEXT("Zero"), Tree.NumNodes(), 1);
	TestEqual(TEXT("Zero"), Tree.Size(), 0);

	Tree.Insert(TEXT("Foobar"));
	Tree.Insert(TEXT("Foo"));

	Tree.Remove(TEXT("Foo"));
	TestFalse(TEXT("Remove One"), Tree.Contains(TEXT("Foo")));
	TestTrue(TEXT("Remove One"), Tree.Contains(TEXT("Foobar")));
	TestEqual(TEXT("Remove One"), Tree.NumNodes(), 3);
	TestEqual(TEXT("Remove One"), Tree.Size(), 1);

	Tree.Remove(TEXT("Foobar"));

	TestFalse(TEXT("Remove Two"), Tree.Contains(TEXT("Foo")));
	TestEqual(TEXT("Remove Two"), Tree.NumNodes(), 1);
	TestEqual(TEXT("Remove Two"), Tree.Size(), 0);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStringPrefixTreeTests_DumpToString, "StringPrefixTree.DumpToString", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FStringPrefixTreeTests_DumpToString::RunTest(const FString& Parameters)
{
	FStringPrefixTree Tree;
	
	Tree.Insert(TEXT("A"));
	Tree.Insert(TEXT("A.1"));
	Tree.Insert(TEXT("A.1.Z"));
	Tree.Insert(TEXT("A.2.Y"));
	Tree.Insert(TEXT("A.2.X"));
	Tree.Insert(TEXT("B.3"));
	Tree.Insert(TEXT("B.3.W"));
	Tree.Insert(TEXT("C"));

	const FString Dumped = Tree.DumpToString();

	TArray<FString> Lines;
	Dumped.ParseIntoArrayLines(Lines);
	TestEqual(TEXT("Lines"), Lines.Num(), Tree.NumNodes());

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
