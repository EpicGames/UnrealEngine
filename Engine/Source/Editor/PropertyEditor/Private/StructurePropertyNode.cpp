// Copyright Epic Games, Inc. All Rights Reserved.


#include "StructurePropertyNode.h"
#include "ItemPropertyNode.h"
#include "PropertyEditorHelpers.h"

void FStructurePropertyNode::InitChildNodes()
{
	InternalInitChildNodes(FName());
}

void FStructurePropertyNode::InternalInitChildNodes(FName SinglePropertyName)
{
	const bool bShouldShowHiddenProperties = !!HasNodeFlags(EPropertyNodeFlags::ShouldShowHiddenProperties);
	const bool bShouldShowDisableEditOnInstance = !!HasNodeFlags(EPropertyNodeFlags::ShouldShowDisableEditOnInstance);

	const UStruct* Struct = GetBaseStructure();

	TArray<FProperty*> StructMembers;

	for (TFieldIterator<FProperty> It(Struct); It; ++It)
	{
		FProperty* StructMember = *It;
		if (PropertyEditorHelpers::ShouldBeVisible(*this, StructMember))
		{
			if (SinglePropertyName == NAME_None || StructMember->GetFName() == SinglePropertyName)
			{
				StructMembers.Add(StructMember);
				if (SinglePropertyName != NAME_None)
				{
					break;
				}
			}
		}
	}

	// Cache the init time base struct so that we can determine if the struct has changed.
	// Store the cached base struct before calling AddChildNode() as they may call back to this node.
	WeakCachedBaseStruct = Struct;

	PropertyEditorHelpers::OrderPropertiesFromMetadata(StructMembers);

	for (FProperty* StructMember : StructMembers)
	{
		TSharedPtr<FItemPropertyNode> NewItemNode(new FItemPropertyNode);

		FPropertyNodeInitParams InitParams;
		InitParams.ParentNode = SharedThis(this);
		InitParams.Property = StructMember;
		InitParams.ArrayOffset = 0;
		InitParams.ArrayIndex = INDEX_NONE;
		InitParams.bAllowChildren = SinglePropertyName == NAME_None;
		InitParams.bForceHiddenPropertyVisibility = bShouldShowHiddenProperties;
		InitParams.bCreateDisableEditOnInstanceNodes = bShouldShowDisableEditOnInstance;
		InitParams.bCreateCategoryNodes = false;

		NewItemNode->InitNode(InitParams);
		AddChildNode(NewItemNode);
	}
}

void FStructurePropertyNode::InitBeforeNodeFlags()
{
	// Cache the base struct. It is used to check if the struct changes later.
	// The struct will be cached on each call to InternalInitChildNodes() as well.
	// We'll cache it here too, so that a FStructurePropertyNode which is initialized
	// with "InitParams.bAllowChildren = false" has it properly set up.
	WeakCachedBaseStruct = GetBaseStructure();
}

bool FStructurePropertyNode::GetReadAddressUncached(const FPropertyNode& InPropertyNode, FReadAddressListData& OutAddresses) const
{
	if (!HasValidStructData())
	{
		return false;
	}
	check(StructProvider.IsValid());

	const FProperty* InItemProperty = InPropertyNode.GetProperty();
	if (!InItemProperty)
	{
		return false;
	}

	UStruct* OwnerStruct = InItemProperty->GetOwnerStruct();
	if (!OwnerStruct || OwnerStruct->IsStructTrashed())
	{
		// Verify that the property is not part of an invalid trash class
		return false;
	}

	TArray<TSharedPtr<FStructOnScope>> Instances;
	StructProvider->GetInstances(Instances, WeakCachedBaseStruct.Get());
	bool bHasData = false;

	for (TSharedPtr<FStructOnScope>& Instance : Instances)
	{
		uint8* ReadAddress = Instance.IsValid() ? Instance->GetStructMemory() : nullptr;
		if (ReadAddress)
		{
			OutAddresses.Add(nullptr, InPropertyNode.GetValueBaseAddress(ReadAddress, InPropertyNode.HasNodeFlags(EPropertyNodeFlags::IsSparseClassData) != 0, /*bIsStruct=*/true), /*bIsStruct=*/true);
			bHasData = true;
		}
	}
	return bHasData;
}

bool FStructurePropertyNode::GetReadAddressUncached(const FPropertyNode& InPropertyNode,
	bool InRequiresSingleSelection,
	FReadAddressListData* OutAddresses,
	bool bComparePropertyContents,
	bool bObjectForceCompare,
	bool bArrayPropertiesCanDifferInSize) const
{
	if (!HasValidStructData())
	{
		return false;
	}
	check(StructProvider.IsValid());

	const FProperty* InItemProperty = InPropertyNode.GetProperty();
	if (!InItemProperty)
	{
		return false;
	}

	const UStruct* OwnerStruct = InItemProperty->GetOwnerStruct();
	if (!OwnerStruct || OwnerStruct->IsStructTrashed())
	{
		// Verify that the property is not part of an invalid trash class
		return false;
	}

	bool bAllTheSame = true;

	TArray<TSharedPtr<FStructOnScope>> Instances;
	StructProvider->GetInstances(Instances, WeakCachedBaseStruct.Get());
	
	if (Instances.IsEmpty())
	{
		return false;
	}

	if (bComparePropertyContents || bObjectForceCompare)
	{
		const bool bIsSparse = InPropertyNode.HasNodeFlags(EPropertyNodeFlags::IsSparseClassData) != 0;
		const uint8* BaseAddress = nullptr;
		const UStruct* BaseStruct = nullptr;

		for (TSharedPtr<FStructOnScope>& Instance : Instances)
		{
			if (Instance.IsValid())
			{
				if (const UStruct* Struct = Instance->GetStruct())
				{
					if (const uint8* ReadAddress = InPropertyNode.GetValueBaseAddress(Instance->GetStructMemory(), bIsSparse, /*bIsStruct=*/true))
					{
						if (!BaseAddress)
						{
							BaseAddress = ReadAddress;
							BaseStruct = Struct;
						}
						else
						{
							if (BaseStruct != Struct)
							{
								bAllTheSame = false;
								break;
							}
							if (!InItemProperty->Identical(BaseAddress, ReadAddress))
							{
								bAllTheSame = false;
								break;
							}
						}
					}
				}
			}
		}

		// If none of the instances have data, treat it as if the instance data was empty.
		if (!BaseStruct)
		{
			bAllTheSame = false;
		}
	}
	else
	{
		// Check that all are valid or invalid.
		const UStruct* BaseStruct = Instances[0].IsValid() ? Instances[0]->GetStruct() : nullptr;
		for (int32 Index = 1; Index < Instances.Num(); Index++)
		{
			const UStruct* Struct = Instances[Index].IsValid() ? Instances[Index]->GetStruct() : nullptr;
			if (BaseStruct != Struct)
			{
				bAllTheSame = false;
				break;
			}
		}
		
		// If none of the instances have data, treat it as if the instance data was empty.
		if (!BaseStruct)
		{
			bAllTheSame = false;
		}
	}

	if (OutAddresses)
	{
		for (TSharedPtr<FStructOnScope>& Instance : Instances)
		{
			uint8* ReadAddress = Instance.IsValid() ? Instance->GetStructMemory() : nullptr;
			if (ReadAddress)
			{
				OutAddresses->Add(nullptr, InPropertyNode.GetValueBaseAddress(ReadAddress, InPropertyNode.HasNodeFlags(EPropertyNodeFlags::IsSparseClassData) != 0, /*bIsStruct=*/true), /*bIsStruct=*/true);
			}
		}
	}

	return bAllTheSame;
}
	
uint8* FStructurePropertyNode::GetValueBaseAddress(uint8* StartAddress, bool bIsSparseData, bool bIsStruct) const
{
	// If called with struct data, we expect that it is compatible with the first structure node down in the property node chain, return the address as is.
	// This gets called usually when the calling code is dealing with a parent complex node.
	if (bIsStruct)
	{
		return StartAddress;
	}

	if (StructProvider)
	{
		// Assume that this code gets called with an object or object sparse data.
		//
		// The passed object might not be the one that contains the values provided by struct provider.
		// For example this function might get called on an edited object's template object.
		// In that case the data structure is expected to match between the data edited by this node and the foreign object.

		// If the structure provider is set up as indirection, then it knows how to translate parent node's value address to
		// new value address even on data that is not the same as in the structure provider.
		if (StructProvider->IsPropertyIndirection())
		{
			const TSharedPtr<FPropertyNode> ParentNode = ParentNodeWeakPtr.Pin();
			if (!ensureMsgf(ParentNode, TEXT("Expecting valid parent node when indirection structure provider is called with Object data.")))
			{
				return nullptr;
			}
			// Resolve from parent nodes data.
			uint8* ParentValueAddress = ParentNode->GetValueAddress(StartAddress, bIsSparseData);
			uint8* ValueAddress = StructProvider->GetValueBaseAddress(ParentValueAddress, WeakCachedBaseStruct.Get());
			return ValueAddress;
		}

		// The struct is really standalone, in which case we always return the standalone struct data.
		// In that case we can only support one instance, since we cannot discern them.
		// Note: Multiple standalone structure instances are supported when bIsStruct is true (e.g. when the structure property is root node).
		TArray<TSharedPtr<FStructOnScope>> Instances;
		StructProvider->GetInstances(Instances, WeakCachedBaseStruct.Get());
		ensureMsgf(Instances.Num() <= 1, TEXT("Expecting max one instance on standalone structure provider."));
		if (Instances.Num() == 1 && Instances[0].IsValid())
		{
			return Instances[0]->GetStructMemory();
		}
	}
	
	return nullptr;
}

TSharedPtr<FPropertyNode> FStructurePropertyNode::GenerateSingleChild(FName ChildPropertyName)
{
	constexpr bool bDestroySelf = false;
	DestroyTree(bDestroySelf);

	// No category nodes should be created in single property mode
	SetNodeFlags(EPropertyNodeFlags::ShowCategories, false);

	InternalInitChildNodes(ChildPropertyName);

	if (ChildNodes.Num() > 0)
	{
		// only one node should be been created
		check(ChildNodes.Num() == 1);

		return ChildNodes[0];
	}

	return nullptr;
}

EPropertyDataValidationResult FStructurePropertyNode::EnsureDataIsValid()
{
	CachedReadAddresses.Reset();

	// If the struct has changed, rebuild children.
	const UStruct* CachedBaseStruct = WeakCachedBaseStruct.Get();
	if (GetBaseStructure() != CachedBaseStruct)
	{
		RebuildChildren();
		// Invalidate the object to prevent EditCondition evaluating on old nodes.
		return EPropertyDataValidationResult::ObjectInvalid;
	}
	
	return FPropertyNode::EnsureDataIsValid();
}
