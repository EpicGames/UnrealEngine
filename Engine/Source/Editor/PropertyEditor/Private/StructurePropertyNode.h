// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PropertyNode.h"
#include "UObject/StructOnScope.h"
#include "IStructureDataProvider.h"
#include "ObjectPropertyNode.h"

//-----------------------------------------------------------------------------
//	FStructPropertyNode - Used for the root and various sub-nodes
//-----------------------------------------------------------------------------

class FStructurePropertyNode : public FComplexPropertyNode
{
public:
	FStructurePropertyNode() : FComplexPropertyNode() {}
	virtual ~FStructurePropertyNode() override = default;

	virtual FStructurePropertyNode* AsStructureNode() override { return this; }
	virtual const FStructurePropertyNode* AsStructureNode() const override { return this; }

	void RemoveStructure(bool bInDestroySelf = true)
	{
		ClearCachedReadAddresses(true);
		DestroyTree(bInDestroySelf);
		StructProvider = nullptr;
		WeakCachedBaseStruct.Reset();
	}

	void SetStructure(TSharedPtr<FStructOnScope> InStructData)
	{
		RemoveStructure(false);
		if (InStructData)
		{
			StructProvider = MakeShared<FStructOnScopeStructureDataProvider>(InStructData);
		}
	}

	void SetStructure(TSharedPtr<IStructureDataProvider> InStructProvider)
	{
		RemoveStructure(false);
		StructProvider = InStructProvider;
	}

	bool HasValidStructData() const
	{
		return StructProvider.IsValid() && StructProvider->IsValid();
	}

	// Returns just the first structure. Please use GetStructProvider() or GetAllStructureData() when dealing with multiple struct instances.
	TSharedPtr<FStructOnScope> GetStructData() const
	{
		if (StructProvider)
		{
			TArray<TSharedPtr<FStructOnScope>> Instances;
			StructProvider->GetInstances(Instances, WeakCachedBaseStruct.Get());
			
			if (Instances.Num() > 0)
			{
				return Instances[0];
			}
		}
		return nullptr;
	}

	void GetAllStructureData(TArray<TSharedPtr<FStructOnScope>>& OutStructs) const
	{
		if (StructProvider)
		{
			StructProvider->GetInstances(OutStructs, WeakCachedBaseStruct.Get());
		}
	}

	TSharedPtr<IStructureDataProvider> GetStructProvider() const
	{
		return StructProvider;
	}

	virtual bool GetReadAddressUncached(const FPropertyNode& InPropertyNode, FReadAddressListData& OutAddresses) const override;

	virtual bool GetReadAddressUncached(const FPropertyNode& InPropertyNode,
		bool InRequiresSingleSelection,
		FReadAddressListData* OutAddresses,
		bool bComparePropertyContents,
		bool bObjectForceCompare,
		bool bArrayPropertiesCanDifferInSize) const override;

	void GetOwnerPackages(TArray<UPackage*>& OutPackages) const
	{
		if (StructProvider)
		{
			// Walk up until we find the objects that contain this struct property to get the packages
			const FPropertyNode* Parent = this;
		
			while(Parent)
			{
				const FComplexPropertyNode* ComplexParent = Parent->FindComplexParent();
				if (!ensureMsgf(ComplexParent != nullptr, TEXT("Expected to find a complex parent")))
				{
					return;
				}

				if (const FObjectPropertyNode* ObjectNode = ComplexParent->AsObjectNode())
				{
					for (int32 ObjectIndex = 0, ObjectCount = ObjectNode->GetNumObjects(); ObjectIndex < ObjectCount; ++ObjectIndex)
					{
						const UPackage* Package = ObjectNode->GetUPackage(ObjectIndex);
						OutPackages.Add(const_cast<UPackage*>(Package));
					}
					break;
				}

				if (const FStructurePropertyNode* StructNode = ComplexParent->AsStructureNode())
				{
					if (StructNode->StructProvider && StructNode->StructProvider->IsPropertyIndirection())
					{
						// Skip this and keep walking up to the next complex parent. This is assumed to be a struct that is pointed
						// to be a parent indirection.
						// Note: in InstancedStruct case, will re-enter GetOwnerPackages during EnumerateInstances if calling GetInstances here.
						// We want to avoid that otherwise it can cause poor performance as recursion branches into multiple recursions on each level.
						Parent = ComplexParent->GetParentNode();
						Parent = Parent ? Parent->FindComplexParent() : nullptr;
						if (Parent == nullptr)
						{
							// If no owning property, then there is no package.
							// Returning null to match instance count.
							OutPackages.Add(nullptr);
							break;
						}
					}
					else
					{
						if (StructNode->StructProvider)
						{
							TArray<TSharedPtr<FStructOnScope>> Instances;
							StructNode->StructProvider->GetInstances(Instances, StructNode->WeakCachedBaseStruct.Get());
							for (int32 Index = 0, ObjectCount = Instances.Num(); Index < ObjectCount; ++Index)
							{
								UPackage* Package = Instances[Index]->GetPackage();
								OutPackages.Add(Package);
							}
						}
						break;
					}
				}
			}
		}
	}

	/** FComplexPropertyNode Interface */
	virtual const UStruct* GetBaseStructure() const override
	{ 
		if (StructProvider)
		{
			return StructProvider->GetBaseStructure();
		}
		return nullptr; 
	}
	virtual UStruct* GetBaseStructure() override
	{
		if (StructProvider)
		{
			return const_cast<UStruct*>(StructProvider->GetBaseStructure());
		}
		return nullptr; 
	}
	virtual TArray<UStruct*> GetAllStructures() override
	{
		TArray<UStruct*> RetVal;
		if (StructProvider)
		{
			TArray<TSharedPtr<FStructOnScope>> Instances;
			StructProvider->GetInstances(Instances, WeakCachedBaseStruct.Get());
			
			for (TSharedPtr<FStructOnScope>& Instance : Instances)
			{
				const UStruct* Struct = Instance.IsValid() ? Instance->GetStruct() : nullptr;
				if (Struct)
				{
					RetVal.AddUnique(const_cast<UStruct*>(Struct));
				}
			}
		}

		return RetVal;
	}
	virtual TArray<const UStruct*> GetAllStructures() const override
	{
		TArray<const UStruct*> RetVal;
		if (StructProvider)
		{
			TArray<TSharedPtr<FStructOnScope>> Instances;
			StructProvider->GetInstances(Instances, WeakCachedBaseStruct.Get());
			
			for (TSharedPtr<FStructOnScope>& Instance : Instances)
			{
				const UStruct* Struct = Instance.IsValid() ? Instance->GetStruct() : nullptr;
				if (Struct)
				{
					RetVal.AddUnique(Struct);
				}
			}
		}
		return RetVal;
	}
	virtual int32 GetInstancesNum() const override
	{
		// Can't get instance count directly from standalone structures, need to walk to the next parent that is an object
		// to get number of instances from that
		// Note: This uses iteration over StructProvider->GetInstances and counting returned instances since InstancedStructProvider
		// calls GetInstancesNum from within it's implementation of GetInstances, leading to exponential recursion up the
		// property tree.
		const FComplexPropertyNode* CurrentNode = this;	
	
		while(CurrentNode)
		{
			if (const FObjectPropertyNode* ObjectNode = CurrentNode->AsObjectNode())
			{
				// Found owning UObject
				const int32 ObjectCount = ObjectNode->GetInstancesNum();
				return ObjectCount;
			}
			else if (const FStructurePropertyNode* StructNode = CurrentNode->AsStructureNode())
			{
				if (TSharedPtr<IStructureDataProvider> TempStructProvider = StructNode->GetStructProvider())
				{
					if (TempStructProvider->IsPropertyIndirection())
					{
						// If the struct provider is marked as property indirection, it is assumed that it handles indirection between it's
						// parent property, and some data inside that property (e.g. FInstancedStruct).
						const FPropertyNode* ParentNode = CurrentNode->GetParentNode();
						CurrentNode = ParentNode->FindComplexParent();
					}
					else
					{
						TArray<TSharedPtr<FStructOnScope>> Instances;
						TempStructProvider->GetInstances(Instances, StructNode->WeakCachedBaseStruct.Get());
						return Instances.Num();
					}
				}
				else
				{
					return 0;
				}
			}
		}
		return 0;
		
	}
	virtual uint8* GetMemoryOfInstance(int32 Index) const override
	{
		if (StructProvider)
		{
			TArray<TSharedPtr<FStructOnScope>> Instances;
			StructProvider->GetInstances(Instances, WeakCachedBaseStruct.Get());
			if (Instances.IsValidIndex(Index) && Instances[Index].IsValid())
			{
				return Instances[Index]->GetStructMemory();
			}
		}
		return nullptr;
	}
	virtual uint8* GetValuePtrOfInstance(int32 Index, const FProperty* InProperty, const FPropertyNode* InParentNode) const override
	{ 
		if (InProperty == nullptr || InParentNode == nullptr)
		{
			return nullptr;
		}

		uint8* StructBaseAddress = GetMemoryOfInstance(Index);
		if (StructBaseAddress == nullptr)
		{
			return nullptr;
		}

		uint8* ParentBaseAddress = InParentNode->GetValueAddress(StructBaseAddress, false, /*bIsStruct=*/true);
		if (ParentBaseAddress == nullptr)
		{
			return nullptr;
		}

		return InProperty->ContainerPtrToValuePtr<uint8>(ParentBaseAddress);
	}

	virtual TWeakObjectPtr<UObject> GetInstanceAsUObject(int32 Index) const override
	{
		return nullptr;
	}
	virtual EPropertyType GetPropertyType() const override
	{
		return EPT_StandaloneStructure;
	}

	virtual void Disconnect() override
	{
		ClearCachedReadAddresses(true);
		DestroyTree();
		StructProvider = nullptr;
	}

	/** Generates a single child from the provided property name.  Any existing children are destroyed */
	virtual TSharedPtr<FPropertyNode> GenerateSingleChild(FName ChildPropertyName) override;

protected:

	virtual EPropertyDataValidationResult EnsureDataIsValid() override;

	/** FPropertyNode interface */
	virtual void InitChildNodes() override;

    virtual void InitBeforeNodeFlags() override;
	void InternalInitChildNodes(FName SinglePropertyName);

	virtual uint8* GetValueBaseAddress(uint8* Base, bool bIsSparseData, bool bIsStruct) const override;

	virtual bool GetQualifiedName(FString& PathPlusIndex, const bool bWithArrayIndex, const FPropertyNode* StopParent = nullptr, bool bIgnoreCategories = false) const override
	{
		bool bAddedAnything = false;
		const TSharedPtr<FPropertyNode> ParentNode = ParentNodeWeakPtr.Pin();
		if (ParentNode && StopParent != ParentNode.Get())
		{
			bAddedAnything = ParentNode->GetQualifiedName(PathPlusIndex, bWithArrayIndex, StopParent, bIgnoreCategories);
		}

		if (bAddedAnything)
		{
			PathPlusIndex += TEXT(".");
		}

		PathPlusIndex += TEXT("Struct");
		bAddedAnything = true;

		return bAddedAnything;
	}

private:
	TSharedPtr<IStructureDataProvider> StructProvider;

	/** The base struct at the time InitChildNodes() was called. */
	TWeakObjectPtr<const UStruct> WeakCachedBaseStruct = nullptr;
};
