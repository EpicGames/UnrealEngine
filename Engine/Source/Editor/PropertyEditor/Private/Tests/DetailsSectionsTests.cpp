// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "GameFramework/Actor.h"
#include "Editor/PropertyEditorTestObject.h"
#include "Modules/ModuleManager.h"
#include "Misc/AutomationTest.h"
#include "PropertyEditorModule.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDetailsViewTests_SectionsBasic, "DetailsView.Sections.Basic", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FDetailsViewTests_SectionsBasic::RunTest(const FString& Parameters)
{
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	const FText DisplayName = NSLOCTEXT("DetailsViewTests", "Test", "Test");
	TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("Object", "Test", DisplayName);

	TestTrue(TEXT("Name"),				Section->GetName().IsEqual("Test"));
	TestTrue(TEXT("Display Name"),		Section->GetDisplayName().CompareTo(DisplayName) == 0);
	
	TestFalse(TEXT("Initialized"),		Section->HasAddedCategory("Test"));
	TestFalse(TEXT("Initialized"),		Section->HasRemovedCategory("Test"));

	Section->AddCategory("Test");
	TestTrue(TEXT("After Added"),		Section->HasAddedCategory("Test"));
	TestFalse(TEXT("After Added"),		Section->HasRemovedCategory("Test"));

	TArray<TSharedPtr<FPropertySection>> Sections = PropertyModule.FindSectionsForCategory(UObject::StaticClass(), "Test");
	if (TestEqual(TEXT("After Added"),	Sections.Num(), 1))
	{
		TestTrue(TEXT("After Added"),	Sections[0]->HasAddedCategory("Test"));
	}

	Section->RemoveCategory("Test");
	TestFalse(TEXT("After Removed"),	Section->HasAddedCategory("Test"));
	TestTrue(TEXT("After Removed"),		Section->HasRemovedCategory("Test"));

	Sections = PropertyModule.FindSectionsForCategory(UObject::StaticClass(), "Test");
	TestEqual(TEXT("After Removed"),	Sections.Num(), 0);

	PropertyModule.RemoveSection("Object", "Test");

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDetailsViewTests_SectionsInherited, "DetailsView.Sections.Inherited", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FDetailsViewTests_SectionsInherited::RunTest(const FString& Parameters)
{
	const FText DisplayName = NSLOCTEXT("DetailsViewTests", "Test", "Test");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedRef<FPropertySection> BaseSection = PropertyModule.FindOrCreateSection("Object", "Test", DisplayName);
	BaseSection->AddCategory("Test");

	TArray<TSharedPtr<FPropertySection>> Sections = PropertyModule.FindSectionsForCategory(UObject::StaticClass(), "Test");
	if (TestEqual(TEXT("After Added"),		Sections.Num(), 1))
	{
		TestTrue(TEXT("After Added"),		Sections[0]->HasAddedCategory("Test"));
	}

	Sections = PropertyModule.FindSectionsForCategory(AActor::StaticClass(), "Test");
	if (TestEqual(TEXT("Inherited"),		Sections.Num(), 1))
	{
		TestTrue(TEXT("Inherited"),		Sections[0]->HasAddedCategory("Test"));
		TestFalse(TEXT("Inherited"),		Sections[0]->HasRemovedCategory("Test"));
	}

	TSharedRef<FPropertySection> InheritedSection = PropertyModule.FindOrCreateSection("Actor", "Test", DisplayName);
	InheritedSection->RemoveCategory("Test");

	Sections = PropertyModule.FindSectionsForCategory(AActor::StaticClass(), "Test");

	TestEqual(TEXT("After Removed"),		Sections.Num(), 0);

	const FText SecondDisplayName = NSLOCTEXT("DetailsViewTests", "Second", "Second");
	TSharedRef<FPropertySection> NewSection = PropertyModule.FindOrCreateSection("Actor", "Second", DisplayName);
	NewSection->AddCategory("Test");

	Sections = PropertyModule.FindSectionsForCategory(AActor::StaticClass(), "Test");

	if (TestEqual(TEXT("Second Section"),	Sections.Num(), 1))
	{
		TestTrue(TEXT("Second Section"),	Sections[0]->GetName() == "Second");
		TestTrue(TEXT("Second Section"),	Sections[0]->HasAddedCategory("Test"));
		TestFalse(TEXT("Second Section"),	Sections[0]->HasRemovedCategory("Test"));
	}

	InheritedSection->AddCategory("Test");

	Sections = PropertyModule.FindSectionsForCategory(AActor::StaticClass(), "Test");

	TestEqual(TEXT("Second Section"),		Sections.Num(), 3); // Object -> Added to Test, Actor -> Added to Test, Actor -> Added to Second
	
	PropertyModule.RemoveSection("Actor", "Test");
	PropertyModule.RemoveSection("Actor", "Second");
	PropertyModule.RemoveSection("Object", "Test");

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS