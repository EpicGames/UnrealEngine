// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "DetailTreeNode.h"
#include "Editor/PropertyEditorTestObject.h"
#include "Misc/AutomationTest.h"
#include "SDetailsView.h"
#include "UObject/UnrealType.h"

namespace UE::PropertyEditor::Tests
{

/** Helper: create a details view displaying a single object */
static TSharedRef<SDetailsView> CreateDetailsView(UObject* InObject)
{
	FDetailsViewArgs Args;
	Args.bAllowSearch = false;
	Args.bShowOptions = false;
	TSharedRef<SDetailsView> View = SNew(SDetailsView, Args);
	View->SetObject(InObject);
	return View;
}

/** Helper: find a head (category) node by name */
static TSharedPtr<FDetailTreeNode> FindHeadNode(SDetailsView& View, FName Name)
{
	TArray<TWeakPtr<FDetailTreeNode>> WeakNodes;
	View.GetHeadNodes(WeakNodes);
	for (const TWeakPtr<FDetailTreeNode>& Weak : WeakNodes)
	{
		if (TSharedPtr<FDetailTreeNode> Node = Weak.Pin())
		{
			if (Node->GetNodeName() == Name)
			{
				return Node;
			}
		}
	}
	return nullptr;
}

/** Helper: find a direct child node by name */
static TSharedPtr<FDetailTreeNode> FindChildNode(FDetailTreeNode& Parent, FName Name)
{
	FDetailNodeList Children;
	const bool bIgnoreVisibility = true;
	Parent.GetChildren(Children, bIgnoreVisibility);
	for (const TSharedRef<FDetailTreeNode>& Child : Children)
	{
		if (Child->GetNodeName() == Name)
		{
			return Child;
		}
	}
	return nullptr;
}

/** Helper: add default elements to an array UPROPERTY by name (avoids accessing private members) */
static void AddArrayElements(UObject* Object, FName PropertyName, int32 Count)
{
	FArrayProperty* ArrayProp = FindFProperty<FArrayProperty>(Object->GetClass(), PropertyName);
	check(ArrayProp);
	FScriptArrayHelper ArrayHelper(ArrayProp, ArrayProp->ContainerPtrToValuePtr<void>(Object));
	ArrayHelper.AddValues(Count);
}

/** Helper: find a child node by index (for array elements) */
static TSharedPtr<FDetailTreeNode> FindChildByIndex(FDetailTreeNode& Parent, int32 Index)
{
	FDetailNodeList Children;
	const bool bIgnoreVisibility = true;
	Parent.GetChildren(Children, bIgnoreVisibility);
	if (Children.IsValidIndex(Index))
	{
		return Children[Index];
	}
	return nullptr;
}

// ---------------------------------------------------------------------------
// Test 1: Struct expansion is preserved across ForceRefresh (verifies original CL fix)
// ---------------------------------------------------------------------------
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FDetailsViewExpansionState_StructPreservedAcrossForceRefresh,
	"Editor.PropertyEditor.DetailsView.ExpansionState.StructPreservedAcrossForceRefresh",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FDetailsViewExpansionState_StructPreservedAcrossForceRefresh::RunTest(const FString& Parameters)
{
	UPropertyEditorTestObject* TestObject = NewObject<UPropertyEditorTestObject>();
	TSharedRef<SDetailsView> View = CreateDetailsView(TestObject);

	// Find the struct property node
	TSharedPtr<FDetailTreeNode> Category = FindHeadNode(*View, TEXT("StructTests"));
	if (!TestNotNull(TEXT("StructTests category found"), Category.Get()))
	{
		return false;
	}

	TSharedPtr<FDetailTreeNode> StructNode = FindChildNode(*Category, TEXT("StructWithMultipleInstances1"));
	if (!TestNotNull(TEXT("StructWithMultipleInstances1 node found"), StructNode.Get()))
	{
		return false;
	}

	// Expand the struct
	const bool bShouldSaveState = true;
	StructNode->OnItemExpansionChanged(true, bShouldSaveState);
	TestTrue(TEXT("Struct is expanded after toggle"), StructNode->ShouldBeExpanded());

	// ForceRefresh rebuilds the tree
	View->ForceRefresh();

	// Re-find the node (old references are stale after refresh)
	Category = FindHeadNode(*View, TEXT("StructTests"));
	if (!TestNotNull(TEXT("StructTests category found after refresh"), Category.Get()))
	{
		return false;
	}

	StructNode = FindChildNode(*Category, TEXT("StructWithMultipleInstances1"));
	if (!TestNotNull(TEXT("StructWithMultipleInstances1 node found after refresh"), StructNode.Get()))
	{
		return false;
	}

	TestTrue(TEXT("Struct expansion preserved across ForceRefresh"), StructNode->ShouldBeExpanded());

	return true;
}

// ---------------------------------------------------------------------------
// Test 2: Array expansion is preserved across reselection (regression test)
// ---------------------------------------------------------------------------
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FDetailsViewExpansionState_ArrayPreservedAcrossReselection,
	"Editor.PropertyEditor.DetailsView.ExpansionState.ArrayPreservedAcrossReselection",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FDetailsViewExpansionState_ArrayPreservedAcrossReselection::RunTest(const FString& Parameters)
{
	UPropertyEditorTestObject* ObjectA = NewObject<UPropertyEditorTestObject>();
	UPropertyEditorTestObject* ObjectB = NewObject<UPropertyEditorTestObject>();

	// Add elements so the array is non-empty and expandable
	AddArrayElements(ObjectA, TEXT("ArrayOfStructs"), 2);

	TSharedRef<SDetailsView> View = CreateDetailsView(ObjectA);

	// Find and expand the ArrayOfStructs property
	TSharedPtr<FDetailTreeNode> Category = FindHeadNode(*View, TEXT("StructTests"));
	if (!TestNotNull(TEXT("StructTests category found"), Category.Get()))
	{
		return false;
	}

	TSharedPtr<FDetailTreeNode> ArrayNode = FindChildNode(*Category, TEXT("ArrayOfStructs"));
	if (!TestNotNull(TEXT("ArrayOfStructs node found"), ArrayNode.Get()))
	{
		return false;
	}

	// Expand the array
	ArrayNode->OnItemExpansionChanged(true, true);
	TestTrue(TEXT("Array is expanded"), ArrayNode->ShouldBeExpanded());

	// Switch to object B
	View->SetObject(ObjectB);

	// Switch back to object A
	View->SetObject(ObjectA);

	// Re-find the array node
	Category = FindHeadNode(*View, TEXT("StructTests"));
	if (!TestNotNull(TEXT("StructTests category found after reselect"), Category.Get()))
	{
		return false;
	}

	ArrayNode = FindChildNode(*Category, TEXT("ArrayOfStructs"));
	if (!TestNotNull(TEXT("ArrayOfStructs node found after reselect"), ArrayNode.Get()))
	{
		return false;
	}

	TestTrue(TEXT("Array expansion preserved across reselection"), ArrayNode->ShouldBeExpanded());

	return true;
}

// ---------------------------------------------------------------------------
// Test 3: Struct array elements maintain independent expansion states (key collision test)
// ---------------------------------------------------------------------------
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FDetailsViewExpansionState_StructArrayElementsIndependent,
	"Editor.PropertyEditor.DetailsView.ExpansionState.StructArrayElementsIndependent",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FDetailsViewExpansionState_StructArrayElementsIndependent::RunTest(const FString& Parameters)
{
	UPropertyEditorTestObject* TestObject = NewObject<UPropertyEditorTestObject>();
	AddArrayElements(TestObject, TEXT("StructPropertyArray"), 3);

	TSharedRef<SDetailsView> View = CreateDetailsView(TestObject);

	// Find the array property
	TSharedPtr<FDetailTreeNode> Category = FindHeadNode(*View, TEXT("ArraysOfProperties"));
	if (!TestNotNull(TEXT("ArraysOfProperties category found"), Category.Get()))
	{
		return false;
	}

	TSharedPtr<FDetailTreeNode> ArrayNode = FindChildNode(*Category, TEXT("StructPropertyArray"));
	if (!TestNotNull(TEXT("StructPropertyArray node found"), ArrayNode.Get()))
	{
		return false;
	}

	// Expand the array to reveal elements
	ArrayNode->OnItemExpansionChanged(true, true);

	// Get array element nodes
	TSharedPtr<FDetailTreeNode> Element0 = FindChildByIndex(*ArrayNode, 0);
	TSharedPtr<FDetailTreeNode> Element1 = FindChildByIndex(*ArrayNode, 1);
	TSharedPtr<FDetailTreeNode> Element2 = FindChildByIndex(*ArrayNode, 2);

	if (!TestNotNull(TEXT("Element [0] found"), Element0.Get())
		|| !TestNotNull(TEXT("Element [1] found"), Element1.Get())
		|| !TestNotNull(TEXT("Element [2] found"), Element2.Get()))
	{
		return false;
	}

	// Expand only element [0], leave [1] and [2] collapsed
	Element0->OnItemExpansionChanged(true, true);
	TestTrue(TEXT("Element [0] expanded"), Element0->ShouldBeExpanded());

	// ForceRefresh rebuilds the tree
	View->ForceRefresh();

	// Re-find all nodes
	Category = FindHeadNode(*View, TEXT("ArraysOfProperties"));
	if (!TestNotNull(TEXT("ArraysOfProperties category found after refresh"), Category.Get()))
	{
		return false;
	}

	ArrayNode = FindChildNode(*Category, TEXT("StructPropertyArray"));
	if (!TestNotNull(TEXT("StructPropertyArray node found after refresh"), ArrayNode.Get()))
	{
		return false;
	}

	Element0 = FindChildByIndex(*ArrayNode, 0);
	Element1 = FindChildByIndex(*ArrayNode, 1);
	Element2 = FindChildByIndex(*ArrayNode, 2);

	if (!TestNotNull(TEXT("Element [0] found after refresh"), Element0.Get())
		|| !TestNotNull(TEXT("Element [1] found after refresh"), Element1.Get())
		|| !TestNotNull(TEXT("Element [2] found after refresh"), Element2.Get()))
	{
		return false;
	}

	// Element [0] should still be expanded; [1] and [2] should remain collapsed
	TestTrue(TEXT("Element [0] expansion preserved after ForceRefresh"), Element0->ShouldBeExpanded());
	TestFalse(TEXT("Element [1] still collapsed after ForceRefresh"), Element1->ShouldBeExpanded());
	TestFalse(TEXT("Element [2] still collapsed after ForceRefresh"), Element2->ShouldBeExpanded());

	return true;
}

// ---------------------------------------------------------------------------
// Test 4: Struct expansion preserved across repeated ForceRefresh (simulates property changes)
// ---------------------------------------------------------------------------
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FDetailsViewExpansionState_StructPreservedAcrossRepeatedForceRefresh,
	"Editor.PropertyEditor.DetailsView.ExpansionState.StructPreservedAcrossRepeatedForceRefresh",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FDetailsViewExpansionState_StructPreservedAcrossRepeatedForceRefresh::RunTest(const FString& Parameters)
{
	UPropertyEditorTestObject* TestObject = NewObject<UPropertyEditorTestObject>();
	TSharedRef<SDetailsView> View = CreateDetailsView(TestObject);

	// Find and expand the struct
	TSharedPtr<FDetailTreeNode> Category = FindHeadNode(*View, TEXT("StructTests"));
	if (!TestNotNull(TEXT("StructTests category found"), Category.Get()))
	{
		return false;
	}

	TSharedPtr<FDetailTreeNode> StructNode = FindChildNode(*Category, TEXT("StructWithMultipleInstances1"));
	if (!TestNotNull(TEXT("Struct node found"), StructNode.Get()))
	{
		return false;
	}

	StructNode->OnItemExpansionChanged(true, true);
	TestTrue(TEXT("Struct expanded initially"), StructNode->ShouldBeExpanded());

	// Simulate multiple property changes by doing repeated ForceRefresh calls
	for (int32 Iteration = 0; Iteration < 3; ++Iteration)
	{
		View->ForceRefresh();

		Category = FindHeadNode(*View, TEXT("StructTests"));
		if (!TestNotNull(*FString::Printf(TEXT("Category found after refresh #%d"), Iteration + 1), Category.Get()))
		{
			return false;
		}

		StructNode = FindChildNode(*Category, TEXT("StructWithMultipleInstances1"));
		if (!TestNotNull(*FString::Printf(TEXT("Struct node found after refresh #%d"), Iteration + 1), StructNode.Get()))
		{
			return false;
		}

		TestTrue(
			*FString::Printf(TEXT("Struct still expanded after refresh #%d"), Iteration + 1),
			StructNode->ShouldBeExpanded());
	}

	return true;
}

// ---------------------------------------------------------------------------
// Test 5: Struct expansion preserved after changing a child property value
// ---------------------------------------------------------------------------
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FDetailsViewExpansionState_StructPreservedAfterPropertyChange,
	"Editor.PropertyEditor.DetailsView.ExpansionState.StructPreservedAfterPropertyChange",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FDetailsViewExpansionState_StructPreservedAfterPropertyChange::RunTest(const FString& Parameters)
{
	UPropertyEditorTestObject* TestObject = NewObject<UPropertyEditorTestObject>();
	TSharedRef<SDetailsView> View = CreateDetailsView(TestObject);

	// Find and expand the struct
	TSharedPtr<FDetailTreeNode> Category = FindHeadNode(*View, TEXT("StructTests"));
	if (!TestNotNull(TEXT("Category found"), Category.Get()))
	{
		return false;
	}

	TSharedPtr<FDetailTreeNode> StructNode = FindChildNode(*Category, TEXT("StructWithMultipleInstances1"));
	if (!TestNotNull(TEXT("Struct node found"), StructNode.Get()))
	{
		return false;
	}

	StructNode->OnItemExpansionChanged(true, true);
	TestTrue(TEXT("Struct expanded"), StructNode->ShouldBeExpanded());

	// Modify a child property through the property system to simulate a real edit
	FProperty* IntProp = FindFProperty<FProperty>(FPropertyEditorTestBasicStruct::StaticStruct(), TEXT("IntPropertyInsideAStruct"));
	if (TestNotNull(TEXT("IntPropertyInsideAStruct found"), IntProp))
	{
		// Find the child property handle and change its value
		FDetailNodeList StructChildren;
		StructNode->GetChildren(StructChildren, true);
		for (const TSharedRef<FDetailTreeNode>& Child : StructChildren)
		{
			if (TSharedPtr<IDetailPropertyRow> Row = Child->GetRow())
			{
				if (TSharedPtr<IPropertyHandle> Handle = Row->GetPropertyHandle())
				{
					if (Handle->GetProperty() == IntProp)
					{
						Handle->SetValue(42);
						break;
					}
				}
			}
		}
	}

	// ForceRefresh simulates the details panel refresh after the edit
	View->ForceRefresh();

	// Re-find and verify
	Category = FindHeadNode(*View, TEXT("StructTests"));
	if (!TestNotNull(TEXT("Category found after change+refresh"), Category.Get()))
	{
		return false;
	}

	StructNode = FindChildNode(*Category, TEXT("StructWithMultipleInstances1"));
	if (!TestNotNull(TEXT("Struct found after change+refresh"), StructNode.Get()))
	{
		return false;
	}

	TestTrue(TEXT("Struct expansion preserved after property change + ForceRefresh"), StructNode->ShouldBeExpanded());

	return true;
}

} // namespace UE::PropertyEditor::Tests

#endif // WITH_DEV_AUTOMATION_TESTS
