// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "DetailsViewPropertyHandleTestBase.h"

DETAILS_VIEW_PROPERTY_HANDLE_TEST(FDetailsViewPropertyHandleArrayTest, "Editor.PropertyEditor.DetailsView.PropertyHandleArray")
{
	FDetailsViewPropertyHandleArrayTest() : FDetailsViewPropertyHandleTestBase("Properties", "TestValueSoftPtrArray") {}

	TSharedPtr<IPropertyHandleArray> PropertyHandleArray;

	BEFORE_EACH()
	{
		FDetailsViewPropertyHandleTestBase::Setup();
		ASSERT_THAT(IsNotNull(PropertyHandle));

		PropertyHandleArray = PropertyHandle->AsArray();
		ASSERT_THAT(IsNotNull(PropertyHandleArray, TEXT("Property handle is an array check")));
	}
	
	// The purpose of the "AddItem" test is to verify functionality that could not be covered in the test code implemented outside of this module
	// due to a lack of access to the private headers. In particular, this test checks whether users can add an item to the array property
	// through the SDetailsView widget.
	// 
	// Note: The test method may need to be updated soon due to changes in the PropertyHandle operation.
	TEST_METHOD(AddItem)
	{
		const int32 InitialSize = TestObject->TestValueSoftPtrArray.Num();
		const FPropertyAccess::Result Result = PropertyHandleArray->AddItem();
		ASSERT_THAT(AreEqual(FPropertyAccess::Result::Success, Result, TEXT("Property handle added an item check")));
		ASSERT_THAT(AreEqual(1, TestObject->TestValueSoftPtrArray.Num() - InitialSize, TEXT("Array size of the editing object's property increased by one check")));
	}
};

#endif // WITH_DEV_AUTOMATION_TESTS