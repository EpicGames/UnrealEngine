// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "DetailsViewPropertyHandleTestBase.h"
#include "PropertyHandleImpl.h"

#define DETAILS_VIEW_PROPERTY_HANDLE_ENUM_TEST(_ClassName, _TestDir) TEST_CLASS_WITH_BASE_AND_FLAGS(_ClassName, _TestDir, TDetailsViewPropertyHandleEnumTestBase, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

template<typename Derived, typename AsserterType>
using TDetailsViewPropertyHandleEnumTestBase = FDetailsViewPropertyHandleTestBase<Derived, AsserterType>;

/**
 * Tests that uint8-based enums resolve to FPropertyHandleByte
 */
DETAILS_VIEW_PROPERTY_HANDLE_ENUM_TEST(FDetailsViewPropertyHandleByteEnumTest, "Editor.PropertyEditor.DetailsView.PropertyHandleByteEnum")
{
	FDetailsViewPropertyHandleByteEnumTest() : FDetailsViewPropertyHandleTestBase("Properties", "TestByteEnum") {}

	BEFORE_EACH()
	{
		FDetailsViewPropertyHandleTestBase::Setup();
		ASSERT_THAT(IsNotNull(PropertyHandle));
	}

	TEST_METHOD(GetValue_uint8)
	{
		TestObject->TestByteEnum = ETestByteEnum::Value1;

		uint8 Value = 0;
		const FPropertyAccess::Result Result = PropertyHandle->GetValue(Value);
		ASSERT_THAT(AreEqual(FPropertyAccess::Result::Success, Result, TEXT("Property handle retrieved value check")));
		ASSERT_THAT(AreEqual(static_cast<uint8>(ETestByteEnum::Value1), Value, TEXT("The retrieved value is correct check")));
	}

	TEST_METHOD(SetValue_uint8)
	{
		const FPropertyAccess::Result Result = PropertyHandle->SetValue(static_cast<uint8>(ETestByteEnum::Value2));
		ASSERT_THAT(AreEqual(FPropertyAccess::Result::Success, Result, TEXT("Property handle set value check")));
		ASSERT_THAT(AreEqual(ETestByteEnum::Value2, TestObject->TestByteEnum, TEXT("Property of the editing object is set correctly check")));
	}
};

/**
 * Tests that uint32-based enums resolve to FPropertyHandleInt
 */
DETAILS_VIEW_PROPERTY_HANDLE_ENUM_TEST(FDetailsViewPropertyHandleIntEnumTest, "Editor.PropertyEditor.DetailsView.PropertyHandleIntEnum")
{
	FDetailsViewPropertyHandleIntEnumTest() : FDetailsViewPropertyHandleTestBase("Properties", "TestIntEnum") {}

	BEFORE_EACH()
	{
		FDetailsViewPropertyHandleTestBase::Setup();
		ASSERT_THAT(IsNotNull(PropertyHandle));
	}

	TEST_METHOD(GetValue_uint32)
	{
		TestObject->TestIntEnum = ETestIntEnum::Value1;

		uint32 Value = 0;
		const FPropertyAccess::Result Result = PropertyHandle->GetValue(Value);
		ASSERT_THAT(AreEqual(FPropertyAccess::Result::Success, Result, TEXT("Property handle retrieved value check")));
		ASSERT_THAT(AreEqual(static_cast<uint32>(ETestIntEnum::Value1), Value, TEXT("The retrieved value is correct check")));
	}

	TEST_METHOD(SetValue_uint32)
	{
		const FPropertyAccess::Result Result = PropertyHandle->SetValue(static_cast<uint32>(ETestIntEnum::Value2));
		ASSERT_THAT(AreEqual(FPropertyAccess::Result::Success, Result, TEXT("Property handle set value check")));
		ASSERT_THAT(AreEqual(ETestIntEnum::Value2, TestObject->TestIntEnum, TEXT("Property of the editing object is set correctly check")));
	}
};

#endif // WITH_DEV_AUTOMATION_TESTS
