// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "DetailsViewPropertyHandleTestBase.h"

DETAILS_VIEW_PROPERTY_HANDLE_TEST(FDetailsViewPropertyHandleObjectTest, "Editor.PropertyEditor.DetailsView.PropertyHandleObject")
{
	FDetailsViewPropertyHandleObjectTest() : FDetailsViewPropertyHandleTestBase("Properties", "TestValueSoftPtr") {}

	// The purpose of the "SetValue" and "GetValue" tests is to verify functionality that could not be covered in the test code implemented outside of this module
	// due to a lack of access to the private headers. In particular, these tests check whether users can set a property value through the SDetailsView widget.
	// 
	// Note: The test methods may need to be updated soon due to changes in the PropertyHandle operation.
	TEST_METHOD(SetValue_FAssetData)
	{
		UDetailsViewPropertyHandleTestValueClass* const TestValueObject = NewObject<UDetailsViewPropertyHandleTestValueClass>();
		const FPropertyAccess::Result Result = PropertyHandle->SetValue(FAssetData(TestValueObject));
		ASSERT_THAT(AreEqual(FPropertyAccess::Result::Success, Result, TEXT("Property handle set value check")));
		ASSERT_THAT(AreEqual(TestValueObject, TestObject->TestValueSoftPtr.Get(), TEXT("Property of the editing object is set correctly check")));
	}

	TEST_METHOD(SetValue_UObject)
	{
		UDetailsViewPropertyHandleTestValueClass* const TestValueObject = NewObject<UDetailsViewPropertyHandleTestValueClass>();
		const FPropertyAccess::Result Result = PropertyHandle->SetValue(TestValueObject);
		ASSERT_THAT(AreEqual(FPropertyAccess::Result::Success, Result, TEXT("Property handle set value check")));
		ASSERT_THAT(AreEqual(TestValueObject, TestObject->TestValueSoftPtr.Get(), TEXT("Property of the editing object is set correctly check")));
	}
	
	TEST_METHOD(GetValue_FAssetData)
	{
		UDetailsViewPropertyHandleTestValueClass* const TestValueObject = NewObject<UDetailsViewPropertyHandleTestValueClass>();
		TestObject->TestValueSoftPtr = TestValueObject;
		
		FAssetData Value;
		const FPropertyAccess::Result Result = PropertyHandle->GetValue(Value);
		ASSERT_THAT(AreEqual(FPropertyAccess::Result::Success, Result, TEXT("Property handle retrieved value check")));
		ASSERT_THAT(AreEqual(FAssetData(TestValueObject), Value, TEXT("The retrieved value is correct check")));
	}
	
	TEST_METHOD(GetValue_UObject)
	{
		UDetailsViewPropertyHandleTestValueClass* const TestValueObject = NewObject<UDetailsViewPropertyHandleTestValueClass>();
		TestObject->TestValueSoftPtr = TestValueObject;
		
		UObject* Value = nullptr;
		const FPropertyAccess::Result Result = PropertyHandle->GetValue(Value);
		ASSERT_THAT(AreEqual(FPropertyAccess::Result::Success, Result, TEXT("Property handle retrieved value check")));
		ASSERT_THAT(AreEqual(Cast<UObject>(TestValueObject), Value, TEXT("The retrieved value is correct check")));
	}
};

#endif // WITH_DEV_AUTOMATION_TESTS