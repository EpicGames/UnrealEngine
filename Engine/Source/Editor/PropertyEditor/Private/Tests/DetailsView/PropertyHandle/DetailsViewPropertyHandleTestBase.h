// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_DEV_AUTOMATION_TESTS

#include "CoreMinimal.h"
#include "CQTest.h"
#include "DetailTreeNode.h"
#include "DetailsViewPropertyHandleTestClass.h"
#include "PropertyHandle.h"
#include "SDetailsView.h"

template<typename Derived, typename AsserterType>
class FDetailsViewPropertyHandleTestBase : public TTest<Derived, AsserterType>
{
protected:
	using TTest<Derived, AsserterType>::Assert;

	FDetailsViewPropertyHandleTestBase(const FString& InCategoryName, const FString& InPropertyName) :
		CategoryName(InCategoryName),
		PropertyName(InPropertyName)
	{
	}

	virtual void Setup() override
	{
		TestObject = NewObject<UDetailsViewPropertyHandleTestClass>();
		DetailsView = CreateDetailsViewForObject(TestObject);
		FindPropertyHandleInDetailsView();
	}

private:
	static TSharedRef<SDetailsView> CreateDetailsViewForObject(UObject* InObject)
	{
		TSharedRef<SDetailsView> DetailsView = SNew(SDetailsView, FDetailsViewArgs());
		DetailsView->SetObject(InObject);
		return DetailsView;
	}

	static TSharedPtr<FDetailTreeNode> GetHeadNodeByName(SDetailsView& DetailsView, const FString& HeadNodeName)
	{
		TArray<TWeakPtr<FDetailTreeNode>> WeakNodes;
		DetailsView.GetHeadNodes(WeakNodes);

		for (const TWeakPtr<FDetailTreeNode>& WeakNode : WeakNodes)
		{
			if (TSharedPtr<FDetailTreeNode> Node = WeakNode.Pin())
			{
				if (HeadNodeName.Equals(Node->GetNodeName().ToString())) { return Node; }
			}
		}

		return nullptr;
	}

	static TSharedPtr<FDetailTreeNode> GetChildNodeByName(FDetailTreeNode& ParentNode, const FString& ChildNodeName)
	{
		FDetailNodeList ChildNodes;
		ParentNode.GetChildren(ChildNodes, true);

		for (const TSharedRef<FDetailTreeNode>& Node : ChildNodes)
		{
			if (ChildNodeName.Equals(Node->GetNodeName().ToString())) { return Node; }
		}

		return nullptr;
	}

	void FindPropertyHandleInDetailsView()
	{
		const TSharedPtr<FDetailTreeNode> CategoryNode = GetHeadNodeByName(*DetailsView.Get(), CategoryName);
		ASSERT_THAT(IsNotNull(CategoryNode, FString::Printf(TEXT("Head node with name '%s' retrieved check"), *CategoryName)));

		const TSharedPtr<FDetailTreeNode> PropertyNode = GetChildNodeByName(*CategoryNode.Get(), PropertyName);
		ASSERT_THAT(IsNotNull(PropertyNode, FString::Printf(TEXT("Child node with name '%s' retrieved check"), *PropertyName)));

		const TSharedPtr<IDetailPropertyRow> PropertyRow = PropertyNode->GetRow();
		ASSERT_THAT(IsNotNull(PropertyRow, TEXT("Property row retrieved check")));

		PropertyHandle = PropertyRow->GetPropertyHandle();
		ASSERT_THAT(IsNotNull(PropertyHandle, TEXT("Property handle retrieved check")));
	}

protected:
	UDetailsViewPropertyHandleTestClass* TestObject = nullptr;
	TSharedPtr<IPropertyHandle> PropertyHandle;

private:
	TSharedPtr<SDetailsView> DetailsView;
	const FString CategoryName;
	const FString PropertyName;
};

#define DETAILS_VIEW_PROPERTY_HANDLE_TEST(_ClassName, _TestDir) TEST_CLASS_WITH_BASE_AND_FLAGS(_ClassName, _TestDir, FDetailsViewPropertyHandleTestBase, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

#endif // WITH_DEV_AUTOMATION_TESTS