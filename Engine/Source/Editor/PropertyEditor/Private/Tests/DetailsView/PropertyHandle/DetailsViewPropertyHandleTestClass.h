// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"

#include "DetailsViewPropertyHandleTestClass.generated.h"

#if WITH_EDITORONLY_DATA

/** Test enum with uint8 underlying type (should resolve to FPropertyHandleByte) */
UENUM()
enum class ETestByteEnum : uint8
{
	Value0 = 0,
	Value1 = 1,
	Value2 = 2
};

/** Test enum with uint32 underlying type (should resolve to FPropertyHandleInt) */
UENUM()
enum class ETestIntEnum : uint32
{
	Value0 = 0,
	Value1 = 1,
	Value2 = 2
};

UCLASS()
class UDetailsViewPropertyHandleTestValueClass : public UObject
{
	GENERATED_BODY()
};

UCLASS()
class UDetailsViewPropertyHandleTestClass : public UObject
{
	GENERATED_BODY()

public:	
	UPROPERTY(EditAnywhere, Category = "Properties")
	TSoftObjectPtr<UDetailsViewPropertyHandleTestValueClass> TestValueSoftPtr;
	
	UPROPERTY(EditAnywhere, Category = "Properties")
	TArray<TSoftObjectPtr<UDetailsViewPropertyHandleTestValueClass>> TestValueSoftPtrArray;

	/** uint8-based enum. */
	UPROPERTY(EditAnywhere, Category = "Properties")
	ETestByteEnum TestByteEnum = ETestByteEnum::Value0;

	/** uint32-based enum */
	UPROPERTY(EditAnywhere, Category = "Properties")
	ETestIntEnum TestIntEnum = ETestIntEnum::Value0;
};

#endif // WITH_EDITORONLY_DATA