// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyPermissionTests.h"

#include "Misc/AutomationTest.h"
#include "PropertyPermissionList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyPermissionTests)

#if WITH_DEV_AUTOMATION_TESTS

class FTestPropertyPermissionList : public FPropertyPermissionList
{
};

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPropertyEditorTests_PropertyPermissions, "PropertyEditor.PropertyPermissions", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FPropertyEditorTests_PropertyPermissions::RunTest(const FString& Parameters)
{
	FTestPropertyPermissionList PermissionList;
	PermissionList.SetEnabled(true);

	auto AddPermissionList =
		[&PermissionList](const UStruct* Struct, TConstArrayView<FName> AllowPropertyNames, TConstArrayView<FName> DenyPropertyNames, const EPropertyPermissionListRules Rules)
		{
			FNamePermissionList PropertyNameList;
			for (const FName PropertyName : AllowPropertyNames)
			{
				PropertyNameList.AddAllowListItem(FName(), PropertyName);
			}
			for (const FName PropertyName : DenyPropertyNames)
			{
				PropertyNameList.AddDenyListItem(FName(), PropertyName);
			}
			PermissionList.AddPermissionList(Struct, PropertyNameList, Rules, {});
		};

	auto TestDoesPropertyPassFilter =
		[this, &PermissionList](const UStruct* Struct, FName PropertyName, const bool bShouldPass, const TCHAR* What)
		{
			const bool bDidPass = PermissionList.DoesPropertyPassFilter(Struct, PropertyName);
			TestEqual(What, bDidPass, bShouldPass);
		};

	{
		// Allow COne and CTwo from struct C
		// Properties from struct A or struct B for an instance of struct C should be denied
		// Properties from struct A or struct B for an instance of struct A or B should be allowed
		// Properties from struct C for an instance of struct C should be allowed
		AddPermissionList(FPropertyEditorPermissionTestStructC::StaticStruct(), { "COne", "CTwo" }, {}, EPropertyPermissionListRules::UseExistingPermissionList);
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructA::StaticStruct(), "AOne", true, TEXT("[Test1] AOne on struct A"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructA::StaticStruct(), "ATwo", true, TEXT("[Test1] ATwo on struct A"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "AOne", true, TEXT("[Test1] AOne on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "ATwo", true, TEXT("[Test1] ATwo on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "BOne", true, TEXT("[Test1] BOne on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "BTwo", true, TEXT("[Test1] BTwo on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "AOne", false, TEXT("[Test1] AOne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "ATwo", false, TEXT("[Test1] ATwo on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "BOne", false, TEXT("[Test1] BOne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "BTwo", false, TEXT("[Test1] BTwo on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "COne", true, TEXT("[Test1] COne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "CTwo", true, TEXT("[Test1] CTwo on struct C"));
		
		// Allow AOne property from struct A
		// AOne for an instance of struct A, B, or C should be allowed
		// ATwo for an instance of struct A, B, or C should be denied
		// Properties from struct B for an instance of struct B or C should be denied
		// Properties from struct C for an instance of struct C should be allowed (due to the previous rule)
		AddPermissionList(FPropertyEditorPermissionTestStructA::StaticStruct(), { "AOne" }, {}, EPropertyPermissionListRules::UseExistingPermissionList);
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructA::StaticStruct(), "AOne", true, TEXT("[Test2] AOne on struct A"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructA::StaticStruct(), "ATwo", false, TEXT("[Test2] ATwo on struct A"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "AOne", true, TEXT("[Test2] AOne on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "ATwo", false, TEXT("[Test2] ATwo on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "BOne", false, TEXT("[Test2] BOne on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "BTwo", false, TEXT("[Test2] BTwo on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "AOne", true, TEXT("[Test2] AOne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "ATwo", false, TEXT("[Test2] ATwo on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "BOne", false, TEXT("[Test2] BOne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "BTwo", false, TEXT("[Test2] BTwo on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "COne", true, TEXT("[Test2] COne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "CTwo", true, TEXT("[Test2] CTwo on struct C"));
	}

	PermissionList.ClearPermissionList();
	
	{
		// Allow all properties from struct C
		// Properties from struct A or struct B for an instance of struct A, B, or C should be allowed
		// Properties from struct C for an instance of struct C should be allowed
		AddPermissionList(FPropertyEditorPermissionTestStructC::StaticStruct(), {}, {}, EPropertyPermissionListRules::AllowListAllProperties);
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructA::StaticStruct(), "AOne", true, TEXT("[Test3] AOne on struct A"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructA::StaticStruct(), "ATwo", true, TEXT("[Test3] ATwo on struct A"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "AOne", true, TEXT("[Test3] AOne on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "ATwo", true, TEXT("[Test3] ATwo on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "BOne", true, TEXT("[Test3] BOne on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "BTwo", true, TEXT("[Test3] BTwo on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "AOne", true, TEXT("[Test3] AOne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "ATwo", true, TEXT("[Test3] ATwo on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "BOne", true, TEXT("[Test3] BOne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "BTwo", true, TEXT("[Test3] BTwo on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "COne", true, TEXT("[Test3] COne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "CTwo", true, TEXT("[Test3] CTwo on struct C"));
		
		// Allow AOne property from struct A
		// AOne for an instance of struct A, B, or C should be allowed
		// ATwo for an instance of struct A, B, or C should be denied
		// Properties from struct B for an instance of struct B or C should be denied
		// Properties from struct C for an instance of struct C should be allowed (due to the previous rule)
		AddPermissionList(FPropertyEditorPermissionTestStructA::StaticStruct(), { "AOne" }, {}, EPropertyPermissionListRules::UseExistingPermissionList);
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructA::StaticStruct(), "AOne", true, TEXT("[Test4] AOne on struct A"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructA::StaticStruct(), "ATwo", false, TEXT("[Test4] ATwo on struct A"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "AOne", true, TEXT("[Test4] AOne on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "ATwo", false, TEXT("[Test4] ATwo on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "BOne", false, TEXT("[Test4] BOne on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "BTwo", false, TEXT("[Test4] BTwo on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "AOne", true, TEXT("[Test4] AOne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "ATwo", false, TEXT("[Test4] ATwo on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "BOne", false, TEXT("[Test4] BOne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "BTwo", false, TEXT("[Test4] BTwo on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "COne", true, TEXT("[Test4] COne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "CTwo", true, TEXT("[Test4] CTwo on struct C"));
	}

	PermissionList.ClearPermissionList();
	
	{
		// Allow all sub-class properties from struct B
		// Properties from struct A for an instance of struct A should be allowed
		// Properties from struct A or B for an instance of struct B or C should be denied
		// Properties from struct C for an instance of struct C should be allowed
		AddPermissionList(FPropertyEditorPermissionTestStructB::StaticStruct(), { "Dummy" }, {}, EPropertyPermissionListRules::AllowListAllSubclassProperties);
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructA::StaticStruct(), "AOne", true, TEXT("[Test5] AOne on struct A"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructA::StaticStruct(), "ATwo", true, TEXT("[Test5] ATwo on struct A"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "AOne", false, TEXT("[Test5] AOne on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "ATwo", false, TEXT("[Test5] ATwo on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "BOne", false, TEXT("[Test5] BOne on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "BTwo", false, TEXT("[Test5] BTwo on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "AOne", false, TEXT("[Test5] AOne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "ATwo", false, TEXT("[Test5] ATwo on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "BOne", false, TEXT("[Test5] BOne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "BTwo", false, TEXT("[Test5] BTwo on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "COne", true, TEXT("[Test5] COne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "CTwo", true, TEXT("[Test5] CTwo on struct C"));
		
		// Allow all properties from struct A
		// Properties from struct A for an instance of struct A, B, or C should be allowed
		// Properties from struct B for an instance of struct B or C should be denied
		// Properties from struct C for an instance of struct C should be allowed
		AddPermissionList(FPropertyEditorPermissionTestStructA::StaticStruct(), {}, {}, EPropertyPermissionListRules::AllowListAllProperties);
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructA::StaticStruct(), "AOne", true, TEXT("[Test6] AOne on struct A"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructA::StaticStruct(), "ATwo", true, TEXT("[Test6] ATwo on struct A"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "AOne", true, TEXT("[Test6] AOne on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "ATwo", true, TEXT("[Test6] ATwo on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "BOne", false, TEXT("[Test6] BOne on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "BTwo", false, TEXT("[Test6] BTwo on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "AOne", true, TEXT("[Test6] AOne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "ATwo", true, TEXT("[Test6] ATwo on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "BOne", false, TEXT("[Test6] BOne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "BTwo", false, TEXT("[Test6] BTwo on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "COne", true, TEXT("[Test6] COne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "CTwo", true, TEXT("[Test6] CTwo on struct C"));
	}

	PermissionList.ClearPermissionList();

	{
		// Allow BOne property from struct B
		// Properties from struct A for an instance of struct A should be allowed
		// Properties from struct A for an instance of struct B or C should be denied
		// BOne for an instance of struct B or C should be allowed
		// BTwo for an instance of struct B or C should be denied
		// Properties from struct C for an instance of struct C should be denied
		AddPermissionList(FPropertyEditorPermissionTestStructB::StaticStruct(), { "BOne" }, {}, EPropertyPermissionListRules::UseExistingPermissionList);
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructA::StaticStruct(), "AOne", true, TEXT("[Test7] AOne on struct A"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructA::StaticStruct(), "ATwo", true, TEXT("[Test7] ATwo on struct A"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "AOne", false, TEXT("[Test7] AOne on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "ATwo", false, TEXT("[Test7] ATwo on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "BOne", true, TEXT("[Test7] BOne on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "BTwo", false, TEXT("[Test7] BTwo on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "AOne", false, TEXT("[Test7] AOne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "ATwo", false, TEXT("[Test7] ATwo on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "BOne", true, TEXT("[Test7] BOne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "BTwo", false, TEXT("[Test7] BTwo on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "COne", false, TEXT("[Test7] COne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "CTwo", false, TEXT("[Test7] CTwo on struct C"));
		
		// Allow all properties from struct A
		// Properties from struct A for an instance of struct A, B, or C should be allowed
		// BOne for an instance of struct B or C should be allowed
		// BTwo for an instance of struct B or C should be denied
		// Properties from struct C for an instance of struct C should be denied
		AddPermissionList(FPropertyEditorPermissionTestStructA::StaticStruct(), {}, {}, EPropertyPermissionListRules::AllowListAllProperties);
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructA::StaticStruct(), "AOne", true, TEXT("[Test8] AOne on struct A"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructA::StaticStruct(), "ATwo", true, TEXT("[Test8] ATwo on struct A"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "AOne", true, TEXT("[Test8] AOne on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "ATwo", true, TEXT("[Test8] ATwo on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "BOne", true, TEXT("[Test8] BOne on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "BTwo", false, TEXT("[Test8] BTwo on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "AOne", true, TEXT("[Test8] AOne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "ATwo", true, TEXT("[Test8] ATwo on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "BOne", true, TEXT("[Test8] BOne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "BTwo", false, TEXT("[Test8] BTwo on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "COne", false, TEXT("[Test8] COne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "CTwo", false, TEXT("[Test8] CTwo on struct C"));
	}
	
	PermissionList.ClearPermissionList();

	{
		// Allow all sub-class properties from struct A
		// Deny BOne from struct B
		// Properties from struct A for an instance of struct A, B or C should be denied
		// BOne for an instance of struct B or C should be denied
		// BTwo for an instance of struct B or C should be allowed
		// Properties from struct C for an instance of struct C should be allowed
		AddPermissionList(FPropertyEditorPermissionTestStructA::StaticStruct(), { "Dummy" }, {}, EPropertyPermissionListRules::AllowListAllSubclassProperties);
		AddPermissionList(FPropertyEditorPermissionTestStructB::StaticStruct(), {}, { "BOne" }, EPropertyPermissionListRules::UseExistingPermissionList);
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructA::StaticStruct(), "AOne", false, TEXT("[Test9] AOne on struct A"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructA::StaticStruct(), "ATwo", false, TEXT("[Test9] ATwo on struct A"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "AOne", false, TEXT("[Test9] AOne on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "ATwo", false, TEXT("[Test9] ATwo on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "BOne", false, TEXT("[Test9] BOne on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructB::StaticStruct(), "BTwo", true, TEXT("[Test9] BTwo on struct B"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "AOne", false, TEXT("[Test9] AOne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "ATwo", false, TEXT("[Test9] ATwo on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "BOne", false, TEXT("[Test9] BOne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "BTwo", true, TEXT("[Test9] BTwo on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "COne", true, TEXT("[Test9] COne on struct C"));
		TestDoesPropertyPassFilter(FPropertyEditorPermissionTestStructC::StaticStruct(), "CTwo", true, TEXT("[Test9] CTwo on struct C"));
	}

	PermissionList.ClearPermissionList();
	
	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
