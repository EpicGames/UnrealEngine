// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyPermissionTests.generated.h"

USTRUCT()
struct FPropertyEditorPermissionTestStructA
{
	GENERATED_BODY()
	
public:
	UPROPERTY(EditAnywhere, Category="Properties")
	int32 AOne = 0;
	
	UPROPERTY(EditAnywhere, Category="Properties")
	int32 ATwo = 0;
};

USTRUCT()
struct FPropertyEditorPermissionTestStructB : public FPropertyEditorPermissionTestStructA
{
	GENERATED_BODY()
	
public:
	UPROPERTY(EditAnywhere, Category="Properties")
	int32 BOne = 0;
	
	UPROPERTY(EditAnywhere, Category="Properties")
	int32 BTwo = 0;
};

USTRUCT()
struct FPropertyEditorPermissionTestStructC : public FPropertyEditorPermissionTestStructB
{
	GENERATED_BODY()
	
public:
	UPROPERTY(EditAnywhere, Category="Properties")
	int32 COne = 0;
	
	UPROPERTY(EditAnywhere, Category="Properties")
	int32 CTwo = 0;
};
