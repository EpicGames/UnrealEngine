// Copyright Epic Games, Inc. All Rights Reserved.

#include "SinglePropertyTests.h"
#include "ISinglePropertyView.h"
#include "Misc/AutomationTest.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SinglePropertyTests)

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPropertyEditorTests_SingleProperty, "PropertyEditor.SingleProperty", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

#define REQUIRE(expr) \
do { \
	if (!(expr)) { \
		AddError(FString::Printf(TEXT("Expression '%s' to be true."), TEXT(#expr))); \
		return false; \
	} \
} while(false)

#define EXPECT(expr) \
do { \
	if (!(expr)) { \
		AddError(FString::Printf(TEXT("Expression '%s' to be true."), TEXT(#expr))); \
	} \
} while(false)

bool FPropertyEditorTests_SingleProperty::RunTest(const FString& Parameters)
{
	const FVector ExpectedValue(1.0, 2.0, 3.0);

	UPropertyEditorSinglePropertyTestClass* TestObject = NewObject<UPropertyEditorSinglePropertyTestClass>();
	TestObject->Vector = ExpectedValue;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedPtr<ISinglePropertyView> PropertyView =
		PropertyEditorModule.CreateSingleProperty(TestObject, "Vector", FSinglePropertyParams{});

	REQUIRE(PropertyView.IsValid());

	const TSharedPtr<IPropertyHandle> PropertyHandle = PropertyView->GetPropertyHandle();

	REQUIRE(PropertyHandle.IsValid());
	REQUIRE(PropertyHandle->IsValidHandle());

	FVector ReadVector;
	FPropertyAccess::Result ReadResult = PropertyHandle->GetValue(ReadVector);

	EXPECT(FPropertyAccess::Success == ReadResult);
	EXPECT(ExpectedValue.X == ReadVector.X);
	EXPECT(ExpectedValue.Y == ReadVector.Y);
	EXPECT(ExpectedValue.Z == ReadVector.Z);

	return true;
}

#undef REQUIRE
#undef EXPECT

#endif // WITH_DEV_AUTOMATION_TESTS
