// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"
#include "Math/MathFwd.h"

#include "SinglePropertyTests.generated.h"

#if WITH_EDITORONLY_DATA

UCLASS()
class UPropertyEditorSinglePropertyTestClass : public UObject
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = "Properties")
	FVector Vector;
};

#endif // WITH_EDITORONLY_DATA