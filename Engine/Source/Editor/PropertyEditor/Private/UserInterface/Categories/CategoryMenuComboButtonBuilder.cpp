//  Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/Categories/CategoryMenuComboButtonBuilder.h"

FCategoryMenuComboButtonBuilder::FCategoryMenuComboButtonBuilder(
	TSharedRef<FDetailsDisplayManager> InDetailsDisplayManager):
	FToolElementRegistrationArgs("FCategoryMenuComboButtonBuilder"),
	DisplayManager(InDetailsDisplayManager)
{
}

FCategoryMenuComboButtonBuilder& FCategoryMenuComboButtonBuilder::Set_OnGetContent(FOnGetContent InOnGetContent)
{
	OnGetContent.Unbind();
	OnGetContent = InOnGetContent;
	return *this;
}

FCategoryMenuComboButtonBuilder& FCategoryMenuComboButtonBuilder::Bind_IsVisible(TAttribute<EVisibility> InIsVisible)
{
	IsVisible = InIsVisible;
	return *this;
}

TSharedPtr<SWidget> FCategoryMenuComboButtonBuilder::GenerateWidget()
{
	return 	DisplayManager->ShouldShowCategoryMenu()  ?
		        SNew( SComboButton )
				.ComboButtonStyle( FAppStyle::Get(), "DetailsView.CategoryComboButton" )
				.Visibility(EVisibility::Visible)
				.HasDownArrow(true)
				.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("DetailsView.CategoryComboButton")))
				.OnGetMenuContent(OnGetContent) :
		        SNullWidget::NullWidget;
}

TSharedRef<SWidget> FCategoryMenuComboButtonBuilder::operator*()
{
	return GenerateWidget().ToSharedRef();
}

FCategoryMenuComboButtonBuilder::~FCategoryMenuComboButtonBuilder()
{
	OnGetContent.Unbind();
}