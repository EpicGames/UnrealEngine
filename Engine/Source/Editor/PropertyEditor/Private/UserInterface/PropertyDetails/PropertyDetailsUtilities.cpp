// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyDetails/PropertyDetailsUtilities.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "IDetailsViewPrivate.h"

FPropertyDetailsUtilities::FPropertyDetailsUtilities(IDetailsViewPrivate& InDetailsView)
	: DetailsViewPtr(StaticCastWeakPtr<IDetailsViewPrivate>(InDetailsView.AsWeak()))
{
}

class FNotifyHook* FPropertyDetailsUtilities::GetNotifyHook() const
{
	if (TSharedPtr<IDetailsViewPrivate> DetailsView = DetailsViewPtr.Pin())
	{
		return DetailsView->GetNotifyHook();
	}
	return nullptr;
}

bool FPropertyDetailsUtilities::AreFavoritesEnabled() const
{
	// not implemented
	return false;
}

void FPropertyDetailsUtilities::ToggleFavorite( const TSharedRef< FPropertyEditor >& PropertyEditor ) const
{
	// not implemented
}

void FPropertyDetailsUtilities::CreateColorPickerWindow( const TSharedRef< FPropertyEditor >& PropertyEditor, bool bUseAlpha ) const
{
	if (TSharedPtr<IDetailsViewPrivate> DetailsView = DetailsViewPtr.Pin())
	{
		DetailsView->CreateColorPickerWindow(PropertyEditor, bUseAlpha);
	}
}

void FPropertyDetailsUtilities::EnqueueDeferredAction( FSimpleDelegate DeferredAction )
{
	if (TSharedPtr<IDetailsViewPrivate> DetailsView = DetailsViewPtr.Pin())
	{
		DetailsView->EnqueueDeferredAction(DeferredAction);
	}
}

bool FPropertyDetailsUtilities::IsPropertyEditingEnabled() const
{
	if (TSharedPtr<IDetailsViewPrivate> DetailsView = DetailsViewPtr.Pin())
	{
		return DetailsView->IsPropertyEditingEnabled();
	}
	return false;
}

void FPropertyDetailsUtilities::ForceRefresh()
{
	if (TSharedPtr<IDetailsViewPrivate> DetailsView = DetailsViewPtr.Pin())
	{
		DetailsView->ForceRefresh();
	}
}

void FPropertyDetailsUtilities::RequestRefresh()
{
	if (TSharedPtr<IDetailsViewPrivate> DetailsView = DetailsViewPtr.Pin())
	{
		DetailsView->RefreshTree();
	}
}

void FPropertyDetailsUtilities::RequestForceRefresh()
{
	if (TSharedPtr<IDetailsViewPrivate> DetailsView = DetailsViewPtr.Pin())
	{
		DetailsView->RequestForceRefresh();
	}
}

TSharedPtr<class FAssetThumbnailPool> FPropertyDetailsUtilities::GetThumbnailPool() const
{
	if (TSharedPtr<IDetailsViewPrivate> DetailsView = DetailsViewPtr.Pin())
	{
		return DetailsView->GetThumbnailPool();
	}
	return {};
}

const TArray<TSharedRef<class IClassViewerFilter>>& FPropertyDetailsUtilities::GetClassViewerFilters() const
{
	if (TSharedPtr<IDetailsViewPrivate> DetailsView = DetailsViewPtr.Pin())
	{
		return DetailsView->GetClassViewerFilters();
	}

	static TArray<TSharedRef<class IClassViewerFilter>> Empty;
	return Empty;
}

void FPropertyDetailsUtilities::NotifyFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (TSharedPtr<IDetailsViewPrivate> DetailsView = DetailsViewPtr.Pin())
	{
		DetailsView->NotifyFinishedChangingProperties(PropertyChangedEvent);
	}
}

bool FPropertyDetailsUtilities::DontUpdateValueWhileEditing() const
{
	if (TSharedPtr<IDetailsViewPrivate> DetailsView = DetailsViewPtr.Pin())
	{
		return DetailsView->DontUpdateValueWhileEditing();
	}
	return true;
}

const TArray<TWeakObjectPtr<UObject>>& FPropertyDetailsUtilities::GetSelectedObjects() const
{
	if (TSharedPtr<IDetailsViewPrivate> DetailsView = DetailsViewPtr.Pin())
	{
		return DetailsView->GetSelectedObjects();
	}
	
	static TArray<TWeakObjectPtr<UObject>> Empty;
	return Empty;
}

bool FPropertyDetailsUtilities::HasClassDefaultObject() const
{
	if (TSharedPtr<IDetailsViewPrivate> DetailsView = DetailsViewPtr.Pin())
	{
		return DetailsView->HasClassDefaultObject();
	}
	return false;
}
