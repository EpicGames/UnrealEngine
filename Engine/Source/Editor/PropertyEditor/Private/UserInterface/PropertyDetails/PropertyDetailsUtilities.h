// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "AssetThumbnail.h"
#include "IPropertyUtilities.h"

class IDetailsViewPrivate;
struct FPropertyChangedEvent;

class FPropertyDetailsUtilities : public IPropertyUtilities
{
public:
	FPropertyDetailsUtilities(IDetailsViewPrivate& InDetailsView);
	/** IPropertyUtilities interface */
	virtual class FNotifyHook* GetNotifyHook() const override;
	virtual bool AreFavoritesEnabled() const override;
	virtual void ToggleFavorite( const TSharedRef< class FPropertyEditor >& PropertyEditor ) const override;
	virtual void CreateColorPickerWindow( const TSharedRef< class FPropertyEditor >& PropertyEditor, bool bUseAlpha ) const override;
	virtual void EnqueueDeferredAction( FSimpleDelegate DeferredAction ) override;
	virtual bool IsPropertyEditingEnabled() const override;
	virtual void ForceRefresh() override;
	virtual void RequestRefresh() override;
	virtual void RequestForceRefresh() override;
	virtual TSharedPtr<class FAssetThumbnailPool> GetThumbnailPool() const override;
	virtual void NotifyStartedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) override {};
	virtual void NotifyFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool DontUpdateValueWhileEditing() const override;
	const TArray<TWeakObjectPtr<UObject>>& GetSelectedObjects() const override;
	virtual bool HasClassDefaultObject() const override;
	virtual const TArray<TSharedRef<class IClassViewerFilter>>& GetClassViewerFilters() const override;

private:
	TWeakPtr<IDetailsViewPrivate> DetailsViewPtr;
};
