// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace PropertyEditorAssetConstants
{
	static const FVector2D ContentBrowserWindowSize(300.0f, 300.0f);
	static const FVector2D SceneOutlinerWindowSize(350.0f, 300.0f);
};
