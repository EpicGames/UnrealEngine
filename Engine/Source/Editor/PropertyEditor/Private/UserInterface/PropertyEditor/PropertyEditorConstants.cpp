// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"

#include "Math/Color.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "UObject/NameTypes.h"
#include "Internationalization/Text.h"

const FName PropertyEditorConstants::PropertyFontStyle( TEXT("PropertyWindow.NormalFont") );
const FName PropertyEditorConstants::CategoryFontStyle( TEXT("DetailsView.CategoryFontStyle") );

const FName PropertyEditorConstants::MD_Bitmask( TEXT("Bitmask") );
const FName PropertyEditorConstants::MD_BitmaskEnum( TEXT("BitmaskEnum") );
const FName PropertyEditorConstants::MD_UseEnumValuesAsMaskValuesInEditor( TEXT("UseEnumValuesAsMaskValuesInEditor") );

const FText PropertyEditorConstants::DefaultUndeterminedText(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"));

const FSlateBrush* PropertyEditorConstants::GetOverlayBrush(const TSharedRef<class FPropertyEditor> PropertyEditor )
{
	return FAppStyle::GetBrush( TEXT("PropertyWindow.NoOverlayColor") );
}

FSlateColor PropertyEditorConstants::GetRowBackgroundColor(int32 IndentLevel, bool IsHovered) 
{
	int32 ColorIndex = 0;
	int32 Increment = 1;

	for (int i = 0; i < IndentLevel; ++i)
	{
		ColorIndex += Increment;

		if (ColorIndex == 0 || ColorIndex == 3)
		{
			Increment = -Increment;
		}
	}

	static const uint8 ColorOffsets[] =
	{
		0, 4, (4 + 2), (6 + 4), (10 + 6)
	};

	const FSlateColor BaseSlateColor = IsHovered ? 
		FAppStyle::Get().GetSlateColor("Colors.Header") : 
		FAppStyle::Get().GetSlateColor("Colors.Panel");

	const FColor BaseColor = BaseSlateColor.GetSpecifiedColor().ToFColor(true);

	const FColor ColorWithOffset(
		BaseColor.R + ColorOffsets[ColorIndex], 
		BaseColor.G + ColorOffsets[ColorIndex], 
		BaseColor.B + ColorOffsets[ColorIndex]);

	return FSlateColor(FLinearColor::FromSRGBColor(ColorWithOffset));
}
