// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/Platform.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"

class FName;
class FText;
struct FSlateBrush;

namespace UE::PropertyEditor::Private
{
	constexpr float PulseAnimationLength = 0.5f;
}

class PropertyEditorConstants
{
public:

	static const FName PropertyFontStyle;
	static const FName CategoryFontStyle;

	static const FName MD_Bitmask;
	static const FName MD_BitmaskEnum;
	static const FName MD_UseEnumValuesAsMaskValuesInEditor;

	static constexpr float PropertyRowHeight = 26.0f;

	static const FText DefaultUndeterminedText;

	static const FSlateBrush* GetOverlayBrush(const TSharedRef<class FPropertyEditor> PropertyEditor);

	static FSlateColor GetRowBackgroundColor(int32 IndentLevel, bool IsHovered);
};
