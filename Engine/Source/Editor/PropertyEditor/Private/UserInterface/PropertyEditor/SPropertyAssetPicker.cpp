// Copyright Epic Games, Inc. All Rights Reserved.
#include "UserInterface/PropertyEditor/SPropertyAssetPicker.h"
#include "Widgets/Layout/SBorder.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#define LOCTEXT_NAMESPACE "PropertyAssetPicker"

void SPropertyAssetPicker::Construct( const FArguments& InArgs )
{
	OnAssetSelected = InArgs._OnAssetSelected;
	OnGetAllowedClasses = InArgs._OnGetAllowedClasses;
	PropertyHandle = InArgs._PropertyHandle;

	ChildSlot
	[
		SNew( SVerticalBox )
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		[
			SAssignNew( AssetPickerAnchor, SMenuAnchor )
			.Placement( MenuPlacement_AboveAnchor )
			.OnGetMenuContent( this, &SPropertyAssetPicker::OnGenerateAssetPicker )
		]
		+ SVerticalBox::Slot()
		.FillHeight(1)
		[
			SNew( SButton )
			.ButtonStyle( FAppStyle::Get(), "HoverHintOnly" )
			.OnClicked( this, &SPropertyAssetPicker::OnClicked )
			.ToolTipText(LOCTEXT("PickButtonLabel", "Pick Asset"))
			.ContentPadding(0.0f)
			.ForegroundColor( FSlateColor::UseForeground() )
			.IsFocusable(false)
			[ 
				SNew( SImage )
				.Image( FAppStyle::GetBrush("PropertyWindow.Button_PickAsset") )
				.ColorAndOpacity( FSlateColor::UseForeground() )
			]
		]
	];
}
 
FReply SPropertyAssetPicker::OnClicked()
{	
	AssetPickerAnchor->SetIsOpen( true );
	return FReply::Handled();
}

TSharedRef<SWidget> SPropertyAssetPicker::OnGenerateAssetPicker()
{
	TArray<const UClass*> AllowedClasses;
	OnGetAllowedClasses.ExecuteIfBound( AllowedClasses );

	if( AllowedClasses.Num() == 0 )
	{
		// Assume all classes are allowed
		AllowedClasses.Add( UObject::StaticClass() );
	}
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	FAssetPickerConfig AssetPickerConfig;
	for ( auto ClassIt = AllowedClasses.CreateConstIterator(); ClassIt; ++ClassIt )
	{
		const UClass* Class = (*ClassIt);
		AssetPickerConfig.Filter.ClassPaths.Add( Class->GetClassPathName() );
	}
	// Allow child classes
	AssetPickerConfig.Filter.bRecursiveClasses = true;
	// Set a delegate for setting the asset from the picker
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SPropertyAssetPicker::OnAssetSelectedFromPicker);
	AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateSP(this, &SPropertyAssetPicker::OnAssetEnterPressedFromPicker);
	AssetPickerConfig.bAllowDragging = false;
	// Use the list view by default
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	// Populate the referencing assets via property handle
	AssetPickerConfig.PropertyHandle = PropertyHandle;
	
	TSharedRef<SWidget> MenuContent = 
		SNew(SBox)
		.HeightOverride(300.0f)
		.WidthOverride(300.0f)
		[
			SNew( SBorder )
			.BorderImage( FAppStyle::GetBrush("Menu.Background") )
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			]
		];

	return MenuContent;
}

void SPropertyAssetPicker::OnAssetSelectedFromPicker( const struct FAssetData& AssetData )
{
	// Close the asset picker
	AssetPickerAnchor->SetIsOpen( false );

	OnAssetSelected.ExecuteIfBound( AssetData.GetAsset() );
}

void SPropertyAssetPicker::OnAssetEnterPressedFromPicker( const TArray<struct FAssetData>& AssetData )
{
	// Close the asset picker
	AssetPickerAnchor->SetIsOpen( false );

	if(AssetData.Num() > 0)
	{
		OnAssetSelected.ExecuteIfBound( AssetData[0].GetAsset() );
	}
}

#undef LOCTEXT_NAMESPACE
