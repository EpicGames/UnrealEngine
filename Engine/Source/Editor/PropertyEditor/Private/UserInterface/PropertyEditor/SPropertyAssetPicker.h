// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SMenuAnchor.h"

class SPropertyAssetPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SPropertyAssetPicker ) {}
		SLATE_EVENT( FOnGetAllowedClasses, OnGetAllowedClasses )
		SLATE_EVENT( FOnAssetSelected, OnAssetSelected )
		SLATE_ARGUMENT( TSharedPtr<IPropertyHandle>, PropertyHandle )
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs );
private:

	FReply OnClicked();

	TSharedRef<SWidget> OnGenerateAssetPicker();

	void OnAssetSelectedFromPicker( const struct FAssetData& AssetData );
	void OnAssetEnterPressedFromPicker( const TArray<struct FAssetData>& AssetData );
private:
	/** Menu anchor for opening and closing the asset picker */
	TSharedPtr< class SMenuAnchor > AssetPickerAnchor;

	FOnGetAllowedClasses OnGetAllowedClasses;
	FOnAssetSelected OnAssetSelected;
	TSharedPtr<IPropertyHandle> PropertyHandle;
};
