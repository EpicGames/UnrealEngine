// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyEditor/SPropertyComboBox.h"

#include "Delegates/Delegate.h"
#include "HAL/PlatformMath.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Templates/TypeHash.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
struct FGeometry;

#define LOCTEXT_NAMESPACE "PropertyComboBox"

void SPropertyComboBox::Construct( const FArguments& InArgs )
{
	ComboItemList = InArgs._ComboItemList.Get();
	RestrictedList = InArgs._RestrictedList.Get();
	RichToolTips = InArgs._RichToolTipList;
	OnSelectionChanged = InArgs._OnSelectionChanged;
	ShowSearchForItemCount = InArgs._ShowSearchForItemCount;
	Font = InArgs._Font;

	// find the initially selected item, if any
	const FString VisibleText = InArgs._VisibleText.Get();
	TSharedPtr<FString> InitiallySelectedItem = NULL;
	for(int32 ItemIndex = 0; ItemIndex < ComboItemList.Num(); ++ItemIndex)
	{
		if(*ComboItemList[ItemIndex].Get() == VisibleText)
		{
			if (RichToolTips.IsValidIndex(ItemIndex))
			{
				SetToolTip(RichToolTips[ItemIndex]);
			}
			else
			{
				SetToolTip(nullptr);
			}

			InitiallySelectedItem = ComboItemList[ItemIndex];
			break;
		}
	}

	auto VisibleTextAttr = InArgs._VisibleText;
	SSearchableComboBox::Construct(SSearchableComboBox::FArguments()
		.Content()
		[
			SNew( STextBlock )
			.Text_Lambda( [=] { return (VisibleTextAttr.IsSet()) ? FText::FromString(VisibleTextAttr.Get()) : FText::GetEmpty(); } )
			.Font( Font )
		]
		.OptionsSource(&ComboItemList)
		.OnGenerateWidget(this, &SPropertyComboBox::OnGenerateComboWidget)
		.OnSelectionChanged(this, &SPropertyComboBox::OnSelectionChangedInternal)
		.OnComboBoxOpening(InArgs._OnComboBoxOpening)
		.InitiallySelectedItem(InitiallySelectedItem)
		.SearchVisibility(this, &SPropertyComboBox::GetSearchVisibility)
		);
}

SPropertyComboBox::~SPropertyComboBox()
{
	if (IsOpen())
	{
		SetIsOpen(false);
	}
}

void SPropertyComboBox::SetSelectedItem( const FString& InSelectedItem )
{
	// Look for the item, due to drag and dropping of Blueprints that may not be in this list.
	for(int32 ItemIndex = 0; ItemIndex < ComboItemList.Num(); ++ItemIndex)
	{
		if(*ComboItemList[ItemIndex].Get() == InSelectedItem)
		{
			if(RichToolTips.IsValidIndex(ItemIndex))
			{
				SetToolTip(RichToolTips[ItemIndex]);
			}
			else
			{
				SetToolTip(nullptr);
			}

			SSearchableComboBox::SetSelectedItem(ComboItemList[ItemIndex]);
			return;
		}
	}

	// Clear selection in this case
	SSearchableComboBox::ClearSelection();

}

void SPropertyComboBox::SetItemList(TArray< TSharedPtr< FString > >& InItemList, TArray< TSharedPtr< SToolTip > >& InRichTooltips, TArray<bool>& InRestrictedList)
{
	ComboItemList = InItemList;
	RichToolTips = InRichTooltips;
	RestrictedList = InRestrictedList;
	RefreshOptions();
}

void SPropertyComboBox::OnSelectionChangedInternal( TSharedPtr<FString> InSelectedItem, ESelectInfo::Type SelectInfo )
{
	bool bEnabled = true;

	if (!InSelectedItem.IsValid())
	{
		return;
	}

	if (RestrictedList.Num() > 0)
	{
		int32 Index = 0;
		for( ; Index < ComboItemList.Num() ; ++Index )
		{
			if( *ComboItemList[Index] == *InSelectedItem )
				break;
		}

		if ( Index < ComboItemList.Num() )
		{
			bEnabled = !RestrictedList[Index];
		}
	}

	if( bEnabled )
	{
		OnSelectionChanged.ExecuteIfBound( InSelectedItem, SelectInfo );
		SetSelectedItem(*InSelectedItem);
	}
}

TSharedRef<SWidget> SPropertyComboBox::OnGenerateComboWidget( TSharedPtr<FString> InComboString )
{
	//Find the corresponding tool tip for this combo entry if any
	TSharedPtr<SToolTip> RichToolTip = nullptr;
	bool bEnabled = true;
	if (RichToolTips.Num() > 0)
	{
		int32 Index = ComboItemList.IndexOfByKey(InComboString);
		if (Index >= 0)
		{
			//A list of tool tips should have been populated in a 1 to 1 correspondance
			check(ComboItemList.Num() == RichToolTips.Num());
			RichToolTip = RichToolTips[Index];

			if( RestrictedList.Num() > 0 )
			{
				bEnabled = !RestrictedList[Index];
			}
		}
	}

	return
		SNew( STextBlock )
		.Text( FText::FromString(*InComboString) )
		.Font( Font )
		.ToolTip(RichToolTip)
		.IsEnabled(bEnabled);
}

EVisibility SPropertyComboBox::GetSearchVisibility() const
{
	if (ShowSearchForItemCount >= 0 && ComboItemList.Num() >= ShowSearchForItemCount)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

FReply SPropertyComboBox::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	const FKey Key = InKeyEvent.GetKey();

	if(Key == EKeys::Up)
	{
		const int32 SelectionIndex = ComboItemList.Find( GetSelectedItem() );
		if ( SelectionIndex >= 1 )
		{
			if (RestrictedList.Num() > 0)
			{
				// find & select the previous unrestricted item
				for(int32 TestIndex = SelectionIndex - 1; TestIndex >= 0; TestIndex--)
				{
					if(!RestrictedList[TestIndex])
					{
						SSearchableComboBox::SetSelectedItem(ComboItemList[TestIndex]);
						break;
					}
				}
			}
			else
			{
				SSearchableComboBox::SetSelectedItem(ComboItemList[SelectionIndex - 1]);
			}
		}

		return FReply::Handled();
	}
	else if(Key == EKeys::Down)
	{
		const int32 SelectionIndex = ComboItemList.Find( GetSelectedItem() );
		if ( SelectionIndex < ComboItemList.Num() - 1 )
		{
			if (RestrictedList.Num() > 0)
			{
				// find & select the next unrestricted item
				for(int32 TestIndex = SelectionIndex + 1; TestIndex < RestrictedList.Num() && TestIndex < ComboItemList.Num(); TestIndex++)
				{
					if(!RestrictedList[TestIndex])
					{
						SSearchableComboBox::SetSelectedItem(ComboItemList[TestIndex]);
						break;
					}
				}
			}
			else
			{
				SSearchableComboBox::SetSelectedItem(ComboItemList[SelectionIndex + 1]);
			}
		}

		return FReply::Handled();
	}

	return SSearchableComboBox::OnKeyDown( MyGeometry, InKeyEvent );
}

#undef LOCTEXT_NAMESPACE
