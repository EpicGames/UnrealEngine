// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Editor/PropertyEditor/Private/UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"

class SToolTip;
class SWidget;
struct FGeometry;
struct FKeyEvent;

class SPropertyComboBox : public SSearchableComboBox
{
public:

	SLATE_BEGIN_ARGS( SPropertyComboBox )
		: _Font( FAppStyle::GetFontStyle( PropertyEditorConstants::PropertyFontStyle ) )
		, _ShowSearchForItemCount(-1)
	{}
		SLATE_ATTRIBUTE( TArray< TSharedPtr< FString > >, ComboItemList )
		SLATE_ATTRIBUTE( TArray< bool >, RestrictedList )
		SLATE_ATTRIBUTE( FString, VisibleText )
		SLATE_ARGUMENT( TArray< TSharedPtr< SToolTip > >, RichToolTipList)
		SLATE_EVENT( FOnSelectionChanged, OnSelectionChanged )
		SLATE_EVENT( FOnComboBoxOpening, OnComboBoxOpening )
		SLATE_ARGUMENT( FSlateFontInfo, Font )
		SLATE_ARGUMENT( int32, ShowSearchForItemCount )
	SLATE_END_ARGS()

	
	void Construct( const FArguments& InArgs );

	~SPropertyComboBox();

	/**
	 *	Sets the currently selected item for the combo box.
	 *	@param InSelectedItem			The name of the item to select.
	 */
	DLLEXPORT void SetSelectedItem( const FString& InSelectedItem );

	/** 
	 *	Sets the item list for the combo box.
	 *	@param InItemList			The item list to use.
	 *	@param InTooltipList		The list of tooltips to use.
	 *	@param InRestrictedList		The list of restricted items.
	 */
	void SetItemList(TArray< TSharedPtr< FString > >& InItemList, TArray< TSharedPtr< SToolTip > >& InRichTooltips, TArray<bool>& InRestrictedList);


private:

	void OnSelectionChangedInternal( TSharedPtr<FString> InSelectedItem, ESelectInfo::Type SelectInfo );

	TSharedRef<SWidget> OnGenerateComboWidget( TSharedPtr<FString> InComboString );
	
	EVisibility GetSearchVisibility() const;

	/** SWidget interface */
	FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;

private:

	/** List of items in our combo box. Only generated once as combo items dont change at runtime */
	TArray< TSharedPtr<FString> > ComboItemList;
	TArray< TSharedPtr< SToolTip > > RichToolTips;
	FOnSelectionChanged OnSelectionChanged;
	FSlateFontInfo Font;
	TArray< bool > RestrictedList;
	int32 ShowSearchForItemCount;
};
