// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPropertyEditor.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "UObject/UnrealType.h"
#include "Widgets/Input/SEditableTextBox.h"

void SPropertyEditor::Construct(const FArguments& InArgs, const TSharedRef<FPropertyEditor>& InPropertyEditor)
{
	PropertyEditor = InPropertyEditor;

	if (ShouldShowValue(InPropertyEditor))
	{
		ChildSlot
		[
			// Make a read only text box so that copy still works
			SNew(SEditableTextBox)
			.Text(InPropertyEditor, &FPropertyEditor::GetValueAsText)
			.ToolTipText(InPropertyEditor, &FPropertyEditor::GetValueAsText)
			.Font(InArgs._Font)
			.IsReadOnly(true)
		];
	}
}

void SPropertyEditor::GetDesiredWidth(float& OutMinDesiredWidth, float& OutMaxDesiredWidth) const
{
	OutMinDesiredWidth = 125.0f;
	OutMaxDesiredWidth = 0.0f;

	if (PropertyEditor.IsValid())
	{
		const FProperty* Property = PropertyEditor->GetProperty();
		if (Property && Property->IsA<FStructProperty>())
		{
			// Struct headers with nothing in them have no min width
			OutMinDesiredWidth = 0;
			OutMaxDesiredWidth = 130.0f;
		}
	}
}

bool SPropertyEditor::ShouldShowValue(const TSharedRef<FPropertyEditor>& InPropertyEditor) const
{
	return PropertyEditor->GetProperty() && !PropertyEditor->GetProperty()->IsA<FStructProperty>();
}
