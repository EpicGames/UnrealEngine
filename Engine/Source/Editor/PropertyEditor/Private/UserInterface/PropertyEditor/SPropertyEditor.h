// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Fonts/SlateFontInfo.h"
#include "Styling/AppStyle.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FPropertyEditor;

class SPropertyEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPropertyEditor)
		: _Font(FAppStyle::GetFontStyle(PropertyEditorConstants::PropertyFontStyle))
	{}
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FPropertyEditor>& InPropertyEditor);
	void GetDesiredWidth(float& OutMinDesiredWidth, float& OutMaxDesiredWidth) const;

private:
	bool ShouldShowValue(const TSharedRef<FPropertyEditor>& InPropertyEditor) const;

private:
	TSharedPtr<FPropertyEditor> PropertyEditor;
};
