// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPropertyEditorArray.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "PropertyEditorHelpers.h"
#include "SDropTarget.h"
#include "Widgets/Text/STextBlock.h"
#include "PropertyEditorConstants.h"

#define LOCTEXT_NAMESPACE "PropertyEditor"

void SPropertyEditorArray::Construct(const FArguments& InArgs, const TSharedRef<FPropertyEditor>& InPropertyEditor)
{
	PropertyEditor = InPropertyEditor;

	TAttribute<FText> TextAttr;
	if (PropertyEditorHelpers::IsStaticArray(*InPropertyEditor->GetPropertyNode()))
	{
		// Static arrays need special case handling for their values
		TextAttr.Set(GetArrayTextValue());
	}
	else
	{
		TextAttr.Bind(this, &SPropertyEditorArray::GetArrayTextValue);
	}

	ChildSlot
	.Padding(0.0f, 0.0f, 2.0f, 0.0f)
	[
		SNew(SDropTarget)
		.OnDropped(this, &SPropertyEditorArray::OnDragDropTarget)
		.OnAllowDrop(this, &SPropertyEditorArray::WillAddValidElements)
		.OnIsRecognized(this, &SPropertyEditorArray::IsValidAssetDropOp)
		.Content()
		[
			SNew(STextBlock)
			.Text(TextAttr)
			.Font(InArgs._Font)
		]
	];

	SetEnabled(TAttribute<bool>(this, &SPropertyEditorArray::CanEdit));
}

bool SPropertyEditorArray::Supports(const TSharedRef<FPropertyEditor>& InPropertyEditor)
{
	const FProperty* NodeProperty = InPropertyEditor->GetProperty();

	return PropertyEditorHelpers::IsStaticArray(*InPropertyEditor->GetPropertyNode())
		|| PropertyEditorHelpers::IsDynamicArray(*InPropertyEditor->GetPropertyNode());
}

void SPropertyEditorArray::GetDesiredWidth(float& OutMinDesiredWidth, float& OutMaxDesiredWidth) const
{
	OutMinDesiredWidth = 170.0f;
	OutMaxDesiredWidth = 170.0f;
}

FText SPropertyEditorArray::GetArrayTextValue() const
{
	check(PropertyEditor.IsValid());
	return PropertyEditorHelpers::GetNumElementsText(LOCTEXT("ArrayText", "Array"), *PropertyEditor.Get());
}

bool SPropertyEditorArray::CanEdit() const
{
	return PropertyEditor.IsValid() ? !PropertyEditor->IsEditConst() : true;
}

FReply SPropertyEditorArray::OnDragDropTarget(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	FObjectPropertyBase* ObjectProperty = nullptr;

	if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(PropertyEditor->GetProperty()))
	{
		ObjectProperty = CastField<FObjectPropertyBase>(ArrayProperty->Inner);
	}

	// Only try to add entries if we are dropping on an asset array
	TSharedPtr<FDragDropOperation> DragOperation = InDragDropEvent.GetOperation();
	if (ObjectProperty && DragOperation && DragOperation->IsOfType<FAssetDragDropOp>())
	{
		TSharedPtr<FAssetDragDropOp> DragDropOp = StaticCastSharedPtr<FAssetDragDropOp>(DragOperation);
		if (DragDropOp.IsValid())
		{
			for (FAssetData AssetData : DragDropOp->GetAssets())
			{
				// if the type matches
				if (AssetData.IsInstanceOf(ObjectProperty->PropertyClass))
				{
					PropertyEditor->AddGivenItem(AssetData.GetObjectPathString());
				}
			}
			// Let this bubble up to the rest of the row
			return FReply::Unhandled();

		}
	}
	return FReply::Unhandled();
}

bool SPropertyEditorArray::IsValidAssetDropOp(TSharedPtr<FDragDropOperation> InOperation)
{
	FObjectPropertyBase* ObjectProperty = nullptr;

	if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(PropertyEditor->GetProperty()))
	{
		ObjectProperty = CastField<FObjectPropertyBase>(ArrayProperty->Inner);
	}

	// Only try to add entries if we are dropping on an asset array
	if (ObjectProperty && InOperation->IsOfType<FAssetDragDropOp>())
	{
		return true;
	}
	return false;
}

bool SPropertyEditorArray::WillAddValidElements(TSharedPtr<FDragDropOperation> InOperation)
{
	FObjectPropertyBase* ObjectProperty = nullptr;

	if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(PropertyEditor->GetProperty()))
	{
		ObjectProperty = CastField<FObjectPropertyBase>(ArrayProperty->Inner);
	}

	// Only try to add entries if we are dropping on an asset array
	if (ObjectProperty && InOperation->IsOfType<FAssetDragDropOp>())
	{
		bool bHasOnlyValidElements = true;
		TSharedPtr<FAssetDragDropOp> DragDropOp = StaticCastSharedPtr<FAssetDragDropOp>(InOperation);
		if (DragDropOp.IsValid())
		{
			for (FAssetData AssetData : DragDropOp->GetAssets())
			{
				// if the type does not match
				if (!AssetData.IsInstanceOf(ObjectProperty->PropertyClass))
				{
					bHasOnlyValidElements = false;
					break;
				}
			}
		}
		return bHasOnlyValidElements;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
