// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Fonts/SlateFontInfo.h"
#include "Internationalization/Text.h"
#include "Styling/AppStyle.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FDragDropEvent;
class FDragDropOperation;
class FPropertyEditor;

class SPropertyEditorArray : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SPropertyEditorArray)
		: _Font(FAppStyle::GetFontStyle(PropertyEditorConstants::PropertyFontStyle))
	{}
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FPropertyEditor>& InPropertyEditor);
	void GetDesiredWidth(float& OutMinDesiredWidth, float& OutMaxDesiredWidth) const;

	static bool Supports(const TSharedRef<FPropertyEditor>& InPropertyEditor);

private:
	FText GetArrayTextValue() const;

	/** @return True if the property can be edited */
	bool CanEdit() const;

	FReply OnDragDropTarget(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent);
	bool IsValidAssetDropOp(TSharedPtr<FDragDropOperation> InOperation);
	bool WillAddValidElements(TSharedPtr<FDragDropOperation> InOperation);

private:
	TSharedPtr<FPropertyEditor> PropertyEditor;
};
