// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyEditor/SPropertyEditorArrayItem.h"

#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "PropertyHandle.h"
#include "PropertyNode.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"
#include "Widgets/Text/STextBlock.h"

/*static*/ TSharedPtr<FTitleMetadataFormatter> FTitleMetadataFormatter::TryParse(TSharedPtr<IPropertyHandle> RootProperty, const FString& TitlePropertyRaw)
{
	if (RootProperty.IsValid() && !TitlePropertyRaw.IsEmpty())
	{
		TSharedRef<FTitleMetadataFormatter> TitleFormatter = MakeShared<FTitleMetadataFormatter>();

		// Simple solution to quickly discover if we need to do the more complex scan for FText formatting, or if
		// we can just take what's there and use it directly.
		if (TitlePropertyRaw.Contains(TEXT("{")))
		{
			TitleFormatter->Format = FText::FromString(TitlePropertyRaw);

			TArray<FString> OutParameterNames;
			FText::GetFormatPatternParameters(TitleFormatter->Format, OutParameterNames);
			for (const FString& ParameterName : OutParameterNames)
			{
				TSharedPtr<IPropertyHandle> Handle = RootProperty->GetChildHandle(FName(*ParameterName), true);
				if (Handle.IsValid())
				{
					TitleFormatter->PropertyHandles.Add(Handle);
				}
				else
				{
					// title property doesn't exist, display an error message
					TitleFormatter->PropertyHandles.Empty();
					TitleFormatter->Format = FText::FromString(TEXT("Invalid Title Property!"));
					break;
				}
			}
		}
		else // Support the old style where it was just a name of a property with no formatting.
		{
			TSharedPtr<IPropertyHandle> Handle = RootProperty->GetChildHandle(FName(*TitlePropertyRaw), true);
			if (Handle.IsValid())
			{
				TitleFormatter->PropertyHandles.Add(Handle);
				TitleFormatter->Format = FText::FromString(TEXT("{") + TitlePropertyRaw + TEXT("}"));
			}
			else
			{
				// title property doesn't exist, display an error message
				TitleFormatter->Format = FText::FromString(TEXT("Invalid Title Property!"));
			}
		}

		return TitleFormatter;
	}

	return TSharedPtr<FTitleMetadataFormatter>();
}

FPropertyAccess::Result FTitleMetadataFormatter::GetDisplayText(FText& OutText) const
{
	FFormatNamedArguments FormatArgs;
	for(TSharedPtr<IPropertyHandle> PropertyHandle : PropertyHandles)
	{
		FText ReplaceValue;
		FPropertyAccess::Result Result = PropertyHandle->GetValueAsDisplayText(ReplaceValue);
		if (Result == FPropertyAccess::Success)
		{
			FormatArgs.Add(PropertyHandle->GetProperty()->GetName(), ReplaceValue);
		}
		else
		{
			return Result;
		}
	}

	OutText = FText::Format(Format, FormatArgs);

	return FPropertyAccess::Success;
}

void SPropertyEditorArrayItem::Construct( const FArguments& InArgs, const TSharedRef< class FPropertyEditor>& InPropertyEditor )
{
	static const FName TitlePropertyFName = FName(TEXT("TitleProperty"));

	PropertyEditor = InPropertyEditor;

	ChildSlot
	.Padding( 0.0f, 0.0f, 5.0f, 0.0f )
	[
		SNew( STextBlock )
		.Text( this, &SPropertyEditorArrayItem::GetValueAsString )
		.Font( InArgs._Font )
	];

	SetEnabled( TAttribute<bool>( this, &SPropertyEditorArrayItem::CanEdit ) );

	// if this is a struct property, try to find a representative element to use as our stand in
	if (PropertyEditor->PropertyIsA( FStructProperty::StaticClass() ))
	{
		const FProperty* MainProperty = PropertyEditor->GetProperty();
		const FProperty* ArrayProperty = MainProperty ? MainProperty->GetOwner<const FProperty>() : nullptr;
		if (ArrayProperty) // should always be true
		{
			TitlePropertyFormatter = FTitleMetadataFormatter::TryParse(PropertyEditor->GetPropertyHandle(), ArrayProperty->GetMetaData(TitlePropertyFName));
		}
	}
}

void SPropertyEditorArrayItem::GetDesiredWidth( float& OutMinDesiredWidth, float& OutMaxDesiredWidth )
{
	OutMinDesiredWidth = 130.0f;
	OutMaxDesiredWidth = 500.0f;
}

bool SPropertyEditorArrayItem::Supports( const TSharedRef< class FPropertyEditor >& PropertyEditor )
{
	const TSharedRef< FPropertyNode > PropertyNode = PropertyEditor->GetPropertyNode();
	const FProperty* Property = PropertyEditor->GetProperty();

	if (!CastField<const FClassProperty>(Property) && PropertyNode->HasNodeFlags(EPropertyNodeFlags::SingleSelectOnly))
	{
		if (Property->GetOwner<FArrayProperty>())
		{
			return true;
		}

		if (Property->GetOwner<FMapProperty>())
		{
			return true;
		}
	}
	return false;
}

FText SPropertyEditorArrayItem::GetValueAsString() const
{
	if (TitlePropertyFormatter.IsValid())
	{
		FText TextOut;
		if (FPropertyAccess::Success == TitlePropertyFormatter->GetDisplayText(TextOut))
		{
			return TextOut;
		}
	}
	
	if( PropertyEditor->GetProperty() && PropertyEditor->PropertyIsA( FStructProperty::StaticClass() ) )
	{
		return FText::Format( NSLOCTEXT("PropertyEditor", "NumStructItems", "{0} members"), FText::AsNumber( PropertyEditor->GetPropertyNode()->GetNumChildNodes() ) );
	}

	return PropertyEditor->GetValueAsDisplayText();
}

bool SPropertyEditorArrayItem::CanEdit() const
{
	return PropertyEditor.IsValid() ? !PropertyEditor->IsEditConst() : true;
}
