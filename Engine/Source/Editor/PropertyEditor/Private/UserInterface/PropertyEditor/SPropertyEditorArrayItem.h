// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Fonts/SlateFontInfo.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "PropertyEditorModule.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FPropertyEditor;
class IPropertyHandle;

struct FTitleMetadataFormatter
{
	FText Format;
	TArray<TSharedPtr<IPropertyHandle>> PropertyHandles;

	FPropertyAccess::Result GetDisplayText(FText& OutText) const;

	static TSharedPtr<FTitleMetadataFormatter> TryParse(TSharedPtr<IPropertyHandle> RootProperty, const FString& TitlePropertyRaw);
};

class SPropertyEditorArrayItem : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SPropertyEditorArrayItem ) 
		: _Font( FAppStyle::GetFontStyle( PropertyEditorConstants::PropertyFontStyle ) ) 
		{}
		SLATE_ATTRIBUTE( FSlateFontInfo, Font )
	SLATE_END_ARGS()

	static bool Supports( const TSharedRef< class FPropertyEditor >& PropertyEditor );

	void Construct( const FArguments& InArgs, const TSharedRef< class FPropertyEditor>& InPropertyEditor );

	void GetDesiredWidth( float& OutMinDesiredWidth, float& OutMaxDesiredWidth );

private:
	FText GetValueAsString() const;

	/** @return True if the property can be edited */
	bool CanEdit() const;

private:
	TSharedPtr<FPropertyEditor> PropertyEditor;
	TSharedPtr<FTitleMetadataFormatter> TitlePropertyFormatter;
};
