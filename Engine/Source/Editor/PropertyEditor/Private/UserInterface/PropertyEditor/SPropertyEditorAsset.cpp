// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyEditor/SPropertyEditorAsset.h"
#include "AssetDefinitionRegistry.h"
#include "AssetDefinition.h"
#include "Engine/Texture.h"
#include "Engine/SkeletalMesh.h"
#include "Components/StaticMeshComponent.h"
#include "Containers/VersePath.h"
#include "Engine/StaticMesh.h"
#include "Engine/LevelScriptActor.h"
#include "Engine/DataAsset.h"
#include "Editor.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Components/SkeletalMeshComponent.h"
#include "Particles/ParticleSystem.h"
#include "UserInterface/PropertyEditor/SPropertyEditorAsset.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "PropertyEditorHelpers.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "AssetToolsModule.h"
#include "SAssetDropTarget.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/Selection.h"
#include "ObjectPropertyNode.h"
#include "PropertyHandleImpl.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "FileHelpers.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "AssetThumbnail.h"
#include "DetailWidgetRow.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "PropertyEditorConstants.h"
#include "PropertyEditorUtils.h"
#include "Misc/EditorPathHelper.h"
#include "Settings/EditorStyleSettings.h"

#define LOCTEXT_NAMESPACE "PropertyEditor"

static const FLazyName NAME_DisallowLevelActorReference = FLazyName("DisallowLevelActorReference");

// Helper to retrieve the correct property that has the applicable metadata.
static const FProperty* GetActualMetadataProperty(const FProperty* Property)
{
	if (FProperty* OuterProperty = Property->GetOwner<FProperty>())
	{
		if (OuterProperty->IsA<FArrayProperty>()
			|| OuterProperty->IsA<FSetProperty>()
			|| OuterProperty->IsA<FMapProperty>())
		{
			return OuterProperty;
		}
	}

	return Property;
}

// Helper to support both meta=(TagName) and meta=(TagName=true) syntaxes
static bool GetTagOrBoolMetadata(const FProperty* Property, FName TagName, bool bDefault)
{
	bool bResult = bDefault;

	if (Property->HasMetaData(TagName))
	{
		bResult = true;

		const FString ValueString = Property->GetMetaData(TagName);
		if (!ValueString.IsEmpty())
		{
			if (ValueString == TEXT("true"))
			{
				bResult = true;
			}
			else if (ValueString == TEXT("false"))
			{
				bResult = false;
			}
		}
	}

	return bResult;
}

static bool GetEditorPathOwnerFromPropertyHandle(const TSharedPtr<IPropertyHandle>& PropertyHandle, UObject*& OutEditorPathOwner)
{
	// If we don't get a proper Handle then consider the context null and valid
	OutEditorPathOwner = nullptr;

	if(PropertyHandle.IsValid())
	{ 
		TArray<UObject*> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);

		if (OuterObjects.Num() > 0)
		{
			UObject* OutReferencer = OuterObjects[0];
			OutEditorPathOwner = FEditorPathHelper::GetEditorPathOwner(OutReferencer);
			for (int32 i = 1; i < OuterObjects.Num(); ++i)
			{
				if (OutEditorPathOwner != FEditorPathHelper::GetEditorPathOwner(OuterObjects[i]))
				{
					OutEditorPathOwner = nullptr;
					return false;
				}
			}
		}
	}

	return true;
}

static FString GetActorEditorPathTooltip(const AActor* InActor)
{
	check(InActor);
	
	TArray<FString> EditorPathOwners;
	const UObject* Context = InActor;

	while (UObject* EditorPathOwner = FEditorPathHelper::GetEditorPathOwner(Context))
	{
		if (AActor* ActorEditorPathOwner = Cast<AActor>(EditorPathOwner))
		{
			EditorPathOwners.Add(ActorEditorPathOwner->GetActorLabel());
		}
		else
		{
			EditorPathOwners.Add(EditorPathOwner->GetName());
		}
		Context = EditorPathOwner;
	}

	// If there are no owners, we don't want the tooltip, as the actor label is already visible in the field
	if (EditorPathOwners.IsEmpty())
	{
		return FString();
	}

	TStringBuilder<256> LabelBuilder;
	for (int32 i = EditorPathOwners.Num() - 1; i >= 0; --i)
	{
		LabelBuilder.Append(EditorPathOwners[i]);
		LabelBuilder.Append(TEXT(" \u2192 "));
	}

	LabelBuilder.Append(InActor->GetActorLabel());
	return LabelBuilder.ToString();
}

bool SPropertyEditorAsset::ShouldDisplayThumbnail(const FArguments& InArgs, const UClass* InObjectClass) const
{
	if (!InArgs._DisplayThumbnail || !InArgs._ThumbnailPool.IsValid())
	{
		return false;
	}

	bool bShowThumbnail = InObjectClass == nullptr || !InObjectClass->IsChildOf(AActor::StaticClass());

	// also check metadata for thumbnail & text display
	const FProperty* PropertyToCheck = nullptr;
	if (PropertyEditor.IsValid())
	{
		PropertyToCheck = PropertyEditor->GetProperty();
	}
	else if (PropertyHandle.IsValid())
	{
		PropertyToCheck = PropertyHandle->GetProperty();
	}

	if (PropertyToCheck != nullptr)
	{
		PropertyToCheck = GetActualMetadataProperty(PropertyToCheck);

		return GetTagOrBoolMetadata(PropertyToCheck, TEXT("DisplayThumbnail"), bShowThumbnail);
	}

	return bShowThumbnail;
}

const FSlateBrush* SPropertyEditorAsset::GetThumbnailBorder() const
{
	static const FName HoveredBorderName("PropertyEditor.AssetThumbnailBorderHovered");
	static const FName RegularBorderName("PropertyEditor.AssetThumbnailBorder");

	return ThumbnailBorder->IsHovered() ? FAppStyle::Get().GetBrush(HoveredBorderName) : FAppStyle::Get().GetBrush(RegularBorderName);
}

void SPropertyEditorAsset::InitializeClassFilters(const FProperty* Property)
{
	if (Property == nullptr)
	{
		AllowedClassFilters.Add(ObjectClass);
		return;
	}

	// Account for the allowed classes specified in the property metadata
	const FProperty* MetadataProperty = GetActualMetadataProperty(Property);

	bExactClass = GetTagOrBoolMetadata(MetadataProperty, "ExactClass", false);
	
	TArray<UObject*> ObjectList;
	if (PropertyEditor && PropertyEditor->GetPropertyHandle()->IsValidHandle())
	{
		PropertyEditor->GetPropertyHandle()->GetOuterObjects(ObjectList);
	}
	else if (PropertyHandle.IsValid())
	{
		PropertyHandle->GetOuterObjects(ObjectList);
	}
	
	PropertyEditorUtils::GetAllowedAndDisallowedClasses(ObjectList, *MetadataProperty, AllowedClassFilters, DisallowedClassFilters, bExactClass, ObjectClass);
	
	if (AllowedClassFilters.Num() == 0)
	{
		// always add the object class to the filters
		AllowedClassFilters.Add(ObjectClass);
	}
}

void SPropertyEditorAsset::InitializeAssetDataTags(const FProperty* Property)
{
	if (Property == nullptr)
	{
		return;
	}

	const FProperty* MetadataProperty = GetActualMetadataProperty(Property);
	const FString DisallowedAssetDataTagsFilterString = MetadataProperty->GetMetaData("DisallowedAssetDataTags");
	if (!DisallowedAssetDataTagsFilterString.IsEmpty())
	{
		TArray<FString> DisallowedAssetDataTagsAndValues;
		DisallowedAssetDataTagsFilterString.ParseIntoArray(DisallowedAssetDataTagsAndValues, TEXT(","), true);

		for (const FString& TagAndOptionalValueString : DisallowedAssetDataTagsAndValues)
		{
			TArray<FString> TagAndOptionalValue;
			TagAndOptionalValueString.ParseIntoArray(TagAndOptionalValue, TEXT("="), true);
			size_t NumStrings = TagAndOptionalValue.Num();
			check((NumStrings == 1) || (NumStrings == 2)); // there should be a single '=' within a tag/value pair

			if (!DisallowedAssetDataTags.IsValid())
			{
				DisallowedAssetDataTags = MakeShared<FAssetDataTagMap>();
			}
			DisallowedAssetDataTags->Add(FName(*TagAndOptionalValue[0]), (NumStrings > 1) ? TagAndOptionalValue[1] : FString());
		}
	}

	const FString RequiredAssetDataTagsFilterString = MetadataProperty->GetMetaData("RequiredAssetDataTags");
	if (!RequiredAssetDataTagsFilterString.IsEmpty())
	{
		TArray<FString> RequiredAssetDataTagsAndValues;
		RequiredAssetDataTagsFilterString.ParseIntoArray(RequiredAssetDataTagsAndValues, TEXT(","), true);

		for (const FString& TagAndOptionalValueString : RequiredAssetDataTagsAndValues)
		{
			TArray<FString> TagAndOptionalValue;
			TagAndOptionalValueString.ParseIntoArray(TagAndOptionalValue, TEXT("="), true);
			size_t NumStrings = TagAndOptionalValue.Num();
			check((NumStrings == 1) || (NumStrings == 2)); // there should be a single '=' within a tag/value pair

			if (!RequiredAssetDataTags.IsValid())
			{
				RequiredAssetDataTags = MakeShared<FAssetDataTagMap>();
			}
			RequiredAssetDataTags->Add(FName(*TagAndOptionalValue[0]), (NumStrings > 1) ? TagAndOptionalValue[1] : FString());
		}
	}
}

bool SPropertyEditorAsset::IsAssetFiltered(const FAssetData& InAssetData)
{
	if (DisallowedAssetDataTags.IsValid())
	{
		for (const auto& DisallowedTagAndValue : *DisallowedAssetDataTags.Get())
		{
			if (InAssetData.TagsAndValues.ContainsKeyValue(DisallowedTagAndValue.Key, DisallowedTagAndValue.Value))
			{
				return true;
			}
		}
	}
	if (RequiredAssetDataTags.IsValid())
	{
		for (const auto& RequiredTagAndValue : *RequiredAssetDataTags.Get())
		{
			if (!InAssetData.TagsAndValues.ContainsKeyValue(RequiredTagAndValue.Key, RequiredTagAndValue.Value))
			{
				// For backwards compatibility compare against short name version of the tag value.
				if (!FPackageName::IsShortPackageName(RequiredTagAndValue.Value) &&
					InAssetData.TagsAndValues.ContainsKeyValue(RequiredTagAndValue.Key, FPackageName::ObjectPathToObjectName(RequiredTagAndValue.Value)))
				{
					continue;
				}
				return true;
			}
		}
	}
	return false;
}

void SPropertyEditorAsset::GenerateCustomAssetPickerButtons(const FAssetData& InAssetData, const TArray<FAssetButtonActionExtension>& InExtensions)
{
	if (!CustomAssetPickerButtonBox.IsValid())
	{
		return;
	}

	CustomAssetPickerButtonBox->ClearChildren();

	for (FAssetButtonActionExtension Extension : InExtensions)
	{
		CustomAssetPickerButtonBox->AddSlot()
			.Padding(2.0f, 0.0f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.WidthOverride(22.0f)
					.HeightOverride(22.0f)
					.IsEnabled(true)
					.ToolTipText(Extension.PickTooltipAttribute)
					[
						SNew(SButton)
							.ButtonStyle(FAppStyle::Get(), "SimpleButton")
							.OnClicked_Lambda([Extension]()-> FReply { return Extension.OnClicked.Execute(); })
							.ContentPadding(0.0f)
							.IsFocusable(false)
							[
								SNew(SImage)
									.Image(Extension.PickBrushAttribute)
									.ColorAndOpacity(FSlateColor::UseForeground())
							]
					]
			];
	}
}

// Awful hack to deal with UClass::FindCommonBase taking an array of non-const classes...
static TArray<UClass*> ConstCastClassArray(TArray<const UClass*>& Classes)
{
	TArray<UClass*> Result;
	for (const UClass* Class : Classes)
	{
		Result.Add(const_cast<UClass*>(Class));
	}

	return Result;
}

void SPropertyEditorAsset::Construct(const FArguments& InArgs, const TSharedPtr<FPropertyEditor>& InPropertyEditor)
{
	PropertyEditor = InPropertyEditor;
	PropertyHandle = InArgs._PropertyHandle;
	OwnerAssetDataArray = InArgs._OwnerAssetDataArray;
	OnIsEnabled = InArgs._IsEnabled;
	OnSetObject = InArgs._OnSetObject;
	OnShouldFilterActor = InArgs._OnShouldFilterActor;
	ObjectPath = InArgs._ObjectPath;
	bDisplayUseSelected = InArgs._DisplayUseSelected;
	bDisplayTransient = InArgs._DisplayTransient;
	NewAssetFactoriesOverride = InArgs._NewAssetFactories;

	// Override this as we stole the value to use as OnIsEnabled for the inner widgets
	SetEnabled(true);

	if(InArgs._InWidgetRow.IsSet() && InArgs._InWidgetRow.GetValue() != nullptr)
	{
		if (!InArgs._InWidgetRow.GetValue()->CopyMenuAction.IsBound())
		{
			InArgs._InWidgetRow.GetValue()->CopyMenuAction = FUIAction(
				FExecuteAction::CreateSP(this, &SPropertyEditorAsset::OnCopy),
				FCanExecuteAction());
		}
		if (!InArgs._InWidgetRow.GetValue()->PasteMenuAction.IsBound())
		{
			InArgs._InWidgetRow.GetValue()->PasteMenuAction = FUIAction(
				FExecuteAction::CreateSP(this, &SPropertyEditorAsset::OnPaste),
				FCanExecuteAction::CreateSP(this, &SPropertyEditorAsset::CanPaste));
		}
	}

	FProperty* Property = nullptr;
	if (PropertyEditor.IsValid())
	{
		Property = PropertyEditor->GetPropertyNode()->GetProperty();
	}
	else if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
	{
		Property = PropertyHandle->GetProperty();
	}

	ObjectClass = InArgs._Class != nullptr ? InArgs._Class : GetObjectPropertyClass(Property);
	bAllowClear = InArgs._AllowClear.IsSet() ? InArgs._AllowClear.GetValue() : (Property ? !(Property->PropertyFlags & CPF_NoClear) : true);
	bAllowCreate = InArgs._AllowCreate.IsSet() ? InArgs._AllowCreate.GetValue() : (Property ? !Property->HasMetaData("NoCreate") : true);
	bIsSoftObjectPath = CastField<FSoftObjectProperty>(Property) != nullptr;
	
	InitializeAssetDataTags(Property);
	InitializeClassFilters(Property);

	// Make the ObjectClass more specific if we only have one class filter
	// eg. if ObjectClass was set to Actor, but AllowedClasses="PointLight", we can limit it to PointLight immediately
	if (AllowedClassFilters.Num() == 1 && DisallowedClassFilters.Num() == 0)
	{
		ObjectClass = const_cast<UClass*>(AllowedClassFilters[0]);
	}
	else
	{
		ObjectClass = UClass::FindCommonBase(ConstCastClassArray(AllowedClassFilters));
	}

	bIsActor = ObjectClass->IsChildOf(AActor::StaticClass());

	auto AppendOnShouldFilterAssetCallback = [this](FOnShouldFilterAsset&& OnShouldFilterAssetCallback)
	{
		check(OnShouldFilterAssetCallback.IsBound());
		if (OnShouldFilterAsset.IsBound())
		{
			OnShouldFilterAsset.BindLambda([BaseOnShouldFilterAsset = OnShouldFilterAsset, OnShouldFilterAssetCallback = MoveTemp(OnShouldFilterAssetCallback)](const FAssetData& InAssetData)
			{
				return BaseOnShouldFilterAsset.Execute(InAssetData) || OnShouldFilterAssetCallback.Execute(InAssetData);
			});
		}
		else
		{
			OnShouldFilterAsset = MoveTemp(OnShouldFilterAssetCallback);
		}
	};

	OnShouldFilterAsset = InArgs._OnShouldFilterAsset;
	
	auto AppendOnShouldFilterActorCallback = [this](FOnShouldFilterActor&& OnShouldFilterActorCallback)
	{
		check(OnShouldFilterActorCallback.IsBound());
		if (OnShouldFilterActor.IsBound())
		{
			OnShouldFilterActor.BindLambda([BaseOnShouldFilterActor = OnShouldFilterActor, OnShouldFilterActorCallback = MoveTemp(OnShouldFilterActorCallback)](const AActor* InActor)
			{
				return BaseOnShouldFilterActor.Execute(InActor) || OnShouldFilterActorCallback.Execute(InActor);
			});
		}
		else
		{
			OnShouldFilterActor = MoveTemp(OnShouldFilterActorCallback);
		}
	};

	if (DisallowedAssetDataTags.IsValid() || RequiredAssetDataTags.IsValid())
	{
		// re-route the filter delegate to our own if we have our own asset data tags filter :
		AppendOnShouldFilterAssetCallback(FOnShouldFilterAsset::CreateRaw(this, &SPropertyEditorAsset::IsAssetFiltered));
	}

	static const FName GetAssetFilterMetaName = "GetAssetFilter";
	static const FName GetActorFilterMetaName = "GetActorFilter";

	const FName GetFilterMetaName = bIsActor ? GetActorFilterMetaName : GetAssetFilterMetaName;
	if (Property && Property->GetOwnerProperty()->HasMetaData(GetFilterMetaName))
	{
		// Add MetaData asset filter
		const FString GetFilterFunctionName = Property->GetOwnerProperty()->GetMetaData(GetFilterMetaName);
		if (!GetFilterFunctionName.IsEmpty())
		{
			TArray<UObject*> ObjectList;
			if (PropertyEditor.IsValid())
			{
				PropertyEditor->GetPropertyHandle()->GetOuterObjects(ObjectList);
			}
			else if (PropertyHandle.IsValid())
			{
				PropertyHandle->GetOuterObjects(ObjectList);
			}
			for (UObject* Object : ObjectList)
			{
				if (!Object)
				{
					continue;
				}
				const UFunction* GetFilterFunction = Object->FindFunction(*GetFilterFunctionName);
				if (ensureMsgf(GetFilterFunction, TEXT("Could not find UFunction %s on %s"), *GetFilterFunctionName, *GetNameSafe(Object)))
				{
					if (UActorComponent* ActorComponent = Cast<UActorComponent>(Object) )
					{
						// Create a soft reference on the component
						TSoftObjectPtr<UActorComponent> ComponentSoftPtr(ActorComponent);
						FSoftComponentReference ComponentReference;
						ComponentReference.OtherActor = ActorComponent->GetOwner();
						ComponentReference.PathToComponent = ActorComponent->GetPathName(ActorComponent->GetOwner());

						if (bIsActor)
						{
							AppendOnShouldFilterActorCallback(FOnShouldFilterActor::CreateLambda([ComponentSoftPtr, ComponentReference, FunctionName =  GetFilterFunction->GetFName()](const AActor* InActor) mutable {
								if (!ComponentSoftPtr.IsValid())
								{
									ComponentSoftPtr = ComponentReference.GetComponent(nullptr);
								}
								if (UActorComponent* Component = ComponentSoftPtr.Get())
								{
									FOnShouldFilterActor UFunctionDelegate = FOnShouldFilterActor::CreateUFunction(Component, FunctionName);
									if (UFunctionDelegate.IsBound())
									{
										return UFunctionDelegate.Execute(InActor);
									}
								}
								return false;
							}));

							continue;
						}

						AppendOnShouldFilterAssetCallback(FOnShouldFilterAsset::CreateLambda([ComponentSoftPtr, ComponentReference, FunctionName =  GetFilterFunction->GetFName()](const FAssetData& AssetData) mutable {
							if (!ComponentSoftPtr.IsValid())
							{
								ComponentSoftPtr = ComponentReference.GetComponent(nullptr);
							}
							if (UActorComponent* Component = ComponentSoftPtr.Get())
							{
								FOnShouldFilterAsset UFunctionDelegate = FOnShouldFilterAsset::CreateUFunction(Component, FunctionName);
								if (UFunctionDelegate.IsBound())
								{
									return UFunctionDelegate.Execute(AssetData);
								}
							}
							return false;
						}));
						
					}
					else if (bIsActor)
					{
						AppendOnShouldFilterActorCallback(FOnShouldFilterActor::CreateUFunction(Object, GetFilterFunction->GetFName()));
					}
					else
					{
						AppendOnShouldFilterAssetCallback(FOnShouldFilterAsset::CreateUFunction(Object, GetFilterFunction->GetFName()));
					}
				}
			}
		}
	}

	TSharedPtr<SHorizontalBox> ValueContentBox = nullptr;
	ChildSlot
	[
		SNew( SAssetDropTarget )
		.bOnlyRecognizeOnDragEnter(InArgs._bOnlyRecognizeOnDragEnter)
		.OnAreAssetsAcceptableForDropWithReason( this, &SPropertyEditorAsset::OnAssetDraggedOver )
		.OnAssetsDropped( this, &SPropertyEditorAsset::OnAssetDropped )
		[
			SAssignNew( ValueContentBox, SHorizontalBox )	
		]
	];

	TAttribute<bool> IsEnabledAttribute(this, &SPropertyEditorAsset::CanEdit);
	TAttribute<FText> TooltipAttribute(this, &SPropertyEditorAsset::OnGetToolTip);

	EditorPathOwner = nullptr;
	if (bIsActor && bIsSoftObjectPath && FEditorPathHelper::IsEnabled())
	{
		if (!GetEditorPathOwnerFromPropertyHandle(GetMostSpecificPropertyHandle(), EditorPathOwner))
		{
			IsEnabledAttribute.Set(false);
			TooltipAttribute.Set(LOCTEXT("InvalidActorEditorPathOwner", "Editing this value with different referencing context is not allowed"));
		}
	}

	if (Property)
	{
		TArray<UObject*> ObjectList;
		if (PropertyEditor.IsValid())
		{
			PropertyEditor->GetPropertyHandle()->GetOuterObjects(ObjectList);
		}

		const FProperty* PropToConsider = GetActualMetadataProperty(Property);
		if (PropToConsider->HasAnyPropertyFlags(CPF_EditConst | CPF_DisableEditOnTemplate))
		{
			// NOTE: This code decides that 99% of structs are "defaults" which is not technically correct, 
			// but we want to stop hard actor references from being set in places like data tables without banning soft references.
			// The actor check should get refactored to be independent of EditOnTemplate and do more explicit checks for world-owned object references
			if (ObjectList.Num() == 0)
			{
				IsEnabledAttribute.Set(false);
				TooltipAttribute.Set(LOCTEXT("VariableHasDisableEditOnTemplate", "Editing this value in structure's defaults is not allowed"));
			}
			else
			{
				// Go through all the found objects and see if any are a CDO, we can't set an actor in a CDO default.
				for (const UObject* Obj : ObjectList)
				{
					if (Obj->IsTemplate() && !Obj->IsA<ALevelScriptActor>())
					{
						IsEnabledAttribute.Set(false);
						TooltipAttribute.Set(LOCTEXT("VariableHasDisableEditOnTemplateTooltip", "Editing this value in a Class Default Object is not allowed"));
						break;
					}

				}
			}

		}

		if (bIsActor && IsEnabledAttribute.Get())
		{
			for (const UObject* Obj : ObjectList)
			{
				if (const UClass* Class = Obj->GetClass())
				{
					if (Class->GetBoolMetaDataHierarchical(NAME_DisallowLevelActorReference))
					{
						IsEnabledAttribute.Set(false);
						TooltipAttribute.Set(LOCTEXT("ObjectDisallowLevelActorReference", "Object doesn't allow level actor reference"));
						break;
					}
				}
			}
		}
	}

	bool bOldEnableAttribute = IsEnabledAttribute.Get();
	if (bOldEnableAttribute && !InArgs._EnableContentPicker)
	{
		IsEnabledAttribute.Set(false);
	}

	AssetComboButton = SNew(SComboButton)
		.ToolTipText(TooltipAttribute)
		.OnGetMenuContent( this, &SPropertyEditorAsset::OnGetMenuContent )
		.OnMenuOpenChanged( this, &SPropertyEditorAsset::OnMenuOpenChanged )
		.IsEnabled(IsEnabledAttribute)
		.ButtonContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image( this, &SPropertyEditorAsset::GetStatusIcon )
			]

			+SHorizontalBox::Slot()
			.FillWidth(1)
			.VAlign(VAlign_Center)
			[
				// Show the name of the asset or actor
				SNew(STextBlock)
				.Font( FAppStyle::GetFontStyle( PropertyEditorConstants::PropertyFontStyle ) )
				.Text(this,&SPropertyEditorAsset::OnGetAssetName)
                .OverflowPolicy(
                    GetDefault<UEditorStyleSettings>()->bEnableMiddleEllipsis
                        ? TOptional(ETextOverflowPolicy::MiddleEllipsis)
                        : NullOpt
                )
			]
		];

	if (bOldEnableAttribute && !InArgs._EnableContentPicker)
	{
		IsEnabledAttribute.Set(true);
	}

	TSharedPtr<SWidget> ButtonBoxWrapper;
	TSharedRef<SHorizontalBox> ButtonBox = SNew( SHorizontalBox );
	
	TSharedPtr<SHorizontalBox> CustomContentBox;

	if (ShouldDisplayThumbnail(InArgs, ObjectClass))
	{
		FObjectOrAssetData Value; 
		GetValue( Value, FObjectOrAssetData::EAssetDataOptions::None );

		AssetThumbnail = MakeShareable( new FAssetThumbnail( Value.AssetData, InArgs._ThumbnailSize.X, InArgs._ThumbnailSize.Y, InArgs._ThumbnailPool ) );

		float ThumbnailPadding = 1.f;
		FAssetThumbnailConfig AssetThumbnailConfig;
		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			ThumbnailPadding = 0.f;
			AssetThumbnailConfig.BorderPadding = FMargin(1.f);
			AssetThumbnailConfig.AssetBorderImageOverride = TAttribute<const FSlateBrush*>::CreateSP(this, &SPropertyEditorAsset::GetThumbnailBorder);
		}
		else
		{
			//Only override the color if we have a meaningful type specified.
			//=> When override is done, it will internally prevent to switch the color strip to the current asset type color, when updated the asset.
			if (ObjectClass != nullptr && ObjectClass != UObject::StaticClass())
			{
				FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
				TSharedPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(ObjectClass).Pin();

				if (AssetTypeActions.IsValid())
				{
					AssetThumbnailConfig.AssetTypeColorOverride = AssetTypeActions->GetTypeColor();
				}
			}
		}

		TSharedRef<SWidget> ThumbnailWidget = AssetThumbnail->MakeThumbnailWidget(AssetThumbnailConfig);
		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			ThumbnailWidget->SetToolTip(TSharedPtr<IToolTip>());
		}

		ValueContentBox->AddSlot()
		.Padding(0.0f,3.0f,5.0f,0.0f)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SBorder)
			.Visibility(EVisibility::SelfHitTestInvisible)
			.Padding(FMargin(0.0f, 0.0f, 4.0f, 4.0f))
			.BorderImage(FAppStyle::Get().GetBrush("PropertyEditor.AssetTileItem.DropShadow"))
			[
				SNew(SOverlay)
				+SOverlay::Slot()
				.Padding(ThumbnailPadding)
				[
					SAssignNew(ThumbnailBorder, SBorder)
					.Padding(0)
					.BorderImage(FStyleDefaults::GetNoBrush())
					.OnMouseDoubleClick(this, &SPropertyEditorAsset::OnAssetThumbnailDoubleClick)
					[
						SNew(SBox)
						.ToolTipText(TooltipAttribute)
						.WidthOverride(static_cast<float>(InArgs._ThumbnailSize.X))
						.HeightOverride(static_cast<float>(InArgs._ThumbnailSize.Y))
						[
							ThumbnailWidget
						]
					]
				]
				+ SOverlay::Slot()
				[
					SNew(SImage)
					.Image(this, &SPropertyEditorAsset::GetThumbnailBorder)
					.Visibility(UE::Editor::ContentBrowser::IsNewStyleEnabled() ? EVisibility::Collapsed : EVisibility::SelfHitTestInvisible)
				]
			]
		];

	
		ValueContentBox->AddSlot()
		.Padding(0.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoHeight()
			[
				AssetComboButton.ToSharedRef()
			]
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoHeight()
			[
				SAssignNew(CustomContentBox, SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SAssignNew(ButtonBoxWrapper, SBox)
					.Padding(FMargin(0.0f, 2.0f, 4.0f, 2.0f))
					[
						ButtonBox
					]
				]
			]
		];
	}
	else
	{
		ValueContentBox->AddSlot()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.VAlign( VAlign_Center )
			[
				SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				[
					AssetComboButton.ToSharedRef()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SAssignNew(ButtonBoxWrapper, SBox)
					.Padding(FMargin(4.0f,0.0f))
					[
						ButtonBox
					]
				]
			]
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoHeight()
			[
				SAssignNew(CustomContentBox, SHorizontalBox)
			]
		];
	}

	if( InArgs._CustomContentSlot.Widget != SNullWidget::NullWidget )
	{
		CustomContentBox->AddSlot()
		.VAlign( VAlign_Center )
		.Padding( FMargin( 0.0f, 2.0f ) )
		[
			InArgs._CustomContentSlot.Widget
		];
	}

	if( bDisplayUseSelected )
	{
		ButtonBox->AddSlot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding( 2.0f, 0.0f )
		[
			PropertyCustomizationHelpers::MakeUseSelectedButton( FSimpleDelegate::CreateSP( this, &SPropertyEditorAsset::OnUse ), FText(), IsEnabledAttribute, bIsActor )
		];
	}

	if( InArgs._DisplayBrowse )
	{
		FSimpleDelegate OnBrowseDelegate = InArgs._OnBrowseOverride.IsBound() ? InArgs._OnBrowseOverride : FSimpleDelegate::CreateSP(this, &SPropertyEditorAsset::OnBrowse);

		// Only the default SPropertyEditorAsset::OnBrowse delegate supports alt+click to directly open the asset by default, so we only modify the icon if there is no browse override
		TAttribute<const FSlateBrush*> IconDelegate = InArgs._OnBrowseIconOverride.IsSet() ? InArgs._OnBrowseIconOverride
			: !InArgs._OnBrowseOverride.IsBound() ? TAttribute<const FSlateBrush*>::CreateSP(this, &SPropertyEditorAsset::GetOnBrowseIcon)
			: TAttribute<const FSlateBrush*>();

		TAttribute<FText> OnBrowseToolTip = InArgs._OnBrowseToolTipOverride.IsSet() ? InArgs._OnBrowseToolTipOverride : TAttribute<FText>(this, &SPropertyEditorAsset::GetOnBrowseToolTip);

		// Browse to is readonly. There are cases the reference is set elsewhere other than from the content picker, and we want to allow browsing to.
		bool bIsPickerEnabled = IsEnabledAttribute.Get();
		TAttribute<bool> IsBrowseEnabledAttribute = TAttribute<bool>::CreateSPLambda(this,
				[bIsPickerEnabled, this]()
				{
					if (bIsPickerEnabled)
					{
						return true;
					}

					FObjectOrAssetData Value;
					GetValue(Value, FObjectOrAssetData::EAssetDataOptions::None);

					return Value.IsValid();
				});

		ButtonBox->AddSlot()
		.Padding( 2.0f, 0.0f )
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			PropertyCustomizationHelpers::MakeBrowseButton(
				OnBrowseDelegate,
				OnBrowseToolTip,
				IsBrowseEnabledAttribute,
				bIsActor,
				IconDelegate
				)
		];
	}

	if( bIsActor )
	{
		TSharedRef<SWidget> ActorPicker = PropertyCustomizationHelpers::MakeInteractiveActorPicker( FOnGetAllowedClasses::CreateSP(this, &SPropertyEditorAsset::OnGetAllowedClasses), FOnShouldFilterActor::CreateSP(this, &SPropertyEditorAsset::IsFilteredActor), FOnActorSelected::CreateSP(this, &SPropertyEditorAsset::OnActorSelected));
		ActorPicker->SetEnabled( IsEnabledAttribute );

		ButtonBox->AddSlot()
		.Padding( 2.0f, 0.0f )
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			ActorPicker
		];
	}
		
	FObjectOrAssetData Value;
	GetValue(Value, FObjectOrAssetData::EAssetDataOptions::SkipAssetRegistryTagsGathering);

	if (const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForAsset(Value.AssetData))
	{
		TArray<FAssetButtonActionExtension> AssetButtonActionExtension;
		AssetDefinition->GetAssetActionButtonExtensions(Value.AssetData, AssetButtonActionExtension);

		if (!AssetButtonActionExtension.IsEmpty())
		{
			CustomAssetPickerButtonBox = SNew(SHorizontalBox);
			ButtonBox->AddSlot()
			[
				CustomAssetPickerButtonBox.ToSharedRef()
			];

			GenerateCustomAssetPickerButtons(Value.AssetData, AssetButtonActionExtension);
		}
	}	

	if (InArgs._CustomButtonSlot.Widget != SNullWidget::NullWidget)
	{
		ButtonBox->AddSlot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.0f, 2.0f))
			[
				InArgs._CustomButtonSlot.Widget
			];
	}

	NumButtons = ButtonBox->NumSlots();
	
	if (ButtonBoxWrapper.IsValid())
	{
		ButtonBoxWrapper->SetVisibility(NumButtons > 0 ? EVisibility::Visible : EVisibility::Collapsed);
	}

	// Create a default empty tooltip to prevent SPropertyValueWidget::Construct from setting our tooltip to the default FPropertyEditor::GetValueAsText.
	// We do this because we create a user friendly tooltip in SPropertyEditorAsset::OnGetToolTip, but only want it to appear on certain child widgets to avoid it covering up any buttons.
	SetToolTipText(TAttribute<FText>::CreateLambda([]() { return FText::GetEmpty(); }));
}

void SPropertyEditorAsset::GetDesiredWidth( float& OutMinDesiredWidth, float &OutMaxDesiredWidth )
{
	OutMinDesiredWidth = 250.f;
	OutMaxDesiredWidth = 350.f;

	if (!AssetThumbnail.IsValid())
	{
		static const float ButtonWidth = 20.0f /* button width */ + 4.0f /* padding */;

		const float AdditionalButtonSize = NumButtons * ButtonWidth + 8.0f /* button box padding */;
		OutMinDesiredWidth += AdditionalButtonSize;
		OutMaxDesiredWidth += AdditionalButtonSize;
	}
}

const FSlateBrush* SPropertyEditorAsset::GetStatusIcon() const
{
	static FSlateNoResource EmptyBrush = FSlateNoResource();

	EActorReferenceState State = GetActorReferenceState();

	if (State == EActorReferenceState::Unknown)
	{
		return FAppStyle::GetBrush("Icons.Warning");
	}
	else if (State == EActorReferenceState::Error)
	{
		return FAppStyle::GetBrush("Icons.Error");
	}

	return &EmptyBrush;
}

SPropertyEditorAsset::EActorReferenceState SPropertyEditorAsset::GetActorReferenceState() const
{
	if (bIsActor)
	{
		FObjectOrAssetData Value;
		GetValue(Value, FObjectOrAssetData::EAssetDataOptions::SkipAssetRegistryTagsGathering );

		if (Value.Object != nullptr)
		{
			// If this is not an actual actor, this is broken
			if (!Value.Object->IsA(AActor::StaticClass()))
			{
				return EActorReferenceState::Error;
			}

			return EActorReferenceState::Loaded;
		}
		else if (Value.ObjectPath.IsNull())
		{
			return EActorReferenceState::Null;
		}
		else
		{
			// Get a path pointing to the owning map
			FSoftObjectPath MapObjectPath = Value.ObjectPath.GetWithoutSubPath();

			if (UObject* MapObject = MapObjectPath.ResolveObject())
			{
				UWorld* World = Cast<UWorld>(MapObject);

				// In a partitioned world, the world object will exist but the actor itself can be unloaded
				if (World && World->IsPartitionedWorld())
				{
					UObject* Object = nullptr;
					if (World->ResolveSubobject(*Value.ObjectPath.GetSubPathString(), Object, /*bLoadIfExists*/false))
					{
						return EActorReferenceState::Exists;
					}
				}

				return EActorReferenceState::Error;
			}

			return EActorReferenceState::Unknown;
		}
	}
	return EActorReferenceState::NotAnActor;
}

void SPropertyEditorAsset::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if( AssetThumbnail.IsValid() && !UE::IsSavingPackage() && !IsGarbageCollecting() )
	{
		// Ensure the thumbnail is up to date
		FObjectOrAssetData Value;
		GetValue( Value, FObjectOrAssetData::EAssetDataOptions::SkipAssetRegistryTagsGathering );

		// If the thumbnail is not the same as the object value set the thumbnail to the new value
		if( !(AssetThumbnail->GetAssetData() == Value.AssetData) )
		{
			AssetThumbnail->SetAsset( Value.AssetData );
		}
	}
}

bool SPropertyEditorAsset::Supports(const TSharedRef<FPropertyEditor >& InPropertyEditor)
{
	const TSharedRef<FPropertyNode> PropertyNode = InPropertyEditor->GetPropertyNode();
	if (PropertyNode->HasNodeFlags(EPropertyNodeFlags::EditInlineNew))
	{
		return false;
	}

	if (PropertyNode->HasNodeFlags(EPropertyNodeFlags::SupportsDynamicInstancing) && PropertyNode->HasNodeFlags(EPropertyNodeFlags::DynamicInstance))
	{
		return false;
	}

	return Supports(PropertyNode->GetProperty());
}

bool SPropertyEditorAsset::Supports(const FProperty* NodeProperty)
{
	const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(NodeProperty);
	const FInterfaceProperty* InterfaceProperty = CastField<const FInterfaceProperty>(NodeProperty);

	if ((ObjectProperty != nullptr || InterfaceProperty != nullptr)
		 && !NodeProperty->IsA(FClassProperty::StaticClass()) 
		 && !NodeProperty->IsA(FSoftClassProperty::StaticClass()))
	{
		return true;
	}
	
	return false;
}

bool SPropertyEditorAsset::ParseAssetText(const FString& InText, FAssetData& OutAssetData)
{
	OutAssetData = {};

	FString PossibleObjectPath = FPackageName::ExportTextPathToObjectPath(InText);

	if (PossibleObjectPath.IsEmpty())
	{
		return false;
	}

	if (PossibleObjectPath == TEXT("None"))
	{
		return true;
	}
	
	// All supported paths start with a /
	if (PossibleObjectPath[0] == TEXT('/'))
	{
		constexpr bool bIncludeOnlyOnDiskAssets = false;
		constexpr bool bSkipARFilteredAssets = true;

		const bool bTextIsExportTextPath = PossibleObjectPath != InText;

		// Check if we just have a package name.
		int32 ObjectNameDelimiterIndex;
		if (!bTextIsExportTextPath && !PossibleObjectPath.FindChar(TEXT('.'), ObjectNameDelimiterIndex))
		{
			// Assume that the object we're trying to load is the main asset inside of the package 
			// which usually has the same name as the short package name.
			PossibleObjectPath = PossibleObjectPath + TEXT('.') + FPackageName::GetShortName(PossibleObjectPath);
		}

		if (PossibleObjectPath.Len() < NAME_SIZE)
		{
			OutAssetData = IAssetRegistry::GetChecked().GetAssetByObjectPath(FSoftObjectPath(PossibleObjectPath), bIncludeOnlyOnDiskAssets, bSkipARFilteredAssets);
			if (OutAssetData.IsValid())
			{
				return true;
			}
		}

		// If it wasn't an export text, object or package path, it might be a Verse path.
		if (!bTextIsExportTextPath)
		{
			FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
			if (AssetToolsModule.Get().ShowingContentVersePath())
			{
				UE::Core::FVersePath VersePath;
				if (UE::Core::FVersePath::TryMake(VersePath, InText))
				{
					OutAssetData = AssetToolsModule.Get().FindAssetByVersePath(VersePath, bIncludeOnlyOnDiskAssets, bSkipARFilteredAssets);
					if (OutAssetData.IsValid())
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

TSharedRef<SWidget> SPropertyEditorAsset::OnGetMenuContent()
{
	FObjectOrAssetData Value;
	GetValue(Value, FObjectOrAssetData::EAssetDataOptions::None);

	if (bIsActor)
	{
		bool bShowTransient = false;
		if (bDisplayTransient)
		{
			FProperty* Property = nullptr;
			if (PropertyEditor.IsValid())
			{
				Property = PropertyEditor->GetPropertyNode()->GetProperty();
			}
			else if (PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
			{
				Property = PropertyHandle->GetProperty();
			}

			if (Property)
			{
				// Transient objects need to be keyable to ensure they can be handle properly through their lifetime.
				bShowTransient = Property->HasAnyPropertyFlags(EPropertyFlags::CPF_Interp);
			}
		}
		return PropertyCustomizationHelpers::MakeActorPickerWithMenu(Cast<AActor>(Value.Object),
																	 bAllowClear,
																	 bIsSoftObjectPath && FEditorPathHelper::IsEnabled(),
																	 FOnShouldFilterActor::CreateSP( this, &SPropertyEditorAsset::IsFilteredActor ),
																	 FOnActorSelected::CreateSP( this, &SPropertyEditorAsset::OnActorSelected),
																	 FSimpleDelegate::CreateSP( this, &SPropertyEditorAsset::CloseComboButton ),
																	 FSimpleDelegate::CreateSP( this, &SPropertyEditorAsset::OnUse), 
																	 bDisplayUseSelected,
																	 bShowTransient);
	}
	else
	{
		TArray<UFactory*> NewAssetFactories;
		if (bAllowCreate)
		{
			if (NewAssetFactoriesOverride.IsSet())
			{
				NewAssetFactories = NewAssetFactoriesOverride.GetValue();
			}
			// If there are more allowed classes than just UObject
			else if (AllowedClassFilters.Num() > 1 || !AllowedClassFilters.Contains(UObject::StaticClass()))
			{
				NewAssetFactories = PropertyCustomizationHelpers::GetNewAssetFactoriesForClasses(AllowedClassFilters, DisallowedClassFilters);
			}
		}

		return PropertyCustomizationHelpers::MakeAssetPickerWithMenu(Value.AssetData,
																	 bAllowClear,
																	 AllowedClassFilters,
																	 DisallowedClassFilters,
																	 NewAssetFactories,
																	 OnShouldFilterAsset,
																	 FOnAssetSelected::CreateSP(this, &SPropertyEditorAsset::OnAssetSelected),
																	 FSimpleDelegate::CreateSP(this, &SPropertyEditorAsset::CloseComboButton),
																	 GetMostSpecificPropertyHandle(),
																	 OwnerAssetDataArray);
	}
}

void SPropertyEditorAsset::OnMenuOpenChanged(bool bOpen)
{
	if (bOpen == false)
	{
		AssetComboButton->SetMenuContent(SNullWidget::NullWidget);
	}
}

bool SPropertyEditorAsset::IsFilteredActor( const AActor* const Actor ) const
{
	bool IsAllowed = Actor != nullptr && Actor->IsA(ObjectClass) && !Actor->IsChildActor() && IsClassAllowed(Actor->GetClass());

	if (IsAllowed)
	{
		// If we have an EditorPathOwner referenced actor needs to be in same EditorPathOwner
		IsAllowed = !EditorPathOwner || FEditorPathHelper::IsInEditorPath(EditorPathOwner, Actor);
	}

	if (IsAllowed && OnShouldFilterActor.IsBound())
	{
		IsAllowed = OnShouldFilterActor.Execute(Actor);
	}
	return IsAllowed;
}

void SPropertyEditorAsset::OpenComboButton()
{
	if (AssetComboButton.IsValid())
	{
		AssetComboButton->SetIsOpen(true);
	}
}

void SPropertyEditorAsset::CloseComboButton()
{
	AssetComboButton->SetIsOpen(false);
}

FText SPropertyEditorAsset::OnGetAssetName() const
{
	FObjectOrAssetData Value; 
	FPropertyAccess::Result Result = GetValue( Value, FObjectOrAssetData::EAssetDataOptions::SkipAssetRegistryTagsGathering );

	FText Name = LOCTEXT("None", "None");
	if( Result == FPropertyAccess::Success )
	{
		if(Value.Object != nullptr)
		{
			if( bIsActor )
			{
				AActor* Actor = Cast<AActor>(Value.Object);

				if (Actor)
				{
					Name = FText::AsCultureInvariant(Actor->GetActorLabel());
				}
				else
				{
					Name = FText::AsCultureInvariant(Value.Object->GetName());
				}
			}
			else if (UField* AsField = Cast<UField>(Value.Object))
			{
				Name = AsField->GetDisplayNameText();
			}
			else if (UDataAsset* DataAsset = Cast<UDataAsset>(Value.Object))
			{
				Name = DataAsset->GetDisplayNameText();
			}
			else
			{
				Name = FText::AsCultureInvariant(Value.Object->GetName());
			}
		}
		else if( Value.AssetData.IsValid() )
		{
			Name = FText::AsCultureInvariant(Value.AssetData.AssetName.ToString());
		}
		else if (Value.ObjectPath.IsValid())
		{
			Name = FText::AsCultureInvariant(Value.ObjectPath.ToString());
		}
	}
	else if( Result == FPropertyAccess::MultipleValues )
	{
		Name = PropertyEditorConstants::DefaultUndeterminedText;
	}

	return Name;
}

FText SPropertyEditorAsset::OnGetAssetClassName() const
{
	const UClass* Class = GetDisplayedClass();
	if(Class)
	{
		return FText::AsCultureInvariant(Class->GetName());
	}
	return FText::GetEmpty();
}

FText SPropertyEditorAsset::OnGetToolTip() const
{
	FObjectOrAssetData Value; 
	FPropertyAccess::Result Result = GetValue( Value, FObjectOrAssetData::EAssetDataOptions::SkipAssetRegistryTagsGathering );

	FText ToolTipText = FText::GetEmpty();

	if( Result == FPropertyAccess::Success )
	{
		if ( bIsActor )
		{
			// Always show full path instead of label
			EActorReferenceState State = GetActorReferenceState();
			FFormatNamedArguments Args;
			Args.Add(TEXT("Actor"), FText::AsCultureInvariant(Value.ObjectPath.ToString()));
			if (State == EActorReferenceState::Null)
			{
				ToolTipText = LOCTEXT("EmptyActorReference", "None");
			}
			else if (State == EActorReferenceState::Error)
			{
				ToolTipText = FText::Format(LOCTEXT("BrokenActorReference", "Broken reference to Actor ID '{Actor}', it was deleted or renamed"), Args);
			}
			else if (State == EActorReferenceState::Exists)
			{
				ToolTipText = FText::Format(LOCTEXT("ExistsActorReference", "Unloaded reference to Actor ID '{Actor}', use Browse to pin actor"), Args);
			}
			else if (State == EActorReferenceState::Unknown)
			{
				ToolTipText = FText::Format(LOCTEXT("UnknownActorReference", "Unloaded reference to Actor ID '{Actor}', use Browse to load level"), Args);
			}
			else
			{
				ToolTipText = FText::Format(LOCTEXT("GoodActorReference", "Reference to Actor ID '{Actor}'"), Args);

				if (const AActor* Actor = Cast<AActor>(Value.Object))
				{
					FString Path = GetActorEditorPathTooltip(Actor);
					if (!Path.IsEmpty())
					{
						FText OwnerPath = FText::Format(
							LOCTEXT("ReferenceOwnerPath", "Actor path: {0}"),
							FText::AsCultureInvariant(MoveTemp(Path))
						);

						ToolTipText = FText::Format(
							INVTEXT("{0}\n\n{1}"),
							MoveTemp(OwnerPath),
							MoveTemp(ToolTipText)
						);
					}
				}
			}
		}
		else if( Value.Object != nullptr )
		{
			UE::Core::FVersePath VersePath;

			if (FAssetToolsModule::GetModule().Get().ShowingContentVersePath())
			{
				VersePath = Value.Object->GetVersePath();
			}

			if (VersePath.IsValid())
			{
				ToolTipText = MoveTemp(VersePath).AsText();
			}
			else
			{
				// Display the package name which is a valid path to the object without redundant information
				ToolTipText = FText::AsCultureInvariant(Value.Object->GetOutermost()->GetName());
			}
		}
		else if( Value.AssetData.IsValid() )
		{
			UE::Core::FVersePath VersePath;

			if (FAssetToolsModule::GetModule().Get().ShowingContentVersePath())
			{
				VersePath = Value.AssetData.GetVersePath();
			}

			if (VersePath.IsValid())
			{
				ToolTipText = MoveTemp(VersePath).AsText();
			}
			else
			{
				ToolTipText = FText::AsCultureInvariant(Value.AssetData.PackageName.ToString());
			}
		}
	}
	else if( Result == FPropertyAccess::MultipleValues )
	{
		ToolTipText = PropertyEditorConstants::DefaultUndeterminedText;
	}

	if( ToolTipText.IsEmpty() )
	{
		UE::Core::FVersePath VersePath;

		if (FAssetToolsModule::GetModule().Get().ShowingContentVersePath())
		{
			FSoftObjectPath SoftObjectPath(ObjectPath.Get());
			if (SoftObjectPath.IsValid())
			{
				const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
				FAssetData AssetData;
				if (AssetRegistryModule.Get().TryGetAssetByObjectPath(SoftObjectPath, AssetData) == UE::AssetRegistry::EExists::Exists)
				{
					VersePath = AssetData.GetVersePath();
				}
			}
		}

		if (VersePath.IsValid())
		{
			ToolTipText = MoveTemp(VersePath).AsText();
		}
		else
		{
			ToolTipText = FText::AsCultureInvariant(ObjectPath.Get());
		}
	}

	return ToolTipText;
}

void SPropertyEditorAsset::SetValue( const FAssetData& AssetData )
{
	AssetComboButton->SetIsOpen(false);

	if (CanSetBasedOnCustomClasses(AssetData))
	{
		FText AssetReferenceFilterFailureReason;
		if (CanSetBasedOnAssetReferenceFilter(AssetData, &AssetReferenceFilterFailureReason))
		{
			if (PropertyEditor.IsValid())
			{
				PropertyEditor->GetPropertyHandle()->SetValue(AssetData);

				if (const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForAsset(AssetData))
				{
					TArray<FAssetButtonActionExtension> AssetButtonActionExtension;
					AssetDefinition->GetAssetActionButtonExtensions(AssetData, AssetButtonActionExtension);
					GenerateCustomAssetPickerButtons(AssetData, AssetButtonActionExtension);
				}
			}

			OnSetObject.ExecuteIfBound(AssetData);
		}
		else if (!AssetReferenceFilterFailureReason.IsEmpty())
		{
			FNotificationInfo Info(AssetReferenceFilterFailureReason);
			Info.ExpireDuration = 4.f;
			FSlateNotificationManager::Get().AddNotification(Info);
		}
	}
}

FPropertyAccess::Result SPropertyEditorAsset::GetValue( FObjectOrAssetData& OutValue, FObjectOrAssetData::EAssetDataOptions AssetDataOptions ) const
{
	// Potentially accessing the value while garbage collecting or saving the package could trigger a crash.
	// so we fail to get the value when that is occurring.
	if ( UE::IsSavingPackage() || IsGarbageCollecting() )
	{
		return FPropertyAccess::Fail;
	}

	FPropertyAccess::Result Result = FPropertyAccess::Fail;

	if( PropertyEditor.IsValid() && PropertyEditor->GetPropertyHandle()->IsValidHandle() )
	{
		UObject* Object = nullptr;
		Result = PropertyEditor->GetPropertyHandle()->GetValue(Object);

		if (Object == nullptr)
		{
			// Check to see if it's pointing to an unloaded object
			FString CurrentObjectPath;
			PropertyEditor->GetPropertyHandle()->GetValueAsFormattedString( CurrentObjectPath );

			if (CurrentObjectPath.Len() > 0 && CurrentObjectPath != TEXT("None"))
			{
				FSoftObjectPath SoftObjectPath = FSoftObjectPath(CurrentObjectPath);

				if (SoftObjectPath.IsAsset())
				{
					if (!CachedAssetData.IsValid() || CachedAssetData.GetObjectPathString() != CurrentObjectPath)
					{
						static FName AssetRegistryName("AssetRegistry");

						FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(AssetRegistryName);
						CachedAssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(CurrentObjectPath));
					}

					Result = FPropertyAccess::Success;
					OutValue = FObjectOrAssetData(CachedAssetData);
				}
				else
				{
					// This is an actor or other subobject reference
					if (CachedAssetData.IsValid())
					{
						CachedAssetData = FAssetData();
					}

					Result = FPropertyAccess::Success;
					OutValue = FObjectOrAssetData(SoftObjectPath);
				}

				return Result;
			}
		}

#if !UE_BUILD_SHIPPING
		if (Object && !Object->IsValidLowLevel())
		{
			const FProperty* Property = PropertyEditor->GetProperty();
			UE_LOGF(LogPropertyNode, Fatal, "Property \"%ls\" (%ls) contains invalid data.", *Property->GetName(), *Property->GetCPPType());
		}
#endif

		OutValue = FObjectOrAssetData( Object, EditorPathOwner, AssetDataOptions );
	}
	else
	{
		FSoftObjectPath SoftObjectPath;
		UObject* Object = nullptr;
		if (PropertyHandle.IsValid())
		{
			Result = PropertyHandle->GetValue(Object);
		}
		else
		{
			SoftObjectPath = FSoftObjectPath(ObjectPath.Get());
			Object = SoftObjectPath.ResolveObject();

			if (Object != nullptr)
			{
				Result = FPropertyAccess::Success;
			}
		}

		if (Object != nullptr)
		{
#if !UE_BUILD_SHIPPING
			if (!Object->IsValidLowLevel())
			{
				const FProperty* Property = PropertyEditor->GetProperty();
				UE_LOGF(LogPropertyNode, Fatal, "Property \"%ls\" (%ls) contains invalid data.", *Property->GetName(), *Property->GetCPPType());
			}
#endif
			UObject* const InEditorPathOwner = nullptr;
			OutValue = FObjectOrAssetData( Object, InEditorPathOwner, AssetDataOptions);
		}
		else
		{
			if (SoftObjectPath.IsNull())
			{
				SoftObjectPath = FSoftObjectPath(ObjectPath.Get());
			}

			if (SoftObjectPath.IsAsset())
			{
				const FSoftObjectPath CurrentObjectPath = SoftObjectPath;
				if (CurrentObjectPath.IsValid() && (!CachedAssetData.IsValid() || CachedAssetData.GetSoftObjectPath() != CurrentObjectPath))
				{
					static FName AssetRegistryName("AssetRegistry");

					FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(AssetRegistryName);
					CachedAssetData = AssetRegistryModule.Get().GetAssetByObjectPath(CurrentObjectPath);
				}

				OutValue = FObjectOrAssetData(CachedAssetData);
				Result = FPropertyAccess::Success;
			}
			else
			{
				// This is an actor or other subobject reference
				if (CachedAssetData.IsValid())
				{
					CachedAssetData = FAssetData();
				}

				OutValue = FObjectOrAssetData(SoftObjectPath);
			}

			if (PropertyHandle.IsValid())
			{
				// No property editor was specified so check if multiple property values are associated with the property handle
				TArray<FString> ObjectValues;
				PropertyHandle->GetPerObjectValues(ObjectValues);

				if (ObjectValues.Num() > 1)
				{
					for (int32 ObjectIndex = 1; ObjectIndex < ObjectValues.Num() && Result == FPropertyAccess::Success; ++ObjectIndex)
					{
						if (ObjectValues[ObjectIndex] != ObjectValues[0])
						{
							Result = FPropertyAccess::MultipleValues;
						}
					}
				}
			}
		}
	}

	return Result;
}

const UClass* SPropertyEditorAsset::GetDisplayedClass() const
{
	FObjectOrAssetData Value;
	GetValue( Value, FObjectOrAssetData::EAssetDataOptions::SkipAssetRegistryTagsGathering );
	if(Value.Object != nullptr)
	{
		return Value.Object->GetClass();
	}
	else
	{
		return ObjectClass;	
	}
}

void SPropertyEditorAsset::OnAssetSelected( const struct FAssetData& AssetData )
{
	SetValue(AssetData);
}

SPropertyEditorAsset::FObjectOrAssetData::FObjectOrAssetData(UObject* InObject, UObject* InEditorPathOwner, EAssetDataOptions AssetDataOptions)
	: Object(InObject)
{
	if (AActor* Actor = Cast<AActor>(InObject))
	{
		ObjectPath = FEditorPathHelper::GetEditorPathFromEditorPathOwner(Actor, InEditorPathOwner);
	}
	else if(InObject != nullptr)
	{
		FAssetData::ECreationFlags CreationFlags =
			(AssetDataOptions == EAssetDataOptions::SkipAssetRegistryTagsGathering)
			? FAssetData::ECreationFlags::SkipAssetRegistryTagsGathering
			: FAssetData::ECreationFlags::None;
		
		AssetData = FAssetData(InObject, CreationFlags);
		ObjectPath = InObject;
	}
}

void SPropertyEditorAsset::OnActorSelected( AActor* InActor )
{
	if (InActor && FEditorPathHelper::IsEnabled() && bIsSoftObjectPath)
	{
		// Even if SetValue ends up calling FSoftObjectProperty::ImportText_Internal the FAssetData validation needs to validate the reference domain which is /Temp when referencing Level Instance objects. So we convert the FAssetData to the EditorPath version to pass validation.
		FSoftObjectPath EditorPath = FEditorPathHelper::GetEditorPathFromEditorPathOwner(InActor, EditorPathOwner);
		if (FSoftObjectPath(InActor) != EditorPath)
		{
			FAssetData EditorAssetData(EditorPath.GetLongPackageName(), EditorPath.ToString(), FTopLevelAssetPath(InActor->GetClass()->GetPathName()));
			SetValue(EditorAssetData);
			return;
		}
	}
	SetValue(InActor);
}

void SPropertyEditorAsset::OnGetAllowedClasses(TArray<const UClass*>& AllowedClasses)
{
	AllowedClasses.Append(AllowedClassFilters);
}

void SPropertyEditorAsset::OnOpenAssetEditor()
{
	FObjectOrAssetData Value;
	GetValue( Value, FObjectOrAssetData::EAssetDataOptions::SkipAssetRegistryTagsGathering );

	UObject* ObjectToEdit = Value.AssetData.GetAsset();
	if( ObjectToEdit )
	{
		if (UWorld* World = Cast<UWorld>(ObjectToEdit))
		{
			constexpr bool bPromptUserToSave = true;
			constexpr bool bSaveMapPackages = true;
			constexpr bool bSaveContentPackages = true;
			if (!FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages))
			{
				return;
			}
		}

		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();

		if(AssetEditorSubsystem)
		{
			FText ErrorMsg;
			if(AssetEditorSubsystem->CanOpenEditorForAsset(ObjectToEdit, EAssetTypeActivationOpenedMethod::Edit, &ErrorMsg))
			{
				AssetEditorSubsystem->OpenEditorForAsset(ObjectToEdit); // Default opens in Edit Mode
			}
			else if(AssetEditorSubsystem->CanOpenEditorForAsset(ObjectToEdit, EAssetTypeActivationOpenedMethod::View, &ErrorMsg))
			{
				AssetEditorSubsystem->OpenEditorForAsset(ObjectToEdit, EToolkitMode::Standalone /* default */, TSharedPtr<IToolkitHost>() /* default */, true /* default */, EAssetTypeActivationOpenedMethod::View);
			}
		}
	}
}

void SPropertyEditorAsset::OnBrowse()
{
	FObjectOrAssetData Value;
	GetValue( Value, FObjectOrAssetData::EAssetDataOptions::None );

	if (bIsActor)
	{
		TSharedPtr<IPropertyHandle> PropertyHandleToUse = GetMostSpecificPropertyHandle();
		if (PropertyHandleToUse)
		{
			// Try to resolve a potentially unloaded object
			if (!Value.Object)
			{
				FSoftObjectPath MapObjectPath = Value.ObjectPath.GetWithoutSubPath();

				if (UObject* MapObject = MapObjectPath.ResolveObject())
				{
					if (UWorld* World = Cast<UWorld>(MapObject); World && World->IsPartitionedWorld())
					{
						if (const FWorldPartitionActorDescInstance* ActorDescInstance = World->GetWorldPartition()->GetActorDescInstanceByPath(Value.ObjectPath))
						{
							World->GetWorldPartition()->PinActors({ ActorDescInstance->GetGuid() });
							GetValue(Value, FObjectOrAssetData::EAssetDataOptions::None);
						}
					}
				}
			}

			if (Value.Object)
			{
				// This code only works on loaded objects
				if (TSharedPtr<FPropertyNode> PropertyNodeToSync = StaticCastSharedPtr<FPropertyHandleBase>(PropertyHandleToUse)->GetPropertyNode())
				{
					FPropertyEditor::SyncToObjectsInNode(PropertyNodeToSync);
				}
			}
		}
	}
	else
	{
		const FModifierKeysState KeyState = FSlateApplication::Get().GetModifierKeys();
		if (KeyState.IsAltDown( ))
		{
			GEditor->EditObject( Value.AssetData.GetAsset(  ) );
		}
		else
		{
			TArray<FAssetData> AssetDataList;
			AssetDataList.Add( Value.AssetData );
			GEditor->SyncBrowserToObjects( AssetDataList );
		}
	}
}

FText SPropertyEditorAsset::GetOnBrowseToolTip() const
{
	FObjectOrAssetData Value;
	GetValue( Value, FObjectOrAssetData::EAssetDataOptions::SkipAssetRegistryTagsGathering );
	
	FFormatNamedArguments Args;
	Args.Add(TEXT("AltOpen"), LOCTEXT( "HoldAltToOpenText", "(hold Alt to Open instead)"));

	if (Value.Object)
	{
		Args.Add(TEXT("Asset"), FText::AsCultureInvariant(Value.Object->GetName()));
		if (bIsActor)
		{
			return FText::Format(LOCTEXT( "SelectSpecificActorInViewport", "Select '{Asset}' in the viewport"), Args);
		}
		else
		{
			return FText::Format(LOCTEXT( "BrowseToSpecificAssetInContentBrowser", "Browse to '{Asset}' in Content Browser {AltOpen}"), Args);
		}
	}
	
	if (bIsActor)
	{
		return FText::Format(LOCTEXT("SelectActorInViewport", "Select Actor in the viewport"), Args);
	}
	else
	{
		return FText::Format(LOCTEXT("BrowseToAssetInContentBrowser", "Browse to Asset in Content Browser {AltOpen}"), Args);
	}
}

const FSlateBrush* SPropertyEditorAsset::GetOnBrowseIcon() const
{
	// If the widget is hovered and alt is held down, show the edit icon
	if (!bIsActor && IsHovered() && FSlateApplication::Get().GetModifierKeys().IsAltDown())
	{
		return FAppStyle::Get().GetBrush("Icons.Edit");
	}
	
	return bIsActor ? FAppStyle::Get().GetBrush("Icons.SelectInViewport") : FAppStyle::Get().GetBrush("Icons.BrowseContent");
}

void SPropertyEditorAsset::OnUse()
{
	// Use the property editor path if it is valid and there is no custom filtering required
	if(PropertyEditor.IsValid()
		&& !OnShouldFilterAsset.IsBound()
		&& !OnShouldFilterActor.IsBound()
		&& AllowedClassFilters.Num() == 0
		&& DisallowedClassFilters.Num() == 0
		&& (GEditor ? !GEditor->MakeAssetReferenceFilter(FAssetReferenceFilterContext()).IsValid() : true))
	{
		PropertyEditor->GetPropertyHandle()->SetObjectValueFromSelection();
	}
	else
	{
		// Load selected assets
		FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

		// try to get a selected object of our class
		const UObject* Selection = nullptr;
		if( ObjectClass && ObjectClass->IsChildOf( AActor::StaticClass() ) )
		{
			Selection = GEditor->GetSelectedActors()->GetTop( ObjectClass );

			// For actors filtered means allowed, unlike for assets (where filtered means NOT allowed)
			if (!IsFilteredActor(static_cast<const AActor*>(Selection)))
			{
				Selection = nullptr;
			}
		}
		else if( ObjectClass )
		{
			// Get the first material selected
			Selection = GEditor->GetSelectedObjects()->GetTop( ObjectClass );
		}

		// Check against custom asset filter
		if (Selection != nullptr
			&& OnShouldFilterAsset.IsBound()
			&& OnShouldFilterAsset.Execute(FAssetData(Selection)))
		{
			Selection = nullptr;
		}

		if( Selection )
		{
			SetValue( Selection );
		}
	}
}

void SPropertyEditorAsset::OnClear()
{
	SetValue(nullptr);
}

FSlateColor SPropertyEditorAsset::GetAssetClassColor()
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));	
	TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(GetDisplayedClass());
	if(AssetTypeActions.IsValid())
	{
		return FSlateColor(AssetTypeActions.Pin()->GetTypeColor());
	}

	return FSlateColor::UseForeground();
}

bool SPropertyEditorAsset::OnAssetDraggedOver( TArrayView<FAssetData> InAssets, FText& OutReason ) const
{
	UObject* AssetObject = InAssets[0].GetAsset();
	if (CanEdit() && (AssetObject != nullptr) && (AssetObject->IsA(ObjectClass) || AssetObject->GetClass()->ImplementsInterface(ObjectClass)))
	{
		FAssetData AssetData(InAssets[0]);
		// Check against custom asset filter
		if (!OnShouldFilterAsset.IsBound()
			|| !OnShouldFilterAsset.Execute(AssetData))
		{
			if (CanSetBasedOnCustomClasses(AssetData))
			{
				return CanSetBasedOnAssetReferenceFilter(AssetData, &OutReason);
			}
		}
	}

	return false;
}

void SPropertyEditorAsset::OnAssetDropped( const FDragDropEvent&, TArrayView<FAssetData> InAssets )
{
	if( CanEdit() )
	{
		SetValue(InAssets[0].GetAsset());
	}
}


void SPropertyEditorAsset::OnCopy()
{
	FObjectOrAssetData Value;
	GetValue( Value, FObjectOrAssetData::EAssetDataOptions::SkipAssetRegistryTagsGathering );

	if( Value.AssetData.IsValid() )
	{
		FPlatformApplicationMisc::ClipboardCopy(*Value.AssetData.GetExportTextName());
	}
	else if (Value.ObjectPath.IsValid())
	{
		FPlatformApplicationMisc::ClipboardCopy(*Value.ObjectPath.ToString());
	}
	else
	{
		// Copy "None" instead of ""
		FPlatformApplicationMisc::ClipboardCopy(*FName().ToString());
	}
}

void SPropertyEditorAsset::OnPaste()
{
	FString DestPath;
	FPlatformApplicationMisc::ClipboardPaste(DestPath);
	
	OnPasteFromText(TEXT(""), DestPath, {});
}

void SPropertyEditorAsset::OnPasteFromText(
	const FString& InTag,
	const FString& InText,
	const TOptional<FGuid>& InOperationId)
{
	FAssetData AssetData;
	if (CanPasteFromText(InTag, InText, AssetData))
	{
		PasteFromText(InTag, AssetData);
	}
}

void SPropertyEditorAsset::PasteFromText(
	const FString& InTag,
	const FAssetData& InAssetData)
{
	if (!InAssetData.IsValid())
	{
		SetValue(nullptr);
	}
	else
	{
		UObject* Object = InAssetData.GetAsset();
		if(Object && Object->IsA(ObjectClass))
		{
			// Check against custom asset filter
			if (!OnShouldFilterAsset.IsBound()
				|| !OnShouldFilterAsset.Execute(InAssetData))
			{
				SetValue(InAssetData);
			}
		}
	}
}

bool SPropertyEditorAsset::CanPaste()
{
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

	FAssetData AssetData;
	return CanPasteFromText(TEXT(""), ClipboardText, AssetData);
}

bool SPropertyEditorAsset::CanPasteFromText(
	const FString& InTag,
	const FString& InText,
	FAssetData& OutAssetData) const
{
	if (!UE::PropertyEditor::TagMatchesProperty(InTag, PropertyHandle))
	{
		return false;
	}

	if (!CanEdit())
	{
		return false;
	}

	return ParseAssetText(InText, OutAssetData);
}

FReply SPropertyEditorAsset::OnAssetThumbnailDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	OnOpenAssetEditor();
	return FReply::Handled();
}

bool SPropertyEditorAsset::CanEdit() const
{
	if (PropertyEditor.IsValid() && PropertyEditor->IsEditConst())
	{
		return false;
	}
	return OnIsEnabled.Get(true);
}

bool SPropertyEditorAsset::CanSetBasedOnCustomClasses( const FAssetData& InAssetData ) const
{
	if (InAssetData.IsValid())
	{
		return IsClassAllowed(InAssetData.GetClass());
	}

	return true;
}

bool SPropertyEditorAsset::IsClassAllowed(const UClass* InClass) const
{
	if (!InClass)
	{
		// A null class will not match any filters. If we have an allow list, this means failure, otherwise it means success.
		return AllowedClassFilters.Num() == 0;
	}

	bool bClassAllowed = true;
	if (AllowedClassFilters.Num() > 0)
	{
		bClassAllowed = false;
		for (const UClass* AllowedClass : AllowedClassFilters)
		{
			const bool bAllowedClassIsInterface = AllowedClass->HasAnyClassFlags(CLASS_Interface);
			bClassAllowed = bExactClass ? InClass == AllowedClass :
				InClass->IsChildOf(AllowedClass) || (bAllowedClassIsInterface && InClass->ImplementsInterface(AllowedClass));

			if (bClassAllowed)
			{
				break;
			}
		}
	}

	if (DisallowedClassFilters.Num() > 0 && bClassAllowed)
	{
		for (const UClass* DisallowedClass : DisallowedClassFilters)
		{
			const bool bDisallowedClassIsInterface = DisallowedClass->HasAnyClassFlags(CLASS_Interface);
			if (InClass->IsChildOf(DisallowedClass) || (bDisallowedClassIsInterface && InClass->ImplementsInterface(DisallowedClass)))
			{
				bClassAllowed = false;
				break;
			}
		}
	}

	return bClassAllowed;
}

bool SPropertyEditorAsset::CanSetBasedOnAssetReferenceFilter( const FAssetData& InAssetData, FText* OutOptionalFailureReason) const
{
	if (GEditor && InAssetData.IsValid())
	{
		FAssetReferenceFilterContext AssetReferenceFilterContext;
		AssetReferenceFilterContext.AddReferencingAssets(OwnerAssetDataArray);
		AssetReferenceFilterContext.AddReferencingAssetsFromPropertyHandle(GetMostSpecificPropertyHandle());
		
		TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
		if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(InAssetData, OutOptionalFailureReason))
		{
			return false;
		}
	}

	return true;
}

TSharedPtr<IPropertyHandle> SPropertyEditorAsset::GetMostSpecificPropertyHandle() const
{
	if (PropertyHandle.IsValid())
	{
		return PropertyHandle;
	}
	else if (PropertyEditor.IsValid())
	{
		return PropertyEditor->GetPropertyHandle();
	}
	
	return TSharedPtr<IPropertyHandle>();
}

UClass* SPropertyEditorAsset::GetObjectPropertyClass(const FProperty* Property)
{
	UClass* Class = nullptr;

	if (CastField<const FObjectPropertyBase>(Property) != nullptr)
	{
		Class = CastField<const FObjectPropertyBase>(Property)->PropertyClass;
		if (Class == nullptr)
		{
			UE_LOGF(LogPropertyNode, Warning, "Object Property (%ls) has a null class, falling back to UObject", *Property->GetFullName());
			Class = UObject::StaticClass();
		}
	}
	else if (CastField<const FInterfaceProperty>(Property) != nullptr)
	{
		Class = CastField<const FInterfaceProperty>(Property)->InterfaceClass;
		if (Class == nullptr)
		{
			UE_LOGF(LogPropertyNode, Warning, "Interface Property (%ls) has a null class, falling back to UObject", *Property->GetFullName());
			Class = UObject::StaticClass();
		}
	}
	else
	{
		ensureMsgf(Class != nullptr, TEXT("Property (%s) is not an object or interface class"), Property ? *Property->GetFullName() : TEXT("null"));
		Class = UObject::StaticClass();
	}
	return Class;
}

#undef LOCTEXT_NAMESPACE
