// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "GameFramework/Actor.h"
#include "Styling/SlateColor.h"
#include "Fonts/SlateFontInfo.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Input/SComboButton.h"
#include "Styling/AppStyle.h"
#include "AssetRegistry/AssetData.h"
#include "AssetThumbnail.h"
#include "PropertyHandle.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "PropertyCustomizationHelpers.h"

class FAssetThumbnail;
class FPropertyEditor;
class IPropertyHandle;
class SBorder;
class SComboButton;
class UFactory;
class FDetailWidgetRow;
struct FAssetButtonActionExtension;

/**
 * A widget used to edit Asset-type properties (UObject-derived properties).
 * Can also be used (with a NULL FPropertyEditor) to edit a raw weak object pointer.
 */
class SPropertyEditorAsset : public SCompoundWidget
{
public:

	enum class EActorReferenceState : uint8
	{
		// This is not pointing to an actor
		NotAnActor,
		// This is specifically pointing at no actor
		Null,
		// The pointed to actor is fully loaded in memory
		Loaded,
		// The pointed to actor exists because the pointed to map is loaded but not the actor
		Exists,
		// The pointed to actor is unknown because the pointed to map is not loaded 
		Unknown,
		// This is a known bad reference, the owning map is loaded but the actor does not exist
		Error,
	};

	SLATE_BEGIN_ARGS(SPropertyEditorAsset)
		: _DisplayThumbnail(true)
		, _DisplayUseSelected(true)
		, _DisplayTransient(true)
		, _DisplayBrowse(true)
		, _EnableContentPicker(true)
		, _DisplayCompactSize(true)
		, _ThumbnailPool(nullptr)
		, _ThumbnailSize(FIntPoint(48, 48))
		, _ObjectPath()
		, _Class(nullptr)
		, _CustomContentSlot()
		, _CustomButtonSlot()
	{}
		SLATE_ARGUMENT(TOptional<bool>, AllowClear)
		SLATE_ARGUMENT(TOptional<bool>, AllowCreate)
		SLATE_ARGUMENT(bool, DisplayThumbnail)
		SLATE_ARGUMENT(bool, DisplayUseSelected)
		SLATE_ARGUMENT(bool, DisplayTransient)
		SLATE_ARGUMENT(bool, DisplayBrowse)
		SLATE_EVENT(FSimpleDelegate, OnBrowseOverride)
		SLATE_ATTRIBUTE(FText, OnBrowseToolTipOverride)
		SLATE_ATTRIBUTE(const FSlateBrush*, OnBrowseIconOverride)
		SLATE_ARGUMENT(bool, EnableContentPicker)
		SLATE_ARGUMENT(bool, DisplayCompactSize)
		SLATE_ARGUMENT(TSharedPtr<FAssetThumbnailPool>, ThumbnailPool)
		SLATE_ARGUMENT(FIntPoint, ThumbnailSize)
		SLATE_ATTRIBUTE(FString, ObjectPath)
		SLATE_ARGUMENT(const UClass*, Class)
		SLATE_ARGUMENT(TOptional<TArray<UFactory*>>, NewAssetFactories)
		SLATE_EVENT(FOnSetObject, OnSetObject)
		SLATE_EVENT(FOnShouldFilterAsset, OnShouldFilterAsset)
		SLATE_NAMED_SLOT(FArguments, CustomContentSlot)
		SLATE_NAMED_SLOT(FArguments, CustomButtonSlot)
		SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, PropertyHandle)
		SLATE_ARGUMENT(TArray<FAssetData>, OwnerAssetDataArray)
		SLATE_EVENT(FOnShouldFilterActor, OnShouldFilterActor)
		SLATE_ARGUMENT(TOptional<FDetailWidgetRow*>, InWidgetRow)
		/** When this is true, the drop target will only get recognized when entering while drag & dropping. */
		SLATE_ATTRIBUTE(bool, bOnlyRecognizeOnDragEnter)

	SLATE_END_ARGS()

	static bool Supports( const TSharedRef<FPropertyEditor>& InPropertyEditor );
	static bool Supports( const FProperty* NodeProperty );

	/**
	 * Parses the input text and finds the corrisponding asset in the asset registry.
	 * Supports various types of paths, such as export text, object and package paths.
	 * @returns True if OutAssetData contains a valid asset, or the input text was None.
	 */
	static bool ParseAssetText(const FString& InText, FAssetData& OutAssetData);

	/**
	 * Construct the widget.
	 * @param	InArgs				Arguments for widget construction
	 * @param	InPropertyEditor	The property editor that this widget will operate on. If this is NULL, we expect
	 *                              the Class member of InArgs to be a valid UClass, so we know what objects this 
	 *                              widget can accept.
	 */
	void Construct( const FArguments& InArgs, const TSharedPtr<FPropertyEditor>& InPropertyEditor = NULL );

	void GetDesiredWidth( float& OutMinDesiredWidth, float &OutMaxDesiredWidth );

	void OpenComboButton();

	/** Override the tick method so we can check for thumbnail differences & update if necessary */
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

private:
	struct FObjectOrAssetData
	{
		enum class EAssetDataOptions
		{
			None,
			// Will not populate AssetRegistry Tags for an AssetData
			SkipAssetRegistryTagsGathering
		};
		
		UObject* Object = nullptr;
		FSoftObjectPath ObjectPath;
		FAssetData AssetData;
		UObject* EditorPathOwner = nullptr;
		
		FObjectOrAssetData() = default;

		FObjectOrAssetData(UObject* InObject, UObject* EditorPathOwner, EAssetDataOptions AssetDataOptions);

		FObjectOrAssetData( const FSoftObjectPath& InObjectPath )
			: Object(nullptr)
			, ObjectPath(InObjectPath)
			, EditorPathOwner(nullptr)
		{}

		FObjectOrAssetData( const FAssetData& InAssetData )
			: Object(nullptr)
			, ObjectPath( InAssetData.ToSoftObjectPath() )
			, AssetData( InAssetData )
			, EditorPathOwner(nullptr)
		{}

		bool IsValid() const
		{
			return Object != nullptr || ObjectPath.IsValid() || AssetData.IsValid();
		}
	};

	/** 
	 * Helper function for Construct - determines whether we should display a thumbnail or not
	 * @param	InArgs	The args passed into Construct()
	 * @returns true if we should display the thumbnail 
	 */
	bool ShouldDisplayThumbnail( const FArguments& InArgs, const UClass* ObjectClass ) const;

	/** Gets the border brush to show around the thumbnail, changes when the user hovers on it. */
	const FSlateBrush* GetThumbnailBorder() const;

	/** Returns the status icon, empty for non actors */
	const FSlateBrush* GetStatusIcon() const;

	/** Returns the state of this actor reference */
	EActorReferenceState GetActorReferenceState() const;

	/** 
	 * Get the content to be displayed in the asset/actor picker menu 
	 * @returns the widget for the menu content
	 */
	TSharedRef<SWidget> OnGetMenuContent();

	/**
	 * Called when the asset menu is closed, we handle this to force the destruction of the asset menu to
	 * ensure any settings the user set are saved.
	 */
	void OnMenuOpenChanged(bool bOpen);

	/**
	 * Returns whether the actor should be filtered out from selection.
	 */
	bool IsFilteredActor( const AActor* const Actor) const;

	/**
	 * Closes the combo button for the asset name.
	 */
	void CloseComboButton();

	/** 
	 * Get the name to be displayed for this asset
	 * @returns the name of the asset
	 */
	FText OnGetAssetName() const;

	/** 
	 * Get the class name to be displayed for this asset
	 * @returns the class name of this asset
	 */
	FText OnGetAssetClassName() const;

	/** 
	 * Get the tooltip to be displayed for this asset
	 * @returns the tooltip string
	 */
	FText OnGetToolTip() const;

	/**
	 * Determine whether the given asset is allowed to be assigned to the property.
	 * This is currently used to prevent non-engine assets being assigned to properties of engine assets.
	 *
	 * @param	AssetData	The asset to be assigned
	 * @return	false		if the asset cannot be assigned,
	 *			true		if it can be assigned or if this is not known (e.g. because a delegate handles the reference assignment)
	 */
	bool CanAssetBeAssigned(const FAssetData& AssetData) const;

	/** 
	 * Set the value of the asset referenced by this property editor.
	 * Will set the underlying property handle if there is one.
	 * @param	AssetData	The asset data for the object to set the reference to
	 */
	void SetValue( const FAssetData& AssetData );

	/** 
	 * Get the value referenced by this widget.
	 * @returns the referenced object
	 */
	FPropertyAccess::Result GetValue( FObjectOrAssetData& OutValue, FObjectOrAssetData::EAssetDataOptions GetValueOptions ) const;

	/** 
	 * Get the UClass we will display in the UI.
	 * This is the class of the object (if valid) or the property handles class (if any)
	 * or the Class value this widget was constructed with.
	 * @returns the UClass to display
	 */
	const UClass* GetDisplayedClass() const;

	/** 
	 * Delegate for handling selection in the asset browser.
	 * @param	Object	The chosen asset
	 */
	void OnAssetSelected( const FAssetData& AssetData );

	/** 
	 * Delegate for handling selection in the scene outliner.
	 * @param	InActor	The chosen actor
	 */
	void OnActorSelected( AActor* InActor );

	/** 
	 * Delegate for handling classes of objects that can be picked.
	 * @param	AllowedClasses	The array of classes we allow
	 */
	void OnGetAllowedClasses(TArray<const UClass*>& AllowedClasses);

	/** 
	 * Opens the asset editor for the viewed object
	 */
	void OnOpenAssetEditor();

	/** 
	 * Browse for the object referenced by this widget, either in the Content Browser
	 * or the scene (for Actors)
	 */
	void OnBrowse();

	/** 
	 * Get the tooltip text for the Browse button
	 */
	FText GetOnBrowseToolTip() const;
	
	/** 
	 * Get the icon for the Browse button
	 */
	const FSlateBrush* GetOnBrowseIcon() const;

	/** 
	 * Use the selected object (replaces the referenced object if valid)
	 */
	void OnUse();

	/** 
	 * Clear the referenced object
	 */
	void OnClear();

	/** 
	 * Assets have an associated color, this is used to supply that color in the UI.
	 * @returns the color to display for this asset
	 */
	FSlateColor GetAssetClassColor();

	/** 
	 * Delegate used to check whether we can drop an object on this widget.
	 * @param	InObject	The object we are dragging
	 * @param	OutReason	When returning false, the reason it was not allowed
	 * @returns true if the object can be dropped
	 */
	bool OnAssetDraggedOver( TArrayView<FAssetData> InAssets, FText& OutReason ) const;

	/** 
	 * Delegate handling dropping an object on this widget
	 * @param	InObject	The object we are dropping
	 */
	void OnAssetDropped( const FDragDropEvent&, TArrayView<FAssetData> InAssets );

	/** 
	 * Delegate handling ctrl+c
	 */
	void OnCopy();

	/** 
	 * Delegate handling ctrl+v
	 */
	void OnPaste();

	/**
	 * @return True if the (optionally tagged) input contents can be pasted
	 */
	bool CanPasteFromText(const FString& InTag, const FString& InText, FAssetData& OutAssetData) const;

	/**
	 * Delegate handling pasting an optionally tagged text snippet
	 */
	void OnPasteFromText(const FString& InTag, const FString& InText, const TOptional<FGuid>& InOperationId);

	/**
	 * Handle pasting an optionally tagged text snippet
	 */
	void PasteFromText(const FString& InTag, const FAssetData& InAssetData);

	/**
	 * @return True of the current clipboard contents can be pasted
	 */
	bool CanPaste();

	/** 
	 * Handle double clicking the asset thumbnail. this 'Edits' the displayed asset 
	 */
	FReply OnAssetThumbnailDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent );

	/** @return True if the property can be edited */
	bool CanEdit() const;

	/** @return true if the passed in AssetData can be used to set the property based on the list of custom classes */
	bool CanSetBasedOnCustomClasses( const FAssetData& InAssetData ) const;

	/** @return true if the passed in AssetData can be used to set the property based on the game specific asset reference filter */
	bool CanSetBasedOnAssetReferenceFilter( const FAssetData& InAssetData, FText* OutOptionalFailureReason = nullptr ) const;

	/** @return Returns the property handle most relevant */
	TSharedPtr<IPropertyHandle> GetMostSpecificPropertyHandle() const;

	/**
	 * Gets the class of the supplied property for use within the PropertyEditorAsset widget. Asserts if the property
	 * is not supported by the widget.
	 *
	 * @param	Property	The supplied property for the widget
	 * @return	The class of the property. Asserts if this tries to return null.
	 */
	static UClass* GetObjectPropertyClass(const FProperty* Property);

	/**
	 * Initialize the AllowedClasses and DisallowedClasses arrays based on the property metadata.
	 */
	void InitializeClassFilters(const FProperty* Property);

	/** Check if the given class is allowed by the AllowedClasses and DisallowedClasses as defined in the property metadata. */
	bool IsClassAllowed(const UClass* InClass) const;

	/**
	 * Initialize DisallowedAssetDataTags based on the property metadata.
	 */
	void InitializeAssetDataTags(const FProperty* Property);

	/** @return Returns true if the asset is excluded for this property*/
	bool IsAssetFiltered(const FAssetData& InAssetData);

	/**
	 *Generates custom Asset Picker Buttons using the provided AssetActionButtonExtensions data.
	 */
	void GenerateCustomAssetPickerButtons(const FAssetData& InAssetData, const TArray<FAssetButtonActionExtension>& InExtensions);

private:

	/** Main combobutton */
	TSharedPtr<SComboButton> AssetComboButton;

	/** The border surrounding the thumbnail image. */
	TSharedPtr<SBorder> ThumbnailBorder;

	/** The property editor, if any */
	TSharedPtr<FPropertyEditor> PropertyEditor;

	/** Optional Custom Asset Picker Buttons*/
	TSharedPtr<SHorizontalBox> CustomAssetPickerButtonBox;

	/** Path to the object being edited instead of accessing the value directly with a property handle */
	TAttribute<FString> ObjectPath;

	/** Cached data */
	mutable FAssetData CachedAssetData;

	/** The class of the object we are editing */
	const UClass* ObjectClass;

	/** Classes that can be used with this property */
	TArray<const UClass*> AllowedClassFilters;

	/** Classes that can NOT be used with this property */
	TArray<const UClass*> DisallowedClassFilters;

	/** A list of the factories we can use to create new assets */
	TOptional<TArray<UFactory*>> NewAssetFactoriesOverride;

	/** Tags (and eventually values) that can NOT be used with this property */
	TSharedPtr<FAssetDataTagMap> DisallowedAssetDataTags;

	/** Tags and values that must be present for an asset to be used with this property */
	TSharedPtr<FAssetDataTagMap> RequiredAssetDataTags;

	/** Whether the asset can be 'None' in this case */
	bool bAllowClear;

	/** Whether the asset can be created from the asset picker */
	bool bAllowCreate;

	/** Whether the object we are editing is an Actor (i.e. requires a Scene Outliner to be displayed) */
	bool bIsActor;

	/** Whether the classes in the AllowedClassFilters list are exact or whether valid classes can be derived. */
	bool bExactClass;

	/** Wheter the property is a soft reference */
	bool bIsSoftObjectPath;

	/** Editor Path context if any */
	UObject* EditorPathOwner;

	/** The number of additional buttons this picker has. */
	int32 NumButtons;

	/** Base attribute passed to specify whether this editor is enabled */ 
	TAttribute<bool> OnIsEnabled;

	/** Delegate to call when our object value is set */
	FOnSetObject OnSetObject;

	/** Delegate for filtering valid assets */
	FOnShouldFilterAsset OnShouldFilterAsset;

	/** Thumbnail for the asset */
	TSharedPtr<FAssetThumbnail> AssetThumbnail;

	/** The property handle, if any */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** Delegate for filtering valid actors */
	FOnShouldFilterActor OnShouldFilterActor;

	/*
	 * The reference object on which the picker will assign the picked asset, if any.
	 * The system will test first the PropertyHandle follow by the PropertyEditor and if nothing this array of assets
	 */
	TArray<FAssetData> OwnerAssetDataArray;

	/*Whether to show the 'Use Selected' item*/
	bool bDisplayUseSelected;

	/*Whether to show transient objects in the picker. */
	bool bDisplayTransient;
};
