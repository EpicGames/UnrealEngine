// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyEditor/SPropertyEditorBool.h"

#include "Delegates/Delegate.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "UObject/UnrealType.h"
#include "Widgets/Input/SCheckBox.h"

struct FGeometry;

void SPropertyEditorBool::Construct( const FArguments& InArgs, const TSharedRef< class FPropertyEditor >& InPropertyEditor )
{
	PropertyEditor = InPropertyEditor;

	CheckBox = SNew( SCheckBox )
		.OnCheckStateChanged( this, &SPropertyEditorBool::OnCheckStateChanged )
		.IsChecked( this, &SPropertyEditorBool::OnGetCheckState )
		.Padding(0.0f);

	ChildSlot
	[	
		CheckBox.ToSharedRef()
	];

	SetEnabled( TAttribute<bool>( this, &SPropertyEditorBool::CanEdit ) );
}

void SPropertyEditorBool::GetDesiredWidth( float& OutMinDesiredWidth, float& OutMaxDesiredWidth )
{
	// No desired width
	OutMinDesiredWidth = 0;
	OutMaxDesiredWidth = 0;
}

bool SPropertyEditorBool::Supports( const TSharedRef< class FPropertyEditor >& PropertyEditor )
{
	return PropertyEditor->PropertyIsA( FBoolProperty::StaticClass() );
}

bool SPropertyEditorBool::SupportsKeyboardFocus() const
{
	return CheckBox->SupportsKeyboardFocus();
}

bool SPropertyEditorBool::HasKeyboardFocus() const
{
	return CheckBox->HasKeyboardFocus();
}

FReply SPropertyEditorBool::OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent )
{
	// Forward keyboard focus to our editable text widget
	return FReply::Handled().SetUserFocus(CheckBox.ToSharedRef(), InFocusEvent.GetCause());
}

ECheckBoxState SPropertyEditorBool::OnGetCheckState() const
{
	ECheckBoxState ReturnState = ECheckBoxState::Undetermined;

	bool Value;
	const TSharedRef< IPropertyHandle > PropertyHandle = PropertyEditor->GetPropertyHandle();
	if( PropertyHandle->GetValue( Value ) == FPropertyAccess::Success )
	{
		if( Value == true )
		{
			ReturnState = ECheckBoxState::Checked;
		}
		else
		{
			ReturnState = ECheckBoxState::Unchecked;
		}
	}

	return ReturnState;
}

void SPropertyEditorBool::OnCheckStateChanged( ECheckBoxState InNewState )
{
	const TSharedRef< IPropertyHandle > PropertyHandle = PropertyEditor->GetPropertyHandle();
	if( InNewState == ECheckBoxState::Checked || InNewState == ECheckBoxState::Undetermined )
	{
		PropertyHandle->SetValue( true );
	}
	else
	{
		PropertyHandle->SetValue( false );
	}
}

FReply SPropertyEditorBool::OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	if ( InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		//toggle the our checkbox
		CheckBox->ToggleCheckedState();

		// Set focus to this object, but don't capture the mouse
		return FReply::Handled().SetUserFocus(AsShared(), EFocusCause::Mouse);
	}

	return FReply::Unhandled();
}

bool SPropertyEditorBool::CanEdit() const
{
	return PropertyEditor.IsValid() ? !PropertyEditor->IsEditConst() : true;
}
