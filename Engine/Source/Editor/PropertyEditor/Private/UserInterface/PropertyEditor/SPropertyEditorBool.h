// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Input/Reply.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

struct FFocusEvent;
struct FGeometry;
struct FPointerEvent;

class SPropertyEditorBool : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SPropertyEditorBool ) {}
	SLATE_END_ARGS()

	static bool Supports( const TSharedRef< class FPropertyEditor >& PropertyEditor );

	void Construct( const FArguments& InArgs, const TSharedRef< class FPropertyEditor >& PropertyEditor );

	void GetDesiredWidth( float& OutMinDesiredWidth, float& OutMaxDesiredWidth );

	// SWidget overrides
	virtual bool SupportsKeyboardFocus() const override;
	virtual bool HasKeyboardFocus() const override;
	virtual FReply OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent ) override;

private:

	virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;

	ECheckBoxState OnGetCheckState() const;

	void OnCheckStateChanged( ECheckBoxState InNewState );

	/** @return True if the property can be edited */
	bool CanEdit() const;
private:

	TSharedPtr< class FPropertyEditor > PropertyEditor;

	TSharedPtr< class SCheckBox> CheckBox;
};
