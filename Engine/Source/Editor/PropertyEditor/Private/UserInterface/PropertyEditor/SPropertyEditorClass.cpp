// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyEditor/SPropertyEditorClass.h"
#include "Engine/Blueprint.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SBox.h"
#include "DragAndDrop/ClassDragDropOp.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "ClassViewerModule.h"
#include "ClassViewerFilter.h"
#include "PropertyEditorUtils.h"
#include "UObject/UObjectIterator.h"
#include "UObject/VerseClassProperty.h"
#include "VerseClassConstraintFilters.h"
#include "PropertyEditorHelpers.h"
#include "PropertyNode.h"
#include "PropertyRestriction.h"

#define LOCTEXT_NAMESPACE "PropertyEditor"

class FPropertyEditorClassFilter : public IClassViewerFilter
{
public:
	/** The meta class for the property that classes must be a child-of. */
	const UClass* ClassPropertyMetaClass = nullptr;

	/** The interface that must be implemented. */
	const UClass* InterfaceThatMustBeImplemented = nullptr;

	/** Whether or not abstract classes are allowed. */
	bool bAllowAbstract = false;

	/** Filters all class types to be UVerseClass types with a <concrete> attribute */
	bool bRequiresVerseConcrete = false;

	/** Filters all class types to be UVerseClass types with a <castable> attribute */
	bool bRequiresVerseCastable = false;

	/** Classes that can be picked */
	TArray<const UClass*> AllowedClassFilters;

	/** Classes that can't be picked */
	TArray<const UClass*> DisallowedClassFilters;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs ) override
	{
		if (bRequiresVerseConcrete && !FVerseClassProperty::StaticIsClassConcrete(InClass))
		{
			return false;
		}
		if (bRequiresVerseCastable && !FVerseClassProperty::StaticIsClassCastable(InClass))
		{
			return false;
		}

		return IsClassAllowedHelper(InClass);
	}
	
	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InBlueprint, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return IsClassAllowedHelper(InBlueprint);
	}

private:

	template <typename TClass>
	bool IsClassAllowedHelper(TClass InClass)
	{
		bool bMatchesFlags = !InClass->HasAnyClassFlags(CLASS_Hidden | CLASS_HideDropDown | CLASS_Deprecated) &&
			(bAllowAbstract || !InClass->HasAnyClassFlags(CLASS_Abstract));

		if (bMatchesFlags && InClass->IsChildOf(ClassPropertyMetaClass)
			&& (!InterfaceThatMustBeImplemented || InClass->ImplementsInterface(InterfaceThatMustBeImplemented)))
		{
			auto PredicateFn = [InClass](const UClass* Class)
			{
				return InClass->IsChildOf(Class);
			};

			if (DisallowedClassFilters.FindByPredicate(PredicateFn) == nullptr &&
				(AllowedClassFilters.Num() == 0 || AllowedClassFilters.FindByPredicate(PredicateFn) != nullptr))
			{
				return true;
			}
		}

		return false;
	}
};

void SPropertyEditorClass::GetDesiredWidth(float& OutMinDesiredWidth, float& OutMaxDesiredWidth)
{
	OutMinDesiredWidth = 200.0f;
	OutMaxDesiredWidth = 400.0f;
}

bool SPropertyEditorClass::Supports(const TSharedRef< FPropertyEditor >& InPropertyEditor)
{
	const TSharedRef< FPropertyNode > PropertyNode = InPropertyEditor->GetPropertyNode();
	const FProperty* Property = InPropertyEditor->GetProperty();
	int32 ArrayIndex = PropertyNode->GetArrayIndex();

	if ((Property->IsA(FClassProperty::StaticClass()) || Property->IsA(FSoftClassProperty::StaticClass())) 
		&& ((ArrayIndex == -1 && Property->ArrayDim == 1) || (ArrayIndex > -1 && Property->ArrayDim > 0)))
	{
		return true;
	}

	return false;
}

/** @return True if the property can be edited */
bool SPropertyEditorClass::CanEdit() const
{
	return PropertyEditor.IsValid() ? !PropertyEditor->IsEditConst() : true;
}

void SPropertyEditorClass::Construct(const FArguments& InArgs, const TSharedPtr< FPropertyEditor >& InPropertyEditor)
{
	PropertyEditor = InPropertyEditor;

	TArray<TSharedRef<class IClassViewerFilter>> ClassViewerFilters = InArgs._ClassViewerFilters;

	if (PropertyEditor.IsValid())
	{
		const TSharedRef<FPropertyNode> PropertyNode = PropertyEditor->GetPropertyNode();
		const FProperty* const Property = PropertyNode->GetProperty();
		if (const FClassProperty* const ClassProp = CastField<FClassProperty>(Property))
		{
			MetaClass = ClassProp->MetaClass;
		}
		else if (const FSoftClassProperty* const SoftClassProperty = CastField<FSoftClassProperty>(Property))
		{
			MetaClass = SoftClassProperty->MetaClass;
		}
		else
		{
			check(false);
		}

		const FString* AllowAbstractString = Property->GetOwnerProperty()->FindMetaData(TEXT("AllowAbstract"));
		bAllowAbstract = AllowAbstractString && (AllowAbstractString->IsEmpty() || AllowAbstractString->ToBool());
		
		bAllowOnlyPlaceable = Property->GetOwnerProperty()->HasMetaData(TEXT("OnlyPlaceable"));
		bIsBlueprintBaseOnly = Property->GetOwnerProperty()->HasMetaData(TEXT("BlueprintBaseOnly"));
		RequiredInterface = Property->GetOwnerProperty()->GetClassMetaData(TEXT("MustImplement"));
		bAllowNone = !(Property->PropertyFlags & CPF_NoClear);
		bShowViewOptions = Property->GetOwnerProperty()->HasMetaData(TEXT("HideViewOptions")) ? false : true;
		bShowTree = Property->GetOwnerProperty()->HasMetaData(TEXT("ShowTreeView"));
		bShowDisplayNames = Property->GetOwnerProperty()->HasMetaData(TEXT("ShowDisplayNames"));
		bShowObjectRootClass = Property->GetOwnerProperty()->HasMetaData(PropertyEditorUtils::ShowObjectRootClassKey);

		if (RequiredInterface != nullptr)
		{
			if (!RequiredInterface->HasAnyClassFlags(CLASS_Interface))
			{
				UE_LOGF(LogPropertyNode, Warning, "Property (%ls) specifies a MustImplement class which isn't an interface (%ls), clearing filter.", *Property->GetFullName(), *RequiredInterface->GetPathName());
				RequiredInterface = nullptr;
			}
			else if (RequiredInterface == UInterface::StaticClass())
			{
				UE_LOGF(LogPropertyNode, Warning, "Property (%ls) specifies a MustImplement class which isn't valid (UInterface), clearing filter.", *Property->GetFullName());
				RequiredInterface = nullptr;
			}
		}

		// Filter based on UPROPERTY meta data
		TArray<UObject*> ObjectList;
		if (PropertyEditor->GetPropertyHandle()->IsValidHandle())
		{
			PropertyEditor->GetPropertyHandle()->GetOuterObjects(ObjectList);
		}
		PropertyEditorUtils::GetAllowedAndDisallowedClasses(ObjectList, *Property, AllowedClassFilters, DisallowedClassFilters, false);

		// Filter based on restrictions
		for (const TSharedRef<const FPropertyRestriction>& ClassRestriction : PropertyNode->GetRestrictions())
		{
			for (TArray<FString>::TConstIterator Iter = ClassRestriction.Get().GetHiddenValuesIterator(); Iter; ++Iter)
			{
				if (const UClass* HiddenClass = PropertyEditorHelpers::FindOrLoadClass(*Iter))
				{
					DisallowedClassFilters.Add(HiddenClass);
				}
			}

			for (TArray<FString>::TConstIterator Iter = ClassRestriction.Get().GetDisabledValuesIterator(); Iter; ++Iter)
			{
				if (const UClass* DisabledClass = PropertyEditorHelpers::FindOrLoadClass(*Iter))
				{
					DisallowedClassFilters.Add(DisabledClass);
				}
			}

			for (TArray<TSharedRef<IClassViewerFilter>>::TConstIterator Iter = ClassRestriction.Get().GeClassViewFilterIterator(); Iter; ++Iter)
			{
				ClassViewerFilters.Add(*Iter);
			}
		}
	}
	else
	{
		check(InArgs._MetaClass);
		check(InArgs._SelectedClass.IsSet());
		check(InArgs._OnSetClass.IsBound());

		MetaClass = InArgs._MetaClass;
		RequiredInterface = InArgs._RequiredInterface;
		bAllowAbstract = InArgs._AllowAbstract;
		bIsBlueprintBaseOnly = InArgs._IsBlueprintBaseOnly;
		bAllowNone = InArgs._AllowNone;
		bAllowOnlyPlaceable = false;
		bShowViewOptions = InArgs._ShowViewOptions;
		bShowTree = InArgs._ShowTree;
		bShowDisplayNames = InArgs._ShowDisplayNames;
		bShowObjectRootClass = false;
		AllowedClassFilters = InArgs._AllowedClasses;
		DisallowedClassFilters = InArgs._DisallowedClasses;
		SelectedClass = InArgs._SelectedClass;
		OnSetClass = InArgs._OnSetClass;
	}

	InvalidObjectDisplayText = InArgs._InvalidObjectDisplayText;

	CreateClassFilter(ClassViewerFilters);

	SAssignNew(ComboButton, SComboButton)
		.OnGetMenuContent(this, &SPropertyEditorClass::GenerateClassPicker)
		.ToolTipText(this, &SPropertyEditorClass::GetDisplayValueAsString)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(this, &SPropertyEditorClass::GetDisplayValueAsString)
			.Font(InArgs._Font)
		];

	ChildSlot
	[
		ComboButton.ToSharedRef()
	];

	SetEnabled(TAttribute<bool>(this, &SPropertyEditorClass::CanEdit));
}

/** Util to give better names for BP generated classes */
static FText GetClassDisplayName(const UObject* Object, bool bShowDisplayNames, const FString& InvalidObjectDisplayText)
{
	const UClass* Class = Cast<UClass>(Object);
	if (Class != nullptr)
	{
		if (bShowDisplayNames)
		{
			return Class->GetDisplayNameText();
		}
		
		UBlueprint* BP = UBlueprint::GetBlueprintFromClass(Class);
		if(BP != nullptr)
		{
			return FText::FromString(BP->GetName());
		}
	}
	return (Object) ? FText::FromString(Object->GetName()) : FText::FromString(InvalidObjectDisplayText);
}

FText SPropertyEditorClass::GetDisplayValueAsString() const
{
	static bool bIsReentrant = false;

	// Guard against re-entrancy which can happen if the delegate executed below (SelectedClass.Get()) forces a slow task dialog to open, thus causing this to lose context and regain focus later starting the loop over again
	if( !bIsReentrant )
	{
		if (bIsMultiOptionalSetter)
		{
			return FText::FromString("Multiple States");
		}

		TGuardValue<bool> Guard( bIsReentrant, true );
		if(PropertyEditor.IsValid())
		{
			UObject* ObjectValue = nullptr;
			FPropertyAccess::Result Result = PropertyEditor->GetPropertyHandle()->GetValue(ObjectValue);

			if(Result == FPropertyAccess::Success && ObjectValue != nullptr)
			{
				return GetClassDisplayName(ObjectValue, bShowDisplayNames, InvalidObjectDisplayText);
			}

			return FText::FromString(FPaths::GetBaseFilename(PropertyEditor->GetValueAsString()));
		}

		return GetClassDisplayName(SelectedClass.Get(), bShowDisplayNames, InvalidObjectDisplayText);
	}
	else
	{
		return FText::GetEmpty();
	}
}

void SPropertyEditorClass::CreateClassFilter(const TArray<TSharedRef<IClassViewerFilter>>& InClassFilters)
{
	ClassViewerOptions.bShowBackgroundBorder = false;
	ClassViewerOptions.bShowUnloadedBlueprints = true;
	ClassViewerOptions.bShowNoneOption = bAllowNone;

	if (PropertyEditor.IsValid())
	{
		ClassViewerOptions.PropertyHandle = PropertyEditor->GetPropertyHandle();
	}

	ClassViewerOptions.bIsBlueprintBaseOnly = bIsBlueprintBaseOnly;
	ClassViewerOptions.bIsPlaceableOnly = bAllowOnlyPlaceable;
	ClassViewerOptions.NameTypeToDisplay = (bShowDisplayNames ? EClassViewerNameTypeToDisplay::DisplayName : EClassViewerNameTypeToDisplay::ClassName);
	ClassViewerOptions.DisplayMode = bShowTree ? EClassViewerDisplayMode::TreeView : EClassViewerDisplayMode::ListView;
	ClassViewerOptions.bAllowViewOptions = bShowViewOptions;
	ClassViewerOptions.ClassFilters.Append(InClassFilters);
	ClassViewerOptions.bShowObjectRootClass	= bShowObjectRootClass;

	TSharedRef<FPropertyEditorClassFilter> PropEdClassFilter = MakeShared<FPropertyEditorClassFilter>();
	PropEdClassFilter->ClassPropertyMetaClass = MetaClass;
	PropEdClassFilter->InterfaceThatMustBeImplemented = RequiredInterface;
	PropEdClassFilter->bAllowAbstract = bAllowAbstract;
	PropEdClassFilter->AllowedClassFilters = AllowedClassFilters;
	PropEdClassFilter->DisallowedClassFilters = DisallowedClassFilters;

	if (PropertyEditor.IsValid())
	{
		const TSharedRef<FPropertyNode> PropertyNode = PropertyEditor->GetPropertyNode();
		const FProperty* const Property = PropertyNode->GetProperty();
		if (const FVerseClassProperty* const VerseClassProp = CastField<FVerseClassProperty>(Property))
		{
			PropEdClassFilter->bRequiresVerseConcrete = VerseClassProp->bRequiresConcrete;
			PropEdClassFilter->bRequiresVerseCastable = VerseClassProp->bRequiresCastable;
		}
	}

	ClassViewerOptions.ClassFilters.Add(PropEdClassFilter);

	ClassFilter = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassFilter(ClassViewerOptions);
	ClassFilterFuncs = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateFilterFuncs();
}

TSharedRef<SWidget> SPropertyEditorClass::GenerateClassPicker()
{
	FOnClassPicked OnPicked(FOnClassPicked::CreateSP(this, &SPropertyEditorClass::OnClassPicked));

	return SNew(SBox)
		.WidthOverride(280.0f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.MaxHeight(500.0f)
			[
				FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(ClassViewerOptions, OnPicked)
			]			
		];
}

void SPropertyEditorClass::OnClassPicked(UClass* InClass)
{
	if (!InClass)
	{
		SendToObjects(TEXT("None"));
	}
	else
	{
		SendToObjects(InClass->GetPathName());
	}

	ComboButton->SetIsOpen(false);
}

void SPropertyEditorClass::SendToObjects(const FString& NewValue)
{
	if (PropertyEditor.IsValid())
	{
		const TSharedRef<IPropertyHandle> PropertyHandle = PropertyEditor->GetPropertyHandle();
		PropertyHandle->SetValueFromFormattedString(NewValue);
	}
	else if (!NewValue.IsEmpty() && NewValue != TEXT("None"))
	{
		const UClass* NewClass = PropertyEditorHelpers::FindOrLoadClass(NewValue);
		OnSetClass.Execute(NewClass);
	}
	else
	{
		OnSetClass.Execute(nullptr);
	}
}

static UObject* LoadDragDropObject(TSharedPtr<FAssetDragDropOp> UnloadedClassOp)
{
	FString AssetPath;

	// Find the class/blueprint path
	if (UnloadedClassOp->HasAssets())
	{
		AssetPath = UnloadedClassOp->GetAssets()[0].GetObjectPathString();
	}
	else if (UnloadedClassOp->HasAssetPaths())
	{
		AssetPath = UnloadedClassOp->GetAssetPaths()[0];
	}

	// Check to see if the asset can be found, otherwise load it.
	UObject* Object = FindObject<UObject>(nullptr, *AssetPath);
	if (Object == nullptr)
	{
		// Load the package.
		GWarn->BeginSlowTask(LOCTEXT("OnDrop_LoadPackage", "Fully Loading Package For Drop"), true, false);

		Object = LoadObject<UObject>(nullptr, *AssetPath);

		GWarn->EndSlowTask();
	}

	return Object;
}

void SPropertyEditorClass::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FAssetDragDropOp> UnloadedClassOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
	if (UnloadedClassOp.IsValid())
	{
		const UObject* Object = LoadDragDropObject(UnloadedClassOp);

		bool bOK = false;

		if (const UClass* Class = Cast<UClass>(Object))
		{
			bOK = ClassFilter->IsClassAllowed(ClassViewerOptions, Class, ClassFilterFuncs.ToSharedRef());
		}
		else if (const UBlueprint* Blueprint = Cast<UBlueprint>(Object))
		{
			if (Blueprint->GeneratedClass)
			{
				bOK = ClassFilter->IsClassAllowed(ClassViewerOptions, Blueprint->GeneratedClass, ClassFilterFuncs.ToSharedRef());
			}
		}
		
		if (bOK)
		{
			UnloadedClassOp->SetToolTip(FText::GetEmpty(), FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")));
		}
		else
		{
			UnloadedClassOp->SetToolTip(FText::GetEmpty(), FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")));
		}
	}
}

void SPropertyEditorClass::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FAssetDragDropOp> UnloadedClassOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
	if (UnloadedClassOp.IsValid())
	{
		UnloadedClassOp->ResetToDefaultToolTip();
	}
}

FReply SPropertyEditorClass::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FClassDragDropOp> ClassOperation = DragDropEvent.GetOperationAs<FClassDragDropOp>();
	if (ClassOperation.IsValid())
	{
		// We can only drop one item into the combo box, so drop the first one.
		const FString ClassPath = ClassOperation->ClassesToDrop[0]->GetPathName();

		// Set the property, it will be verified as valid.
		SendToObjects(ClassPath);

		return FReply::Handled();
	}
	
	TSharedPtr<FAssetDragDropOp> UnloadedClassOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
	if (UnloadedClassOp.IsValid())
	{
		FString AssetPath;

		// Find the class/blueprint path
		if (UnloadedClassOp->HasAssets())
		{
			AssetPath = UnloadedClassOp->GetAssets()[0].GetObjectPathString();
		}
		else if (UnloadedClassOp->HasAssetPaths())
		{
			AssetPath = UnloadedClassOp->GetAssetPaths()[0];
		}

		// Check to see if the asset can be found, otherwise load it.
		UObject* Object = FindObject<UObject>(nullptr, *AssetPath);
		if(Object == nullptr)
		{
			// Load the package.
			GWarn->BeginSlowTask(LOCTEXT("OnDrop_LoadPackage", "Fully Loading Package For Drop"), true, false);

			Object = LoadObject<UObject>(nullptr, *AssetPath);

			GWarn->EndSlowTask();
		}

		if (const UClass* Class = Cast<UClass>(Object))
		{
			if (ClassFilter->IsClassAllowed(ClassViewerOptions, Class, ClassFilterFuncs.ToSharedRef()))
			{
				// This was pointing to a class directly
				SendToObjects(Class->GetPathName());
			}
		}
		else if (const UBlueprint* Blueprint = Cast<UBlueprint>(Object))
		{
			if (Blueprint->GeneratedClass)
			{
				if (ClassFilter->IsClassAllowed(ClassViewerOptions, Blueprint->GeneratedClass, ClassFilterFuncs.ToSharedRef()))
				{
					// This was pointing to a blueprint, get generated class
					SendToObjects(Blueprint->GeneratedClass->GetPathName());
				}
			}
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
