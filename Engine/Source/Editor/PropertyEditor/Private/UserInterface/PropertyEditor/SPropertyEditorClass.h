// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Misc/Attribute.h"
#include "Fonts/SlateFontInfo.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboButton.h"
#include "Styling/AppStyle.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "PropertyCustomizationHelpers.h"
#include "ClassViewerModule.h"

class IClassViewerFilter;
class FClassViewerFilterFuncs;

/**
 * A widget used to edit Class properties (UClass type properties).
 * Can also be used (with a null FPropertyEditor) to edit a raw weak class pointer.
 */
class SPropertyEditorClass : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPropertyEditorClass)
		: _Font(FAppStyle::GetFontStyle(PropertyEditorConstants::PropertyFontStyle)) 
		, _MetaClass(UObject::StaticClass())
		, _RequiredInterface(nullptr)
		, _AllowAbstract(false)
		, _IsBlueprintBaseOnly(false)
		, _AllowNone(true)
		, _ShowDisplayNames(false)
		, _InvalidObjectDisplayText("None")
		, _bIsMultiOptionalSetter(false)
		{}
		SLATE_ARGUMENT(FSlateFontInfo, Font)

		/** Arguments used when constructing this outside of a PropertyEditor (PropertyEditor == null), ignored otherwise */
		/** The meta class that the selected class must be a child-of (required if PropertyEditor == null) */
		SLATE_ARGUMENT(const UClass*, MetaClass)
		/** An interface that the selected class must implement (optional) */
		SLATE_ARGUMENT(const UClass*, RequiredInterface)
		/** Allowed class that the selected class must be a child-of. (optional) */
		SLATE_ARGUMENT(TArray<const UClass*>, AllowedClasses)
		/** Classes that the selected class cannot be a child-of. (optional) */
		SLATE_ARGUMENT(TArray<const UClass*>, DisallowedClasses)
		/** Whether or not abstract classes are allowed (optional) */
		SLATE_ARGUMENT(bool, AllowAbstract)
		/** Should only base blueprints be displayed? (optional) */
		SLATE_ARGUMENT(bool, IsBlueprintBaseOnly)
		/** Should we be able to select "None" as a class? (optional) */
		SLATE_ARGUMENT(bool, AllowNone)
		/** Attribute used to get the currently selected class (required if PropertyEditor == null) */
		SLATE_ATTRIBUTE(const UClass*, SelectedClass)
		/** Should we show the view options button at the bottom of the class picker?*/
		SLATE_ARGUMENT(bool, ShowViewOptions)
		/** Should we show the class picker in tree mode or list mode?*/
		SLATE_ARGUMENT(bool, ShowTree)
		/** Should we prettify class names on the class picker? (ie show their display name) */
		SLATE_ARGUMENT(bool, ShowDisplayNames)
		/** Delegate used to set the currently selected class (required if PropertyEditor == null) */
		SLATE_EVENT(FOnSetClass, OnSetClass)
		/** Custom class filter(s) to be applied to the class picker widget (may be empty) */
		SLATE_ARGUMENT(TArray<TSharedRef<class IClassViewerFilter>>, ClassViewerFilters)
		/** Text to show when no class is selected (Default of "None") */
		SLATE_ARGUMENT(FString, InvalidObjectDisplayText)
		/** True if this class editor is for setting a multiple-value optional */
		SLATE_ARGUMENT(bool, bIsMultiOptionalSetter)
	SLATE_END_ARGS()

	static bool Supports(const TSharedRef< class FPropertyEditor >& InPropertyEditor);

	void Construct(const FArguments& InArgs, const TSharedPtr< class FPropertyEditor >& InPropertyEditor = nullptr);

	void GetDesiredWidth(float& OutMinDesiredWidth, float& OutMaxDesiredWidth);

private:
	void SendToObjects(const FString& NewValue);

	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	TSharedRef<SWidget> ConstructClassViewer();

	/** 
	 * Generates a class picker with a filter to show only classes allowed to be selected. 
	 *
	 * @return The Class Picker widget.
	 */
	TSharedRef<SWidget> GenerateClassPicker();

	/** 
	 * Callback function from the Class Picker for when a Class is picked.
	 *
	 * @param InClass			The class picked in the Class Picker
	 */
	void OnClassPicked(UClass* InClass);

	/**
	 * Gets the active display value as a string
	 */
	FText GetDisplayValueAsString() const;

	bool CanEdit() const;

private:
	/** The property editor we were constructed for, or null if we're editing using the construction arguments */
	TSharedPtr<class FPropertyEditor> PropertyEditor;

	/** Used when the property deals with Classes and will display a Class Picker. */
	TSharedPtr<class SComboButton> ComboButton;

	/** Class filter that the class viewer is using. */
	TSharedPtr<IClassViewerFilter> ClassFilter;

	/** Filter functions for class viewer. */
	TSharedPtr<FClassViewerFilterFuncs> ClassFilterFuncs;

	/** Options used for creating the class viewer. */
	FClassViewerInitializationOptions ClassViewerOptions;

	/** The meta class that the selected class must be a child-of */
	const UClass* MetaClass;
	/** An interface that the selected class must implement */
	const UClass* RequiredInterface;
	/** Whether or not abstract classes are allowed */
	bool bAllowAbstract;
	/** Should only base blueprints be displayed? */
	bool bIsBlueprintBaseOnly;
	/** Should we be able to select "None" as a class? */
	bool bAllowNone;
	/** Should only placeable classes be displayed? */
	bool bAllowOnlyPlaceable;
	/** Should we show the view options button at the bottom of the class picker?*/
	bool bShowViewOptions;
	/** Should we show the class picker in tree mode or list mode?*/
	bool bShowTree;
	/** Should we prettify class names on the class picker? (ie show their display name) */
	bool bShowDisplayNames;
	/** Should we include /Script/CoreUObject.Object in the list? */
	bool bShowObjectRootClass;
	/** Classes that can be picked with this property */
	TArray<const UClass*> AllowedClassFilters;
	/** Classes that can NOT be picked with this property */
	TArray<const UClass*> DisallowedClassFilters;
	/** Text which is displayed instead of the normal class-text (used to show 'Set to Value' on optionals) */
	FString InvalidObjectDisplayText;
	/** True if this class editor is for setting a multiple-value optional */
	bool bIsMultiOptionalSetter;

	/** Attribute used to get the currently selected class (required if PropertyEditor == null) */
	TAttribute<const UClass*> SelectedClass;
	/** Delegate used to set the currently selected class (required if PropertyEditor == null) */
	FOnSetClass OnSetClass;

	void CreateClassFilter(const TArray<TSharedRef<IClassViewerFilter>>& InClassFilters);
};
