// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyEditor/SPropertyEditorColor.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "Components/LightComponent.h"
#include "Editor.h"
#include "PropertyNode.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Widgets/Colors/SColorPicker.h"

void SPropertyEditorColor::Construct( const FArguments& InArgs, const TSharedRef<FPropertyEditor>& InPropertyEditor, const TSharedRef<IPropertyUtilities>& InPropertyUtilities )
{
	PropertyEditor = InPropertyEditor;
	PropertyUtilities = InPropertyUtilities;

	//@todo Slate Property window: This should probably be controlled via metadata and then the actual alpha channel property hidden if its not used.
	bIgnoreAlpha = ShouldDisplayIgnoreAlpha();

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[
			ConstructColorBlock()
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[
			ConstructAlphaColorBlock()
		]
	];

	SetEnabled( TAttribute<bool>( this, &SPropertyEditorColor::CanEdit ) );
}

bool SPropertyEditorColor::Supports( const TSharedRef<FPropertyEditor>& InPropertyEditor )
{
	const FProperty* NodeProperty = InPropertyEditor->GetProperty();
	return ( CastField<const FStructProperty>(NodeProperty) && (CastField<const FStructProperty>(NodeProperty)->Struct->GetFName()==NAME_Color || CastField<const FStructProperty>(NodeProperty)->Struct->GetFName()==NAME_LinearColor) );
}

TSharedRef< SColorBlock > SPropertyEditorColor::ConstructColorBlock()
{
	return SNew( SColorBlock )
		.Color( this, &SPropertyEditorColor::OnGetColor )
		.ShowBackgroundForAlpha(true)
		.AlphaDisplayMode(bIgnoreAlpha ? EColorBlockAlphaDisplayMode::Ignore : EColorBlockAlphaDisplayMode::Combined)
		.OnMouseButtonDown( this, &SPropertyEditorColor::ColorBlock_OnMouseButtonDown )
		.Size( FVector2D( 120.0f, 20.0f ) )
		.CornerRadius(FVector4(4.0f,4.0f,4.0f,4.0f));

}

TSharedRef< SColorBlock > SPropertyEditorColor::ConstructAlphaColorBlock()
{
	// If the color is alpha'd, we always want to display the color as opaque as well as with the appropriate alpha.
	return SNew( SColorBlock )
		.Color( this, &SPropertyEditorColor::OnGetColor )
		.ShowBackgroundForAlpha(false)
		.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
		.Visibility( this, &SPropertyEditorColor::GetVisibilityForOpaqueDisplay )
		.OnMouseButtonDown( this, &SPropertyEditorColor::ColorBlock_OnMouseButtonDown )
		.Size( FVector2D( 120.0f, 20.0f ) )
		.CornerRadius(FVector4(4.0f,4.0f,4.0f,4.0f));
}

bool SPropertyEditorColor::ShouldDisplayIgnoreAlpha() const
{
	// Determines whether we should ignore the alpha value and just display the color fully opaque (such as
	// when associated with lights, for example).

	const FStructProperty* NodeStructProperty = CastField<const FStructProperty>( PropertyEditor->GetProperty() );
	check(NodeStructProperty);

	return NodeStructProperty->GetOwnerClass() && 
		( NodeStructProperty->GetOwnerClass()->IsChildOf(ULightComponent::StaticClass()) || NodeStructProperty->GetOwnerClass()->IsChildOf(UMaterialExpressionConstant3Vector::StaticClass()));
}

EVisibility SPropertyEditorColor::GetVisibilityForOpaqueDisplay() const
{
	// If the color has a non-opaque alpha, we want to display half of the box as Opaque so you can always see
	// the color even if it's mostly/entirely transparent.  But if the color is rendered as completely opaque,
	// we might as well collapse the extra opaque display rather than drawing two separate boxes.

	EVisibility OpaqueDisplayVisibility = EVisibility::Collapsed;
	if( !bIgnoreAlpha )
	{
		const bool bColorIsAlreadyOpaque = (OnGetColor().A == 1.0);
		if (bColorIsAlreadyOpaque)
		{
			OpaqueDisplayVisibility = EVisibility::Collapsed;
		}
		else
		{
			OpaqueDisplayVisibility = EVisibility::Visible;
		}
	}

	return OpaqueDisplayVisibility;
}

FReply SPropertyEditorColor::ColorBlock_OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton )
	{
		return FReply::Unhandled();
	}

	const FProperty* Property = PropertyEditor->GetProperty();
	check(Property);

	bool bRefreshOnlyOnOk = Property->GetOwnerClass() && Property->GetOwnerClass()->IsChildOf(UMaterialExpressionConstant3Vector::StaticClass());
	//@todo Slate Property window: This should probably be controlled via metadata and then the actual alpha channel property hidden if its not used.
	const bool bUseAlpha = !(Property->GetOwnerClass() && (Property->GetOwnerClass()->IsChildOf(ULightComponent::StaticClass()) || bRefreshOnlyOnOk));
	
	CreateColorPickerWindow(PropertyEditor.ToSharedRef(), bUseAlpha, bRefreshOnlyOnOk);

	return FReply::Handled();
}

FLinearColor SPropertyEditorColor::OnGetColor() const
{
	const TSharedRef< FPropertyNode > PropertyNode = PropertyEditor->GetPropertyNode();
	const FProperty* Property = PropertyEditor->GetProperty();
	check(Property);

	FLinearColor Color = FLinearColor::Black;

	FReadAddressList ReadAddresses;
	PropertyNode->GetReadAddress( false, ReadAddresses, false );

	if( ReadAddresses.Num() ) 
	{
		const uint8* Addr = ReadAddresses.GetAddress(0);
		if( Addr )
		{
			if( CastField<const FStructProperty>(Property)->Struct->GetFName() == NAME_Color )
			{
				Color = (*(FColor*)Addr).ReinterpretAsLinear();
			}
			else
			{
				check( CastField<const FStructProperty>(Property)->Struct->GetFName() == NAME_LinearColor );
				Color = *(FLinearColor*)Addr;
			}
		}
	}

	return Color;
}

void SPropertyEditorColor::CreateColorPickerWindow(const TSharedRef< class FPropertyEditor >& InPropertyEditor, bool bUseAlpha, bool bOnlyRefreshOnOk)
{
	const TSharedRef< FPropertyNode > PropertyNode = InPropertyEditor->GetPropertyNode();

	FProperty* Property = PropertyNode->GetProperty();
	check(Property);

	FReadAddressList ReadAddresses;
	PropertyNode->GetReadAddress( false, ReadAddresses, false );

	FLinearColor InitialColor = FLinearColor(ForceInit);
	if( ReadAddresses.Num() )
	{
		OriginalColors.Empty( ReadAddresses.Num() );
		OriginalColors.AddUninitialized( ReadAddresses.Num() );

		bool bClampValue = false;
		// Store off the original colors in the case that the user cancels the color picker. We'll revert to the original colors in that case
		for( int32 AddrIndex = 0; AddrIndex <  ReadAddresses.Num(); ++AddrIndex )
		{
		
			const uint8* Addr = ReadAddresses.GetAddress(AddrIndex);
			if( Addr )
			{
				if( CastField<FStructProperty>(Property)->Struct->GetFName() == NAME_Color )
				{
					OriginalColors[AddrIndex] = ((FColor*)Addr)->ReinterpretAsLinear();
					bClampValue = true;
				}
				else
				{
					check( CastField<FStructProperty>(Property)->Struct->GetFName() == NAME_LinearColor );
					OriginalColors[AddrIndex] = *(FLinearColor*)Addr;
				}
			}
		}

		// Only one color can be the initial color.  Just use the first color property
		InitialColor = OriginalColors[0];

		FColorPickerArgs PickerArgs;
		PickerArgs.bOnlyRefreshOnMouseUp = true;
		PickerArgs.ParentWidget = AsShared();
		PickerArgs.bUseAlpha = bUseAlpha;
		PickerArgs.bOnlyRefreshOnOk = bOnlyRefreshOnOk;
		PickerArgs.bClampValue = bClampValue;
		PickerArgs.DisplayGamma = TAttribute<float>::Create( TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma) );
		PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP( this, &SPropertyEditorColor::SetColor);
		PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateSP( this, &SPropertyEditorColor::OnColorPickerCancelled );
		PickerArgs.InitialColor = InitialColor;

		OpenColorPicker(PickerArgs);
	}
	

}

void SPropertyEditorColor::SetColor(FLinearColor NewColor)
{
	const TSharedRef< FPropertyNode > PropertyNode = PropertyEditor->GetPropertyNode();

	FProperty* Property = PropertyNode->GetProperty();
	check(Property);
	
	FReadAddressList ReadAddresses;
	PropertyNode->GetReadAddress( false, ReadAddresses, false );

	if( ReadAddresses.Num() ) 
	{
		GEditor->BeginTransaction( NSLOCTEXT("UnrealEd", "SetColorProperty", "Set Color Property") );

		PropertyNode->NotifyPreChange(Property, PropertyUtilities->GetNotifyHook());
		for( int32 AddrIndex = 0; AddrIndex <  ReadAddresses.Num(); ++AddrIndex )
		{
			const uint8* Addr = ReadAddresses.GetAddress(AddrIndex);
			if( Addr )
			{
				if( CastField<FStructProperty>(Property)->Struct->GetFName() == NAME_Color )
				{
					*(FColor*)Addr = NewColor.ToFColor(false);
				}
				else
				{
					check( CastField<FStructProperty>(Property)->Struct->GetFName() == NAME_LinearColor );
					*(FLinearColor*)Addr = NewColor;
				}
			}
		}

		FPropertyChangedEvent ChangeEvent(Property, EPropertyChangeType::ValueSet);
		PropertyNode->NotifyPostChange( ChangeEvent, PropertyUtilities->GetNotifyHook() );

		GEditor->EndTransaction();		
	}
}

void SPropertyEditorColor::OnColorPickerCancelled( FLinearColor OriginalColor )
{
	const TSharedRef< FPropertyNode > PropertyNode = PropertyEditor->GetPropertyNode();

	FProperty* Property = PropertyNode->GetProperty();
	check(Property);

	FReadAddressList ReadAddresses;
	PropertyNode->GetReadAddress( false, ReadAddresses, false );

	if( ReadAddresses.Num() ) 
	{
		check( OriginalColors.Num() == ReadAddresses.Num() );
		PropertyNode->NotifyPreChange(Property, PropertyUtilities->GetNotifyHook());

		for( int32 AddrIndex = 0; AddrIndex < ReadAddresses.Num(); ++AddrIndex )
		{
			const uint8* Addr = ReadAddresses.GetAddress(AddrIndex);
			if( Addr )
			{
				if( CastField<FStructProperty>(Property)->Struct->GetFName() == NAME_Color )
				{
					*(FColor*)Addr = OriginalColors[AddrIndex].ToFColor(false);
				}
				else
				{
					check( CastField<FStructProperty>(Property)->Struct->GetFName() == NAME_LinearColor );
					*(FLinearColor*)Addr = OriginalColors[AddrIndex];
				}
			}
		}

		FPropertyChangedEvent ChangeEvent(Property, EPropertyChangeType::ValueSet);
		PropertyNode->NotifyPostChange( ChangeEvent, PropertyUtilities->GetNotifyHook() );
	}

	OriginalColors.Empty();
}

bool SPropertyEditorColor::CanEdit() const
{
	return PropertyEditor.IsValid() ? !PropertyEditor->IsEditConst() : true;
}
