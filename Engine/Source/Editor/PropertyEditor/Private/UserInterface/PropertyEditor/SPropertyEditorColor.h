// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "IPropertyUtilities.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"

class SColorBlock;

class SPropertyEditorColor : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SPropertyEditorColor ) {}
	SLATE_END_ARGS()

	static bool Supports( const TSharedRef< class FPropertyEditor >& PropertyEditor );

	void Construct( const FArguments& InArgs, const TSharedRef< class FPropertyEditor >& InPropertyEditor, const TSharedRef< class IPropertyUtilities >& InPropertyUtilities );

protected:

	/**	@return A newly constructed SColorBlock */
	TSharedRef< SColorBlock > ConstructColorBlock();

	/**	@return A newly constructed SColorBlock setup specifically for handling alpha */
	TSharedRef< SColorBlock > ConstructAlphaColorBlock();

	/**
	 * ShouldDisplayIgnoreAlpha determines whether or not to ignore the alpha value and just display the color as fully
	 * opaque (such as when associated with lights, for example).
	 *
	 * @return Whether to display the color as completely opaque regardless of its alpha value.
	 */
	bool ShouldDisplayIgnoreAlpha() const;

	/** @return Gets the color for the color block to use for its TAttribute */
	FLinearColor OnGetColor() const;

	/**
	* The system calls this method to notify the widget that a mouse button was pressed within it. This event is bubbled.
	* The color block will spawn a new color picker and destroy the old one if applicable.
	*
	* @param MyGeometry The Geometry of the widget receiving the event
	* @param MouseEvent Information about the input event
	*
	* @return Whether the event was handled along with possible requests for the system to take action.
	*/
	FReply ColorBlock_OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/**
	 * If the color has a non-opaque alpha, we want to display half of the box as opaque so you can always see
	 * the color even if it's mostly/entirely transparent.  But if the color is rendered as completely opaque,
	 * we might as well collapse the extra opaque display rather than drawing two separate boxes.
	 *
	 * @return Whether the always-opaque part of the display is visible or collapsed (the latter is if the main
	 * part of the display is already being rendered as opaque).
	 */
	EVisibility GetVisibilityForOpaqueDisplay() const;

	void CreateColorPickerWindow(const TSharedRef< class FPropertyEditor >& PropertyEditor, bool bUseAlpha, bool bOnlyRefreshOnOk=false);

	void SetColor(FLinearColor NewColor);

	/**
	 * Called when the color picker is cancelled
	 */
	void OnColorPickerCancelled( FLinearColor OriginalColor );

	/** @return True if the property can be edited */
	bool CanEdit() const;

protected:
	/** Original colors to set in the case that the user cancel's the color picker */
	TArray< FLinearColor > OriginalColors;

	TSharedPtr< class FPropertyEditor > PropertyEditor;

	TSharedPtr< class IPropertyUtilities > PropertyUtilities;

	/** Whether or not alpha should be displayed.  Some color properties (such as lights) do not use alpha. */
	bool bIgnoreAlpha;
};
