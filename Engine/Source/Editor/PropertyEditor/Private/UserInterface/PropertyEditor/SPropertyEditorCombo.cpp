// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyEditor/SPropertyEditorCombo.h"
#include "IDocumentation.h"
#include "Widgets/SToolTip.h"
#include "PropertyEditorHelpers.h"
#include "UserInterface/PropertyEditor/SPropertyComboBox.h"


#define LOCTEXT_NAMESPACE "PropertyEditor"

void SPropertyEditorCombo::GetDesiredWidth( float& OutMinDesiredWidth, float& OutMaxDesiredWidth )
{
	OutMinDesiredWidth = 125.0f;
	OutMaxDesiredWidth = 400.0f;
}

bool SPropertyEditorCombo::Supports( const TSharedRef< class FPropertyEditor >& InPropertyEditor )
{
	const TSharedRef< FPropertyNode > PropertyNode = InPropertyEditor->GetPropertyNode();
	const FProperty* Property = InPropertyEditor->GetProperty();
	int32 ArrayIndex = PropertyNode->GetArrayIndex();

	if(	((Property->IsA(FByteProperty::StaticClass()) && CastField<const FByteProperty>(Property)->Enum)
		||	Property->IsA(FEnumProperty::StaticClass())
		|| (Property->IsA(FStrProperty::StaticClass()) && Property->HasMetaData(TEXT("Enum")))
		|| !PropertyEditorHelpers::GetPropertyOptionsMetaDataKey(Property).IsNone()
		)
		&&	( ( ArrayIndex == -1 && Property->ArrayDim == 1 ) || ( ArrayIndex > -1 && Property->ArrayDim > 0 ) ) )
	{
		return true;
	}

	return false;
}

void SPropertyEditorCombo::Construct( const FArguments& InArgs, const TSharedPtr< class FPropertyEditor >& InPropertyEditor )
{
	PropertyEditor = InPropertyEditor;
	ComboArgs = InArgs._ComboArgs;

	bool bSegmentedDisplay = false;
	if (PropertyEditor.IsValid())
	{
		ComboArgs.PropertyHandle = PropertyEditor->GetPropertyHandle();
		if (ComboArgs.PropertyHandle.IsValid())
		{
			ComboArgs.PropertyHandle->SetOnPropertyResetToDefault(FSimpleDelegate::CreateSP(this, &SPropertyEditorCombo::OnResetToDefault));
			bSegmentedDisplay = ComboArgs.PropertyHandle->GetBoolMetaData(TEXT("SegmentedDisplay"));
		}
	}

	ensureMsgf(ComboArgs.PropertyHandle.IsValid() || (ComboArgs.OnGetStrings.IsBound() && ComboArgs.OnGetValue.IsBound() && ComboArgs.OnValueSelected.IsBound()), TEXT("Either PropertyEditor or ComboArgs.PropertyHandle must be set!"));

	TArray<TSharedPtr<FString>> ComboItems;
	TArray<bool> Restrictions;
	TArray<TSharedPtr<SToolTip>> RichToolTips;

	if (!ComboArgs.Font.HasValidFont())
	{
		ComboArgs.Font = FAppStyle::GetFontStyle(PropertyEditorConstants::PropertyFontStyle);
	}

	GenerateComboBoxStrings(ComboItems, RichToolTips, Restrictions);

	if (bSegmentedDisplay)
	{
		ParameterTextStyle = FTextBlockStyle(FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText"))
			.SetFont(ComboArgs.Font);
		
		SAssignNew(SegmentControl, SSegmentedControl<FString>)
			.UniformPadding(FMargin(10, 5))
			.TextStyle(&ParameterTextStyle)
			.Value(this, &SPropertyEditorCombo::GetDisplayValueAsString)
			.OnValueChanged(this, &SPropertyEditorCombo::OnSegmentedControlSelectionChanged);

		for (int ItemIndex = 0; ItemIndex < ComboItems.Num(); ItemIndex++)
		{
			const FString& ComboItem = *ComboItems[ItemIndex];
			SSegmentedControl<FString>::FScopedWidgetSlotArguments Slot = SegmentControl->AddSlot(ComboItem);
			
			FText DisplayName = FText::FromString(ComboItem);
			FText TooltipText = DisplayName;
			if (RichToolTips.IsValidIndex(ItemIndex) && !RichToolTips[ItemIndex]->IsEmpty())
			{
				Slot.ToolTipWidget(RichToolTips[ItemIndex]);
			}
			else
			{
				Slot.ToolTip(DisplayName);
			}
			
			Slot
			  .HAlign(HAlign_Center)
			  .VAlign(VAlign_Center)
			  .Text(DisplayName);
		}
		
		ChildSlot
		[
			SegmentControl.ToSharedRef()
		];
	}
	else
	{
		SAssignNew(ComboBox, SPropertyComboBox)
			.Font( ComboArgs.Font )
			.RichToolTipList( RichToolTips )
			.ComboItemList( ComboItems )
			.RestrictedList( Restrictions )
			.OnSelectionChanged( this, &SPropertyEditorCombo::OnComboSelectionChanged )
			.OnComboBoxOpening( this, &SPropertyEditorCombo::OnComboOpening )
			.VisibleText( this, &SPropertyEditorCombo::GetDisplayValueAsString )
			.ToolTipText( this, &SPropertyEditorCombo::GetValueToolTip )
			.ShowSearchForItemCount( ComboArgs.ShowSearchForItemCount );

		ChildSlot
		[
			ComboBox.ToSharedRef()
		];
	}

	SetEnabled( TAttribute<bool>( this, &SPropertyEditorCombo::CanEdit ) );
	SetToolTipText( TAttribute<FText>( this, &SPropertyEditorCombo::GetValueToolTip) );
}

FString SPropertyEditorCombo::GetDisplayValueAsString() const
{
	if (ComboArgs.OnGetValue.IsBound())
	{
		return ComboArgs.OnGetValue.Execute();
	}
	else if (bUsesAlternateDisplayValues)
	{
		{
			FString RawValueString;
			ComboArgs.PropertyHandle->GetValueAsFormattedString(RawValueString, PPF_None);

			if (const FString* AlternateDisplayValuePtr = InternalValueToAlternateDisplayValue.Find(RawValueString))
			{
				return *AlternateDisplayValuePtr;
			}
		}

		if (PropertyEditor.IsValid())
		{
			return PropertyEditor->GetValueAsDisplayString();
		}

		FString ValueString;
		ComboArgs.PropertyHandle->GetValueAsDisplayString(ValueString);
		return ValueString;
	}
	else
	{
		if (PropertyEditor.IsValid())
		{
			return PropertyEditor->GetValueAsString();
		}

		FString ValueString;
		ComboArgs.PropertyHandle->GetValueAsFormattedString(ValueString);
		return ValueString;
	}
}

FText SPropertyEditorCombo::GetValueToolTip() const
{
	if (bUsesAlternateDisplayValues)
	{
		FString RawValueString;
		ComboArgs.PropertyHandle->GetValueAsFormattedString(RawValueString, PPF_None);

		if (const FString* AlternateDisplayValuePtr = InternalValueToAlternateDisplayValue.Find(RawValueString))
		{
			return FText::AsCultureInvariant(*AlternateDisplayValuePtr);
		}
	}

	if (PropertyEditor.IsValid())
	{
		return PropertyEditor->GetValueAsText();
	}

	return FText();
}

void SPropertyEditorCombo::GenerateComboBoxStrings( TArray< TSharedPtr<FString> >& OutComboBoxStrings, TArray<TSharedPtr<SToolTip>>& RichToolTips, TArray<bool>& OutRestrictedItems )
{
	if (ComboArgs.OnGetStrings.IsBound())
	{
		ComboArgs.OnGetStrings.Execute(OutComboBoxStrings, RichToolTips, OutRestrictedItems);
		return;
	}

	TArray<FString> ValueStrings;
	TArray<FText> BasicTooltips;
	TArray<FText> DisplayNames;
	
	bUsesAlternateDisplayValues = ComboArgs.PropertyHandle->GeneratePossibleValues(ValueStrings, BasicTooltips, OutRestrictedItems, &DisplayNames);

	// Build the reverse LUT for alternate display values
	AlternateDisplayValueToInternalValue.Reset();
	InternalValueToAlternateDisplayValue.Reset();
	if (const FProperty* Property = ComboArgs.PropertyHandle->GetProperty();
		bUsesAlternateDisplayValues)
	{
		if (ensureMsgf(ValueStrings.Num() == DisplayNames.Num(), TEXT("Mismatched Value and DisplayNames Array")))
		{
			AlternateDisplayValueToInternalValue.Reserve(ValueStrings.Num());
			InternalValueToAlternateDisplayValue.Reserve(ValueStrings.Num());
			for (int32 Index = 0, End = ValueStrings.Num(); Index < End; ++Index)
			{
				AlternateDisplayValueToInternalValue.Emplace(DisplayNames[Index].ToString(), ValueStrings[Index]);
				InternalValueToAlternateDisplayValue.Emplace(ValueStrings[Index], DisplayNames[Index].ToString());
			}
			Algo::Transform(DisplayNames, OutComboBoxStrings, [](const FText& Str) { return MakeShared<FString>(Str.ToString()); });
		}
		else
		{
			bUsesAlternateDisplayValues = false;
			Algo::Transform(ValueStrings, OutComboBoxStrings, [](const FString& Str) { return MakeShared<FString>(Str); });
		}
	}
	else
	{
		Algo::Transform(ValueStrings, OutComboBoxStrings, [](const FString& Str) { return MakeShared<FString>(Str); });
	}
	
	// If we regenerate the entries, let's make sure that the currently selected item has the same shared pointer as
	// the newly generated item with the same value, so that the generation of elements won't immediately result in a
	// value changed event (i.e. at every single `OnComboOpening`).
	if (ComboBox)
	{
		if (const TSharedPtr<FString>& SelectedItem = ComboBox->GetSelectedItem())
		{
			for (TSharedPtr<FString>& Item : OutComboBoxStrings)
			{
				if (Item)
				{
					if (*SelectedItem.Get() == *Item.Get())
					{
						Item = SelectedItem;
						break;
					}
				}
			}
		}
	}

	// For enums, look for rich tooltip information
	if(ComboArgs.PropertyHandle.IsValid())
	{
		if(const FProperty* Property = ComboArgs.PropertyHandle->GetProperty())
		{
			UEnum* Enum = nullptr;

			if(const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
			{
				Enum = ByteProperty->Enum;
			}
			else if(const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
			{
				Enum = EnumProperty->GetEnum();
			}

			if(Enum)
			{
				TArray<FName> ValidPropertyEnums = PropertyEditorHelpers::GetValidEnumsFromPropertyOverride(Property, Enum);
				TArray<FName> InvalidPropertyEnums = PropertyEditorHelpers::GetInvalidEnumsFromPropertyOverride(Property, Enum);
				
				// Get enum doc link (not just GetDocumentationLink as that is the documentation for the struct we're in, not the enum documentation)
				FString DocLink = PropertyEditorHelpers::GetEnumDocumentationLink(Property);

				for(int32 EnumIdx = 0; EnumIdx < Enum->NumEnums() - 1; ++EnumIdx)
				{
					FString Excerpt = Enum->GetNameStringByIndex(EnumIdx);

					bool bShouldBeHidden = Enum->HasMetaData(TEXT("Hidden"), EnumIdx) || Enum->HasMetaData(TEXT("Spacer"), EnumIdx);
					if(!bShouldBeHidden)
					{
						if (ValidPropertyEnums.Num() > 0)
						{
							bShouldBeHidden = ValidPropertyEnums.Find(Enum->GetNameByIndex(EnumIdx)) == INDEX_NONE;
						}

						// If both are specified, the metadata "InvalidEnumValues" takes precedence
						if (InvalidPropertyEnums.Num() > 0)
						{
							bShouldBeHidden = InvalidPropertyEnums.Find(Enum->GetNameByIndex(EnumIdx)) != INDEX_NONE;
						}
					}

					if(!bShouldBeHidden)
					{
						bShouldBeHidden = ComboArgs.PropertyHandle->IsHidden(Excerpt);
					}
				
					if(!bShouldBeHidden)
					{
						RichToolTips.Add(IDocumentation::Get()->CreateToolTip(MoveTemp(BasicTooltips[EnumIdx]), nullptr, DocLink, MoveTemp(Excerpt)));
					}
				}
			}
		}
	}
}

void SPropertyEditorCombo::OnComboSelectionChanged( TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo )
{
	if ( NewValue.IsValid() )
	{
		SendToObjects( *NewValue );
	}
}

void SPropertyEditorCombo::OnSegmentedControlSelectionChanged(FString NewValue)
{
	SendToObjects(NewValue);
}

void SPropertyEditorCombo::OnResetToDefault()
{
	FString CurrentDisplayValue = GetDisplayValueAsString();
	if (ComboBox)
	{
		ComboBox->SetSelectedItem(CurrentDisplayValue);
	}
	if (SegmentControl)
	{
		SegmentControl->SetValue(CurrentDisplayValue);
	}
}

void SPropertyEditorCombo::OnComboOpening()
{
	TArray<TSharedPtr<FString>> ComboItems;
	TArray<TSharedPtr<SToolTip>> RichToolTips;
	TArray<bool> Restrictions;
	GenerateComboBoxStrings(ComboItems, RichToolTips, Restrictions);

	ComboBox->SetItemList(ComboItems, RichToolTips, Restrictions);

	// try and re-sync the selection in the combo list in case it was changed since Construct was called
	// this would fail if the displayed value doesn't match the equivalent value in the combo list
	FString CurrentDisplayValue = GetDisplayValueAsString();
	ComboBox->SetSelectedItem(CurrentDisplayValue);
}

void SPropertyEditorCombo::SendToObjects( const FString& NewValue )
{
	FString Value = NewValue;
	if (ComboArgs.OnValueSelected.IsBound())
	{
		ComboArgs.OnValueSelected.Execute(NewValue);
	}
	else if (ComboArgs.PropertyHandle.IsValid())
	{
		FProperty* Property = ComboArgs.PropertyHandle->GetProperty();

		if (bUsesAlternateDisplayValues)
		{
			if (const FString* InternalValuePtr = AlternateDisplayValueToInternalValue.Find(Value))
			{
				Value = *InternalValuePtr;
			}
		}

		ComboArgs.PropertyHandle->SetValueFromFormattedString(Value);
	}
}

bool SPropertyEditorCombo::CanEdit() const
{
	if (ComboArgs.PropertyHandle.IsValid())
	{
		return ComboArgs.PropertyHandle->IsEditable();
	}
	return true;
}

#undef LOCTEXT_NAMESPACE
