// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/AppStyle.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SSegmentedControl.h"

class SPropertyComboBox;

class SPropertyEditorCombo : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SPropertyEditorCombo )
		{}
		SLATE_ARGUMENT( FPropertyComboBoxArgs, ComboArgs )
	SLATE_END_ARGS()

	static bool Supports( const TSharedRef< class FPropertyEditor >& InPropertyEditor );

	/** Constructs widget, if InPropertyEditor is null then PropertyHandle must be set */
	void Construct( const FArguments& InArgs, const TSharedPtr< class FPropertyEditor >& InPropertyEditor = nullptr);

	void GetDesiredWidth( float& OutMinDesiredWidth, float& OutMaxDesiredWidth );
private:
	void GenerateComboBoxStrings( TArray< TSharedPtr<FString> >& OutComboBoxStrings, TArray<TSharedPtr<class SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems );
	void OnComboSelectionChanged( TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo );
	void OnSegmentedControlSelectionChanged( FString NewValue );
	void OnResetToDefault();
	void OnComboOpening();

	virtual void SendToObjects( const FString& NewValue );

	/**
	 * Gets the active display value as a string
	 */
	FString GetDisplayValueAsString() const;

	/**
	 * Get the active display value tooltip.
	 */
	FText GetValueToolTip() const;

	/** @return True if the property can be edited */
	bool CanEdit() const;
private:

	/** Property editor this was created from, may be null */
	TSharedPtr< class FPropertyEditor > PropertyEditor;

	/** Fills out with generated strings. */
	TSharedPtr<class SPropertyComboBox> ComboBox;

	/** Alternative display widget. */
	TSharedPtr<SSegmentedControl<FString>> SegmentControl;
	FTextBlockStyle ParameterTextStyle;

	/** Arguments used to construct the combo box */
	FPropertyComboBoxArgs ComboArgs;

	/**
	 * Indicates that this combo box's values are friendly names for the real values; currently only used for enum drop-downs.
	 */
	bool bUsesAlternateDisplayValues;
	TMap<FString, FString> AlternateDisplayValueToInternalValue;
	TMap<FString, FString> InternalValueToAlternateDisplayValue;
};
