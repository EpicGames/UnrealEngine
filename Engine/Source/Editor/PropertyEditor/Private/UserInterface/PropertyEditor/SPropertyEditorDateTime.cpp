// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyEditor/SPropertyEditorDateTime.h"

#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "PropertyHandle.h"
#include "UObject/Field.h"
#include "UObject/UnrealType.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SWidget.h"

class FPropertyNode;
class UScriptStruct;


void SPropertyEditorDateTime::Construct( const FArguments& InArgs, const TSharedRef< class FPropertyEditor >& InPropertyEditor )
{
	PropertyEditor = InPropertyEditor;

	ChildSlot
	[
		SAssignNew( PrimaryWidget, SEditableTextBox )
		.Text( InPropertyEditor, &FPropertyEditor::GetValueAsText )
		.Font( InArgs._Font )
		.SelectAllTextWhenFocused(true)
		.ClearKeyboardFocusOnCommit(false)
		.OnTextCommitted(this, &SPropertyEditorDateTime::HandleTextCommitted)
		.SelectAllTextOnCommit(true)
	];

	if( InPropertyEditor->PropertyIsA( FObjectPropertyBase::StaticClass() ) )
	{
		// Object properties should display their entire text in a tooltip
		PrimaryWidget->SetToolTipText( TAttribute<FText>( InPropertyEditor, &FPropertyEditor::GetValueAsText ) );
	}

	SetEnabled(TAttribute<bool>(this, &SPropertyEditorDateTime::CanEdit));
}


bool SPropertyEditorDateTime::Supports( const TSharedRef< FPropertyEditor >& InPropertyEditor )
{
	const TSharedRef< FPropertyNode > PropertyNode = InPropertyEditor->GetPropertyNode();
	const FProperty* Property = InPropertyEditor->GetProperty();

	if (Property->IsA(FStructProperty::StaticClass()))
	{
		const FStructProperty* StructProp = CastField<const FStructProperty>(Property);
		if (TBaseStructure<FDateTime>::Get() == StructProp->Struct)
		{
			return true;
		}
	}

	return false;
}


void SPropertyEditorDateTime::HandleTextCommitted( const FText& NewText, ETextCommit::Type /*CommitInfo*/ )
{
	const TSharedRef<FPropertyNode> PropertyNode = PropertyEditor->GetPropertyNode();
	const TSharedRef<IPropertyHandle> PropertyHandle = PropertyEditor->GetPropertyHandle();

	PropertyHandle->SetValueFromFormattedString(NewText.ToString());
}

/** @return True if the property can be edited */
bool SPropertyEditorDateTime::CanEdit() const
{
	return PropertyEditor.IsValid() ? !PropertyEditor->IsEditConst() : true;
}
