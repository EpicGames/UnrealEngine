// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Fonts/SlateFontInfo.h"
#include "Misc/Attribute.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FText;

class SPropertyEditorDateTime : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SPropertyEditorDateTime )
		: _Font( FAppStyle::GetFontStyle( PropertyEditorConstants::PropertyFontStyle ) ) 
		{}
		SLATE_ATTRIBUTE( FSlateFontInfo, Font )
	SLATE_END_ARGS()

	static bool Supports( const TSharedRef< class FPropertyEditor >& InPropertyEditor );

	void Construct( const FArguments& InArgs, const TSharedRef< class FPropertyEditor >& InPropertyEditor );

	void GetDesiredWidth( float& OutMinDesiredWidth, float& OutMaxDesiredWidth );

private:

	virtual void HandleTextCommitted( const FText& NewText, ETextCommit::Type CommitInfo );

	bool CanEdit() const;

private:

	TSharedPtr< class FPropertyEditor > PropertyEditor;

	TSharedPtr< class SWidget > PrimaryWidget;
};
