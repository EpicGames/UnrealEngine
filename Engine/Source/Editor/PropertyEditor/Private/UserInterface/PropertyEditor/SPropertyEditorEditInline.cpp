// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyEditor/SPropertyEditorEditInline.h"
#include "Algo/AnyOf.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Images/SImage.h"
#include "PropertyEditorHelpers.h"
#include "PropertyRestriction.h"
#include "ObjectPropertyNode.h"
#include "PropertyHandleImpl.h"
#include "ClassViewerModule.h"
#include "ClassViewerFilter.h"
#include "Styling/SlateIconFinder.h"
#include "UObject/ConstructorHelpers.h"
#include "Editor.h"
#include "PropertyEditorHelpers.h"
#include "PropertyEditorUtils.h"

class FPropertyEditorInlineClassFilter : public IClassViewerFilter
{
public:
	/** The Object Property, classes are examined for a child-of relationship of the property's class. */
	FObjectPropertyBase* ObjProperty;

	/** The Interface Property, classes are examined for implementing the property's class. */
	FInterfaceProperty* IntProperty;

	/** Whether or not abstract classes are allowed. */
	bool bAllowAbstract;

	/** Hierarchy of objects that own this property. Used to check against ClassWithin. */
	TSet< const UObject* > OwningObjects;
	
	/** The interface that must be implemented. */
	const UClass* InterfaceThatMustBeImplemented;

	/** Classes that can be picked */
	TArray<const UClass*> AllowedClassFilters;

	/** Classes that can't be picked */
	TArray<const UClass*> DisallowedClassFilters;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs ) override
	{
		auto IsClassChildOf = [InClass](const UClass* FilterClass)
		{
			return InClass->IsChildOf(FilterClass);
		};
		
		return IsClassAllowedHelper(InClass, IsClassChildOf, InFilterFuncs);
	}
	
	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InBlueprint, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		auto IsClassChildOf = [InBlueprint](const UClass* FilterClass)
		{
			if (FilterClass && InBlueprint->GetClassPathName() == FilterClass->GetClassPathName())
			{
				return true;
			}
			
			return InBlueprint->IsChildOf(FilterClass);
		};
		
		return IsClassAllowedHelper(InBlueprint, IsClassChildOf, InFilterFuncs);
	}

private:

	template <typename TClass, typename TIsChildOfFunction>
	bool IsClassAllowedHelper(TClass InClass, TIsChildOfFunction IsClassChildOf, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs)
	{
		const bool bMatchesFlags = InClass->HasAnyClassFlags(CLASS_EditInlineNew) &&
			!InClass->HasAnyClassFlags(CLASS_Hidden | CLASS_HideDropDown | CLASS_Deprecated) &&
			(bAllowAbstract || !InClass->HasAnyClassFlags(CLASS_Abstract));

		if (!bMatchesFlags)
		{
			return false;
		}

		// If it's neither an object property that is a child of the object class or an interface property that doesn't implement the interface, skip
		const bool bChildOfObjectClass = ObjProperty && InClass->IsChildOf(ObjProperty->PropertyClass);
		const bool bDerivedInterfaceClass = IntProperty && InClass->ImplementsInterface(IntProperty->InterfaceClass);
		if (!bChildOfObjectClass && !bDerivedInterfaceClass)
		{
			return false;
		}

		// If the class doesn't satisfy an interface requirement, skip
		if ((InterfaceThatMustBeImplemented != nullptr) && !InClass->ImplementsInterface(InterfaceThatMustBeImplemented))
		{
			return false;
		}

		// If the the class is explicitly present in the disallowed class filter set, we can't allow it
		if (Algo::AnyOf(DisallowedClassFilters, IsClassChildOf))
		{
			return false;
		}

		// If the class is explicitly not present in the allowed class filter set, we can't allow it
		if (!AllowedClassFilters.IsEmpty() && Algo::NoneOf(AllowedClassFilters, IsClassChildOf))
		{
			return false;
		}

		// Verify that the Owners of the property satisfy the ClassWithin constraint of the given class.
		// When ClassWithin is null, assume it can be owned by anything.
		const UClass* ClassWithin = GetClassWithin(InClass);
		if (ClassWithin != nullptr && InFilterFuncs->IfMatchesAll_ObjectsSetIsAClass(OwningObjects, ClassWithin) ==
			EFilterReturn::Failed)
		{
			return false;
		}

		return true;
	}

	static const UClass* GetClassWithin(const UClass* InClass)
	{
		return InClass->ClassWithin;
	}

	static const UClass* GetClassWithin(const TSharedRef< const IUnloadedBlueprintData > InBlueprint)
	{
		return InBlueprint->GetClassWithin();
	}
};

void SPropertyEditorEditInline::Construct( const FArguments& InArgs, const TSharedRef< class FPropertyEditor >& InPropertyEditor )
{
	PropertyEditor = InPropertyEditor;

	TWeakPtr<IPropertyHandle> WeakHandlePtr = InPropertyEditor->GetPropertyHandle();

	ChildSlot
	[
		SAssignNew(ComboButton, SComboButton)
		.IsEnabled(this, &SPropertyEditorEditInline::IsValueEnabled, WeakHandlePtr)
		.OnGetMenuContent(FOnGetContent::CreateStatic(&SPropertyEditorEditInline::GenerateClassPicker, 
														InPropertyEditor->GetPropertyHandle(), 
														FOnClassPicked::CreateSP(this, &SPropertyEditorEditInline::OnClassPickedInternal, InPropertyEditor->GetPropertyHandle()),
														TSharedPtr<IClassViewerFilter>(nullptr)))
		.ContentPadding(0.0f)
		.ToolTipText(InPropertyEditor, &FPropertyEditor::GetValueAsText )
		.ButtonContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew( SImage )
				.Image( this, &SPropertyEditorEditInline::GetDisplayValueIcon )
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew( STextBlock )
				.Text( this, &SPropertyEditorEditInline::GetDisplayValueAsString )
				.Font( InArgs._Font )
			]
		]
	];
}

bool SPropertyEditorEditInline::IsValueEnabled(TWeakPtr<IPropertyHandle> WeakHandlePtr) const
{
	if (WeakHandlePtr.IsValid())
	{
		return !WeakHandlePtr.Pin()->IsEditConst();
	}

	return false;
}

FText SPropertyEditorEditInline::GetDisplayValueAsString() const
{
	UObject* CurrentValue = NULL;
	FPropertyAccess::Result Result = PropertyEditor->GetPropertyHandle()->GetValue( CurrentValue );
	if( Result == FPropertyAccess::Success && CurrentValue != NULL )
	{
		return CurrentValue->GetClass()->GetDisplayNameText();
	}
	else
	{
		return PropertyEditor->GetValueAsText();
	}
}

const FSlateBrush* SPropertyEditorEditInline::GetDisplayValueIcon() const
{
	UObject* CurrentValue = nullptr;
	FPropertyAccess::Result Result = PropertyEditor->GetPropertyHandle()->GetValue( CurrentValue );
	if( Result == FPropertyAccess::Success && CurrentValue != nullptr )
	{
		return FSlateIconFinder::FindIconBrushForClass(CurrentValue->GetClass());
	}

	return nullptr;
}

void SPropertyEditorEditInline::OnClassPickedInternal(UClass* InClass, TSharedRef<IPropertyHandle> PropertyHandle)
{
	SPropertyEditorEditInline::OnClassPicked(InClass, PropertyHandle);
	PropertyEditor->ForceRefresh();
	ComboButton->SetIsOpen(false);
}

void SPropertyEditorEditInline::GetDesiredWidth( float& OutMinDesiredWidth, float& OutMaxDesiredWidth )
{
	OutMinDesiredWidth = 250.0f;
	OutMaxDesiredWidth = 600.0f;
}

bool SPropertyEditorEditInline::Supports( const FPropertyNode* InTreeNode, int32 InArrayIdx )
{
	return InTreeNode
		&& (InTreeNode->HasNodeFlags(EPropertyNodeFlags::EditInlineNew) || InTreeNode->HasNodeFlags(EPropertyNodeFlags::DynamicInstance))
		&& InTreeNode->FindObjectItemParent()
		&& !InTreeNode->IsPropertyConst();
}

bool SPropertyEditorEditInline::Supports( const TSharedRef< class FPropertyEditor >& InPropertyEditor )
{
	const TSharedRef< FPropertyNode > PropertyNode = InPropertyEditor->GetPropertyNode();
	return SPropertyEditorEditInline::Supports( &PropertyNode.Get(), PropertyNode->GetArrayIndex() );
}
 

TSharedRef<SWidget> SPropertyEditorEditInline::GenerateClassPicker(TSharedRef<IPropertyHandle> PropertyHandle, FOnClassPicked OnPicked, TSharedPtr<IClassViewerFilter> AdditionalClassFilter) // static
{
	FClassViewerInitializationOptions Options;
	Options.bShowBackgroundBorder = false;
	Options.bShowUnloadedBlueprints = true;
	Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;

	if (AdditionalClassFilter)
	{
		Options.ClassFilters.Add(AdditionalClassFilter.ToSharedRef());
	}

	TSharedPtr<FPropertyEditorInlineClassFilter> ClassFilter = MakeShareable(new FPropertyEditorInlineClassFilter);
	Options.ClassFilters.Add(ClassFilter.ToSharedRef());
	ClassFilter->bAllowAbstract = false;

	FProperty* Property = PropertyHandle->GetProperty();
	ClassFilter->ObjProperty = CastField<FObjectPropertyBase>(Property);
	ClassFilter->IntProperty = CastField<FInterfaceProperty>(Property);

	// Filter based on UPROPERTY meta data
	const FProperty* MetadataProperty = Property->GetOwnerProperty();

	static const FName NAME_MustImplement(ANSITEXTVIEW("MustImplement"));
	ClassFilter->InterfaceThatMustBeImplemented = MetadataProperty->GetClassMetaData(NAME_MustImplement);

	if (ClassFilter->ObjProperty)
	{
		// Verse generated interfaces are not fully supported by yet, for now they are stored in FObjectProperty metadata
		ClassFilter->InterfaceThatMustBeImplemented = ClassFilter->ObjProperty->GetOwnerProperty()->GetClassMetaData(TEXT("ObjectMustImplement"));
	}

	if (ClassFilter->InterfaceThatMustBeImplemented != nullptr)
	{
		if (!ClassFilter->InterfaceThatMustBeImplemented->HasAnyClassFlags(CLASS_Interface))
		{
			UE_LOGF(LogPropertyNode, Warning, "Property (%ls) specifies a MustImplement class which isn't an interface (%ls), clearing filter.", *Property->GetFullName(), *ClassFilter->InterfaceThatMustBeImplemented->GetPathName());
			ClassFilter->InterfaceThatMustBeImplemented = nullptr;
		}
		else if (ClassFilter->InterfaceThatMustBeImplemented == UInterface::StaticClass())
		{
			UE_LOGF(LogPropertyNode, Warning, "Property (%ls) specifies a MustImplement class which isn't valid (UInterface), clearing filter.", *Property->GetFullName());
			ClassFilter->InterfaceThatMustBeImplemented = nullptr;
		}
	}

	TArray<UObject*> ObjectList;
	if (PropertyHandle->IsValidHandle())
	{
		PropertyHandle->GetOuterObjects(ObjectList);
	}

	TArray<const UClass*> AllowedClassFilters;
	TArray<const UClass*> DisallowedClassFilters;
	PropertyEditorUtils::GetAllowedAndDisallowedClasses(ObjectList, *Property, AllowedClassFilters, DisallowedClassFilters, false);

	TSharedRef<FPropertyHandleBase> PropertyHandleImpl = StaticCastSharedRef<FPropertyHandleBase>(PropertyHandle);
	TSharedPtr<FPropertyNode> PropertyNode = PropertyHandleImpl->GetPropertyNode();

	// Filter based on restrictions
	for (const TSharedRef<const FPropertyRestriction>& ClassRestriction : PropertyNode->GetRestrictions())
	{
		for (TArray<FString>::TConstIterator Iter = ClassRestriction.Get().GetHiddenValuesIterator(); Iter; ++Iter)
		{
			if (const UClass* HiddenClass = PropertyEditorHelpers::FindOrLoadClass(*Iter))
			{
				DisallowedClassFilters.Add(HiddenClass);
			}
		}

		for (TArray<FString>::TConstIterator Iter = ClassRestriction.Get().GetDisabledValuesIterator(); Iter; ++Iter)
		{
			if (const UClass* DisabledClass = PropertyEditorHelpers::FindOrLoadClass(*Iter))
			{
				DisallowedClassFilters.Add(DisabledClass);
			}
		}

		for (TArray<TSharedRef<IClassViewerFilter>>::TConstIterator Iter = ClassRestriction.Get().GeClassViewFilterIterator(); Iter; ++Iter)
		{
			Options.ClassFilters.Add(*Iter);
		}
	}

	ClassFilter->AllowedClassFilters = MoveTemp(AllowedClassFilters);
	ClassFilter->DisallowedClassFilters = MoveTemp(DisallowedClassFilters);

	bool bContainerHasNoClear = false;
	if (PropertyNode->GetArrayIndex() != INDEX_NONE)
	{
		if (const TSharedPtr<FPropertyNode>& ParentNode = PropertyNode->GetParentNodeSharedPtr())
		{
			bContainerHasNoClear = ParentNode->GetProperty()->HasAllPropertyFlags(CPF_NoClear);
		}
	}
	Options.bShowNoneOption = !Property->HasAllPropertyFlags(CPF_NoClear) && !bContainerHasNoClear && !Property->HasAllPropertyFlags(CPF_NonNullable);

	FObjectPropertyNode* ObjectPropertyNode = PropertyNode->FindObjectItemParent();
	if (ObjectPropertyNode)
	{
		for (TPropObjectIterator Itor(ObjectPropertyNode->ObjectIterator()); Itor; ++Itor)
		{
			UObject* OwnerObject = Itor->Get();
			ClassFilter->OwningObjects.Add(OwnerObject);
		}
	}

	Options.PropertyHandle = PropertyHandle;

	return FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, OnPicked);
}

void SPropertyEditorEditInline::OnClassPicked(UClass* InClass, TSharedRef<IPropertyHandle> PropertyHandle, EPropertyValueSetFlags::Type Flags) // static
{
	TSharedRef<FPropertyHandleBase> PropertyHandleImpl = StaticCastSharedRef<FPropertyHandleBase>(PropertyHandle);
	TSharedPtr<FPropertyNode> PropertyNode = PropertyHandleImpl->GetPropertyNode();

	TArray<FObjectBaseAddress> ObjectsToModify;
	TArray<FString> NewValues;

	FObjectPropertyNode* ObjectNode = PropertyNode->FindObjectItemParent();

	if( ObjectNode )
	{
		GEditor->BeginTransaction(TEXT("PropertyEditor"), NSLOCTEXT("PropertyEditor", "OnClassPicked", "Set Class"), nullptr /* PropertyNode->GetProperty()) */ );

		FString NewObjectName;
		UObject* NewObjectTemplate = nullptr;

		// If we've picked the same class as our archetype, then we want to create an object with the same name and properties
		if (InClass)
		{
			FString DefaultValue = PropertyNode->GetDefaultValueAsString(/*bUseDisplayName=*/false);
			if (!DefaultValue.IsEmpty() && DefaultValue != FName(NAME_None).ToString())
			{
				FStringView ClassPath, ObjectPath;
				if (FPackageName::ParseExportTextPath(DefaultValue, &ClassPath, &ObjectPath))
				{
					if (InClass->GetPathName() == ClassPath)
					{
						NewObjectName = FPackageName::ObjectPathToObjectName(ObjectPath);

						ConstructorHelpers::StripObjectClass(DefaultValue);
						NewObjectTemplate = StaticFindObject(InClass, nullptr, *DefaultValue);
					}
				}
			}
		}

		// If this is an instanced component property collect current names so we can clean them properly if necessary
		TArray<FString> PrevPerObjectValues;
		FObjectProperty* ObjectProperty = CastFieldChecked<FObjectProperty>(PropertyHandle->GetProperty());
		if (ObjectProperty && ObjectProperty->HasAnyPropertyFlags(CPF_InstancedReference))
		{
			PropertyHandle->GetPerObjectValues(PrevPerObjectValues);
		}

		for ( TPropObjectIterator Itor( ObjectNode->ObjectIterator() ) ; Itor ; ++Itor )
		{
			FString NewValue;
			if (InClass)
			{
				FStringView CurClassPath, CurObjectName;
				if (PrevPerObjectValues.IsValidIndex(NewValues.Num()) && PrevPerObjectValues[NewValues.Num()] != FName(NAME_None).ToString())
				{
					FStringView CurObjectPath;
					if (FPackageName::ParseExportTextPath(PrevPerObjectValues[NewValues.Num()], &CurClassPath, &CurObjectPath))
					{
						CurObjectName = FPackageName::ObjectPathToObjectName(CurObjectPath);
					}
				}

				if (CurObjectName == NewObjectName && InClass->GetPathName() == CurClassPath)
				{
					NewValue = MoveTemp(PrevPerObjectValues[NewValues.Num()]);
					PrevPerObjectValues[NewValues.Num()].Reset();
				}
				else
				{
					UObject*		Object = Itor->Get();
					UObject*		UseOuter = (InClass->IsChildOf(UClass::StaticClass()) ? Cast<UClass>(Object)->GetDefaultObject() : Object);
					EObjectFlags	MaskedOuterFlags = UseOuter ? UseOuter->GetMaskedFlags(RF_PropagateToSubObjects) : RF_NoFlags;
					if (UseOuter && UseOuter->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
					{
						MaskedOuterFlags |= RF_ArchetypeObject;
					}

					if (NewObjectName.Len() > 0)
					{
						if (UObject* SubObject = StaticFindObject(UObject::StaticClass(), UseOuter, *NewObjectName))
						{
							SubObject->Rename(*MakeUniqueObjectName(GetTransientPackage(), SubObject->GetClass()).ToString(), GetTransientPackage(), REN_DontCreateRedirectors);

							// If we've renamed the object out of the way here, we don't need to do it again below
							if (PrevPerObjectValues.IsValidIndex(NewValues.Num()))
							{
								PrevPerObjectValues[NewValues.Num()].Reset();
							}
						}
					}

					UObject* NewUObject = NewObject<UObject>(UseOuter, InClass, *NewObjectName, MaskedOuterFlags, NewObjectTemplate);

					// Wrap the value in quotes before setting - in some cases for editinline-instanced values, the outer object path
					// can potentially contain a space token (e.g. if the outer object was instanced as a Blueprint template object
					// based on a user-facing variable name). While technically such characters should not be allowed, historically there
					// has been no issue with most tokens in the INVALID_OBJECTNAME_CHARACTERS set being present in in-memory object names,
					// other than some systems failing to resolve the object's path if the string representation of the value is not quoted.
					NewValue = FString::Printf(TEXT("\"%s\""), *NewUObject->GetPathName().ReplaceQuotesWithEscapedQuotes());
				}
			}
			else
			{
				NewValue = FName(NAME_None).ToString();
			}
			NewValues.Add(MoveTemp(NewValue));
		}

		PropertyHandle->SetPerObjectValues(NewValues, Flags);
		check(PrevPerObjectValues.Num() == 0 || PrevPerObjectValues.Num() == NewValues.Num());

		for (int32 Index = 0; Index < PrevPerObjectValues.Num(); ++Index)
		{
			if (PrevPerObjectValues[Index].Len() > 0 && PrevPerObjectValues[Index] != NewValues[Index])
			{
				// Move the old subobject to the transient package so GetObjectsWithOuter will not return it
				// This is particularly important for UActorComponent objects so resetting owned components on the parent doesn't find it
				ConstructorHelpers::StripObjectClass(PrevPerObjectValues[Index]);
				if (UObject* SubObject = StaticFindObject(UObject::StaticClass(), nullptr, *PrevPerObjectValues[Index]))
				{
					SubObject->Rename(nullptr, GetTransientOuterForRename(SubObject->GetClass()), REN_DontCreateRedirectors);
				}
			}
		}

		// End the transaction if we called PreChange
		GEditor->EndTransaction();

		// Force a rebuild of the children when this node changes
		PropertyNode->RebuildChildren();
	}
}
