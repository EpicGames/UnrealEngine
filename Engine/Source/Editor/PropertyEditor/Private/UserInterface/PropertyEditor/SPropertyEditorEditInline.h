// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/SWidget.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "Widgets/Input/SComboButton.h"
#include "ClassViewerModule.h"

class SPropertyEditorEditInline : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SPropertyEditorEditInline ) 
		: _Font( FAppStyle::GetFontStyle( PropertyEditorConstants::PropertyFontStyle ) )
		{}
		SLATE_ARGUMENT( FSlateFontInfo, Font )
	SLATE_END_ARGS()

	static bool Supports( const TSharedRef< class FPropertyEditor >& InPropertyEditor );
	static bool Supports( const FPropertyNode* InTreeNode, int32 InArrayIdx );

	void Construct( const FArguments& InArgs, const TSharedRef< class FPropertyEditor >& InPropertyEditor );

	void GetDesiredWidth( float& OutMinDesiredWidth, float& OutMaxDesiredWidth );

	/**
	* Generates a class picker with a filter to show only classes allowed to be selected.
	*
	* @param PropertyHandle			The Property Handle for the instanced UObject whose class is being picked
	* @param OnPicked				The callback to fire when a class is picked.
	* 
	* @return The Class Picker widget.
	*/
	static TSharedRef<SWidget> GenerateClassPicker(TSharedRef<IPropertyHandle> PropertyHandle, FOnClassPicked OnPicked, TSharedPtr<IClassViewerFilter> AdditionalClassFilter);

	/**
	 * Callback function from the Class Picker for when a Class is picked.
	 *
	 * @param InClass			The class picked in the Class Picker
	 * @param PropertyHandle	The Property Handle for the instanced UObject whose class is being picked
	 */
	static void OnClassPicked(UClass* InClass, TSharedRef<IPropertyHandle> PropertyHandle, EPropertyValueSetFlags::Type Flags=EPropertyValueSetFlags::DefaultFlags);

private:
	/**
	 * Called to see if the value is enabled for editing
	 *
	 * @param WeakHandlePtr	Handle to the property that the new value is for
	 *
	 * @return	true if the property is enabled
	 */
	bool IsValueEnabled(TWeakPtr<IPropertyHandle> WeakHandlePtr) const;

	/**
	 * @return The current display value for the combo box as a string
	 */
	FText GetDisplayValueAsString() const;

	/**
	 * @return The current display value's icon, if any. Returns nullptr if we have no valid value.
	 */
	const FSlateBrush* GetDisplayValueIcon() const;

	/*
	* Internal delegate called when a class is picked, used to close the combo box after a class is picked 
	*/
	void OnClassPickedInternal(UClass* InClass, TSharedRef<IPropertyHandle> PropertyHandle);


private:

	TSharedPtr<class FPropertyEditor > PropertyEditor;

	TSharedPtr<class SComboButton> ComboButton;
};
