// Copyright Epic Games, Inc. All Rights Reserved.
#include "UserInterface/PropertyEditor/SPropertyEditorInteractiveActorPicker.h"

#include "Delegates/Delegate.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Widgets/Images/SImage.h"

struct FGeometry;

#define LOCTEXT_NAMESPACE "PropertyPicker"

SPropertyEditorInteractiveActorPicker::~SPropertyEditorInteractiveActorPicker()
{
	if (FActorPickerModeModule* ActorPickerMode = FModuleManager::Get().GetModulePtr<FActorPickerModeModule>("ActorPickerMode"))
	{
		// make sure we are unregistered when this widget goes away
		if (ActorPickerMode->IsInActorPickingMode())
		{
			ActorPickerMode->EndActorPickingMode();
		}
	}
}

void SPropertyEditorInteractiveActorPicker::Construct( const FArguments& InArgs )
{
	OnActorSelected = InArgs._OnActorSelected;
	OnGetAllowedClasses = InArgs._OnGetAllowedClasses;
	OnShouldFilterActor = InArgs._OnShouldFilterActor;

	SButton::Construct(
		SButton::FArguments()
		.ButtonStyle( FAppStyle::Get(), "HoverHintOnly" )
		.OnClicked( this, &SPropertyEditorInteractiveActorPicker::OnClicked )
		.ContentPadding(4.0f)
		.ForegroundColor( FSlateColor::UseForeground() )
		.IsFocusable(false)
		[ 
			SNew( SImage )
			.Image( FAppStyle::GetBrush("Icons.EyeDropper") )
			.ColorAndOpacity( FSlateColor::UseForeground() )
		]
	);
}

FReply SPropertyEditorInteractiveActorPicker::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if(InKeyEvent.GetKey() == EKeys::Escape)
	{
		FActorPickerModeModule& ActorPickerMode = FModuleManager::Get().GetModuleChecked<FActorPickerModeModule>("ActorPickerMode");

		if(ActorPickerMode.IsInActorPickingMode())
		{
			ActorPickerMode.EndActorPickingMode();
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

bool SPropertyEditorInteractiveActorPicker::SupportsKeyboardFocus() const
{
	return true;
}

FReply SPropertyEditorInteractiveActorPicker::OnClicked()
{
	FActorPickerModeModule& ActorPickerMode = FModuleManager::Get().GetModuleChecked<FActorPickerModeModule>("ActorPickerMode");

	if(ActorPickerMode.IsInActorPickingMode())
	{
		ActorPickerMode.EndActorPickingMode();
	}
	else
	{
		ActorPickerMode.BeginActorPickingMode(OnGetAllowedClasses, OnShouldFilterActor, OnActorSelected);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
