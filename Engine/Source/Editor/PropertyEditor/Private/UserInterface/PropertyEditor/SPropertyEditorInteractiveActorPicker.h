// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ActorPickerMode.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"

struct FGeometry;
struct FKeyEvent;

class SPropertyEditorInteractiveActorPicker : public SButton
{
public:
	SLATE_BEGIN_ARGS( SPropertyEditorInteractiveActorPicker ) {}

	/** Delegate used to filter allowed actors */
	SLATE_EVENT( FOnGetAllowedClasses, OnGetAllowedClasses )

	/** Delegate used to filter allowed actors */
	SLATE_EVENT( FOnShouldFilterActor, OnShouldFilterActor )

	/** Delegate called when an actor is selected */
	SLATE_EVENT( FOnActorSelected, OnActorSelected )

	SLATE_END_ARGS()

	~SPropertyEditorInteractiveActorPicker();

	void Construct( const FArguments& InArgs );

	/** Begin SWidget interface */
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	virtual bool SupportsKeyboardFocus() const override;
	/** End SWidget interface */

private:
	/** Delegate for when the button is clicked */
	FReply OnClicked();

	/** Delegate used to filter allowed actors */
	FOnGetAllowedClasses OnGetAllowedClasses;

	/** Delegate used to filter allowed actors */
	FOnShouldFilterActor OnShouldFilterActor;

	/** Delegate called when an actor is selected */
	FOnActorSelected OnActorSelected;
};
