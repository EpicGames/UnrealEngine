// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyEditor/SPropertyEditorMap.h"

#include "Delegates/Delegate.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "PropertyEditorHelpers.h"
#include "PropertyNode.h"
#include "UObject/UnrealType.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "PropertyEditor"

void SPropertyEditorMap::Construct(const FArguments& InArgs, const TSharedRef< class FPropertyEditor >& InPropertyEditor)
{
	PropertyEditor = InPropertyEditor;

	TAttribute<FText> TextAttr(this, &SPropertyEditorMap::GetMapTextValue);

	ChildSlot
	.Padding(0.0f, 0.0f, 2.0f, 0.0f)
	[
		SNew(STextBlock)
		.Text(TextAttr)
		.Font(InArgs._Font)
	];

	SetToolTipText(GetMapTooltipText());

	SetEnabled(TAttribute<bool>(this, &SPropertyEditorMap::CanEdit));
}

bool SPropertyEditorMap::Supports(const TSharedRef< FPropertyEditor >& InPropertyEditor)
{
	const TSharedRef< FPropertyNode > PropertyNode = InPropertyEditor->GetPropertyNode();
	const FProperty* Property = InPropertyEditor->GetProperty();

	if (!PropertyNode->HasNodeFlags(EPropertyNodeFlags::EditInlineNew)
		&& Property->IsA<FMapProperty>())
	{
		return true;
	}

	return false;
}

void SPropertyEditorMap::GetDesiredWidth(float& OutMinDesiredWidth, float& OutMaxDesiredWidth)
{
	OutMinDesiredWidth = 170.0f;
	OutMaxDesiredWidth = 170.0f;
}

FText SPropertyEditorMap::GetMapTextValue() const
{
	check(PropertyEditor.IsValid());
	return PropertyEditorHelpers::GetNumElementsText(LOCTEXT("MapText", "Map"), *PropertyEditor.Get());
}

FText SPropertyEditorMap::GetMapTooltipText() const
{
	return LOCTEXT("RichMapTooltipText", "Maps are associative, unordered containers that associate a set of keys with a set of values. Each key in a map must be unique, but values can be duplicated.");
}

bool SPropertyEditorMap::CanEdit() const
{
	return PropertyEditor.IsValid() ? !PropertyEditor->IsEditConst() : true;
}

#undef LOCTEXT_NAMESPACE
