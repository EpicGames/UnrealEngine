// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Misc/Attribute.h"
#include "UObject/UnrealType.h"
#include "Fonts/SlateFontInfo.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Layout/Margin.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SCompoundWidget.h"
#include "PropertyNode.h"
#include "Textures/SlateIcon.h"
#include "PropertyHandle.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "PropertyEditorHelpers.h"
#include "NumericPropertyParams.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Editor.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SComboButton.h"
#include "ObjectPropertyNode.h"
#include "PropertyEditorConstants.h"

#include "Math/UnitConversion.h"
#include "Widgets/Input/NumericUnitTypeInterface.inl"

#define LOCTEXT_NAMESPACE "PropertyEditor"

template <typename NumericType>
class SPropertyEditorNumeric : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SPropertyEditorNumeric<NumericType> )
		: _Font( FAppStyle::GetFontStyle( PropertyEditorConstants::PropertyFontStyle ) ) 
		{}
		SLATE_ATTRIBUTE( FSlateFontInfo, Font )
	SLATE_END_ARGS()
	
	void Construct( const FArguments& InArgs, const TSharedRef<FPropertyEditor>& InPropertyEditor )
	{
		bIsUsingSlider = false;

		PropertyEditor = InPropertyEditor;

		const TSharedRef< FPropertyNode > PropertyNode = InPropertyEditor->GetPropertyNode();
		const FProperty* Property = InPropertyEditor->GetProperty();
		TSharedPtr<IPropertyHandle> PropertyHandle = InPropertyEditor->GetPropertyHandle();

		if(!Property->IsA(FFloatProperty::StaticClass()) && !Property->IsA(FDoubleProperty::StaticClass()) && Property->HasMetaData(PropertyEditorConstants::MD_Bitmask))
		{
			auto CreateBitmaskFlagsArray = [PropertyHandle]()
			{
				constexpr int32 BitmaskBitCount = sizeof(NumericType) << 3;

				TArray<FBitmaskFlagInfo> Result;
				Result.Empty(BitmaskBitCount);

				const UEnum* BitmaskEnum = nullptr;
				const FString& BitmaskEnumName = PropertyHandle->GetMetaData(PropertyEditorConstants::MD_BitmaskEnum);
				if (!BitmaskEnumName.IsEmpty())
				{
					// @TODO: Potentially replace this with a parameter passed in from a member variable on the FProperty (e.g. FByteProperty::Enum)
					BitmaskEnum = UClass::TryFindTypeSlow<UEnum>(BitmaskEnumName);
				}

				if (BitmaskEnum)
				{
					const TMap<FName, FText> EnumValueDisplayNameOverrides = PropertyEditorHelpers::GetEnumValueDisplayNamesFromPropertyOverride(PropertyHandle->GetProperty(), BitmaskEnum);

					const bool bUseEnumValuesAsMaskValues = BitmaskEnum->GetBoolMetaData(PropertyEditorConstants::MD_UseEnumValuesAsMaskValuesInEditor);
					auto AddNewBitmaskFlagLambda = [BitmaskEnum, &Result, &EnumValueDisplayNameOverrides](int32 InEnumIndex, NumericType InFlagValue)
					{
						Result.Emplace();
						FBitmaskFlagInfo* BitmaskFlag = &Result.Last();

						BitmaskFlag->Value = InFlagValue;
						BitmaskFlag->DisplayName = EnumValueDisplayNameOverrides.FindRef(BitmaskEnum->GetNameByIndex(InEnumIndex));
						if (BitmaskFlag->DisplayName.IsEmpty())
						{
							BitmaskFlag->DisplayName = BitmaskEnum->GetDisplayNameTextByIndex(InEnumIndex);
						}
						BitmaskFlag->ToolTipText = BitmaskEnum->GetToolTipTextByIndex(InEnumIndex);
						if (BitmaskFlag->ToolTipText.IsEmpty())
						{
							BitmaskFlag->ToolTipText = FText::Format(LOCTEXT("BitmaskDefaultFlagToolTipText", "Toggle {0} on/off"), BitmaskFlag->DisplayName);
						}
					};

					const TArray<FName> AllowedPropertyEnums = PropertyEditorHelpers::GetValidEnumsFromPropertyOverride(PropertyHandle->GetProperty(), BitmaskEnum);
					const TArray<FName> DisallowedPropertyEnums = PropertyEditorHelpers::GetInvalidEnumsFromPropertyOverride(PropertyHandle->GetProperty(), BitmaskEnum);
					// Note: This loop doesn't include (BitflagsEnum->NumEnums() - 1) in order to skip the implicit "MAX" value that gets added to the enum type at compile time.
					for (int32 BitmaskEnumIndex = 0; BitmaskEnumIndex < BitmaskEnum->NumEnums() - 1; ++BitmaskEnumIndex)
					{
						bool bShouldBeHidden = BitmaskEnum->HasMetaData(TEXT("Hidden"), BitmaskEnumIndex);
						if (!bShouldBeHidden)
						{
							if(AllowedPropertyEnums.Num() > 0)
							{
								bShouldBeHidden = AllowedPropertyEnums.Find(BitmaskEnum->GetNameByIndex(BitmaskEnumIndex)) == INDEX_NONE;
							}
							// If both are specified, InvalidEnumValues takes precedence
							else if(DisallowedPropertyEnums.Num() > 0)
							{
								bShouldBeHidden = DisallowedPropertyEnums.Find(BitmaskEnum->GetNameByIndex(BitmaskEnumIndex)) != INDEX_NONE;
							}
						}

						const int64 EnumValue64 = BitmaskEnum->GetValueByIndex(BitmaskEnumIndex);
						const NumericType EnumValue = static_cast<NumericType>(EnumValue64);

						if (EnumValue >= 0 && !bShouldBeHidden)
						{
							if (bUseEnumValuesAsMaskValues)
							{
								if (EnumValue64 < static_cast<int64>(TNumericLimits<NumericType>::Max()) &&FMath::IsPowerOfTwo(EnumValue64))
								{
									AddNewBitmaskFlagLambda(BitmaskEnumIndex, EnumValue);
								}
							}
							else if (EnumValue < BitmaskBitCount)
							{
								AddNewBitmaskFlagLambda(BitmaskEnumIndex, TBitmaskValueHelpers<NumericType>::LeftShift(static_cast<NumericType>(1), EnumValue));
							}
						}
					}
				}
				else
				{
					for (int32 BitmaskFlagIndex = 0; BitmaskFlagIndex < BitmaskBitCount; ++BitmaskFlagIndex)
					{
						Result.Emplace();
						FBitmaskFlagInfo* BitmaskFlag = &Result.Last();

						BitmaskFlag->Value = TBitmaskValueHelpers<NumericType>::LeftShift(static_cast<NumericType>(1), BitmaskFlagIndex);
						BitmaskFlag->DisplayName = FText::Format(LOCTEXT("BitmaskDefaultFlagDisplayName", "Flag {0}"), FText::AsNumber(BitmaskFlagIndex + 1));
						BitmaskFlag->ToolTipText = FText::Format(LOCTEXT("BitmaskDefaultFlagToolTipText", "Toggle {0} on/off"), BitmaskFlag->DisplayName);
					}
				}

				return Result;
			};

			const FComboBoxStyle& ComboBoxStyle = FCoreStyle::Get().GetWidgetStyle< FComboBoxStyle >("ComboBox");

			const auto& GetComboButtonText = [this, CreateBitmaskFlagsArray]() -> FText
			{
				TOptional<NumericType> Value = OnGetValue();
				if (Value.IsSet())
				{
					NumericType BitmaskValue = Value.GetValue();
					if (BitmaskValue != 0)
					{
						TArray<FBitmaskFlagInfo> BitmaskFlags = CreateBitmaskFlagsArray();

						TArray<FText> SetFlags;
						SetFlags.Reserve(BitmaskFlags.Num());

						for (const FBitmaskFlagInfo& FlagInfo : BitmaskFlags)
						{
							if (TBitmaskValueHelpers<NumericType>::BitwiseAND(Value.GetValue(), FlagInfo.Value))
							{
								SetFlags.Add(FlagInfo.DisplayName);
							}
						}
						if (SetFlags.Num() > 3)
						{
							SetFlags.SetNum(3);
							SetFlags.Add(FText::FromString("..."));
						}

						return FText::Join(FText::FromString(" | "), SetFlags);
					}

					return LOCTEXT("BitmaskButtonContentNoFlagsSet", "(No Flags Set)");
				}
				else
				{
					return PropertyEditorConstants::DefaultUndeterminedText;
				}
			};

			// Constructs the UI for bitmask property editing.
			SAssignNew(PrimaryWidget, SComboButton)
			.ComboButtonStyle(&ComboBoxStyle.ComboButtonStyle)
			.ToolTipText_Lambda([this, CreateBitmaskFlagsArray]
			{
				TOptional<NumericType> Value = OnGetValue();
				if (Value.IsSet())
				{
					TArray<FBitmaskFlagInfo> BitmaskFlags = CreateBitmaskFlagsArray();

					TArray<FText> SetFlags;
					SetFlags.Reserve(BitmaskFlags.Num());

					for (const FBitmaskFlagInfo& FlagInfo : BitmaskFlags)
					{
						if (TBitmaskValueHelpers<NumericType>::BitwiseAND(Value.GetValue(), FlagInfo.Value))
						{
							SetFlags.Add(FlagInfo.DisplayName);
						}
					}

					return FText::Join(FText::FromString(" | "), SetFlags);
				}

				return FText::GetEmpty();
			})
			.ButtonContent()
			[
				SNew(STextBlock)
				.Font(InArgs._Font)
				.Text_Lambda(GetComboButtonText)
			]
			.OnGetMenuContent_Lambda([this, CreateBitmaskFlagsArray]()
			{
				FMenuBuilder MenuBuilder(false, nullptr);

				TArray<FBitmaskFlagInfo> BitmaskFlags = CreateBitmaskFlagsArray();
				for (int i = 0; i < BitmaskFlags.Num(); ++i)
				{
					MenuBuilder.AddMenuEntry(
						BitmaskFlags[i].DisplayName,
						BitmaskFlags[i].ToolTipText,
						FSlateIcon(),
						FUIAction
						(
							FExecuteAction::CreateLambda([this, i, BitmaskFlags]()
							{
								TOptional<NumericType> Value = OnGetValue();
								if (Value.IsSet())
								{
									OnValueCommitted(TBitmaskValueHelpers<NumericType>::BitwiseXOR(Value.GetValue(), BitmaskFlags[i].Value), ETextCommit::Default);
								}
							}),
							FCanExecuteAction(),
							FIsActionChecked::CreateLambda([this, i, BitmaskFlags]() -> bool
							{
								TOptional<NumericType> Value = OnGetValue();
								if (Value.IsSet())
								{
									return TBitmaskValueHelpers<NumericType>::BitwiseAND(Value.GetValue(), BitmaskFlags[i].Value) != static_cast<NumericType>(0);
								}

								return false;
							})
						),
						NAME_None,
						EUserInterfaceActionType::Check);
				}

				return MenuBuilder.MakeWidget();
			});

			ChildSlot.AttachWidget(PrimaryWidget.ToSharedRef());
		}
		else
		{
			// Instance metadata overrides per-class metadata.
			const typename TNumericPropertyParams<NumericType>::FMetaDataGetter MetaDataGetter = TNumericPropertyParams<NumericType>::FMetaDataGetter::CreateLambda([&](const FName& Key)
			{
				const FString* InstanceValue = PropertyHandle->GetInstanceMetaData(Key);
				return (InstanceValue != nullptr) ? *InstanceValue : PropertyHandle->GetMetaData(Key);
			});

			TNumericPropertyParams<NumericType> NumericPropertyParams(Property, MetaDataGetter);

			FObjectPropertyNode* ObjectPropertyNode = PropertyNode->FindObjectItemParent();

			// Set up the correct type interface if we want to display units on the property editor

			// First off, check for ForceUnits= meta data. This meta tag tells us to interpret, and always display the value in these units. FUnitConversion::Settings().ShouldDisplayUnits does not apply to such properties
			const FString& ForcedUnits = MetaDataGetter.Execute("ForceUnits");
			TOptional<EUnit> PropertyUnits = FUnitConversion::UnitFromString(*ForcedUnits);
			if (PropertyUnits.IsSet())
			{
				// Create the type interface and set up the default input units if they are compatible
				TypeInterface = MakeShareable(new TNumericUnitTypeInterface<NumericType>(PropertyUnits.GetValue()));
				TypeInterface->UserDisplayUnits = TypeInterface->FixedDisplayUnits = PropertyUnits.GetValue();
			}
			// If that's not set, we fall back to Units=xxx which calculates the most appropriate unit to display in
			else
			{
				if (FUnitConversion::Settings().ShouldDisplayUnits())
				{
					const FString& DynamicUnits = PropertyHandle->GetMetaData(TEXT("Units"));
					if (!DynamicUnits.IsEmpty())
					{
						PropertyUnits = FUnitConversion::UnitFromString(*DynamicUnits);
					}
					else
					{
						PropertyUnits = FUnitConversion::UnitFromString(*MetaDataGetter.Execute("Units"));
					}
				}

				if (!PropertyUnits.IsSet())
				{
					PropertyUnits = EUnit::Unspecified;
				}

				// Create the type interface and set up the default input units if they are compatible
				TypeInterface = MakeShareable(new TNumericUnitTypeInterface<NumericType>(PropertyUnits.GetValue()));

				const FString& MaxFractionalDigitsString = MetaDataGetter.Execute("MaxFractionalDigits");
				if (!MaxFractionalDigitsString.IsEmpty())
				{
					int32 MaxFractionalDigits = 0;
					if (LexTryParseString(MaxFractionalDigits, *MaxFractionalDigitsString) && MaxFractionalDigits > 0)
					{
						TypeInterface->SetMaxFractionalDigits(MaxFractionalDigits);
					}
				}

				TOptional<NumericType> Value = OnGetValue();
				if (Value.IsSet())
				{
					TypeInterface->SetupFixedDisplay(Value.GetValue());
				}
			}

			const bool bAllowSpin = (!ObjectPropertyNode || (1 == ObjectPropertyNode->GetNumObjects())) && !PropertyHandle->GetBoolMetaData("NoSpinbox");

			ChildSlot
			[
				SAssignNew(PrimaryWidget, SNumericEntryBox<NumericType>)
				// Only allow spinning if we have a single value
				.AllowSpin(bAllowSpin)
				.Value(this, &SPropertyEditorNumeric<NumericType>::OnGetValue)
				.Font(InArgs._Font)
				.MinValue(NumericPropertyParams.MinValue)
				.MaxValue(NumericPropertyParams.MaxValue)
				.MinSliderValue(NumericPropertyParams.MinSliderValue)
				.MaxSliderValue(NumericPropertyParams.MaxSliderValue)
				.SliderExponent(NumericPropertyParams.SliderExponent)
				.Delta(NumericPropertyParams.Delta)
				// LinearDeltaSensitivity needs to be left unset if not provided, rather than being set to some default
				.LinearDeltaSensitivity(NumericPropertyParams.GetLinearDeltaSensitivityAttribute())
				.AllowWheel(bAllowSpin)
				.WheelStep(NumericPropertyParams.WheelStep)
				.UndeterminedString(PropertyEditorConstants::DefaultUndeterminedText)
				.OnValueChanged(this, &SPropertyEditorNumeric<NumericType>::OnValueChanged)
				.OnValueCommitted(this, &SPropertyEditorNumeric<NumericType>::OnValueCommitted)
				.OnUndeterminedValueCommitted(this, &SPropertyEditorNumeric<NumericType>::OnUndeterminedValueCommitted)
				.OnBeginSliderMovement(this, &SPropertyEditorNumeric<NumericType>::OnBeginSliderMovement)
				.OnEndSliderMovement(this, &SPropertyEditorNumeric<NumericType>::OnEndSliderMovement)
				.TypeInterface(TypeInterface)
			];
		}

		SetEnabled(TAttribute<bool>(this, &SPropertyEditorNumeric<NumericType>::CanEdit));
	}

	virtual bool SupportsKeyboardFocus() const override
	{
		return PrimaryWidget.IsValid() && PrimaryWidget->SupportsKeyboardFocus();
	}

	virtual FReply OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent ) override
	{
		// Forward keyboard focus to our editable text widget
		return FReply::Handled().SetUserFocus(PrimaryWidget.ToSharedRef(), InFocusEvent.GetCause());
	}
	
	void GetDesiredWidth( float& OutMinDesiredWidth, float& OutMaxDesiredWidth )
	{
		const FProperty* Property = PropertyEditor->GetProperty();
		const bool bIsBitmask = (!Property->IsA(FFloatProperty::StaticClass()) && Property->HasMetaData(PropertyEditorConstants::MD_Bitmask));
		const bool bIsNonEnumByte = (Property->IsA(FByteProperty::StaticClass()) && CastField<const FByteProperty>(Property)->Enum == NULL);

		if( bIsNonEnumByte && !bIsBitmask )
		{
			OutMinDesiredWidth = 75.0f;
			OutMaxDesiredWidth = 75.0f;
		}
		else
		{
			OutMinDesiredWidth = 125.0f;
			OutMaxDesiredWidth = bIsBitmask ? 400.0f : 125.0f;
		}
	}

	static bool Supports( const TSharedRef< FPropertyEditor >& PropertyEditor ) 
	{ 
		const TSharedRef< FPropertyNode > PropertyNode = PropertyEditor->GetPropertyNode();
		
		if (!PropertyNode->HasNodeFlags(EPropertyNodeFlags::EditInlineNew))
		{
			return TTypeToProperty<NumericType>::Match(PropertyEditor->GetProperty());
		}
		
		return false;
	}

private:
	/**
	 *	Helper to provide mapping for a given FProperty object to a property editor type
	 */ 
	template<typename T, typename U = void> 
	struct TTypeToProperty 
	{ 
		static bool Match(const FProperty* InProperty) { return false; } 
	};
	
	template<typename U> 
	struct TTypeToProperty<float, U> 
	{	
		static bool Match(const FProperty* InProperty) { return InProperty->IsA(FFloatProperty::StaticClass()); } 
	};

	template<typename U>
	struct TTypeToProperty<double, U>
	{
		static bool Match(const FProperty* InProperty) { return InProperty->IsA(FDoubleProperty::StaticClass()); }
	};

	template<typename U>
	struct TTypeToProperty<int8, U>
	{
		static bool Match(const FProperty* InProperty) { return InProperty->IsA(FInt8Property::StaticClass()); }
	};

	template<typename U>
	struct TTypeToProperty<int16, U>
	{
		static bool Match(const FProperty* InProperty) { return InProperty->IsA(FInt16Property::StaticClass()); }
	};

	template<typename U>
	struct TTypeToProperty<int32, U>
	{
		static bool Match(const FProperty* InProperty) { return InProperty->IsA(FIntProperty::StaticClass()); }
	};

	template<typename U>
	struct TTypeToProperty<int64, U>
	{
		static bool Match(const FProperty* InProperty) { return InProperty->IsA(FInt64Property::StaticClass()); }
	};

	template<typename U>
	struct TTypeToProperty<uint8, U>
	{
		static bool Match(const FProperty* InProperty) { return (InProperty->IsA(FByteProperty::StaticClass()) && CastField<const FByteProperty>(InProperty)->Enum == NULL); }
	};

	template<typename U>
	struct TTypeToProperty<uint16, U>
	{
		static bool Match(const FProperty* InProperty) { return InProperty->IsA(FUInt16Property::StaticClass()); }
	};

	template<typename U>
	struct TTypeToProperty<uint32, U>
	{
		static bool Match(const FProperty* InProperty) { return InProperty->IsA(FUInt32Property::StaticClass()); }
	};

	template<typename U>
	struct TTypeToProperty<uint64, U>
	{
		static bool Match(const FProperty* InProperty) { return InProperty->IsA(FUInt64Property::StaticClass()); }
	};
		
	
	/** @return The value or unset if properties with multiple values are viewed */
	TOptional<NumericType> OnGetValue() const
	{
		NumericType NumericVal;
		const TSharedRef< IPropertyHandle > PropertyHandle = PropertyEditor->GetPropertyHandle();

		if (PropertyHandle->GetValue( NumericVal ) == FPropertyAccess::Success )
		{
			return NumericVal;
		}

		// Return an unset value so it displays the "multiple values" indicator instead
		return TOptional<NumericType>();
	}

	void OnValueChanged( NumericType NewValue )
	{
		if( bIsUsingSlider )
		{
			const TSharedRef< IPropertyHandle > PropertyHandle = PropertyEditor->GetPropertyHandle();

			NumericType OrgValue(0);
			if (PropertyHandle->GetValue(OrgValue) != FPropertyAccess::Fail)
			{
				// Value hasn't changed, so lets return now
				if (OrgValue == NewValue)
				{
					return;
				}
			}

			// We don't create a transaction for each property change when using the slider.  Only once when the slider first is moved
			EPropertyValueSetFlags::Type Flags = (EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable);
			PropertyHandle->SetValue( NewValue, Flags );

			if (TypeInterface.IsValid())
			{
				TypeInterface->SetupFixedDisplay(NewValue);
			}
		}
	}

	void OnValueCommitted( NumericType NewValue, ETextCommit::Type CommitInfo )
	{
		const TSharedRef< IPropertyHandle > PropertyHandle = PropertyEditor->GetPropertyHandle();
		NumericType OrgValue(0);
		/* sometimes an FProperty may have been destroyed due to 2 different events invoking this handler (with the same
		 * NewValue) and the first one nullifying the current FProperty ~ in this case it's too late to not invoke the
		 * method, but we can not run the code instead */
		if (PropertyHandle->GetProperty())
		{
			if (bIsUsingSlider || (PropertyHandle->GetValue(OrgValue) == FPropertyAccess::Fail || OrgValue != NewValue))
			{
				PropertyHandle->SetValue(NewValue);
				LastSliderCommittedValue = NewValue;
			}

			if (TypeInterface.IsValid())
			{
				TypeInterface->SetupFixedDisplay(NewValue);
			}
		}
	}

	void OnUndeterminedValueCommitted( FText NewValue, ETextCommit::Type CommitType )
	{
		const TSharedRef<IPropertyHandle> PropertyHandle = PropertyEditor->GetPropertyHandle();
		const FString NewValueString = NewValue.ToString();
		TArray<FString> PerObjectValues;
		
		// evaluate expression for each property value
		PropertyHandle->GetPerObjectValues(PerObjectValues);

		for (auto& Value : PerObjectValues)
		{
			NumericType OldNumericValue;
			TTypeFromString<NumericType>::FromString(OldNumericValue, *Value);
			const FString ReplacedNewValueString = NewValueString.Replace(*PropertyEditorConstants::DefaultUndeterminedText.ToString(), *Value);
			TOptional<NumericType> NewNumericValue = TypeInterface->FromString(ReplacedNewValueString, OldNumericValue);

			if (NewNumericValue.IsSet())
			{
				Value = TTypeToString<NumericType>::ToString(NewNumericValue.GetValue());
			}
		}

		PropertyHandle->SetPerObjectValues(PerObjectValues);
	}
	
	/**
	 * Called when the slider begins to move.  We create a transaction here to undo the property
	 */
	void OnBeginSliderMovement()
	{
		bIsUsingSlider = true;

		const TSharedRef<IPropertyHandle> PropertyHandle = PropertyEditor->GetPropertyHandle();
		PropertyHandle->GetValue(LastSliderCommittedValue);
		
		GEditor->BeginTransaction(TEXT("PropertyEditor"), FText::Format(NSLOCTEXT("PropertyEditor", "SetNumericPropertyTransaction", "Edit {0}"), PropertyEditor->GetDisplayName()), nullptr /** PropertyEditor->GetPropertyHandle()->GetProperty() */ );
	}


	/**
	 * Called when the slider stops moving.  We end the previously created transaction
	 */
	void OnEndSliderMovement( NumericType NewValue )
	{
		bIsUsingSlider = false;

		// When the slider end, we may have not called SetValue(NewValue) without the InteractiveChange|NotTransactable flags.
		//That prevents some transaction and callback to be triggered like the NotifyHook.
		if (LastSliderCommittedValue != NewValue)
		{
			const TSharedRef<IPropertyHandle> PropertyHandle = PropertyEditor->GetPropertyHandle();
			PropertyHandle->SetValue(NewValue);
		}
		else
		{
			GEditor->EndTransaction();
		}
	}

	/** @return True if the property can be edited */
	bool CanEdit() const
	{
		return PropertyEditor.IsValid() ? !PropertyEditor->IsEditConst() : true;
	}

	/** Flag data for bitmasks. */
	struct FBitmaskFlagInfo
	{
		NumericType Value;
		FText DisplayName;
		FText ToolTipText;
	};

	/** Integral bitmask value helper methods. */
	template<typename T>
	struct TBitmaskValueHelpers
	{
		static T BitwiseAND(T Base, T Mask) { return Base & Mask; }
		static T BitwiseXOR(T Base, T Mask) { return Base ^ Mask; }
		template <typename U>
		static T LeftShift(T Base, U Shift) { return Base << Shift; }
	};

	/** Explicit specialization for numeric 'float' types (these will not be used). */
	template<>
	struct TBitmaskValueHelpers<float>
	{
		static float BitwiseAND(float Base, float Mask) { return 0.0f; }
		static float BitwiseXOR(float Base, float Mask) { return 0.0f; }
		template <typename U>
		static float LeftShift(float Base, U Shift) { return 0.0f; }
	};

	/** Explicit specialization for numeric 'double' types (these will not be used). */
	template<>
	struct TBitmaskValueHelpers<double>
	{
		static double BitwiseAND(double Base, double Mask) { return 0.0f; }
		static double BitwiseXOR(double Base, double Mask) { return 0.0f; }
		template <typename U>
		static double LeftShift(double Base, U Shift)  { return 0.0f; }
	};

private:

	TSharedPtr<TNumericUnitTypeInterface<NumericType>> TypeInterface;	

	TSharedPtr< class FPropertyEditor > PropertyEditor;

	TSharedPtr< class SWidget > PrimaryWidget;

	/** True if the slider is being used to change the value of the property */
	bool bIsUsingSlider;

	/** When using the slider, what was the last committed value */
	NumericType LastSliderCommittedValue;
};

#undef LOCTEXT_NAMESPACE
