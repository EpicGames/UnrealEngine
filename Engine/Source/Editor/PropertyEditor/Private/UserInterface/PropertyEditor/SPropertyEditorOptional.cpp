// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyEditor/SPropertyEditorOptional.h"

#include "Delegates/Delegate.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "PropertyEditorModule.h"
#include "PropertyNode.h"
#include "UObject/PropertyOptional.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboBox.h"
#include "PropertyEditorHelpers.h"

#define LOCTEXT_NAMESPACE "PropertyEditor"

void SPropertyEditorOptional::Construct(const FArguments& InArgs, const TSharedRef<FPropertyEditor>& InPropertyEditor, TSharedRef<IPropertyUtilities> InPropertyUtilities)
{
	MinDesiredWidth = 0.0f;
	MaxDesiredWidth = 0.0f;
	PropertyEditor = InPropertyEditor;

	uint8* ValueAddress = nullptr;
	TSharedRef<FPropertyNode> PropertyNode = InPropertyEditor->GetPropertyNode();
	const FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(PropertyNode->GetProperty());
	const FProperty* ValueProperty = OptionalProperty->GetValueProperty();

	if (ValueProperty->IsA(FObjectPropertyBase::StaticClass()) || ValueProperty->IsA(FInterfaceProperty::StaticClass()))
	{
		ValueEditorWidget = SNew(SBox)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f)
			[
				PropertyEditorHelpers::MakePropertyButton(EPropertyButton::OptionalPick, InPropertyEditor)
			];
	}
	else if (PropertyNode->GetSingleReadAddress(ValueAddress) == FPropertyAccess::MultipleValues)
	{	
		// Note: multi-select for `OptionalPick` types is handled in `MakePickOptionalButton`
		MultiselectOptions.Add(MakeShareable( new FText(LOCTEXT("OptionalMultiselect_Multiple", "Multiple States"))));
		MultiselectOptions.Add(MakeShareable( new FText(LOCTEXT("OptionalMultiselect_Value", "Set all to Value"))));
		MultiselectOptions.Add(MakeShareable( new FText(LOCTEXT("OptionalMultiselect_None", "Set all to None"))));

		SAssignNew(ValueEditorWidget, SComboBox<TSharedPtr<FText>>)
			.OptionsSource(&MultiselectOptions)
			.OnSelectionChanged(this, &SPropertyEditorOptional::OnOptionChanged)
			.OnGenerateWidget(this, &SPropertyEditorOptional::MakeWidgetForOption)
			.InitiallySelectedItem(MultiselectOptions[0])
			[
				SNew(STextBlock)
					.Text(*MultiselectOptions[0])
			];
	}
	else
	{
		ValueEditorWidget = PropertyEditorHelpers::MakePropertyButton(EPropertyButton::OptionalSet, InPropertyEditor);
	}

	ChildSlot
	[
		ValueEditorWidget.ToSharedRef()
	];

	SetToolTipText(GetOptionalTooltipText());

	SetEnabled(TAttribute<bool>(this, &SPropertyEditorOptional::CanEdit));
}

bool SPropertyEditorOptional::Supports(const TSharedRef< FPropertyEditor >& InPropertyEditor)
{
	const TSharedRef<FPropertyNode> PropertyNode = InPropertyEditor->GetPropertyNode();
	const FProperty* Property = InPropertyEditor->GetProperty();

	if (Property && !PropertyNode->HasNodeFlags(EPropertyNodeFlags::EditInlineNew)
		&& Property->IsA<FOptionalProperty>())
	{
		return true;
	}

	return false;
}

void SPropertyEditorOptional::GetDesiredWidth(float& OutMinDesiredWidth, float& OutMaxDesiredWidth)
{
	// Did we get desired width's from our set value? If not just set to a default value of 170
	if (MinDesiredWidth && MaxDesiredWidth)
	{
		OutMinDesiredWidth = MinDesiredWidth;
		OutMaxDesiredWidth = MaxDesiredWidth;
	}
	else
	{
		OutMinDesiredWidth = 170.0f;
		OutMaxDesiredWidth = 170.0f;
	}
}

FText SPropertyEditorOptional::GetOptionalTooltipText() const
{
	return LOCTEXT("RichOptionalTooltipText", "This is an optional property. It can either be `set`, which means it contains a `value` property, or it can be unset which means it contains nothing.");
}

bool SPropertyEditorOptional::CanEdit() const
{
	return PropertyEditor.IsValid() ? !PropertyEditor->IsEditConst() : true;
}

TSharedRef<SWidget> SPropertyEditorOptional::MakeWidgetForOption(TSharedPtr<FText> InOption)
{
	return SNew(STextBlock).Text(*InOption);
}

void SPropertyEditorOptional::OnOptionChanged(TSharedPtr<FText> NewOption, ESelectInfo::Type)
{
	TSharedPtr<IPropertyHandleOptional> OptionalHandle = PropertyEditor->GetPropertyHandle()->AsOptional();
	if (!OptionalHandle.IsValid())
	{
		// We cannot do anything if we don't have a valid handle
		return;
	}

	if (NewOption->EqualTo(*MultiselectOptions[1]) /*"Set all to Value"*/)
	{
		OptionalHandle->ClearOptionalValue(); // Clear all values
		OptionalHandle->SetOptionalValue(nullptr, nullptr); // Set values to default-initialized
	}
	else if (NewOption->EqualTo(*MultiselectOptions[2]) /*"Set all to None"*/)
	{
		OptionalHandle->ClearOptionalValue();
	}
}

#undef LOCTEXT_NAMESPACE
