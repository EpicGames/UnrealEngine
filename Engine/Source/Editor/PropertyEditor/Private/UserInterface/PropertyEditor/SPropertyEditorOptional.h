// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Fonts/SlateFontInfo.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#include "IPropertyUtilities.h"

class FPropertyEditor;

/*
* This class represents a `value editor widget` for optional types in the details panel.
* 
* It has three possible states which it can display:
*	1. (Unset Single-Select) 
*		The optional has no set value. A button `Set to Value` is shown and
*		pressing this button will set the optional's value to a default initialized state.
*	2. (Set Single-Select OR Multi-Select) 
*		The optional has a set value. The `value editor widget` for the set value is shown
*		(example: SPropertyEditorArray) along with a `Clear Optional` button which un-sets the optional.
*	3. (Multi-Select with mix of set + unset)
*		A combo-box with `Multiple States` is shown which when clicked displays a dropdown menu offering:
*		`Set all to Value` (sets all values to default initialized)
*		`Set all to None` (unsets all values)
*/
class SPropertyEditorOptional : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPropertyEditorOptional)
		: _Font(FAppStyle::GetFontStyle(PropertyEditorConstants::PropertyFontStyle))
		{}
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)
	SLATE_END_ARGS()

	static bool Supports(const TSharedRef<FPropertyEditor>& InPropertyEditor);

	void Construct(const FArguments& InArgs, const TSharedRef<FPropertyEditor>& InPropertyEditor, TSharedRef<IPropertyUtilities> InPropertyUtilities);

	void GetDesiredWidth(float& OutMinDesiredWidth, float& OutMaxDesiredWidth);

private:
	FText GetOptionalTooltipText() const;

	/** @return True if the property can be edited */
	bool CanEdit() const;

	TSharedRef<SWidget> MakeWidgetForOption(TSharedPtr<FText> InOption);
	void OnOptionChanged(TSharedPtr<FText> NewOption, ESelectInfo::Type);

	TSharedPtr< FPropertyEditor > PropertyEditor;
	TSharedPtr<SWidget> ValueEditorWidget;
	TArray<TSharedPtr<FText>> MultiselectOptions;
	float MinDesiredWidth;
	float MaxDesiredWidth;
};
