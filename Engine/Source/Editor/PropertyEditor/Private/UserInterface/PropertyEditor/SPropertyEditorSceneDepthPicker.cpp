// Copyright Epic Games, Inc. All Rights Reserved.
#include "UserInterface/PropertyEditor/SPropertyEditorSceneDepthPicker.h"

#include "ActorPickerMode.h"
#include "Delegates/Delegate.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Widgets/Images/SImage.h"

struct FGeometry;

#define LOCTEXT_NAMESPACE "SceneDepthPicker"

SPropertyEditorSceneDepthPicker::~SPropertyEditorSceneDepthPicker()
{
	if (FActorPickerModeModule* ActorPickerMode = FModuleManager::Get().GetModulePtr<FActorPickerModeModule>("ActorPickerMode"))
	{
		// make sure we are unregistered when this widget goes away
		ActorPickerMode->EndActorPickingMode();
	}
}

void SPropertyEditorSceneDepthPicker::Construct(const FArguments& InArgs)
{
	OnSceneDepthLocationSelected = InArgs._OnSceneDepthLocationSelected;

	SButton::Construct(
		SButton::FArguments()
		.ButtonStyle( FAppStyle::Get(), "HoverHintOnly" )
		.OnClicked(this, &SPropertyEditorSceneDepthPicker::OnClicked)
		.ContentPadding(4.0f)
		.ForegroundColor( FSlateColor::UseForeground() )
		.IsFocusable(false)
		[ 
			SNew( SImage )
			.Image( FAppStyle::GetBrush("Icons.EyeDropper") )
			.ColorAndOpacity( FSlateColor::UseForeground() )
		]
	);
}

FReply SPropertyEditorSceneDepthPicker::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if(InKeyEvent.GetKey() == EKeys::Escape)
	{
		FSceneDepthPickerModeModule& SceneDepthPickerMode = FModuleManager::Get().GetModuleChecked<FSceneDepthPickerModeModule>("SceneDepthPickerMode");

		if (SceneDepthPickerMode.IsInSceneDepthPickingMode())
		{
			SceneDepthPickerMode.EndSceneDepthPickingMode();
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

bool SPropertyEditorSceneDepthPicker::SupportsKeyboardFocus() const
{
	return true;
}

FReply SPropertyEditorSceneDepthPicker::OnClicked()
{
	FSceneDepthPickerModeModule& SceneDepthPickerMode = FModuleManager::Get().GetModuleChecked<FSceneDepthPickerModeModule>("SceneDepthPickerMode");

	if (SceneDepthPickerMode.IsInSceneDepthPickingMode())
	{
		SceneDepthPickerMode.EndSceneDepthPickingMode();
	}
	else
	{
		SceneDepthPickerMode.BeginSceneDepthPickingMode(OnSceneDepthLocationSelected);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
