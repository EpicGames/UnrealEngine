// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Input/Reply.h"
#include "SceneDepthPickerMode.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"

struct FGeometry;
struct FKeyEvent;

class SPropertyEditorSceneDepthPicker : public SButton
{
public:
	SLATE_BEGIN_ARGS(SPropertyEditorSceneDepthPicker) {}

	SLATE_EVENT(FOnSceneDepthLocationSelected, OnSceneDepthLocationSelected)

	SLATE_END_ARGS()

	~SPropertyEditorSceneDepthPicker();

	void Construct( const FArguments& InArgs );

	/** Begin SWidget interface */
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	virtual bool SupportsKeyboardFocus() const override;
	/** End SWidget interface */

private:
	/** Delegate for when the button is clicked */
	FReply OnClicked();

	/** Called when scene depth location is picked */
	FOnSceneDepthLocationSelected OnSceneDepthLocationSelected;
};
