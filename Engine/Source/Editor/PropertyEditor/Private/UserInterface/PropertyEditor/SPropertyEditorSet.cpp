// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyEditor/SPropertyEditorSet.h"

#include "Delegates/Delegate.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "PropertyEditorHelpers.h"
#include "PropertyNode.h"
#include "UObject/UnrealType.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "PropertyEditor"

void SPropertyEditorSet::Construct( const FArguments& InArgs, const TSharedRef< class FPropertyEditor >& InPropertyEditor )
{
	PropertyEditor = InPropertyEditor;

	TAttribute<FText> TextAttr(this, &SPropertyEditorSet::GetSetTextValue);

	ChildSlot
	.Padding(0.0f, 0.0f, 2.0f, 0.0f)
	[
		SNew(STextBlock)
		.Text(TextAttr)
		.Font(InArgs._Font)
	];

	SetToolTipText(GetSetTooltipText());

	SetEnabled(TAttribute<bool>(this, &SPropertyEditorSet::CanEdit));
}

bool SPropertyEditorSet::Supports( const TSharedRef< FPropertyEditor >& InPropertyEditor)
{
	const TSharedRef< FPropertyNode > PropertyNode = InPropertyEditor->GetPropertyNode();
	const FProperty* Property = InPropertyEditor->GetProperty();

	if (!PropertyNode->HasNodeFlags(EPropertyNodeFlags::EditInlineNew)
		&& Property->IsA<FSetProperty>())
	{
		return true;
	}

	return false;
}

void SPropertyEditorSet::GetDesiredWidth( float& OutMinDesiredWidth, float& OutMaxDesiredWidth )
{
	OutMinDesiredWidth = 170.0f;
	OutMaxDesiredWidth = 170.0f;
}

FText SPropertyEditorSet::GetSetTextValue() const
{
	check(PropertyEditor.IsValid());
	return PropertyEditorHelpers::GetNumElementsText(LOCTEXT("SetText", "Set"), *PropertyEditor.Get());
}

FText SPropertyEditorSet::GetSetTooltipText() const
{
	return LOCTEXT("RichSetTooltipText", "Sets are unordered containers. Each element in a set must be unique.");
}

bool SPropertyEditorSet::CanEdit() const
{
	return PropertyEditor.IsValid() ? !PropertyEditor->IsEditConst() : true;
}

#undef LOCTEXT_NAMESPACE
