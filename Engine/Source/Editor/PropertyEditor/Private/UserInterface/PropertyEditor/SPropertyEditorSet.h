// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Fonts/SlateFontInfo.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FPropertyEditor;

class SPropertyEditorSet : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPropertyEditorSet)
		: _Font(FAppStyle::GetFontStyle(PropertyEditorConstants::PropertyFontStyle))
		{}
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)
	SLATE_END_ARGS()

	static bool Supports(const TSharedRef< class FPropertyEditor >& InPropertyEditor);

	void Construct(const FArguments& InArgs, const TSharedRef< class FPropertyEditor >& InPropertyEditor);

	void GetDesiredWidth(float& OutMinDesiredWidth, float& OutMaxDesiredWidth);

private:

	FText GetSetTextValue() const;
	FText GetSetTooltipText() const;

	/** @return True if the property can be edited */
	bool CanEdit() const;

private:
	TSharedPtr< FPropertyEditor > PropertyEditor;
};
