// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyEditor/SPropertyEditorStruct.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SBox.h"
#include "PropertyNode.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "StructViewerModule.h"
#include "StructViewerFilter.h"
#include "StructUtils/UserDefinedStruct.h"

#define LOCTEXT_NAMESPACE "PropertyEditor"

class FPropertyEditorStructFilter : public IStructViewerFilter
{
public:
	/** The meta struct for the property that classes must be a child-of. */
	const UScriptStruct* MetaStruct = nullptr;
	bool bExcludeBaseStruct = false;

	// TODO: Have a flag controlling whether we allow UserDefinedStructs, even when a MetaClass is set (as they cannot support inheritance, but may still be allowed (eg, data tables))?

	virtual bool IsStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const UScriptStruct* InStruct, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
	{
		if (InStruct->IsA<UUserDefinedStruct>())
		{
			// User Defined Structs don't support inheritance, so only include them if we have don't a MetaStruct set
			return MetaStruct == nullptr;
		}

		if (bExcludeBaseStruct && MetaStruct)
		{
			if (InStruct == MetaStruct)
			{
				return false;
			}
		}

		// Query the native struct to see if it has the correct parent type (if any)
		return !MetaStruct || InStruct->IsChildOf(MetaStruct);
	}

	virtual bool IsUnloadedStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const FSoftObjectPath& InStructPath, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
	{
		// User Defined Structs don't support inheritance, so only include them if we have don't a MetaStruct set
		return MetaStruct == nullptr;
	}
};

void SPropertyEditorStruct::GetDesiredWidth(float& OutMinDesiredWidth, float& OutMaxDesiredWidth)
{
	OutMinDesiredWidth = 125.0f;
	OutMaxDesiredWidth = 400.0f;
}

bool SPropertyEditorStruct::Supports(const TSharedRef< class FPropertyEditor >& InPropertyEditor)
{
	const TSharedRef< FPropertyNode > PropertyNode = InPropertyEditor->GetPropertyNode();
	const FProperty* Property = InPropertyEditor->GetProperty();
	int32 ArrayIndex = PropertyNode->GetArrayIndex();

	if (Supports(Property) && ((ArrayIndex == -1 && Property->ArrayDim == 1) || (ArrayIndex > -1 && Property->ArrayDim > 0)))
	{
		return true;
	}

	return false;
}

bool SPropertyEditorStruct::Supports(const FProperty* InProperty)
{
	const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(InProperty);
	return ObjectProperty && ObjectProperty->PropertyClass && ObjectProperty->PropertyClass->IsChildOf(UScriptStruct::StaticClass());
}

void SPropertyEditorStruct::Construct(const FArguments& InArgs, const TSharedPtr< class FPropertyEditor >& InPropertyEditor)
{
	PropertyEditor = InPropertyEditor;
	
	if (PropertyEditor.IsValid())
	{
		const TSharedRef<FPropertyNode> PropertyNode = PropertyEditor->GetPropertyNode();
		FProperty* const Property = PropertyNode->GetProperty();
		if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
		{
			// Read the MetaStruct from the property meta-data
			MetaStruct = nullptr;
			{
				const FString& MetaStructName = Property->GetOwnerProperty()->GetMetaData(TEXT("MetaStruct"));
				if (!MetaStructName.IsEmpty())
				{
					MetaStruct = UClass::TryFindTypeSlow<UScriptStruct>(MetaStructName, EFindFirstObjectOptions::EnsureIfAmbiguous);
					if (!MetaStruct)
					{
						MetaStruct = LoadObject<UScriptStruct>(nullptr, *MetaStructName);
					}
				}
			}
		}
		else
		{
			check(false);
		}
		
		bAllowNone = !(Property->PropertyFlags & CPF_NoClear);
		bShowViewOptions = Property->GetOwnerProperty()->HasMetaData(TEXT("HideViewOptions")) ? false : true;
		bShowTree = Property->GetOwnerProperty()->HasMetaData(TEXT("ShowTreeView"));
		bShowDisplayNames = Property->GetOwnerProperty()->HasMetaData(TEXT("ShowDisplayNames"));
		static const FName NAME_ExcludeBaseStruct = "ExcludeBaseStruct";
		bExcludeBaseStruct = Property->GetOwnerProperty()->HasMetaData(NAME_ExcludeBaseStruct);
	}
	else
	{
		check(InArgs._SelectedStruct.IsSet());
		check(InArgs._OnSetStruct.IsBound());

		MetaStruct = InArgs._MetaStruct;
		bAllowNone = InArgs._AllowNone;
		bShowViewOptions = InArgs._ShowViewOptions;
		bShowTree = InArgs._ShowTree;
		bShowDisplayNames = InArgs._ShowDisplayNames;
		bExcludeBaseStruct = InArgs._bExcludeBaseStruct;

		SelectedStruct = InArgs._SelectedStruct;
		OnSetStruct = InArgs._OnSetStruct;
	}
	
	SAssignNew(ComboButton, SComboButton)
		.OnGetMenuContent(this, &SPropertyEditorStruct::GenerateStructPicker)
		.ContentPadding(FMargin(2.0f, 2.0f))
		.ToolTipText(this, &SPropertyEditorStruct::GetDisplayValue)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(this, &SPropertyEditorStruct::GetDisplayValue)
			.Font(InArgs._Font)
		];

	ChildSlot
	[
		ComboButton.ToSharedRef()
	];

	SetEnabled(TAttribute<bool>(this, &SPropertyEditorStruct::CanEdit));
}

FText SPropertyEditorStruct::GetDisplayValue() const
{
	static bool bIsReentrant = false;

	auto GetStructDisplayName = [this](const UObject* InObject) -> FText
	{
		if (const UScriptStruct* Struct = Cast<UScriptStruct>(InObject))
		{
			return bShowDisplayNames
				? Struct->GetDisplayNameText()
				: FText::AsCultureInvariant(Struct->GetName());
		}
		return LOCTEXT("None", "None");
	};

	// Guard against re-entrancy which can happen if the delegate executed below (SelectedStruct.Get()) forces a slow task dialog to open, thus causing this to lose context and regain focus later starting the loop over again
	if (!bIsReentrant)
	{
		TGuardValue<bool> Guard(bIsReentrant, true);

		if (PropertyEditor.IsValid())
		{
			UObject* ObjectValue = NULL;
			FPropertyAccess::Result Result = PropertyEditor->GetPropertyHandle()->GetValue(ObjectValue);

			if (Result == FPropertyAccess::Success && ObjectValue != NULL)
			{
				return GetStructDisplayName(ObjectValue);
			}

			return FText::AsCultureInvariant(FPaths::GetBaseFilename(PropertyEditor->GetValueAsString()));
		}

		return GetStructDisplayName(SelectedStruct.Get());
	}
	else
	{
		return FText::GetEmpty();
	}
}

TSharedRef<SWidget> SPropertyEditorStruct::GenerateStructPicker()
{
	FStructViewerInitializationOptions Options;
	Options.bShowUnloadedStructs = true;
	Options.bShowNoneOption = bAllowNone; 

	if (PropertyEditor.IsValid())
	{
		Options.PropertyHandle = PropertyEditor->GetPropertyHandle();
	}

	TSharedRef<FPropertyEditorStructFilter> StructFilter = MakeShared<FPropertyEditorStructFilter>();
	Options.StructFilter = StructFilter;
	StructFilter->MetaStruct = MetaStruct;
	StructFilter->bExcludeBaseStruct = bExcludeBaseStruct;
	Options.NameTypeToDisplay = (bShowDisplayNames ? EStructViewerNameTypeToDisplay::DisplayName : EStructViewerNameTypeToDisplay::StructName);
	Options.DisplayMode = bShowTree ? EStructViewerDisplayMode::TreeView : EStructViewerDisplayMode::ListView;
	Options.bAllowViewOptions = bShowViewOptions;

	FOnStructPicked OnPicked(FOnStructPicked::CreateRaw(this, &SPropertyEditorStruct::OnStructPicked));

	return SNew(SBox)
		.WidthOverride(280.0f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.MaxHeight(500.0f)
			[
				FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer").CreateStructViewer(Options, OnPicked)
			]			
		];
}

void SPropertyEditorStruct::OnStructPicked(const UScriptStruct* InStruct)
{
	SendToObjects(InStruct ? InStruct->GetPathName() : TEXT("None"));
	ComboButton->SetIsOpen(false);
}

void SPropertyEditorStruct::SendToObjects(const FString& NewValue)
{
	if(PropertyEditor.IsValid())
	{
		const TSharedRef<IPropertyHandle> PropertyHandle = PropertyEditor->GetPropertyHandle();
		PropertyHandle->SetValueFromFormattedString(NewValue);
	}
	else if (!NewValue.IsEmpty() && NewValue != TEXT("None"))
	{
		const UScriptStruct* NewStruct = FindObject<UScriptStruct>(nullptr, *NewValue);
		if (!NewStruct)
		{
			NewStruct = LoadObject<UScriptStruct>(nullptr, *NewValue);
		}
		OnSetStruct.Execute(NewStruct);
	}
	else
	{
		OnSetStruct.Execute(nullptr);
	}
}

void SPropertyEditorStruct::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FAssetDragDropOp> UnloadedStructOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
	if (UnloadedStructOp.IsValid())
	{
		FString AssetPath;
		FString PathName;

		// Find the struct path
		if (UnloadedStructOp->HasAssets())
		{
			AssetPath = UnloadedStructOp->GetAssets()[0].GetObjectPathString();
		}
		else if (UnloadedStructOp->HasAssetPaths())
		{
			AssetPath = UnloadedStructOp->GetAssetPaths()[0];
		}

		// Check to see if the asset can be found, otherwise load it.
		UObject* Object = FindObject<UObject>(nullptr, *AssetPath);
		if (Object == nullptr)
		{
			// Load the package.
			GWarn->BeginSlowTask(LOCTEXT("OnDrop_LoadPackage", "Fully Loading Package For Drop"), true, false);

			Object = LoadObject<UObject>(nullptr, *AssetPath);

			GWarn->EndSlowTask();
		}

		if (const UScriptStruct* Struct = Cast<UScriptStruct>(Object))
		{
			// This was pointing to a struct directly
			UnloadedStructOp->SetToolTip(FText::GetEmpty(), FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")));
		}
		else
		{
			UnloadedStructOp->SetToolTip(FText::GetEmpty(), FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")));
		}
	}
}

void SPropertyEditorStruct::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FAssetDragDropOp> UnloadedStructOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
	if (UnloadedStructOp.IsValid())
	{
		UnloadedStructOp->ResetToDefaultToolTip();
	}
}

FReply SPropertyEditorStruct::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FAssetDragDropOp> UnloadedStructOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
	if (UnloadedStructOp.IsValid())
	{
		FString AssetPath;

		// Find the struct path
		if (UnloadedStructOp->HasAssets())
		{
			AssetPath = UnloadedStructOp->GetAssets()[0].GetObjectPathString();
		}
		else if (UnloadedStructOp->HasAssetPaths())
		{
			AssetPath = UnloadedStructOp->GetAssetPaths()[0];
		}

		// Check to see if the asset can be found, otherwise load it.
		UObject* Object = FindObject<UObject>(nullptr, *AssetPath);
		if(Object == nullptr)
		{
			// Load the package.
			GWarn->BeginSlowTask(LOCTEXT("OnDrop_LoadPackage", "Fully Loading Package For Drop"), true, false);

			Object = LoadObject<UObject>(nullptr, *AssetPath);

			GWarn->EndSlowTask();
		}

		if (const UScriptStruct* Struct = Cast<UScriptStruct>(Object))
		{
			// This was pointing to a struct directly
			SendToObjects(Struct->GetPathName());
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

/** @return True if the property can be edited */
bool SPropertyEditorStruct::CanEdit() const
{
	return PropertyEditor.IsValid() ? !PropertyEditor->IsEditConst() : true;
}

#undef LOCTEXT_NAMESPACE
