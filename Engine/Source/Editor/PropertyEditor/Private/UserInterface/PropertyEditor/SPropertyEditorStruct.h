// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Misc/Attribute.h"
#include "Fonts/SlateFontInfo.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboButton.h"
#include "Styling/AppStyle.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "PropertyCustomizationHelpers.h"

/**
 * A widget used to edit struct type properties (UObject type properties, with a sub-type of UScriptStuct).
 * Can also be used (with a null FPropertyEditor) to edit a raw weak struct pointer.
 */
class SPropertyEditorStruct : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPropertyEditorStruct)
		: _Font(FAppStyle::GetFontStyle(PropertyEditorConstants::PropertyFontStyle)) 
		, _MetaStruct(nullptr)
		, _AllowNone(true)
		, _ShowDisplayNames(false)
		, _bExcludeBaseStruct(false)
		{}
		SLATE_ARGUMENT(FSlateFontInfo, Font)
		/** Arguments used when constructing this outside of a PropertyEditor (PropertyEditor == null), ignored otherwise */

		/** The meta struct that the selected struct must be a child-of (optional) */
		SLATE_ARGUMENT(const UScriptStruct*, MetaStruct)
		/** Should we be able to select "None" as a struct? (optional) */
		SLATE_ARGUMENT(bool, AllowNone)
		/** Attribute used to get the currently selected struct (required if PropertyEditor == null) */
		SLATE_ATTRIBUTE(const UScriptStruct*, SelectedStruct)
		/** Should we show the view options button at the bottom of the struct picker?*/
		SLATE_ARGUMENT(bool, ShowViewOptions)
		/** Should we show the struct picker in tree mode or list mode?*/
		SLATE_ARGUMENT(bool, ShowTree)
		/** Should we prettify struct names on the struct picker? (ie show their display name) */
		SLATE_ARGUMENT(bool, ShowDisplayNames)
		/** Should we hide the specified MetaStruct from the selection list and only display its children? */
		SLATE_ARGUMENT(bool, bExcludeBaseStruct)
		/** Delegate used to set the currently selected struct (required if PropertyEditor == null) */
		SLATE_EVENT(FOnSetStruct, OnSetStruct)
	SLATE_END_ARGS()

	static bool Supports(const TSharedRef< class FPropertyEditor >& InPropertyEditor);
	static bool Supports(const FProperty* InProperty);

	void Construct(const FArguments& InArgs, const TSharedPtr< class FPropertyEditor >& InPropertyEditor = nullptr);

	void GetDesiredWidth(float& OutMinDesiredWidth, float& OutMaxDesiredWidth);

private:
	void SendToObjects(const FString& NewValue);

	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	/** 
	 * Generates a struct picker with a filter to show only structs allowed to be selected. 
	 *
	 * @return The Struct Picker widget.
	 */
	TSharedRef<SWidget> GenerateStructPicker();

	/** 
	 * Callback function from the Struct Picker for when a struct is picked.
	 *
	 * @param InStruct		The struct picked in the Struct Picker
	 */
	void OnStructPicked(const UScriptStruct* InStruct);

	/**
	 * Gets the active display value
	 */
	FText GetDisplayValue() const;

	bool CanEdit() const;

private:
	/** The property editor we were constructed for, or null if we're editing using the construction arguments */
	TSharedPtr<class FPropertyEditor> PropertyEditor;

	/** Used when the property deals with structs and will display a Struct Picker. */
	TSharedPtr<class SComboButton> ComboButton;

	/** The meta struct that the selected class must be a child-of */
	const UScriptStruct* MetaStruct;
	/** Should we be able to select "None" as a struct? */
	bool bAllowNone;
	/** Should we show the view options button at the bottom of the struct picker? */
	bool bShowViewOptions;
	/** Should we show the struct picker in tree mode or list mode?*/
	bool bShowTree;
	/** Should we prettify struct names on the struct picker? (ie show their display name) */
	bool bShowDisplayNames;
	/** Should we hide the specified MetaStruct from the selection list and only display its children? */
	bool bExcludeBaseStruct;

	/** Attribute used to get the currently selected struct (required if PropertyEditor == null) */
	TAttribute<const UScriptStruct*> SelectedStruct;
	/** Delegate used to set the currently selected struct (required if PropertyEditor == null) */
	FOnSetStruct OnSetStruct;
};
