// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyEditor/SPropertyEditorTableRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SListView.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "PropertyEditorHelpers.h"

#include "UserInterface/PropertyTree/PropertyTreeConstants.h"

#include "UserInterface/PropertyEditor/SPropertyEditor.h"
#include "UserInterface/PropertyEditor/SPropertyEditorNumeric.h"
#include "UserInterface/PropertyEditor/SPropertyEditorArray.h"
#include "UserInterface/PropertyEditor/SPropertyEditorCombo.h"
#include "UserInterface/PropertyEditor/SPropertyEditorEditInline.h"
#include "UserInterface/PropertyEditor/SPropertyEditorText.h"
#include "UserInterface/PropertyEditor/SPropertyEditorBool.h"
#include "UserInterface/PropertyEditor/SPropertyEditorColor.h"
#include "UserInterface/PropertyEditor/SPropertyEditorArrayItem.h"
#include "UserInterface/PropertyEditor/SPropertyEditorDateTime.h"


void SPropertyEditorTableRow::Construct( const FArguments& InArgs, const TSharedRef<FPropertyEditor>& InPropertyEditor, const TSharedRef< class IPropertyUtilities >& InPropertyUtilities, const TSharedRef<STableViewBase>& InOwnerTable )
{
	PropertyEditor = InPropertyEditor;
	PropertyUtilities = InPropertyUtilities;
	OnMiddleClicked = InArgs._OnMiddleClicked;
	ConstructExternalColumnCell = InArgs._ConstructExternalColumnCell;

	PropertyPath = FPropertyNode::CreatePropertyPath( PropertyEditor->GetPropertyNode() );

	this->SetToolTipText(PropertyEditor->GetToolTipText());
	this->SetVisibility(TAttribute<EVisibility>::CreateSP(this, &SPropertyEditorTableRow::OnGetRowVisibility));
	SMultiColumnTableRow< TSharedPtr<FPropertyNode*> >::Construct( FSuperRowType::FArguments().Style(&FAppStyle::GetWidgetStyle<FTableRowStyle>("PropertyWindow.PropertyRow")), InOwnerTable );
}

TSharedRef<SWidget> SPropertyEditorTableRow::GenerateWidgetForColumn( const FName& ColumnName )
{
	if (ColumnName == PropertyTreeConstants::ColumnId_Name)
	{
		return ConstructNameColumnWidget();
	}
	else if (ColumnName == PropertyTreeConstants::ColumnId_Property)
	{
		return ConstructValueColumnWidget();
	}
	else if ( ConstructExternalColumnCell.IsBound() )
	{
		return ConstructExternalColumnCell.Execute( ColumnName, SharedThis( this ) );
	}

	return SNew(STextBlock).Text(NSLOCTEXT("PropertyEditor", "UnknownColumnId", "Unknown Column Id"));
}

EVisibility SPropertyEditorTableRow::OnGetRowVisibility() const
{
	if (PropertyEditor->IsOnlyVisibleWhenEditConditionMet() && !PropertyEditor->IsEditConditionMet())
	{
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

TSharedRef< SWidget > SPropertyEditorTableRow::ConstructNameColumnWidget()
{
	TSharedRef< SHorizontalBox > NameColumnWidget =
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding( FMargin( 0.0f, 1.0f, 0.0f, 1.0f ) )
		.VAlign(VAlign_Center)
		[
			SNew(SExpanderArrow, SharedThis( this ) )
		]
		+SHorizontalBox::Slot()
		.Padding( FMargin( 0.0f, 1.0f, 0.0f, 1.0f ) )
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew( SEditConditionWidget )
			.EditConditionValue(PropertyEditor.ToSharedRef(), &FPropertyEditor::IsEditConditionMet)
			.OnEditConditionValueChanged_Lambda([this](bool bValue) { PropertyEditor->ToggleEditConditionState(); })
			.Visibility_Lambda([this]() { return PropertyEditor->SupportsEditConditionToggle() ? EVisibility::Visible : EVisibility::Collapsed; })
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew( SPropertyNameWidget, PropertyEditor )
			.OnDoubleClicked( this, &SPropertyEditorTableRow::OnNameDoubleClicked )
		];

		return NameColumnWidget;
}


TSharedRef< SWidget > SPropertyEditorTableRow::ConstructValueColumnWidget()
{
	TSharedRef<SHorizontalBox> HorizontalBox =
		SNew(SHorizontalBox)
		.IsEnabled(	PropertyEditor.ToSharedRef(), &FPropertyEditor::IsPropertyEditingEnabled );

	ValueEditorWidget = ConstructPropertyEditorWidget();

	HorizontalBox->AddSlot()
		.FillWidth(1) // Fill the entire width if possible
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(20.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SBox)
			.MinDesiredWidth( MinWidth )
			.MaxDesiredWidth( MaxWidth )
			[
				ValueEditorWidget.ToSharedRef()
			]
		];

	// The favorites star for this property
	HorizontalBox->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew( SButton )
			.ButtonStyle( FAppStyle::Get(), "NoBorder" )
			.Visibility( this, &SPropertyEditorTableRow::OnGetFavoritesVisibility )
			.OnClicked( this, &SPropertyEditorTableRow::OnToggleFavoriteClicked )
			.ContentPadding(0.0f)
			[
				SNew( SImage )
				.Image( this, &SPropertyEditorTableRow::OnGetFavoriteImage )
			]
		];

	TArray< TSharedRef<SWidget> > RequiredButtons;
	PropertyEditorHelpers::MakeRequiredPropertyButtons( PropertyEditor.ToSharedRef(), /*OUT*/RequiredButtons, TArray<EPropertyButton::Type>(), false );

	for( int32 ButtonIndex = 0; ButtonIndex < RequiredButtons.Num(); ++ButtonIndex )
	{
		HorizontalBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding( 2.0f, 1.0f )
			[ 
				RequiredButtons[ButtonIndex]
			];
	}

	return SNew(SBorder)
		.Padding( FMargin( 0.0f, 1.0f, 0.0f, 1.0f ) )
		.BorderImage_Static( &PropertyEditorConstants::GetOverlayBrush, PropertyEditor.ToSharedRef() )
		.VAlign(VAlign_Fill)
		[
			HorizontalBox
		];
}

FReply SPropertyEditorTableRow::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	FReply Reply = FReply::Unhandled();

	if ( MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton && OnMiddleClicked.IsBound() )
	{
		OnMiddleClicked.Execute( PropertyPath.ToSharedRef() );
		Reply = FReply::Handled();
	}

	return Reply;
}

TSharedPtr< FPropertyPath > SPropertyEditorTableRow::GetPropertyPath() const
{
	return PropertyPath.ToSharedRef();
}


bool SPropertyEditorTableRow::IsCursorHovering() const
{
	return SWidget::IsHovered();
}


EVisibility SPropertyEditorTableRow::OnGetFavoritesVisibility() const
{
	if( PropertyUtilities->AreFavoritesEnabled() && (!PropertyEditor->IsChildOfFavorite()) )
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

FReply SPropertyEditorTableRow::OnToggleFavoriteClicked() 
{
	PropertyEditor->ToggleFavorite();
	return FReply::Handled();
}

const FSlateBrush* SPropertyEditorTableRow::OnGetFavoriteImage() const
{
	if( PropertyEditor->IsFavorite() )
	{
		return FAppStyle::GetBrush(TEXT("Icons.Star"));
	}

	return FAppStyle::GetBrush(TEXT("PropertyWindow.Favorites_Disabled"));
}

FReply SPropertyEditorTableRow::OnNameDoubleClicked()
{
	FReply Reply = FReply::Unhandled();

	if ( ValueEditorWidget.IsValid() )
	{
		// Get path to editable widget
		FWidgetPath EditableWidgetPath;
		FSlateApplication::Get().GeneratePathToWidgetUnchecked( ValueEditorWidget.ToSharedRef(), EditableWidgetPath );

		// Set keyboard focus directly
		FSlateApplication::Get().SetKeyboardFocus( EditableWidgetPath, EFocusCause::SetDirectly );
		Reply = FReply::Handled();
	}
	else if ( DoesItemHaveChildren() )
	{
		ToggleExpansion();
		Reply = FReply::Handled();
	}

	return Reply;
}

TSharedRef<SWidget> SPropertyEditorTableRow::ConstructPropertyEditorWidget()
{
	TSharedPtr<SWidget> PropertyWidget; 
	const FProperty* const Property = PropertyEditor->GetProperty();

	const TSharedRef< FPropertyEditor > PropertyEditorRef = PropertyEditor.ToSharedRef();
	const TSharedRef< IPropertyUtilities > PropertyUtilitiesRef = PropertyUtilities.ToSharedRef();

	float MinDesiredWidth = 0.0f;
	float MaxDesiredWidth = 0.0f;

	if( Property )
	{
		// ORDER MATTERS: first widget type to support the property node wins!
		if ( SPropertyEditorNumeric<float>::Supports(PropertyEditorRef) )
		{
			TSharedRef<SPropertyEditorNumeric<float>> TempWidget = SAssignNew(PropertyWidget, SPropertyEditorNumeric<float>, PropertyEditorRef );
			TempWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if (SPropertyEditorNumeric<int8>::Supports(PropertyEditorRef))
		{
			TSharedRef<SPropertyEditorNumeric<int8>> TempWidget = SAssignNew(PropertyWidget, SPropertyEditorNumeric<int8>, PropertyEditorRef );
			TempWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if (SPropertyEditorNumeric<int16>::Supports(PropertyEditorRef))
		{
			TSharedRef<SPropertyEditorNumeric<int16>> TempWidget = SAssignNew(PropertyWidget, SPropertyEditorNumeric<int16>, PropertyEditorRef );
			TempWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if (SPropertyEditorNumeric<int32>::Supports(PropertyEditorRef))
		{
			TSharedRef<SPropertyEditorNumeric<int32>> TempWidget = SAssignNew(PropertyWidget, SPropertyEditorNumeric<int32>, PropertyEditorRef );
			TempWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if ( SPropertyEditorNumeric<int64>::Supports(PropertyEditorRef) )
		{
			TSharedRef<SPropertyEditorNumeric<int64>> TempWidget = SAssignNew(PropertyWidget, SPropertyEditorNumeric<int64>, PropertyEditorRef );
			TempWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if ( SPropertyEditorNumeric<uint8>::Supports(PropertyEditorRef) )
		{
			TSharedRef<SPropertyEditorNumeric<uint8>> TempWidget = SAssignNew(PropertyWidget, SPropertyEditorNumeric<uint8>, PropertyEditorRef );
			TempWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if (SPropertyEditorNumeric<uint16>::Supports(PropertyEditorRef))
		{
			TSharedRef<SPropertyEditorNumeric<uint16>> TempWidget = SAssignNew(PropertyWidget, SPropertyEditorNumeric<uint16>, PropertyEditorRef );
			TempWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if (SPropertyEditorNumeric<uint32>::Supports(PropertyEditorRef))
		{
			TSharedRef<SPropertyEditorNumeric<uint32>> TempWidget = SAssignNew(PropertyWidget, SPropertyEditorNumeric<uint32>, PropertyEditorRef );
			TempWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if (SPropertyEditorNumeric<uint64>::Supports(PropertyEditorRef))
		{
			TSharedRef<SPropertyEditorNumeric<uint64>> TempWidget = SAssignNew(PropertyWidget, SPropertyEditorNumeric<uint64>, PropertyEditorRef );
			TempWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if ( SPropertyEditorArray::Supports(PropertyEditorRef) )
		{
			TSharedRef<SPropertyEditorArray> TempWidget = SAssignNew(PropertyWidget, SPropertyEditorArray, PropertyEditorRef );
			TempWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if ( SPropertyEditorCombo::Supports(PropertyEditorRef) )
		{
			TSharedRef<SPropertyEditorCombo> TempWidget = SAssignNew(PropertyWidget, SPropertyEditorCombo, PropertyEditorRef );
			TempWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if ( SPropertyEditorEditInline::Supports(PropertyEditorRef) )
		{
			TSharedRef<SPropertyEditorEditInline> TempWidget = SAssignNew(PropertyWidget, SPropertyEditorEditInline, PropertyEditorRef );
			TempWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if ( SPropertyEditorText::Supports(PropertyEditorRef) )
		{
			TSharedRef<SPropertyEditorText> TempWidget = SAssignNew(PropertyWidget, SPropertyEditorText, PropertyEditorRef );
			TempWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if ( SPropertyEditorBool::Supports(PropertyEditorRef) )
		{
			TSharedRef<SPropertyEditorBool> TempWidget = SAssignNew(PropertyWidget, SPropertyEditorBool, PropertyEditorRef );
			TempWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if ( SPropertyEditorColor::Supports(PropertyEditorRef) )
		{
			TSharedRef<SPropertyEditorColor> TempWidget = SAssignNew(PropertyWidget, SPropertyEditorColor, PropertyEditorRef, PropertyUtilitiesRef );
		}
		else if ( SPropertyEditorArrayItem::Supports(PropertyEditorRef) )
		{
			TSharedRef<SPropertyEditorArrayItem> TempWidget = SAssignNew(PropertyWidget, SPropertyEditorArrayItem, PropertyEditorRef );
			TempWidget->GetDesiredWidth(MinDesiredWidth, MaxDesiredWidth);
		}
		else if ( SPropertyEditorDateTime::Supports(PropertyEditorRef) )
		{
			TSharedRef<SPropertyEditorDateTime> TempWidget = SAssignNew(PropertyWidget, SPropertyEditorDateTime, PropertyEditorRef );
		}
	}

	if(MinDesiredWidth != 0.0f)
	{
		MinWidth = MinDesiredWidth;
	}
	if(MaxDesiredWidth != 0.0f)
	{
		MaxWidth = MaxDesiredWidth;
	}

	if( !PropertyWidget.IsValid() )
	{
		PropertyWidget = SNew( SPropertyEditor, PropertyEditorRef );
	}

	PropertyWidget->SetToolTipText( PropertyEditor->GetToolTipText() );

	return PropertyWidget.ToSharedRef();
}
