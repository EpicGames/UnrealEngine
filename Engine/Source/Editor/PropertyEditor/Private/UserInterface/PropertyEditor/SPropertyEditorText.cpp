// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyEditor/SPropertyEditorText.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "GenericPlatform/GenericApplication.h"
#include "Input/Events.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Misc/CString.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "PropertyNode.h"
#include "SlotBase.h"
#include "UObject/NameTypes.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"

struct FGeometry;

#define LOCTEXT_NAMESPACE "PropertyEditor"

void SPropertyEditorText::Construct( const FArguments& InArgs, const TSharedRef< class FPropertyEditor >& InPropertyEditor )
{
	PropertyEditor = InPropertyEditor;

	static const FName NAME_MaxLength = "MaxLength";
	static const FName NAME_MultiLine = "MultiLine";
	static const FName NAME_PasswordField = "PasswordField";
	static const FName NAME_AllowedCharacters = "AllowedCharacters";
	static const FName NAME_PropertyValidator = "PropertyValidator";
	static const FName NAME_HintText = "HintText";

	TSharedRef<IPropertyHandle> PropertyHandle(InPropertyEditor->GetPropertyHandle());
	bIsMultiLine = PropertyHandle->GetBoolMetaData(NAME_MultiLine);

	MaxLength = PropertyHandle->GetIntMetaData(NAME_MaxLength);
	if (InPropertyEditor->PropertyIsA(FNameProperty::StaticClass()))
	{
		MaxLength = MaxLength <= 0 ? NAME_SIZE - 1 : FMath::Min(MaxLength, NAME_SIZE - 1);
	}

	const bool bIsPassword = PropertyHandle->GetBoolMetaData(NAME_PasswordField);
	AllowedCharacters.InitializeFromString(PropertyHandle->GetMetaData(NAME_AllowedCharacters));

	if (PropertyHandle->HasMetaData(NAME_PropertyValidator))
	{
		const FString PropertyValidatorFunctionName = PropertyHandle->GetMetaData(NAME_PropertyValidator);
		const UClass* OuterBaseClass = PropertyHandle->GetOuterBaseClass();
		if (!PropertyValidatorFunctionName.IsEmpty() && OuterBaseClass)
		{
			static TSet<FString> LoggedWarnings;
			
			UObject* ValidatorObject = OuterBaseClass->GetDefaultObject<UObject>();
			const UFunction* PropertyValidatorFunction = ValidatorObject->FindFunction(*PropertyValidatorFunctionName);
			if (PropertyValidatorFunction)
			{
				if (PropertyValidatorFunction->FunctionFlags & FUNC_Static)
				{
					PropertyValidatorFunc = FPropertyValidatorFunc::CreateUFunction(ValidatorObject, PropertyValidatorFunction->GetFName());
				}
				else
				{
					const FString WarningId = OuterBaseClass->GetName() + TEXT(":") + *PropertyHandle->GetProperty()->GetName() + TEXT(":Static");
				
					UE_CLOGF(!LoggedWarnings.Contains(WarningId), LogPropertyNode, Warning, "PropertyValidator ufunction '%ls' on %ls%ls::%ls must be a static function.",
						*PropertyValidatorFunctionName, OuterBaseClass->GetPrefixCPP(), *OuterBaseClass->GetName(), *PropertyHandle->GetProperty()->GetName());
					LoggedWarnings.Add(WarningId);
				}
			}
			else
			{
				// Let the developer know that the function is missing.
				const FString WarningId = OuterBaseClass->GetName() + TEXT(":") + *PropertyHandle->GetProperty()->GetName() + TEXT(":Missing");
				
				UE_CLOGF(!LoggedWarnings.Contains(WarningId), LogPropertyNode, Warning, "PropertyValidator ufunction '%ls' on %ls%ls::%ls not found.",
					*PropertyValidatorFunctionName, OuterBaseClass->GetPrefixCPP(), *OuterBaseClass->GetName(), *PropertyHandle->GetProperty()->GetName());
				LoggedWarnings.Add(WarningId);
			}
		}
	}

	FString HintText = PropertyHandle->GetMetaData(NAME_HintText);

	TSharedPtr<SHorizontalBox> HorizontalBox;
	if(bIsMultiLine)
	{
		ChildSlot
		[
			SAssignNew(HorizontalBox, SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SAssignNew(MultiLineWidget, SMultiLineEditableTextBox)
				.Text(InPropertyEditor, &FPropertyEditor::GetValueAsText)
				.Font(InArgs._Font)
				.SelectAllTextWhenFocused(false)
				.ClearKeyboardFocusOnCommit(false)
				.MaximumLength(MaxLength)
				.OnTextCommitted(this, &SPropertyEditorText::OnTextCommitted)
				.OnVerifyTextChanged(this, &SPropertyEditorText::OnVerifyTextChanged)
				.SelectAllTextOnCommit(false)
				.IsReadOnly(this, &SPropertyEditorText::IsReadOnly)
				.AutoWrapText(true)
				.ModiferKeyForNewLine(EModifierKey::Shift)
				//.IsPassword( bIsPassword )
			]
		];

		PrimaryWidget = MultiLineWidget;
	}
	else
	{
		ChildSlot
		[
			SAssignNew(HorizontalBox, SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SAssignNew( SingleLineWidget, SEditableTextBox )
				.Text( InPropertyEditor, &FPropertyEditor::GetValueAsText )
				.Font( InArgs._Font )
				.SelectAllTextWhenFocused( true )
				.ClearKeyboardFocusOnCommit(false)
				.MaximumLength(MaxLength)
				.OnTextCommitted( this, &SPropertyEditorText::OnTextCommitted )
				.OnVerifyTextChanged( this, &SPropertyEditorText::OnVerifyTextChanged )
				.SelectAllTextOnCommit( true )
				.IsReadOnly(this, &SPropertyEditorText::IsReadOnly)
				.IsPassword( bIsPassword )
				.HintText(FText::FromString(HintText))
			]
		];

		PrimaryWidget = SingleLineWidget;
	}

	if (bIsPassword)
	{
		// Passwords should be obfuscated rather than reveal the property value in the tooltip
		PrimaryWidget->SetToolTipText(LOCTEXT("PasswordToolTip", "<hidden>"));
	}
	else if (InPropertyEditor->PropertyIsA(FObjectPropertyBase::StaticClass()))
	{
		// Object properties should display their entire text in a tooltip
		PrimaryWidget->SetToolTipText(TAttribute<FText>(InPropertyEditor, &FPropertyEditor::GetValueAsText));
	}
}

void SPropertyEditorText::GetDesiredWidth( float& OutMinDesiredWidth, float& OutMaxDesiredWidth )
{
	if(bIsMultiLine)
	{
		OutMinDesiredWidth = 250.0f;
	}
	else
	{
		OutMinDesiredWidth = 125.0f;
	}
	
	OutMaxDesiredWidth = 600.0f;
}

bool SPropertyEditorText::Supports( const TSharedRef< FPropertyEditor >& InPropertyEditor )
{
	const TSharedRef< FPropertyNode > PropertyNode = InPropertyEditor->GetPropertyNode();
	const FProperty* Property = InPropertyEditor->GetProperty();

	if(	!PropertyNode->HasNodeFlags(EPropertyNodeFlags::EditInlineNew)
		&&	( (Property->IsA(FNameProperty::StaticClass()) && Property->GetFName() != NAME_InitialState)
		||	Property->IsA(FStrProperty::StaticClass())
		||	Property->IsA(FTextProperty::StaticClass())
		||	(Property->IsA(FObjectPropertyBase::StaticClass()) && !Property->HasAnyPropertyFlags(CPF_InstancedReference))
		) )
	{
		return true;
	}

	return false;
}

void SPropertyEditorText::OnTextCommitted( const FText& NewText, ETextCommit::Type /*CommitInfo*/ )
{
	const TSharedRef< FPropertyNode > PropertyNode = PropertyEditor->GetPropertyNode();
	const TSharedRef< IPropertyHandle > PropertyHandle = PropertyEditor->GetPropertyHandle();

	FText CurrentText;
	if( (PropertyHandle->GetValueAsFormattedText( CurrentText ) != FPropertyAccess::MultipleValues || NewText.ToString() != FPropertyEditor::MultipleValuesDisplayName)
		&& !NewText.ToString().Equals(CurrentText.ToString(), ESearchCase::CaseSensitive))
	{
		if (CastField<FTextProperty>(PropertyHandle->GetProperty()))
		{
			PropertyHandle->NotifyPreChange();

			// We should preserve the localizable state of the existing text values when applying a new source string
			PropertyHandle->EnumerateRawData([&NewText](void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
			{
				FText* TextData = static_cast<FText*>(RawData);
				if (NewText.IsEmpty())
				{
					*TextData = NewText;
				}
				else if (TextData->IsCultureInvariant())
				{
					*TextData = FText::AsCultureInvariant(NewText.ToString());
				}
				else
				{
					// TODO: This could attempt to use stable keys, however FText properties should really be edited via STextPropertyEditableTextBox
					*TextData = FText::ChangeKey(TEXT(""), FGuid::NewGuid().ToString(), NewText);
				}
				return true;
			});

			PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
			PropertyHandle->NotifyFinishedChangingProperties();
		}
		else
		{
			PropertyHandle->SetValueFromFormattedString(NewText.ToString());
		}
	}
}

bool SPropertyEditorText::OnVerifyTextChanged(const FText& Text, FText& OutError)
{
	const FString& TextString = Text.ToString();

	if (!AllowedCharacters.IsEmpty())
	{
		if (!TextString.IsEmpty() && !AllowedCharacters.AreAllCharsIncluded(TextString))
		{
			TSet<TCHAR> InvalidCharacters = AllowedCharacters.FindCharsNotIncluded(TextString);
			FString InvalidCharactersString;
			for (TCHAR Char : InvalidCharacters)
			{
				if (!InvalidCharactersString.IsEmpty())
				{
					InvalidCharactersString.AppendChar(TEXT(' '));
				}
				InvalidCharactersString.AppendChar(Char);
			}
			OutError = FText::Format(LOCTEXT("PropertyTextCharactersNotAllowedError", "The value may not contain the following characters: {0}"), FText::FromString(InvalidCharactersString));
			return false;
		}
	}

	if (PropertyValidatorFunc.IsBound())
	{
		FText Result = PropertyValidatorFunc.Execute(TextString); 
		if (!Result.IsEmpty())
		{
			OutError = Result;
			return false;
		}
	}

	return true;
}

bool SPropertyEditorText::SupportsKeyboardFocus() const
{
	return PrimaryWidget.IsValid() && PrimaryWidget->SupportsKeyboardFocus() && CanEdit();
}

FReply SPropertyEditorText::OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent )
{
	// Forward keyboard focus to our editable text widget
	return FReply::Handled().SetUserFocus(PrimaryWidget.ToSharedRef(), InFocusEvent.GetCause());
}

bool SPropertyEditorText::CanEdit() const
{
	return PropertyEditor.IsValid() ? !PropertyEditor->IsEditConst() : true;
}

bool SPropertyEditorText::IsReadOnly() const
{
	return !CanEdit();
}

#undef LOCTEXT_NAMESPACE
