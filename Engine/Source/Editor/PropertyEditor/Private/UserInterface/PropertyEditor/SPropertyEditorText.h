// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Fonts/SlateFontInfo.h"
#include "Framework/Text/CharRangeList.h"
#include "Input/Reply.h"
#include "Misc/Attribute.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FText;
class SEditableTextBox;
class SMultiLineEditableTextBox;
struct FFocusEvent;
struct FGeometry;

class SPropertyEditorText : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SPropertyEditorText )
		: _Font( FAppStyle::GetFontStyle( PropertyEditorConstants::PropertyFontStyle ) ) 
		{}
		SLATE_ATTRIBUTE( FSlateFontInfo, Font )
	SLATE_END_ARGS()

	static bool Supports( const TSharedRef< class FPropertyEditor >& InPropertyEditor );

	void Construct( const FArguments& InArgs, const TSharedRef< class FPropertyEditor >& InPropertyEditor );

	void GetDesiredWidth( float& OutMinDesiredWidth, float& OutMaxDesiredWidth );

	bool SupportsKeyboardFocus() const override;

	FReply OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent ) override;

private:

	void OnTextCommitted( const FText& NewText, ETextCommit::Type CommitInfo );

	bool OnVerifyTextChanged(const FText& Text, FText& OutError);

	/** @return True if the property can be edited */
	bool CanEdit() const;

	/** @return True if the property is Read Only */
	bool IsReadOnly() const;

private:
	DECLARE_DELEGATE_RetVal_OneParam(FText, FPropertyValidatorFunc, const FString&);
	
	TSharedPtr< class FPropertyEditor > PropertyEditor;

	TSharedPtr< class SWidget > PrimaryWidget;

	/** Widget used for the multiline version of the text property */
	TSharedPtr<SMultiLineEditableTextBox> MultiLineWidget;

	/** Widget used for the single line version of the text property */
	TSharedPtr<SEditableTextBox> SingleLineWidget;

	/** Cached flag as we would like multi-line text widgets to be slightly larger */
	bool bIsMultiLine;
	
	/** The maximum length of the value that can be edited, or <=0 for unlimited */
	int32 MaxLength = 0;

	/** The characters that are allowed in the value. If empty, any character is allowed. */
	FCharRangeList AllowedCharacters;

	/** An optional validator function that will be called each time the text is changed */
	FPropertyValidatorFunc PropertyValidatorFunc;
};
