// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "PropertyNode.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/AppStyle.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"
#include "Widgets/Text/STextBlock.h"

class SPropertyEditorTitle : public SCompoundWidget
{

public:

	SLATE_BEGIN_ARGS( SPropertyEditorTitle )
		: _PropertyFont( FAppStyle::GetFontStyle( PropertyEditorConstants::PropertyFontStyle ) )
		, _CategoryFont( FAppStyle::GetFontStyle( PropertyEditorConstants::CategoryFontStyle ) )
		{}
		SLATE_ATTRIBUTE( FSlateFontInfo, PropertyFont )
		SLATE_ATTRIBUTE( FSlateFontInfo, CategoryFont )
		SLATE_EVENT( FOnClicked, OnDoubleClicked )
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const TSharedRef<FPropertyEditor>& InPropertyEditor )
	{
		PropertyEditor = InPropertyEditor;
		OnDoubleClicked = InArgs._OnDoubleClicked;

		const TSharedRef< FPropertyNode > PropertyNode = InPropertyEditor->GetPropertyNode();
		const bool bIsCategory = PropertyNode->AsCategoryNode() != nullptr;
		const TAttribute<FSlateFontInfo> NameFont = bIsCategory ? InArgs._CategoryFont : InArgs._PropertyFont;

		TSharedPtr<STextBlock> NameTextBlock;

		// If our property has title support we want to fetch the value every tick, otherwise we can just use a static value
		static const FName NAME_TitleProperty = FName(TEXT("TitleProperty"));
		const bool bHasTitleProperty = InPropertyEditor->GetProperty() && InPropertyEditor->GetProperty()->HasMetaData(NAME_TitleProperty);
		if (bHasTitleProperty)
		{
			NameTextBlock =
				SNew(STextBlock)
				.Text(InPropertyEditor, &FPropertyEditor::GetDisplayName)
				.Font(NameFont);
		}		
		else
		{
			NameTextBlock =
				SNew(STextBlock)
				.Text(InPropertyEditor->GetDisplayName())
				.Font(NameFont);
		}

		NameTextBlock->SetOverflowPolicy(ETextOverflowPolicy::Ellipsis);

		TSharedPtr<SWidget> NameWidget = NameTextBlock;
		const bool bInArray = PropertyNode->GetProperty() != nullptr && PropertyNode->GetArrayIndex() != INDEX_NONE;
		if (bInArray && !bHasTitleProperty)
		{
			const bool bNameIsIndex = NameTextBlock->GetText().EqualTo(FText::AsNumber(PropertyNode->GetArrayIndex()));
			if (bNameIsIndex)
			{
				NameWidget = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(0, 0, 3, 0)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(NSLOCTEXT("PropertyEditor", "Index", "Index"))
					.Font(NameFont)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(0, 0, 3, 0)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(NSLOCTEXT("PropertyEditor", "OpenBracket", "["))
					.Font(NameFont)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(0, 0, 3, 0)
				.AutoWidth()
				[
					NameTextBlock.ToSharedRef()
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(NSLOCTEXT("PropertyEditor", "CloseBracket", "]"))
					.Font(NameFont)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				];
			}
		}

		ChildSlot
		[
			NameWidget.ToSharedRef()
		];
	}

private:

	FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
	{
		if ( InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
		{
			if( OnDoubleClicked.IsBound() )
			{
				OnDoubleClicked.Execute();
				return FReply::Handled();
			}
		}

		return FReply::Unhandled();
	}


private:

	/** The delegate to execute when this text is double clicked */
	FOnClicked OnDoubleClicked;

	TSharedPtr<FPropertyEditor> PropertyEditor;
};
