// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyEditor/SPropertyMenuActorPicker.h"
#include "Modules/ModuleManager.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "GameFramework/Actor.h"
#include "AssetRegistry/AssetData.h"
#include "Editor.h"
#include "SceneOutlinerPublicTypes.h"
#include "SceneOutlinerModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "UserInterface/PropertyEditor/PropertyEditorAssetConstants.h"
#include "HAL/PlatformApplicationMisc.h"
#include "SceneOutlinerPublicTypes.h"
#include "ActorTreeItem.h"
#include "PropertyNode.h"

#define LOCTEXT_NAMESPACE "PropertyEditor"

void SPropertyMenuActorPicker::Construct( const FArguments& InArgs )
{
	CurrentActor = InArgs._InitialActor;
	bAllowClear = InArgs._AllowClear;
	bAllowPickingLevelInstanceContent = InArgs._AllowPickingLevelInstanceContent;
	ActorFilter = InArgs._ActorFilter;
	OnSet = InArgs._OnSet;
	OnClose = InArgs._OnClose;
	OnUseSelected = InArgs._OnUseSelected;
	bDisplayUseSelected = InArgs._DisplayUseSelected;
	bShowTransient = InArgs._ShowTransient;

	FMenuBuilder MenuBuilder(true, NULL);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("CurrentActorOperationsHeader", "Current Actor"));
	{
		if (bDisplayUseSelected)
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("UseSelected", "Use Selected"),
				LOCTEXT("UseSelected_Tooltip", "Use the currently selected Actor"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SPropertyMenuActorPicker::HandleUseSelected)));
		}

		if( CurrentActor )
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("EditAsset", "Edit"), 
				LOCTEXT("EditAsset_Tooltip", "Edit this asset"),
				FSlateIcon(),
				FUIAction( FExecuteAction::CreateSP( this, &SPropertyMenuActorPicker::OnEdit ) ) );
		}

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CopyAsset", "Copy"),
			LOCTEXT("CopyAsset_Tooltip", "Copies the asset to the clipboard"),
			FSlateIcon(),
			FUIAction( FExecuteAction::CreateSP( this, &SPropertyMenuActorPicker::OnCopy ) )
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("PasteAsset", "Paste"),
			LOCTEXT("PasteAsset_Tooltip", "Pastes an asset from the clipboard to this field"),
			FSlateIcon(),
			FUIAction( 
				FExecuteAction::CreateSP( this, &SPropertyMenuActorPicker::OnPaste ),
				FCanExecuteAction::CreateSP( this, &SPropertyMenuActorPicker::CanPaste ) )
		);

		if( bAllowClear )
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("ClearAsset", "Clear"),
				LOCTEXT("ClearAsset_ToolTip", "Clears the asset set on this field"),
				FSlateIcon(),
				FUIAction( FExecuteAction::CreateSP( this, &SPropertyMenuActorPicker::OnClear ) )
				);
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("BrowseHeader", "Browse"));
	{
		TSharedPtr<SWidget> MenuContent;

		FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::Get().LoadModuleChecked<FSceneOutlinerModule>(TEXT("SceneOutliner"));

		FSceneOutlinerInitializationOptions InitOptions;
		InitOptions.Filters->AddFilterPredicate<FActorTreeItem>(ActorFilter);
		InitOptions.bFocusSearchBoxWhenOpened = true;
		InitOptions.bShowTransient = bShowTransient;

		InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0));
		InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::ActorInfo(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 10));

		TSharedRef<ISceneOutliner> SceneOutliner = SceneOutlinerModule.CreateActorPicker(InitOptions, FOnActorPicked::CreateSP(this, &SPropertyMenuActorPicker::OnActorSelected), nullptr, !bAllowPickingLevelInstanceContent);

		if (CurrentActor)
		{
			SceneOutliner->SetSelection([this](ISceneOutlinerTreeItem& Item)
			{
				if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
				{
					return ActorItem->Actor == CurrentActor;
				}

				return false;
			});

			SceneOutliner->FrameSelectedItems();
		}
		
		MenuContent =
			SNew(SBox)
			.WidthOverride(static_cast<float>(PropertyEditorAssetConstants::SceneOutlinerWindowSize.X))
			.HeightOverride(static_cast<float>(PropertyEditorAssetConstants::SceneOutlinerWindowSize.Y))
			[
				MoveTemp(SceneOutliner)
			];

		MenuBuilder.AddWidget(MenuContent.ToSharedRef(), FText::GetEmpty(), true);
	}
	MenuBuilder.EndSection();

	ChildSlot
	[
		MenuBuilder.MakeWidget()
	];
}

void SPropertyMenuActorPicker::HandleUseSelected()
{
	OnUseSelected.ExecuteIfBound();
}

void SPropertyMenuActorPicker::OnEdit()
{
	if( CurrentActor )
	{
		GEditor->EditObject( CurrentActor );
	}
	OnClose.ExecuteIfBound();
}

void SPropertyMenuActorPicker::OnCopy()
{
	FAssetData CurrentAssetData( CurrentActor );

	if( CurrentAssetData.IsValid() )
	{
		FPlatformApplicationMisc::ClipboardCopy(*CurrentAssetData.GetExportTextName());
	}
	OnClose.ExecuteIfBound();
}

void SPropertyMenuActorPicker::OnPaste()
{
	FString DestPath;
	FPlatformApplicationMisc::ClipboardPaste(DestPath);

	if(DestPath == TEXT("None"))
	{
		SetValue(NULL);
	}
	else
	{
		AActor* Actor = LoadObject<AActor>(NULL, *DestPath);
		if(Actor && (!ActorFilter.IsBound() || ActorFilter.Execute(Actor)))
		{
			SetValue(Actor);
		}
	}
	OnClose.ExecuteIfBound();
}

bool SPropertyMenuActorPicker::CanPaste()
{
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

	FString Class;
	FString PossibleObjectPath = ClipboardText;
	if( ClipboardText.Split( TEXT("'"), &Class, &PossibleObjectPath, ESearchCase::CaseSensitive) )
	{
		// Remove the last item
		PossibleObjectPath.LeftChopInline( 1, EAllowShrinking::No );
	}

	bool bCanPaste = false;

	if( PossibleObjectPath == TEXT("None") )
	{
		bCanPaste = true;
	}
	else
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		bCanPaste = PossibleObjectPath.Len() < NAME_SIZE && AssetRegistryModule.Get().GetAssetByObjectPath( FSoftObjectPath(PossibleObjectPath) ).IsValid();
	}

	return bCanPaste;
}

void SPropertyMenuActorPicker::OnClear()
{
	SetValue(NULL);
	OnClose.ExecuteIfBound();
}

void SPropertyMenuActorPicker::OnActorSelected( AActor* InActor )
{
	FText OutErrorMsg;

	if (IsValid(InActor) && !FName::IsValidXName(InActor->GetName(), INVALID_NAME_CHARACTERS,  &OutErrorMsg))
	{
		FFormatNamedArguments Args;
		Args.Add("ActorName", FText::FromString(FName::SanitizeWhitespace(InActor->GetName())));
		Args.Add("ActorLabel", FText::FromString(FName::SanitizeWhitespace(InActor->GetActorLabel())));

		FText Error = OutErrorMsg.IsEmpty() ?
		     LOCTEXT("InvalidCharactersDefaultErrorMessage", "Names do not support the following characters: \"\' ,\\n\\r\\t") : OutErrorMsg;
		
		Args.Add("ErrorMessage", Error);
		const FText LogMessage = FText::Format(LOCTEXT("InvalidActorName", "The chosen actor, {ActorLabel}, has an invalid name of {ActorName}.\n{ErrorMessage}"), Args);
		
		UE_LOGF(LogPropertyNode, Warning, "%ls", *LogMessage.ToString());
	}


	
	SetValue(InActor);
	OnClose.ExecuteIfBound();
}

void SPropertyMenuActorPicker::SetValue( AActor* InActor )
{
	OnSet.ExecuteIfBound(InActor);
}

#undef LOCTEXT_NAMESPACE
