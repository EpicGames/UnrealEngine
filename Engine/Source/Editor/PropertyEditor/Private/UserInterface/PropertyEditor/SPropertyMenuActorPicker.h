// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "ActorPickerMode.h"

class AActor;

class SPropertyMenuActorPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SPropertyMenuActorPicker )
		: _InitialActor(NULL)
		, _AllowClear(true)
		, _AllowPickingLevelInstanceContent(false)
		, _ActorFilter()
		, _DisplayUseSelected(true)
		, _ShowTransient(false)
	{}
		SLATE_ARGUMENT( AActor*, InitialActor )
		SLATE_ARGUMENT( bool, AllowClear )
		SLATE_ARGUMENT( bool, AllowPickingLevelInstanceContent )
		SLATE_ARGUMENT( FOnShouldFilterActor, ActorFilter )
		SLATE_ARGUMENT(bool, DisplayUseSelected)
		SLATE_EVENT( FOnActorSelected, OnSet )
		SLATE_EVENT( FSimpleDelegate, OnClose )
		SLATE_EVENT( FSimpleDelegate, OnUseSelected )
		SLATE_ARGUMENT(bool, ShowTransient)
	SLATE_END_ARGS()

	/**
	 * Construct the widget.
	 * @param	InArgs				Arguments for widget construction
	 * @param	InPropertyHandle	The property handle that this widget will operate on.
	 */
	void Construct( const FArguments& InArgs );

private:
	/**
	 * Use the currently selected Actor
	 */
	void HandleUseSelected();

	/** 
	 * Edit the object referenced by this widget
	 */
	void OnEdit();

	/** 
	 * Delegate handling ctrl+c
	 */
	void OnCopy();

	/** 
	 * Delegate handling ctrl+v
	 */
	void OnPaste();

	/**
	 * @return True of the current clipboard contents can be pasted
	 */
	bool CanPaste();

	/** 
	 * Clear the referenced object
	 */
	void OnClear();

	/** 
	 * Delegate for handling selection in the scene outliner.
	 * @param	InActor	The chosen actor
	 */
	void OnActorSelected( AActor* InActor );

	/** 
	 * Set the value of the asset referenced by this property editor.
	 * Will set the underlying property handle if there is one.
	 * @param	InObject	The object to set the reference to
	 */
	void SetValue( AActor* InActor );

private:
	AActor* CurrentActor;

	/** Whether the asset can be 'None' in this case */
	bool bAllowClear;

	/** Whether we allow picking actors inside level instances */
	bool bAllowPickingLevelInstanceContent;

	/** Delegate used to test whether an actor should be displayed or not */
	FOnShouldFilterActor ActorFilter;

	/** Delegate to call when our object value should be set */
	FOnActorSelected OnSet;

	/** Delegate for closing the containing menu */
	FSimpleDelegate OnClose;

	/** Delegate for using the currently selected actor */
	FSimpleDelegate OnUseSelected;

	/* Whether to display the 'Use Selected' item*/
	bool bDisplayUseSelected;

	/** Whether to show transient objects */
	bool bShowTransient = false;
};
