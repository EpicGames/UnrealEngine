// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyEditor/SPropertyMenuAssetPicker.h"

#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "Factories/Factory.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "Layout/WidgetPath.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorClipboard.h"
#include "PropertyEditorCopyPastePrivate.h"
#include "PropertyEditorHelpers.h"
#include "Styling/SlateIconFinder.h"
#include "UserInterface/PropertyEditor/SPropertyEditorAsset.h"
#include "UserInterface/PropertyEditor/PropertyEditorAssetConstants.h"
#include "Widgets/Layout/SBox.h"
#include "Settings/EditorStyleSettings.h"

#define LOCTEXT_NAMESPACE "PropertyEditor"

void SPropertyMenuAssetPicker::Construct( const FArguments& InArgs )
{
	CurrentObject = InArgs._InitialObject;
	PropertyHandle = InArgs._PropertyHandle;
	const TArray<FAssetData>& OwnerAssetArray = InArgs._OwnerAssetArray;
	bAllowClear = InArgs._AllowClear;
	bAllowCopyPaste = InArgs._AllowCopyPaste;
	AllowedClasses = InArgs._AllowedClasses;
	DisallowedClasses = InArgs._DisallowedClasses;
	NewAssetFactories = InArgs._NewAssetFactories;
	OnShouldFilterAsset = InArgs._OnShouldFilterAsset;
	OnSet = InArgs._OnSet;
	OnClose = InArgs._OnClose;

	// Get initial view type
	EAssetViewType::Type AssetViewType = EAssetViewType::List;
	static const FName AssetViewTypeMetadata = TEXT("AssetViewInitialViewType");
	if (PropertyHandle && PropertyHandle->HasMetaData(AssetViewTypeMetadata))
	{
		const int32 Value = PropertyHandle->GetIntMetaData(AssetViewTypeMetadata);
		if (Value >= 0 && Value <= (EAssetViewType::Type::MAX - 1))
		{
			AssetViewType = static_cast<EAssetViewType::Type>(Value);
		}
	}

	// Get initial thumbnail size
	EThumbnailSize AssetThumbnailSize = EThumbnailSize::Medium;
	static const FName AssetThumbnailSizeMetadata = TEXT("AssetViewInitialThumbnailSize");
	if (PropertyHandle && PropertyHandle->HasMetaData(AssetThumbnailSizeMetadata))
	{
		const int32 Value = PropertyHandle->GetIntMetaData(AssetThumbnailSizeMetadata);
		if (Value >= 0 && Value <= (static_cast<int32>(EThumbnailSize::MAX) - 1))
		{
			AssetThumbnailSize = static_cast<EThumbnailSize>(Value);
		}
	}

	// Get initial thumbnail scale
	TOptional<float> AssetThumbnailScale;
	static const FName AssetThumbnailScaleMetadata = TEXT("AssetViewInitialThumbnailScale");
	if (PropertyHandle && PropertyHandle->HasMetaData(AssetThumbnailScaleMetadata))
	{
		const int32 Value = PropertyHandle->GetFloatMetaData(AssetThumbnailScaleMetadata);
		if (Value >= 0 && Value <= 1)
		{
			AssetThumbnailScale = Value;
		}
	}

	// Get settings to load/save, use empty string to not use any settings
	static const FName AssetViewSettingsNameMetadata = TEXT("AssetViewSettingsName");
	const FString SaveSettingsName = PropertyHandle && PropertyHandle->HasMetaData(AssetViewSettingsNameMetadata) ? PropertyHandle->GetMetaData(AssetViewSettingsNameMetadata) : TEXT("AssetPropertyPicker");

	// Get show flags
	const bool bForceShowEngineContent = PropertyHandle ? PropertyHandle->HasMetaData(TEXT("ForceShowEngineContent")) : false;
	const bool bForceShowPluginContent = PropertyHandle ? PropertyHandle->HasMetaData(TEXT("ForceShowPluginContent")) : false;

	const bool bInShouldCloseWindowAfterMenuSelection = true;
	const bool bCloseSelfOnly = true;
	const bool bSearchable = false;
	
	FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, nullptr, nullptr, bCloseSelfOnly, &FCoreStyle::Get(), bSearchable);

	if (NewAssetFactories.Num() > 0)
	{
		MenuBuilder.BeginSection(NAME_None, LOCTEXT("CreateNewAsset", "Create New Asset"));
		{
			for (UFactory* Factory : NewAssetFactories)
			{
				TWeakObjectPtr<UFactory> FactoryPtr(Factory);

				MenuBuilder.AddMenuEntry(
					Factory->GetDisplayName(),
					Factory->GetToolTip(),
					FSlateIconFinder::FindIconForClass(Factory->GetSupportedClass()),
					FUIAction(FExecuteAction::CreateSP(this, &SPropertyMenuAssetPicker::OnCreateNewAssetSelected, FactoryPtr))
					);
			}
		}
		MenuBuilder.EndSection();
	}

	if (CurrentObject.IsValid() || bAllowCopyPaste || bAllowClear)
	{
		MenuBuilder.BeginSection(NAME_None, LOCTEXT("CurrentAssetOperationsHeader", "Current Asset"));
		{
			if (CurrentObject.IsValid())
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("EditAsset", "Edit"),
					LOCTEXT("EditAsset_Tooltip", "Edit this asset"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(),"Icons.Edit"),
					FUIAction(FExecuteAction::CreateSP(this, &SPropertyMenuAssetPicker::OnEdit)));
			}

			if (bAllowCopyPaste)
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("CopyAsset", "Copy"),
					LOCTEXT("CopyAsset_Tooltip", "Copies the asset to the clipboard"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(),"GenericCommands.Copy"),
					FUIAction(FExecuteAction::CreateSP(this, &SPropertyMenuAssetPicker::OnCopy))
				);

				MenuBuilder.AddMenuEntry(
					LOCTEXT("PasteAsset", "Paste"),
					LOCTEXT("PasteAsset_Tooltip", "Pastes an asset from the clipboard to this field"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(),"GenericCommands.Paste"),
					FUIAction(
						FExecuteAction::CreateSP(this, &SPropertyMenuAssetPicker::OnPaste),
						FCanExecuteAction::CreateSP(this, &SPropertyMenuAssetPicker::CanPaste))
				);
			}

			if (bAllowClear)
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("ClearAsset", "Clear"),
					LOCTEXT("ClearAsset_ToolTip", "Clears the asset set on this field"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(),"GenericCommands.Delete"),
					FUIAction(FExecuteAction::CreateSP(this, &SPropertyMenuAssetPicker::OnClear))
				);
			}
		}
		MenuBuilder.EndSection();
	}

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("BrowseHeader", "Browse"));
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

		FAssetPickerConfig AssetPickerConfig;
		// Add filter classes - if we have a single filter class of "Object" then don't set a filter since it would always match everything (but slower!)
		if (AllowedClasses.Num() == 1 && AllowedClasses[0] == UObject::StaticClass())
		{
			AssetPickerConfig.Filter.ClassPaths.Reset();
		}
		else
		{
			for(int32 i = 0; i < AllowedClasses.Num(); ++i)
			{
				AssetPickerConfig.Filter.ClassPaths.Add( AllowedClasses[i]->GetClassPathName() );
			}
		}

		for (int32 i = 0; i < DisallowedClasses.Num(); ++i)
		{
			AssetPickerConfig.Filter.RecursiveClassPathsExclusionSet.Add(DisallowedClasses[i]->GetClassPathName());
		}

		// Allow child classes
		AssetPickerConfig.Filter.bRecursiveClasses = true;
		// Set a delegate for setting the asset from the picker
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SPropertyMenuAssetPicker::OnAssetSelected);
		// Set a delegate for setting the asset from the picker via the keyboard
		AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateSP(this, &SPropertyMenuAssetPicker::OnAssetEnterPressed);
		// Use the list view by default
		AssetPickerConfig.InitialAssetViewType = AssetViewType;
		// TODO: Remove "PropertyEditorHelpers.h" include when IsNewStyleEnabled will be removed
		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			// Use the medium size by default if the new style is enabled
			AssetPickerConfig.InitialThumbnailSize = AssetThumbnailSize;
		}
		// Set scale for thumbnail
		if (AssetThumbnailScale.IsSet())
		{
			AssetPickerConfig.ThumbnailScale = AssetThumbnailScale.GetValue();
		}
		// The initial selection should be the current value
		AssetPickerConfig.InitialAssetSelection = CurrentObject;
		// We'll do clearing ourselves
		AssetPickerConfig.bAllowNullSelection = false;
		// Focus search box
		AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
		// Apply custom filter
		AssetPickerConfig.OnShouldFilterAsset = OnShouldFilterAsset;
		// Don't allow dragging
		AssetPickerConfig.bAllowDragging = false;
		// Save the settings into a special section for asset pickers for properties
		AssetPickerConfig.SaveSettingsName = SaveSettingsName;
		// Populate the referencing assets via property handle
		AssetPickerConfig.PropertyHandle = PropertyHandle;
		// Populate the additional referencing assets with the Owner asset data
		AssetPickerConfig.AdditionalReferencingAssets = OwnerAssetArray;
		// Force show engine content if meta data says so
		AssetPickerConfig.bForceShowEngineContent = bForceShowEngineContent;
		// Force show plugin content if meta data says so
		AssetPickerConfig.bForceShowPluginContent = bForceShowPluginContent;

		AssetPickerWidget = ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig);

		// Adjust widget size by Editor Preferences AssetPickerSizeScale
		const UEditorStyleSettings* EditorSettings = GetDefault<UEditorStyleSettings>();
		const float AssetPickerSizeScale = EditorSettings->AssetPickerSizeScale;
		float WidgetWidth = static_cast<float>(PropertyEditorAssetConstants::ContentBrowserWindowSize.X) * AssetPickerSizeScale;
		float WidgetHeight = static_cast<float>(PropertyEditorAssetConstants::ContentBrowserWindowSize.Y) * AssetPickerSizeScale;

		TSharedRef<SWidget> MenuContent =
			SNew(SBox)
			.WidthOverride(WidgetWidth)
			.HeightOverride(WidgetHeight)
			[
				AssetPickerWidget.ToSharedRef()
			];

		MenuBuilder.AddWidget(MenuContent, FText::GetEmpty(), true);
	}
	MenuBuilder.EndSection();

	ChildSlot
	[
		MenuBuilder.MakeWidget()
	];
}

FReply SPropertyMenuAssetPicker::OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (!AssetPickerWidget.IsValid())
	{
		return FReply::Unhandled();
	}
	
	// only give the search box focus if it's not a command like Ctrl+C
	if (InKeyEvent.GetCharacter() == 0 || 
		InKeyEvent.IsAltDown() ||
		InKeyEvent.IsControlDown() ||
		InKeyEvent.IsCommandDown())
	{
		return FReply::Unhandled();
	}

	const FWidgetPath* Path = InKeyEvent.GetEventPath();
	if (Path != nullptr)
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
		TSharedPtr<SWidget> SearchBox = ContentBrowserModule.Get().GetAssetPickerSearchBox(AssetPickerWidget.ToSharedRef());
		if (SearchBox.IsValid())
		{
			if (!Path->ContainsWidget(SearchBox.Get()))
			{
				return FReply::Unhandled().SetUserFocus(SearchBox.ToSharedRef());
			}
		}
	}

	return FReply::Unhandled();
}

void SPropertyMenuAssetPicker::OnEdit()
{
	if( CurrentObject.IsValid() )
	{
		UObject* Asset = CurrentObject.GetAsset();
		if ( Asset )
		{
			GEditor->EditObject( Asset );
		}
	}
	OnClose.ExecuteIfBound();
}

void SPropertyMenuAssetPicker::OnCopy()
{
	if( CurrentObject.IsValid() )
	{
		FPropertyEditorClipboard::ClipboardCopy(*CurrentObject.GetExportTextName());
	}
	OnClose.ExecuteIfBound();
}

void SPropertyMenuAssetPicker::OnPaste()
{
	FString DestPath;
	FPropertyEditorClipboard::ClipboardPaste(DestPath);

	OnPasteFromText(TEXT(""), DestPath, {});
}

void SPropertyMenuAssetPicker::OnPasteFromText(
	const FString& InTag,
	const FString& InText,
	const TOptional<FGuid>& InOperationId)
{
	// Naive check done elsewhere, guard with proper check here 
	FAssetData AssetData;
	if (CanPasteFromText(InTag, InText, AssetData))
	{
		PasteFromText(InTag, AssetData);
	}
}

void SPropertyMenuAssetPicker::PasteFromText(const FString& InTag, const FAssetData& InAssetData)
{
	if (!InAssetData.IsValid())
	{
		SetValue(nullptr);
	}
	else
	{
		UObject* Object = InAssetData.GetAsset();
		bool PassesAllowedClassesFilter = true;
		if (Object && AllowedClasses.Num())
		{
			PassesAllowedClassesFilter = false;
			for(int32 i = 0; i < AllowedClasses.Num(); ++i)
			{
				const bool bIsAllowedClassInterface = AllowedClasses[i]->HasAnyClassFlags(CLASS_Interface);

				if( Object->IsA(AllowedClasses[i]) || (bIsAllowedClassInterface && Object->GetClass()->ImplementsInterface(AllowedClasses[i])) )
				{
					PassesAllowedClassesFilter = true;
					break;
				}
			}
		}
		if( Object && PassesAllowedClassesFilter )
		{
			// Check against custom asset filter
			if (!OnShouldFilterAsset.IsBound()
				|| !OnShouldFilterAsset.Execute(InAssetData))
			{
				SetValue(InAssetData);
			}
		}
	}
	OnClose.ExecuteIfBound();
}

bool SPropertyMenuAssetPicker::CanPaste()
{
	FString ClipboardText;
	FPropertyEditorClipboard::ClipboardPaste(ClipboardText);
	
	FAssetData AssetData;
	return CanPasteFromText(TEXT(""), ClipboardText, AssetData);
}

bool SPropertyMenuAssetPicker::CanPasteFromText(const FString& InTag, const FString& InText, FAssetData& OutAssetData) const
{
	if (!bAllowCopyPaste)
	{
		return false;
	}

	if (!UE::PropertyEditor::TagMatchesProperty(InTag, PropertyHandle))
	{
		return false;
	}

	return SPropertyEditorAsset::ParseAssetText(InText, OutAssetData);
}

void SPropertyMenuAssetPicker::OnClear()
{
	SetValue(nullptr);
	OnClose.ExecuteIfBound();
}

void SPropertyMenuAssetPicker::OnAssetSelected( const FAssetData& AssetData )
{
	SetValue(AssetData);
	OnClose.ExecuteIfBound();
}

void SPropertyMenuAssetPicker::OnAssetEnterPressed( const TArray<FAssetData>& AssetData )
{
	if(AssetData.Num() > 0)
	{
		SetValue(AssetData[0]);
	}
	OnClose.ExecuteIfBound();
}

void SPropertyMenuAssetPicker::SetValue( const FAssetData& AssetData )
{
	OnSet.ExecuteIfBound(AssetData);
}

void SPropertyMenuAssetPicker::OnCreateNewAssetSelected(TWeakObjectPtr<UFactory> FactoryPtr)
{
	if (!FactoryPtr.IsValid())
	{
		return;
	}

	UFactory* FactoryInstance = DuplicateObject<UFactory>(FactoryPtr.Get(), GetTransientPackage());
	// Ensure this object is not GC for the duration of CreateAssetWithDialog
	TStrongObjectPtr<UFactory> FactoryStrong(FactoryInstance);
	FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();

	static const FName ContextNameMetaTag = "AssetFactoryContextName";
	FName ContextName;
	if (PropertyHandle && PropertyHandle->HasMetaData(ContextNameMetaTag))
	{
		ContextName = FName(PropertyHandle->GetMetaData(ContextNameMetaTag));
	}

	if (EnumHasAnyFlags(FactoryInstance->GetSupportedWorkflows(), EFactoryCreateWorkflow::Asynchronous))
	{
		AssetToolsModule.Get().CreateAssetWithDialogAsync(
			FactoryInstance->GetSupportedClass(), 
			FactoryInstance,
			FAssetCreateComplete::CreateSPLambda(
				this,
				[this](UFactory* Factory, UObject* InObject)
				{
					SetValue(InObject);
				}),
			FAssetCreateCancelled(),
			ContextName
		);
	}
	else if (EnumHasAnyFlags(FactoryInstance->GetSupportedWorkflows(), EFactoryCreateWorkflow::Default))
	{
		UObject* NewAsset = AssetToolsModule.Get().CreateAssetWithDialog(FactoryInstance->GetSupportedClass(), FactoryInstance, ContextName);
		if (NewAsset != nullptr)
		{
			SetValue(NewAsset);
		}
	}
}

#undef LOCTEXT_NAMESPACE
