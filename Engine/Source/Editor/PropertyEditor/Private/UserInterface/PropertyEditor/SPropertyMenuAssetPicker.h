// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "PropertyCustomizationHelpers.h"
#include "AssetRegistry/AssetData.h"

class UFactory;

class SPropertyMenuAssetPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SPropertyMenuAssetPicker )
		: _InitialObject(nullptr)
		, _AllowClear(true)
	{}
		SLATE_ARGUMENT( FAssetData, InitialObject )
		SLATE_ARGUMENT( TSharedPtr<IPropertyHandle>, PropertyHandle )
		SLATE_ARGUMENT( TArray<FAssetData>, OwnerAssetArray)
		SLATE_ARGUMENT( bool, AllowClear )
		SLATE_ARGUMENT( bool, AllowCopyPaste )
		SLATE_ARGUMENT( TArray<const UClass*>, AllowedClasses )
		SLATE_ARGUMENT( TArray<const UClass*>, DisallowedClasses )
		SLATE_ARGUMENT( TArray<UFactory*>, NewAssetFactories )
		SLATE_EVENT( FOnShouldFilterAsset, OnShouldFilterAsset )
		SLATE_EVENT( FOnAssetSelected, OnSet )
		SLATE_EVENT( FSimpleDelegate, OnClose )
	SLATE_END_ARGS()

	/**
	 * Construct the widget.
	 * @param	InArgs				Arguments for widget construction
	 * @param	InPropertyHandle	The property handle that this widget will operate on.
	 */
	void Construct( const FArguments& InArgs );

private:
	/** 
	 * Edit the object referenced by this widget
	 */
	void OnEdit();

	/** 
	 * Delegate handling ctrl+c
	 */
	void OnCopy();

	/** 
	 * Delegate handling ctrl+v
	 */
	void OnPaste();

	/**
	 * @return True if the (optionally tagged) input contents can be pasted
	 */
	bool CanPasteFromText(const FString& InTag, const FString& InText, FAssetData& OutAssetData) const;

	/**
	 * Delegate handling pasting an optionally tagged text snippet
	 */
	void OnPasteFromText(const FString& InTag, const FString& InText, const TOptional<FGuid>& InOperationId);

	/**
	 * Handle pasting an optionally tagged text snippet
	 */
	void PasteFromText(const FString& InTag, const FAssetData& InAssetData);

	/**
	 * @return True of the current clipboard contents can be pasted
	 */
	bool CanPaste();

	/** 
	 * Clear the referenced object
	 */
	void OnClear();

	/** 
	 * Delegate for handling selection in the asset browser.
	 * @param	AssetData	The chosen asset data
	 */
	void OnAssetSelected( const FAssetData& AssetData );

	/** 
	 * Delegate for handling keyboard selection in the asset browser.
	 * @param	AssetData	The chosen asset data
	 */
	void OnAssetEnterPressed( const TArray<FAssetData>& AssetData );

	/**
	 * Delegate for handling creating new assets from the menu.
	 * @param	FactoryPtr		Factory which creates asset
	 */
	void OnCreateNewAssetSelected(TWeakObjectPtr<UFactory> Factory);

	/** 
	 * Set the value of the asset referenced by this property editor.
	 * Will set the underlying property handle if there is one.
	 * @param	AssetData	The asset data for the object to set the reference to
	 */
	void SetValue( const FAssetData& AssetData );

	virtual FReply OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:
	FAssetData CurrentObject;
	
	/** The property this asset picker will modify (if any) */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	TSharedPtr<SWidget> AssetPickerWidget;

	/** Whether the asset can be 'None' in this case */
	bool bAllowClear;

	/** Whether the asset can be copied or pasted */
	bool bAllowCopyPaste;

	/** Array of classes to Allow in filter */
	TArray<const UClass*> AllowedClasses;

	/** Array of classes to Disallow in filter */
	TArray<const UClass*> DisallowedClasses;

	/** Array of factories which can create a new asset of suitable type */
	TArray<UFactory*> NewAssetFactories;

	/** Delegate for filtering valid assets */
	FOnShouldFilterAsset OnShouldFilterAsset;

	/** Delegate to call when our object value should be set */
	FOnAssetSelected OnSet;

	/** Delegate for closing the containing menu */
	FSimpleDelegate OnClose;
};
