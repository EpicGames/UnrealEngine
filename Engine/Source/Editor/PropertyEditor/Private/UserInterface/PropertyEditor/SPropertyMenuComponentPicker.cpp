// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyEditor/SPropertyMenuComponentPicker.h"
#include "UserInterface/PropertyEditor/SPropertyMenuComponentPicker.h"

#include "Modules/ModuleManager.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "GameFramework/Actor.h"
#include "AssetRegistry/AssetData.h"
#include "Editor.h"
#include "SceneOutlinerPublicTypes.h"
#include "SceneOutlinerModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "UserInterface/PropertyEditor/PropertyEditorAssetConstants.h"
#include "HAL/PlatformApplicationMisc.h"
#include "SceneOutlinerPublicTypes.h"
#include "ActorTreeItem.h"
#include "ComponentTreeItem.h"

#define LOCTEXT_NAMESPACE "PropertyEditor"

void SPropertyMenuComponentPicker::Construct(const FArguments& InArgs)
{
	InitialComponent = InArgs._InitialComponent;
	bAllowClear = InArgs._AllowClear;
	ActorFilter = InArgs._ActorFilter;
	ComponentFilter = InArgs._ComponentFilter;
	OnSet = InArgs._OnSet;
	OnClose = InArgs._OnClose;

	FMenuBuilder MenuBuilder(true, NULL);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("CurrentComponentOperationsHeader", "Current Component"));
	{
		if (InitialComponent)
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("EditComponent", "Edit"),
				LOCTEXT("EditComponent_Tooltip", "Edit this component"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SPropertyMenuComponentPicker::OnEdit)));
		}

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CopyComponent", "Copy"),
			LOCTEXT("CopyComponent_Tooltip", "Copies the component to the clipboard"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SPropertyMenuComponentPicker::OnCopy))
			);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("PasteComponent", "Paste"),
			LOCTEXT("PasteComponent_Tooltip", "Pastes an component from the clipboard to this field"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SPropertyMenuComponentPicker::OnPaste),
				FCanExecuteAction::CreateSP(this, &SPropertyMenuComponentPicker::CanPaste))
			);

		if (bAllowClear)
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("ClearComponent", "Clear"),
				LOCTEXT("ClearComponent_ToolTip", "Clears the component set on this field"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SPropertyMenuComponentPicker::OnClear))
			);
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("BrowseHeader", "Browse"));
	{
		TSharedPtr<SWidget> MenuContent;

		FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::Get().LoadModuleChecked<FSceneOutlinerModule>(TEXT("SceneOutliner"));

		FSceneOutlinerInitializationOptions InitOptions;
		InitOptions.bFocusSearchBoxWhenOpened = true;
		InitOptions.bShowTransient = InArgs._ShowTransient;

		struct FPickerFilter : public FSceneOutlinerFilter
		{
			FPickerFilter(const FOnShouldFilterActor& InActorFilter, const FOnShouldFilterComponent& InComponentFilter)
				: FSceneOutlinerFilter(FSceneOutlinerFilter::EDefaultBehaviour::Fail)
				, ActorFilter(InActorFilter)
				, ComponentFilter(InComponentFilter)
			{}

			virtual bool PassesFilter(const ISceneOutlinerTreeItem& InItem) const override
			{
				if (const FActorTreeItem* ActorItem = InItem.CastTo<FActorTreeItem>())
				{
					return ActorItem->IsValid() && ActorFilter.Execute(ActorItem->Actor.Get());
				}
				else if (const FComponentTreeItem* ComponentItem = InItem.CastTo<FComponentTreeItem>())
				{
					return ComponentItem->IsValid() && ComponentFilter.Execute(ComponentItem->Component.Get());
				}

				return DefaultBehaviour == FSceneOutlinerFilter::EDefaultBehaviour::Pass;
			}

			virtual bool GetInteractiveState(const ISceneOutlinerTreeItem& InItem) const override
			{
				// All components which pass the filter are interactive
				if (const FComponentTreeItem* ComponentItem = InItem.CastTo<FComponentTreeItem>())
				{
					return true;
				}
				return DefaultBehaviour == FSceneOutlinerFilter::EDefaultBehaviour::Pass;
			}

			FOnShouldFilterActor ActorFilter;
			FOnShouldFilterComponent ComponentFilter;
		};

		TSharedRef<FSceneOutlinerFilter> Filter = MakeShared<FPickerFilter>(ActorFilter, ComponentFilter);
		InitOptions.Filters->Add(Filter);

		InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0));

		MenuContent =
			SNew(SBox)
			.WidthOverride(static_cast<float>(PropertyEditorAssetConstants::SceneOutlinerWindowSize.X))
			.HeightOverride(static_cast<float>(PropertyEditorAssetConstants::SceneOutlinerWindowSize.Y))
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				[
					SceneOutlinerModule.CreateComponentPicker(InitOptions, FOnComponentPicked::CreateSP(this, &SPropertyMenuComponentPicker::OnItemSelected), InArgs._PickerWorld)
				]
			];

		MenuBuilder.AddWidget(MenuContent.ToSharedRef(), FText::GetEmpty(), true);

	}
	MenuBuilder.EndSection();

	ChildSlot
	[
		MenuBuilder.MakeWidget()
	];
}

void SPropertyMenuComponentPicker::OnEdit()
{
	if (InitialComponent)
	{
		GEditor->EditObject(InitialComponent);
	}
	OnClose.ExecuteIfBound();
}

void SPropertyMenuComponentPicker::OnCopy()
{
	if (InitialComponent)
	{
		FPlatformApplicationMisc::ClipboardCopy(*FString::Printf(TEXT("%s %s"), *InitialComponent->GetClass()->GetPathName(), *InitialComponent->GetPathName()));
	}
	OnClose.ExecuteIfBound();
}

void SPropertyMenuComponentPicker::OnPaste()
{
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

	bool bFound = false;

	int32 SpaceIndex = INDEX_NONE;
	if (ClipboardText.FindChar(TEXT(' '), SpaceIndex))
	{
		FString ClassPath = ClipboardText.Left(SpaceIndex);
		FString PossibleObjectPath = ClipboardText.Mid(SpaceIndex);

		if (UClass* ClassPtr = LoadClass<UActorComponent>(nullptr, *ClassPath))
		{
			UActorComponent* Component = FindObject<UActorComponent>(nullptr, *PossibleObjectPath);
			if (Component && Component->IsA(ClassPtr) && Component->GetOwner() && (!ComponentFilter.IsBound() || ComponentFilter.Execute(Component)))
			{
				if (!ActorFilter.IsBound() || ActorFilter.Execute(Component->GetOwner()))
				{
					SetValue(Component);
					bFound = true;
				}
			}
		}
	}

	if (!bFound)
	{
		SetValue(nullptr);
	}

	OnClose.ExecuteIfBound();
}

bool SPropertyMenuComponentPicker::CanPaste()
{
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

	bool bCanPaste = false;

	int32 SpaceIndex = INDEX_NONE;
	if (ClipboardText.FindChar(TEXT(' '), SpaceIndex))
	{
		FString Class = ClipboardText.Left(SpaceIndex);
		FString PossibleObjectPath = ClipboardText.Mid(SpaceIndex);

		bCanPaste = !Class.IsEmpty() && !PossibleObjectPath.IsEmpty()
					&& !Class.Contains(TEXT("//")); // LoadClass does not tolerate '//'
		if (bCanPaste)
		{
			bCanPaste = LoadClass<UActorComponent>(nullptr, *Class, nullptr, LOAD_Quiet | LOAD_NoWarn) != nullptr;
		}
		if (bCanPaste)
		{
			bCanPaste = FindObject<UActorComponent>(nullptr, *PossibleObjectPath) != nullptr;
		}
	}

	return bCanPaste;
}

void SPropertyMenuComponentPicker::OnClear()
{
	SetValue(nullptr);
	OnClose.ExecuteIfBound();
}

void SPropertyMenuComponentPicker::OnItemSelected(UActorComponent* Component)
{
	SetValue(Component);
	OnClose.ExecuteIfBound();
}

void SPropertyMenuComponentPicker::SetValue(UActorComponent* InComponent)
{
	OnSet.ExecuteIfBound(InComponent);
}

#undef LOCTEXT_NAMESPACE
