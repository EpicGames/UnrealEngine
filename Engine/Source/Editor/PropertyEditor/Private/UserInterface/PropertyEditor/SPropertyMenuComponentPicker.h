// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "PropertyCustomizationHelpers.h"

class UActorComponent;

namespace SceneOutliner
{
	struct ISceneOutlinerTreeItem;
}

class SPropertyMenuComponentPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPropertyMenuComponentPicker)
		: _InitialComponent(nullptr)
		, _AllowClear(true)
		, _ShowTransient(false)
		, _ActorFilter()
	{}
		SLATE_ARGUMENT(UActorComponent*, InitialComponent)
		SLATE_ARGUMENT(bool, AllowClear)
		SLATE_ARGUMENT(bool, ShowTransient)
		SLATE_ARGUMENT(FOnShouldFilterActor, ActorFilter)
		SLATE_ARGUMENT(FOnShouldFilterComponent, ComponentFilter)
		SLATE_EVENT(FOnComponentSelected, OnSet)
		SLATE_EVENT(FSimpleDelegate, OnClose)
		SLATE_ARGUMENT(UWorld*, PickerWorld)
	SLATE_END_ARGS()

	/**
	 * Construct the widget.
	 * @param	InArgs				Arguments for widget construction
	 * @param	InPropertyHandle	The property handle that this widget will operate on.
	 */
	void Construct(const FArguments& InArgs);

private:
	/**
	 * Edit the object referenced by this widget
	 */
	void OnEdit();

	/**
	 * Delegate handling ctrl+c
	 */
	void OnCopy();

	/**
	 * Delegate handling ctrl+v
	 */
	void OnPaste();

	/**
	 * @return True of the current clipboard contents can be pasted
	 */
	bool CanPaste();

	/**
	 * Clear the referenced object
	 */
	void OnClear();

	/**
	 * Delegate for handling selection by the actor picker.
	 * @param	Component The chosen component
	 */
	void OnItemSelected(UActorComponent* Component);

	/**
	 * Set the value of the asset referenced by this property editor.
	 * Will set the underlying property handle if there is one.
	 * @param	InObject	The object to set the reference to
	 */
	void SetValue(UActorComponent* InComponent);

private:
	UActorComponent* InitialComponent;

	/** Whether the asset can be 'None' in this case */
	bool bAllowClear;

	/** Delegate used to test whether a item should be displayed or not */
	FOnShouldFilterActor ActorFilter;
	FOnShouldFilterComponent ComponentFilter;

	/** Delegate to call when our object value should be set */
	FOnComponentSelected OnSet;

	/** Delegate for closing the containing menu */
	FSimpleDelegate OnClose;
};
