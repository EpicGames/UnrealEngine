// Copyright Epic Games, Inc. All Rights Reserved.
#include "UserInterface/PropertyEditor/SPropertySceneOutliner.h"
#include "Widgets/Layout/SBorder.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "SceneOutlinerPublicTypes.h"
#include "SceneOutlinerModule.h"

#define LOCTEXT_NAMESPACE "PropertySceneOutliner"

void SPropertySceneOutliner::Construct( const FArguments& InArgs )
{
	OnActorSelected = InArgs._OnActorSelected;
	OnGetActorFilters = InArgs._OnGetActorFilters;

	ChildSlot
	[
		SNew( SVerticalBox )
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		[
			SAssignNew( SceneOutlinerAnchor, SMenuAnchor )
			.Placement( MenuPlacement_AboveAnchor )
			.OnGetMenuContent( this, &SPropertySceneOutliner::OnGenerateSceneOutliner )
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew( SButton )
			.ButtonStyle( FAppStyle::Get(), "HoverHintOnly" )
			.OnClicked( this, &SPropertySceneOutliner::OnClicked )
			.ToolTipText(LOCTEXT("PickButtonLabel", "Pick Actor"))
			.ContentPadding(0.0f)
			.ForegroundColor( FSlateColor::UseForeground() )
			.IsFocusable(false)
			[ 
				SNew( SImage )
				.Image( FAppStyle::GetBrush("PropertyWindow.Button_PickActor") )
				.ColorAndOpacity( FSlateColor::UseForeground() )
			]
		]
	];
}
 
FReply SPropertySceneOutliner::OnClicked()
{	
	SceneOutlinerAnchor->SetIsOpen( true );
	return FReply::Handled();
}

TSharedRef<SWidget> SPropertySceneOutliner::OnGenerateSceneOutliner()
{
	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::Get().LoadModuleChecked<FSceneOutlinerModule>(TEXT("SceneOutliner"));

	FSceneOutlinerInitializationOptions InitOptions;
	OnGetActorFilters.ExecuteIfBound( InitOptions.Filters );

	TSharedRef<SWidget> MenuContent = 
		SNew(SBox)
		.HeightOverride(300.0f)
		.WidthOverride(300.0f)
		[
			SNew( SBorder )
			.BorderImage( FAppStyle::GetBrush("Menu.Background") )
			[
				SceneOutlinerModule.CreateActorPicker(InitOptions, FOnActorPicked::CreateSP(this, &SPropertySceneOutliner::OnActorSelectedFromOutliner))
			]
		];

	return MenuContent;
}

void SPropertySceneOutliner::OnActorSelectedFromOutliner( AActor* InActor )
{
	// Close the scene outliner
	SceneOutlinerAnchor->SetIsOpen( false );

	OnActorSelected.ExecuteIfBound( InActor );
}

#undef LOCTEXT_NAMESPACE
