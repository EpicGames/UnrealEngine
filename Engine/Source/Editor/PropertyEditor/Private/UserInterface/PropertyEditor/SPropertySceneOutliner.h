// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SMenuAnchor.h"

class AActor;

class SPropertySceneOutliner : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SPropertySceneOutliner ) {}
		SLATE_EVENT( FOnGetActorFilters, OnGetActorFilters )
		SLATE_EVENT( FOnActorSelected, OnActorSelected )
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs );
private:

	FReply OnClicked();

	TSharedRef<SWidget> OnGenerateSceneOutliner();

	void OnActorSelectedFromOutliner( AActor* InActor );

private:
	/** Menu anchor for opening and closing the scene outliner */
	TSharedPtr< class SMenuAnchor > SceneOutlinerAnchor;

	FOnGetActorFilters OnGetActorFilters;
	FOnActorSelected OnActorSelected;
};
