// Copyright Epic Games, Inc. All Rights Reserved.
#include "SResetToDefaultPropertyEditor.h"

#include "Containers/UnrealString.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

struct FGeometry;

#define LOCTEXT_NAMESPACE "ResetToDefaultPropertyEditor"

SResetToDefaultPropertyEditor::~SResetToDefaultPropertyEditor()
{
	if (PropertyHandle.IsValid() && OptionalCustomResetToDefault.IsSet())
	{
		PropertyHandle->ClearResetToDefaultCustomized();
	}
}

void SResetToDefaultPropertyEditor::Construct(const FArguments& InArgs, const TSharedPtr<IPropertyHandle>& InPropertyHandle)
{
	PropertyHandle = InPropertyHandle;
	NonVisibleState = InArgs._NonVisibleState;
	bValueDiffersFromDefault = false;
	OptionalCustomResetToDefault = InArgs._CustomResetToDefault;

	if (InPropertyHandle.IsValid() && OptionalCustomResetToDefault.IsSet())
	{
		InPropertyHandle->MarkResetToDefaultCustomized();
	}

	// Indicator for a value that differs from default. Also offers the option to reset to default.
	ChildSlot
	[
		SNew(SButton)
		.IsFocusable(false)
		.ToolTipText(this, &SResetToDefaultPropertyEditor::GetResetToolTip)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.ContentPadding(0.0f)
		.Visibility(this, &SResetToDefaultPropertyEditor::GetDiffersFromDefaultAsVisibility )
		.OnClicked(this, &SResetToDefaultPropertyEditor::OnResetClicked)
		.Content()
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
	];

	UpdateDiffersFromDefaultState();
}

void SResetToDefaultPropertyEditor::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	UpdateDiffersFromDefaultState();
}

FText SResetToDefaultPropertyEditor::GetResetToolTip() const
{
	FString Tooltip;
	Tooltip = LOCTEXT("ResetToDefaultToolTip", "Reset to Default").ToString();

	if( PropertyHandle.IsValid() && !PropertyHandle->IsEditConst() && PropertyHandle->DiffersFromDefault() )
	{
		FString DefaultLabel = PropertyHandle->GetResetToDefaultLabel().ToString();

		if (DefaultLabel.Len() > 0)
		{
			Tooltip += "\n";
			Tooltip += DefaultLabel;
		}
	}

	return FText::FromString(Tooltip);
}

FReply SResetToDefaultPropertyEditor::OnResetClicked()
{
	if (OptionalCustomResetToDefault.IsSet())
	{
		OptionalCustomResetToDefault.GetValue().OnResetToDefaultClicked(PropertyHandle);
	}
	else if (PropertyHandle.IsValid())
	{
		PropertyHandle->ResetToDefault();
	}

	return FReply::Handled();
}

void SResetToDefaultPropertyEditor::UpdateDiffersFromDefaultState()
{
	if (OptionalCustomResetToDefault.IsSet())
	{
		bValueDiffersFromDefault = OptionalCustomResetToDefault.GetValue().IsResetToDefaultVisible(PropertyHandle);
	}
	else if (PropertyHandle.IsValid())
	{
		bValueDiffersFromDefault = PropertyHandle->CanResetToDefault();
	}
}

EVisibility SResetToDefaultPropertyEditor::GetDiffersFromDefaultAsVisibility() const
{
	return bValueDiffersFromDefault ? EVisibility::Visible : NonVisibleState;
}

#undef LOCTEXT_NAMESPACE
