// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyTable/BooleanPropertyTableCellPresenter.h"

#include "HAL/Platform.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "PropertyHandle.h"
#include "SResetToDefaultPropertyEditor.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "UserInterface/PropertyEditor/SPropertyEditorBool.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"

class SWidget;

FBooleanPropertyTableCellPresenter::FBooleanPropertyTableCellPresenter( const TSharedRef< class FPropertyEditor >& InPropertyEditor )
	: PropertyEditor( InPropertyEditor )
{
}

TSharedRef< class SWidget > FBooleanPropertyTableCellPresenter::ConstructDisplayWidget()
{
	TSharedPtr<SHorizontalBox> HorizontalBox;

	TSharedRef<SBorder> Border = SNew( SBorder )
		.Padding( 0 )
		.VAlign( VAlign_Center )
		.HAlign( HAlign_Center )
		.BorderImage( FAppStyle::GetBrush("NoBorder") )
		.Content()
		[	
			SAssignNew( HorizontalBox, SHorizontalBox )
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign( VAlign_Center )
			.HAlign( HAlign_Center )
			.Padding( FMargin( 2, 0, 2, 0 ) )
			[
				SNew( SBorder )
				.BorderImage( FAppStyle::GetBrush( "PropertyTable.ContentBorder" ) )
				.Padding( 0 )
				.Content()
				[
					SAssignNew( FocusWidget, SPropertyEditorBool, PropertyEditor )
					.ToolTipText( PropertyEditor->GetToolTipText() )
				]
			]
		];

	if (!PropertyEditor->GetPropertyHandle()->HasMetaData(TEXT("NoResetToDefault")))
	{
		HorizontalBox->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.Padding(FMargin(0, 0, 2, 0))
		[
			SNew(SResetToDefaultPropertyEditor, PropertyEditor->GetPropertyHandle())
		];
	}

	return Border;
}

bool FBooleanPropertyTableCellPresenter::RequiresDropDown()
{
	return false;
}

TSharedRef< class SWidget > FBooleanPropertyTableCellPresenter::ConstructEditModeDropDownWidget()
{
	return FocusWidget.ToSharedRef();
}

TSharedRef<SWidget> FBooleanPropertyTableCellPresenter::ConstructEditModeCellWidget()
{
	return ConstructDisplayWidget();
}

TSharedRef< class SWidget > FBooleanPropertyTableCellPresenter::WidgetToFocusOnEdit()
{
	return FocusWidget.ToSharedRef();
}

FString FBooleanPropertyTableCellPresenter::GetValueAsString()
{
	return PropertyEditor->GetValueAsString();
}

FText FBooleanPropertyTableCellPresenter::GetValueAsText()
{
	return PropertyEditor->GetValueAsText();
}
