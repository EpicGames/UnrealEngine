// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyTable/ColorPropertyTableCellPresenter.h"

#include "HAL/Platform.h"
#include "IPropertyTableUtilities.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "PropertyHandle.h"
#include "SResetToDefaultPropertyEditor.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "UserInterface/PropertyEditor/SPropertyEditorColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"

class SWidget;

FColorPropertyTableCellPresenter::FColorPropertyTableCellPresenter( const TSharedRef< class FPropertyEditor >& InPropertyEditor, const TSharedRef< class IPropertyTableUtilities >& InPropertyUtilities )
	: PropertyEditor( InPropertyEditor )
	, PropertyUtilities( InPropertyUtilities )
{
}

TSharedRef< class SWidget > FColorPropertyTableCellPresenter::ConstructDisplayWidget()
{
	TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding( FMargin( 3.0, 0, 3.0, 0 ) )
		.FillWidth( 1.0 )
		.VAlign( VAlign_Center )
		[
			SNew( SBorder )
			.BorderImage( FAppStyle::GetBrush( "PropertyTable.CellEditing.Background" ) )
			.Padding( 1 )
			.Content()
			[
				SAssignNew( FocusWidget, SPropertyEditorColor, PropertyEditor, PropertyUtilities )
				.ToolTipText( PropertyEditor->GetToolTipText() )
			]
		];
		
	if (!PropertyEditor->GetPropertyHandle()->HasMetaData(TEXT("NoResetToDefault")))
	{
		HorizontalBox->AddSlot()
		.AutoWidth()
		.VAlign( VAlign_Center )
		.HAlign( HAlign_Center )
		.Padding( FMargin( 0, 0, 2, 0 ) )
		[
			SNew( SResetToDefaultPropertyEditor, PropertyEditor->GetPropertyHandle() )
		];
	}

	return HorizontalBox;
}

bool FColorPropertyTableCellPresenter::RequiresDropDown()
{
	return false;
}

TSharedRef< class SWidget > FColorPropertyTableCellPresenter::ConstructEditModeDropDownWidget()
{
	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> FColorPropertyTableCellPresenter::ConstructEditModeCellWidget()
{
	return ConstructDisplayWidget();
}

TSharedRef< class SWidget > FColorPropertyTableCellPresenter::WidgetToFocusOnEdit()
{
	return FocusWidget.ToSharedRef();
}

FString FColorPropertyTableCellPresenter::GetValueAsString()
{
	return PropertyEditor->GetValueAsString();
}

FText FColorPropertyTableCellPresenter::GetValueAsText()
{
	return PropertyEditor->GetValueAsText();
}
