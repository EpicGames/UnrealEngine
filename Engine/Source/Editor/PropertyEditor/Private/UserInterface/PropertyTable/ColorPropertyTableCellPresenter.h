// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "IPropertyTableCellPresenter.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

class FColorPropertyTableCellPresenter : public TSharedFromThis< FColorPropertyTableCellPresenter >, public IPropertyTableCellPresenter
{
public:

	FColorPropertyTableCellPresenter( const TSharedRef< class FPropertyEditor >& InPropertyEditor, const TSharedRef< class IPropertyTableUtilities >& InPropertyUtilities );

	virtual ~FColorPropertyTableCellPresenter() = default;

	virtual TSharedRef< class SWidget > ConstructDisplayWidget() override;

	virtual bool RequiresDropDown() override;

	virtual TSharedRef< class SWidget > ConstructEditModeCellWidget() override;

	virtual TSharedRef< class SWidget > ConstructEditModeDropDownWidget() override;

	virtual TSharedRef< class SWidget > WidgetToFocusOnEdit() override;

	virtual FString GetValueAsString() override;

	virtual FText GetValueAsText() override;

	virtual bool HasReadOnlyEditMode() override { return false; }


private:

	TSharedPtr< class SWidget > FocusWidget;
	TSharedRef< class FPropertyEditor > PropertyEditor;
	TSharedRef< class IPropertyTableUtilities > PropertyUtilities;
};
