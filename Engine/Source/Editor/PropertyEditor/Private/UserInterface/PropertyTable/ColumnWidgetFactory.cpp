// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyTable/ColumnWidgetFactory.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "UObject/UnrealType.h"
#include "IPropertyTableColumn.h"
#include "IPropertyTable.h"
#include "UserInterface/PropertyTable/SColumnHeader.h"


#include "UserInterface/PropertyTable/SRowHeaderColumnHeader.h"
#include "UserInterface/PropertyTable/STextColumnHeader.h"
#include "UserInterface/PropertyTable/SBoolColumnHeader.h"
#include "UserInterface/PropertyTable/SColorColumnHeader.h"
#include "UserInterface/PropertyTable/SObjectColumnHeader.h"
#include "UserInterface/PropertyTable/SObjectNameColumnHeader.h"
#include "UserInterface/PropertyTable/SPropertyNameColumnHeader.h"

bool ColumnWidgetFactory::Supports( const TSharedRef < IPropertyTableColumn >& Column )
{
	return true;
}

TSharedRef< SColumnHeader > ColumnWidgetFactory::CreateColumnHeaderWidget( const TSharedRef < IPropertyTableColumn >& Column, const TSharedRef< class IPropertyTableUtilities >& Utilities, const TSharedPtr< IPropertyTableCustomColumn >& Customization, const FName& Style )
{
	const TSharedRef< IDataSource > ColumnDataSource = Column->GetDataSource();

	if ( TEXT("ObjectName") == Column->GetId() )
	{
		return SNew( SObjectNameColumnHeader, Column, Utilities )
			.Style( Style )
			.Customization( Customization );
	}

	if ( TEXT("RowHeader") == Column->GetId() )
	{
		return SNew( SRowHeaderColumnHeader, Column, Utilities )
			.Style( Style );
	}

	if( TEXT("PropertyName") == Column->GetId() )
	{
		return SNew( SPropertyNameColumnHeader, Column, Utilities )
			.Style( Style )
			.Customization( Customization );
	}

	//@todo Make this not suck [11/27/2012 Justin.Sargent]
	const TWeakObjectPtr< UObject > Object = ColumnDataSource->AsUObject();
	const TSharedPtr< FPropertyPath > Path = ColumnDataSource->AsPropertyPath();
	if( Path.IsValid() )
	{
		TWeakFieldPtr< const FProperty > Property = Path->GetLeafMostProperty().Property;
		check( Property.IsValid() );

		if ( Property->IsA( FBoolProperty::StaticClass() ) )
		{
			return SNew( SBoolColumnHeader, Column, Utilities )
				.Style( Style )
				.Customization( Customization );
		}
	
		//@todo Move [10/24/2012 Justin.Sargent]
		if ( CastField<const FStructProperty>(Property.Get()) && (CastField<const FStructProperty>(Property.Get())->Struct->GetFName()==NAME_Color || CastField<const FStructProperty>(Property.Get())->Struct->GetFName()==NAME_LinearColor) )
		{
			return SNew( SColorColumnHeader, Column, Utilities )
				.Style( Style )
				.Customization( Customization );
		}
	}
	else if( Object.IsValid() )
	{
		return SNew( SObjectColumnHeader, Column, Utilities )
			.Style( Style )
			.Customization( Customization );
	}

	return SNew( STextColumnHeader, Column, Utilities )
		.Style( Style )
		.Customization( Customization );
}

