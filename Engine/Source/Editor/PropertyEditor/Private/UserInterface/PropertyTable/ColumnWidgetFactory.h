// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class IPropertyTableCustomColumn;

class ColumnWidgetFactory : public TSharedFromThis< ColumnWidgetFactory >	
{
public:

	bool Supports( const TSharedRef < class IPropertyTableColumn >& Column );

	TSharedRef< class SColumnHeader > CreateColumnHeaderWidget( const TSharedRef < class IPropertyTableColumn >& Column, const TSharedRef< class IPropertyTableUtilities >& Utilities, const TSharedPtr< IPropertyTableCustomColumn >& Customization, const FName& Style );

};
