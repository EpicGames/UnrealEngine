// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IPropertyTable.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/Margin.h"
#include "Styling/SlateColor.h"
#include "Widgets/SWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SToolTip.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "IPropertyTableCell.h"
#include "Widgets/Layout/SBox.h"
#include "IPropertyTableCellPresenter.h"
#include "UserInterface/PropertyTable/PropertyTableConstants.h"
#include "Widgets/Input/SEditableTextBox.h"

class FObjectNameTableCellPresenter : public TSharedFromThis< FObjectNameTableCellPresenter >, public IPropertyTableCellPresenter
{
public:

	FObjectNameTableCellPresenter( const TSharedRef< IPropertyTableCell >& InCell )
		: FocusWidget( SNullWidget::NullWidget )
		, Cell( InCell )
	{

	}

	virtual ~FObjectNameTableCellPresenter() = default;

	virtual TSharedRef< class SWidget > ConstructDisplayWidget() override
	{
		return SNew( SBox )
			.VAlign( VAlign_Center)
			.Padding( FMargin( 4, 0, 2, 0 ) )
			[
				ConstructNameWidget( PropertyTableConstants::NormalFontStyle )
			]
			.ToolTip
			(
				SNew( SToolTip )[ ConstructNameWidget( PropertyTableConstants::NormalFontStyle ) ]
			);
	}

	virtual bool RequiresDropDown() override { return false; }

	virtual TSharedRef< class SWidget > ConstructEditModeCellWidget() override 
	{
		TSharedPtr< SEditableTextBox > NewFocusWidget;
		TSharedRef< class SWidget > Result = 
			SNew( SBox )
			.VAlign( VAlign_Center)
			.Padding( FMargin( 2, 0, 2, 0 )  )
			[
				SAssignNew( NewFocusWidget, SEditableTextBox )
				.Text( Cell, &IPropertyTableCell::GetValueAsText )
				.Font( FAppStyle::GetFontStyle( PropertyTableConstants::NormalFontStyle ) )
				.IsReadOnly( true )
			];

		FocusWidget = NewFocusWidget;

		return Result;
	}

	virtual TSharedRef< class SWidget > ConstructEditModeDropDownWidget() override { return SNullWidget::NullWidget; }

	virtual TSharedRef< class SWidget > WidgetToFocusOnEdit() override { return FocusWidget.Pin().ToSharedRef(); }

	virtual FString GetValueAsString() override { return Cell->GetValueAsString(); }

	virtual FText GetValueAsText() override { return Cell->GetValueAsText(); }

	virtual bool HasReadOnlyEditMode() override { return true; }


private:

	TSharedRef< SWidget > ConstructNameWidget( const FName& TextFontStyle )
	{
		TSharedRef< SHorizontalBox > NameBox = SNew( SHorizontalBox );
		const FString& DisplayNameText = Cell->GetValueAsString();

		TArray< FString > DisplayNamePieces;
		DisplayNameText.ParseIntoArray(DisplayNamePieces, TEXT("->"), true);

		for (int Index = 0; Index < DisplayNamePieces.Num(); Index++)
		{
			NameBox->AddSlot()
				.AutoWidth()
				[
					SNew( STextBlock )
					.Font( FAppStyle::GetFontStyle( TextFontStyle ) )
					.Text( FText::FromString(DisplayNamePieces[ Index ]) )
					.ColorAndOpacity(this, &FObjectNameTableCellPresenter::GetTextForegroundColor)
				];

			if ( Index < DisplayNamePieces.Num() - 1 )
			{
				NameBox->AddSlot()
					.AutoWidth()
					.HAlign( HAlign_Center )
					.VAlign( VAlign_Center )
					[
						SNew( SImage )
						.ColorAndOpacity( FSlateColor::UseForeground() )
						.Image( FAppStyle::GetBrush( "PropertyTable.HeaderRow.Column.PathDelimiter" ) )
					];
			}
		}

		return NameBox;
	}

	FSlateColor GetTextForegroundColor() const
	{
		if(Cell->GetTable()->GetSelectedCells().Contains( Cell ))
		{
			return FStyleColors::White;
		}

		return FSlateColor::UseForeground();
	}

private:

	TWeakPtr< class SWidget > FocusWidget;
	TSharedRef< IPropertyTableCell > Cell;
};
