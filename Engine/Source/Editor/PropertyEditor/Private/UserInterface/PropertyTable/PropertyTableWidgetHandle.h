// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "IPropertyTableWidgetHandle.h"
#include "UserInterface/PropertyTable/SPropertyTable.h"

class FPropertyTableWidgetHandle: public IPropertyTableWidgetHandle{
public:
	FPropertyTableWidgetHandle(const TSharedRef<SPropertyTable>& InPropertyTableWidget)
		: PropertyTableWidget(InPropertyTableWidget) {}

	virtual ~FPropertyTableWidgetHandle() = default;

	virtual void RequestRefresh() override
	{
		PropertyTableWidget->RequestRefresh();
	} 

	virtual TSharedRef<SWidget> GetWidget() override
	{
		return PropertyTableWidget;
	}

private:
	TSharedRef<SPropertyTable> PropertyTableWidget;
};
