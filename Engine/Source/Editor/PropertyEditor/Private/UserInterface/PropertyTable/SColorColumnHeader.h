// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "UserInterface/PropertyTable/SColumnHeader.h"
#include "IPropertyTableCell.h"
#include "UserInterface/PropertyTable/SPropertyTableCell.h"
#include "UserInterface/PropertyTable/ColorPropertyTableCellPresenter.h"

class SColorColumnHeader : public SColumnHeader
{
	public:

	SLATE_BEGIN_ARGS( SColorColumnHeader )
		: _Style( TEXT("PropertyTable") )
		, _Customization()
	{}
		SLATE_ARGUMENT( FName, Style )
		SLATE_ARGUMENT( TSharedPtr< IPropertyTableCustomColumn >, Customization )
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const TSharedRef< class IPropertyTableColumn >& InPropertyTableColumn, const TSharedRef< class IPropertyTableUtilities >& InPropertyUtilities )
	{
		Style = InArgs._Style;

		SColumnHeader::FArguments ColumnArgs;
		ColumnArgs.Style( Style );
		ColumnArgs.Customization( InArgs._Customization );

		SColumnHeader::Construct( ColumnArgs, InPropertyTableColumn, InPropertyUtilities );
	}

	virtual TSharedRef< SWidget > GenerateCell( const TSharedRef< class IPropertyTableRow >& PropertyTableRow ) override
	{
		TSharedRef< IPropertyTableCell > Cell = Column->GetCell( PropertyTableRow );

		TSharedPtr< IPropertyTableCellPresenter > CellPresenter( NULL );

		if ( Customization.IsValid() )
		{
			CellPresenter = Customization->CreateCellPresenter( Cell, Utilities.ToSharedRef(), Style );
		}

		if ( !CellPresenter.IsValid() && Cell->IsBound() )
		{
			TSharedRef< FPropertyEditor > PropertyEditor = FPropertyEditor::Create( Cell->GetNode().ToSharedRef(), Utilities.ToSharedRef() );
			CellPresenter = MakeShareable( new FColorPropertyTableCellPresenter( PropertyEditor, Utilities.ToSharedRef() ) );
		}

		return SNew( SPropertyTableCell, Cell )
			.Presenter( CellPresenter )
			.Style( Style );
	}


private:

	FName Style;
};
