// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/Margin.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "IPropertyTable.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SToolTip.h"
#include "Styling/AppStyle.h"
#include "IPropertyTableCustomColumn.h"
#include "PropertyEditorHelpers.h"
#include "IDocumentation.h"

#define LOCTEXT_NAMESPACE "SColumnHeader"


class SColumnHeader : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SColumnHeader ) 
		: _Style( TEXT("PropertyTable") )
		, _Customization()
	{}
		SLATE_ARGUMENT( FName, Style )
		SLATE_ARGUMENT( TSharedPtr< IPropertyTableCustomColumn >, Customization )
	SLATE_END_ARGS()

	virtual TSharedRef< class SWidget > GenerateCell( const TSharedRef< class IPropertyTableRow >& Row ) = 0;


protected:

	void Construct( const FArguments& InArgs, const TSharedRef< class IPropertyTableColumn >& InPropertyTableColumn, const TSharedRef< class IPropertyTableUtilities >& InUtilities )
	{
		Customization = InArgs._Customization;
		Utilities = InUtilities;
		Column = InPropertyTableColumn;

		TSharedPtr< SWidget > ColumnLabel;
		if ( Customization.IsValid() )
		{
			ColumnLabel = Customization->CreateColumnLabel( InPropertyTableColumn, InUtilities, InArgs._Style );
		}

		if ( !ColumnLabel.IsValid() )
		{
			ColumnLabel = ConstructNameWidget( InArgs._Style, "NormalFont" );
		}

		ChildSlot
		.Padding( FMargin( 0, 2, 0, 2 ) )
		.VAlign( VAlign_Center )
		[
			ColumnLabel.ToSharedRef()
		];
		
		TSharedPtr< SToolTip > ColumnHeaderToolTip;

		const TWeakObjectPtr< UObject > Object = Column->GetDataSource()->AsUObject();
		const TSharedPtr< FPropertyPath > PropertyPath = Column->GetDataSource()->AsPropertyPath();

		if( Object.IsValid() )
		{
			ColumnHeaderToolTip = SNew( SToolTip ).Text( FText::FromString(Object->GetName()) );
		}
		else if ( PropertyPath.IsValid() && PropertyPath->GetNumProperties() > 0 )
		{
			const FProperty* Property = PropertyPath->GetLeafMostProperty().Property.Get();
			const FText& DisplayNameText = Column->GetDisplayName();
			const FString DocumentationLink = PropertyEditorHelpers::GetDocumentationLink( Property );
			const FString DocumentationExcerptName = PropertyEditorHelpers::GetDocumentationExcerptName( Property );
			ColumnHeaderToolTip = IDocumentation::Get()->CreateToolTip( DisplayNameText, NULL, DocumentationLink, DocumentationExcerptName );
		}

		this->SetToolTip( ColumnHeaderToolTip );
	}

	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		if ( !Column->IsFrozen() && MouseEvent.IsMouseButtonDown( EKeys::MiddleMouseButton ) )
		{
			Utilities->RemoveColumn( Column.ToSharedRef() );
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}


protected:

	TSharedRef< SWidget > ConstructNameWidget( const FName& Style, const FName& TextFontStyle )
	{
		TSharedRef< SHorizontalBox > NameBox = SNew( SHorizontalBox );
		const FText& DisplayNameText = Column->GetDisplayName();

		TArray< FString > DisplayNamePieces;
		DisplayNameText.ToString().ParseIntoArray(DisplayNamePieces, TEXT("->"), true);

		if(!DisplayNamePieces.IsEmpty())
		{
			// We only show the property name in the column header, the full path in the tooltip
			NameBox->AddSlot()
			.AutoWidth()
			[
				SNew( STextBlock )
				.Font( FAppStyle::GetFontStyle( TextFontStyle ) )
				.Text( FText::FromString(DisplayNamePieces[ DisplayNamePieces.Num() - 1 ]) )
			];
		}
		
		return NameBox;
	}


protected:

	TSharedPtr< IPropertyTableUtilities > Utilities;

	TSharedPtr< IPropertyTableColumn > Column;

	TSharedPtr< IPropertyTableCustomColumn > Customization;
};

#undef LOCTEXT_NAMESPACE

