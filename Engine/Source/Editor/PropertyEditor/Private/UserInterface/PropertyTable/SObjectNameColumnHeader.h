// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "UserInterface/PropertyTable/SColumnHeader.h"
#include "IPropertyTableCell.h"
#include "UserInterface/PropertyTable/SPropertyTableCell.h"
#include "UserInterface/PropertyTable/ObjectNameTableCellPresenter.h"

class SObjectNameColumnHeader : public SColumnHeader
{
	public:

	SLATE_BEGIN_ARGS( SObjectNameColumnHeader ) 
		: _Style( TEXT("PropertyTable") )
		, _Customization()
	{}
		SLATE_ARGUMENT( FName, Style )
		SLATE_ARGUMENT( TSharedPtr< IPropertyTableCustomColumn >, Customization )
	SLATE_END_ARGS()		


	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InViewModel			The UI logic not specific to slate
	 */
	void Construct( const FArguments& InArgs, const TSharedRef< class IPropertyTableColumn >& InPropertyTableColumn, const TSharedRef< class IPropertyTableUtilities >& InPropertyUtilities )
	{
		Style = InArgs._Style;

		SColumnHeader::FArguments ColumnArgs;
		ColumnArgs.Style( Style );
		ColumnArgs.Customization( InArgs._Customization );

		SColumnHeader::Construct( ColumnArgs, InPropertyTableColumn, InPropertyUtilities );
	}

	virtual TSharedRef< SWidget > GenerateCell( const TSharedRef< class IPropertyTableRow >& PropertyTableRow ) override
	{
		TSharedRef< IPropertyTableCell > Cell = Column->GetCell( PropertyTableRow );

		TSharedPtr< IPropertyTableCellPresenter > CellPresenter( NULL );

		if ( Customization.IsValid() )
		{
			CellPresenter = Customization->CreateCellPresenter( Cell, Utilities.ToSharedRef(), Style );
		}

		if ( !CellPresenter.IsValid() && Cell->IsBound() )
		{
			CellPresenter = MakeShareable( new FObjectNameTableCellPresenter( Cell ) );
		}

		return SNew( SPropertyTableCell, Cell )
			.Presenter( CellPresenter )
			.Style( Style );
	}


private:

	FName Style;
};
