// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/Margin.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "UserInterface/PropertyTable/SColumnHeader.h"
#include "Widgets/Layout/SSpacer.h"
#include "IPropertyTableRow.h"
#include "Editor/EditorEngine.h"

class SPropertyNameColumnHeader : public SColumnHeader
{
	public:

	SLATE_BEGIN_ARGS( SPropertyNameColumnHeader ) 
		: _Style( TEXT("PropertyTable") )
		, _Customization()
	{}
		SLATE_ARGUMENT( FName, Style )
		SLATE_ARGUMENT( TSharedPtr< IPropertyTableCustomColumn >, Customization )
	SLATE_END_ARGS()

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InViewModel			The UI logic not specific to slate
	 */
	void Construct( const FArguments& InArgs, const TSharedRef< class IPropertyTableColumn >& InPropertyTableColumn, const TSharedRef< class IPropertyTableUtilities >& InPropertyUtilities )
	{
		Style = InArgs._Style;

		SColumnHeader::FArguments ColumnArgs;
		ColumnArgs.Style( Style );
		ColumnArgs.Customization( InArgs._Customization );

		SColumnHeader::Construct( ColumnArgs, InPropertyTableColumn, InPropertyUtilities );
		ChildSlot
			[
				SNew( SSpacer )
			];
	}

	virtual TSharedRef< SWidget > GenerateCell( const TSharedRef< class IPropertyTableRow >& PropertyTableRow ) override
	{
		FString PropertyName;
			
		if( PropertyTableRow->GetDataSource()->AsPropertyPath().IsValid() )
		{	
			const TWeakFieldPtr< const FProperty > Property = PropertyTableRow->GetDataSource()->AsPropertyPath()->GetLeafMostProperty().Property;
			PropertyName = UEditorEngine::GetFriendlyName( Property.Get() );
		}

		return SNew( SHorizontalBox )
			+SHorizontalBox::Slot()
			[
				SNew( SBorder )
				.BorderImage( FAppStyle::GetBrush( "PropertyTable.HeaderRow.Background" ) )
				.VAlign( VAlign_Center)
				.Padding( FMargin( 2, 0, 2, 0 ) )
				[
					SNew( STextBlock )
					.Text( FText::FromString(PropertyName) )
					.ColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f))
				]
			];
				
	}

	/** Called when the column title has been clicked to change sorting mode */
	FReply OnTitleClicked()
	{
		return FReply::Handled();
	}


private:

	FName Style;
};
