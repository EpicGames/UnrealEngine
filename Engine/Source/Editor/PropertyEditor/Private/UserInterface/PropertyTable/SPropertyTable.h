// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Textures/SlateIcon.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "PropertyPath.h"
#include "PropertyPermissionList.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "IPropertyTableWidgetHandle.h"
#include "IPropertyTable.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Editor/EditorEngine.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "UserInterface/PropertyTable/SPropertyTableHeaderRow.h"
#include "IPropertyTableCell.h"
#include "IPropertyTableRow.h"
#include "UserInterface/PropertyTable/SPropertyTableRow.h"
#include "HAL/PlatformApplicationMisc.h"

#define LOCTEXT_NAMESPACE "PropertyTable"

class SPropertyTable : public STreeView< TSharedRef< IPropertyTableRow > > , public IPropertyTableWidgetHandle
{
public:

	SLATE_BEGIN_ARGS( SPropertyTable )
		: _Style( "PropertyTable" )
		, _ColumnCustomizations()
	{}
		SLATE_ARGUMENT( FName, Style )
		SLATE_ARGUMENT( TArray< TSharedRef< class IPropertyTableCustomColumn > >, ColumnCustomizations )
	SLATE_END_ARGS()

	SPropertyTable()
		: bUpdatingSelection( false )
	{

	}

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InViewModel			The UI logic not specific to slate
	 */
	void Construct( const FArguments& InArgs, const TSharedRef< class IPropertyTable >& InPropertyTable )
	{
		Table = InPropertyTable;
		Style = InArgs._Style;

		STreeView< TSharedRef< IPropertyTableRow > >::FArguments TreeViewArgs;
		TreeViewArgs
			.SelectionMode( Table.ToSharedRef(), &IPropertyTable::GetSelectionMode )
			.OnSelectionChanged( this, &SPropertyTable::OnSelectionChanged )
			.TreeItemsSource( &Table->GetRows() )
			.OnGenerateRow( this, &SPropertyTable::GenerateRow ) 
			.OnGetChildren( this, &SPropertyTable::OnGetChildren )
			.HeaderRow( SAssignNew( HeaderRow, SPropertyTableHeaderRow, InPropertyTable ).Style( Style ).Customizations( InArgs._ColumnCustomizations ) )
			.ItemHeight_Raw( Table.Get(), &IPropertyTable::GetItemHeight )
			.ClearSelectionOnClick( false );

		STreeView< TSharedRef< IPropertyTableRow > >::Construct( TreeViewArgs );

		Table->OnSelectionChanged()->AddSP( this, &SPropertyTable::UpdateSelection );
		Table->OnRowsChanged()->AddSP( this, &STreeView< TSharedRef< IPropertyTableRow > >::RequestTreeRefresh );
		Table->OnRootPathChanged()->AddSP( this, &SPropertyTable::SyncBreadcrumbTrail );

		if ( Table->GetIsUserAllowedToChangeRoot() )
		{
			TSharedRef< SWidget > TreeContent = ChildSlot.GetWidget();

			ChildSlot
			[
				SNew( SVerticalBox )
				+SVerticalBox::Slot()
				.Padding(0.0f, 0.0f, 0.0f, 1.0f)
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage( FAppStyle::GetBrush( "Brushes.Header" ) )
					[
						SAssignNew( BreadcrumbTrail, SBreadcrumbTrail< int32 > )
						.PersistentBreadcrumbs( true )
						.DelimiterImage(FAppStyle::Get().GetBrush("Icons.ChevronRight"))
						.OnCrumbClicked( this, &SPropertyTable::OnCrumbClicked )
						.GetCrumbMenuContent( this, &SPropertyTable::GetCrumbMenuContent )
					]
				]

				+SVerticalBox::Slot()
				.FillHeight( 1.0f )
				[
					SNew(SBorder)
					.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
					[
						TreeContent
					]
				]
			];
		}

		SyncBreadcrumbTrail();

		const TSet< TSharedRef< IPropertyTableRow > > SelectedRows = Table->GetSelectedRows();
		if ( SelectedRows.Num() > 0 )
		{
			for( auto RowIter = SelectedRows.CreateConstIterator(); RowIter; ++RowIter )
			{
				Private_SetItemSelection( *RowIter, true );
			}

			Private_SignalSelectionChanged( ESelectInfo::Direct );
		}
	}

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override
	{
		Table->Tick();
		STreeView< TSharedRef< IPropertyTableRow > >::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	}

	virtual void RequestRefresh() override
	{
		WidgetGenerator.Clear();
		RequestTreeRefresh();
	}

	virtual TSharedRef<SWidget> GetWidget() override
	{
		return SharedThis(this);
	}

	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
	{
		FReply Reply = FReply::Unhandled();

		if ( Table->GetSelectionMode() == ESelectionMode::None )
		{
			return Reply;
		}

		const FKey Key = InKeyEvent.GetKey();
		const EPropertyTableSelectionUnit::Type SelectionUnit = Table->GetSelectionUnit();
			
		const TSharedPtr< IPropertyTableCell > CurrentCell = Table->GetCurrentCell();
		const TSharedPtr< IPropertyTableColumn > CurrentColumn = Table->GetCurrentColumn();
		const TSharedPtr< IPropertyTableRow > CurrentRow = Table->GetCurrentRow();

		const TSharedPtr< IPropertyTableCell > FirstCellInSelection = Table->GetFirstCellInSelection();
		const TSharedPtr< IPropertyTableCell > LastCellInSelection = Table->GetLastCellInSelection();

		if ( Key == EKeys::Escape )
		{
			if ( CurrentCell.IsValid() && CurrentCell->InEditMode() )
			{
				CurrentCell->ExitEditMode();
				Reply = FReply::Handled();
			}
		}
		else if( Key == EKeys::C && InKeyEvent.IsControlDown() ) 
		{
			if ( CurrentCell.IsValid() && FirstCellInSelection.IsValid() && LastCellInSelection.IsValid() )
			{
				const int32 StartingCellRowIndex = Table->GetRows().Find( FirstCellInSelection->GetRow() );
				const int32 EndingCellRowIndex = Table->GetRows().Find( LastCellInSelection->GetRow() );

				const int32 TopMostRowIndex = StartingCellRowIndex < EndingCellRowIndex ? StartingCellRowIndex : EndingCellRowIndex;
				const int32 BottomMostRowIndex = StartingCellRowIndex > EndingCellRowIndex ? StartingCellRowIndex : EndingCellRowIndex;

				const int32 StartingCellColumnIndex = Table->GetColumns().Find( FirstCellInSelection->GetColumn() );
				const int32 EndingCellColumnIndex = Table->GetColumns().Find( LastCellInSelection->GetColumn() );

				const int32 LeftMostColumnIndex = StartingCellColumnIndex < EndingCellColumnIndex ? StartingCellColumnIndex : EndingCellColumnIndex;
				const int32 RightMostColumnIndex = StartingCellColumnIndex > EndingCellColumnIndex ? StartingCellColumnIndex : EndingCellColumnIndex;

				const TSharedRef< IPropertyTableCell > ActualStartingCell = Table->GetColumns()[ LeftMostColumnIndex ]->GetCell( Table->GetRows()[ TopMostRowIndex ] );
				const TSharedRef< IPropertyTableCell > ActualEndingCell = Table->GetColumns()[ RightMostColumnIndex ]->GetCell( Table->GetRows()[ BottomMostRowIndex ] );

				FString CopiedText;
				FString Tab( TEXT("\t") );
				FString NewLine( LINE_TERMINATOR );

				bool IsFirstRow = true;
				TSharedPtr< IPropertyTableCell > RowCell = ActualStartingCell;
				while ( RowCell.IsValid() )
				{
					if ( !IsFirstRow )
					{
						CopiedText += NewLine;
					}

					bool IsFirstColumn = true;
					TSharedPtr< IPropertyTableCell > Cell = RowCell;
					while( Cell.IsValid() )
					{
						if ( IsFirstColumn )
						{
							CopiedText += Cell->GetValueAsString();
						}
						else
						{
							CopiedText += Tab + Cell->GetValueAsString();
						}

						if ( Cell->GetColumn() == ActualEndingCell->GetColumn() )
						{
							break;
						}

						Cell = Table->GetNextCellInRow( Cell.ToSharedRef() );
						IsFirstColumn = false;
					}

					if ( RowCell->GetRow() == ActualEndingCell->GetRow() )
					{
						break;
					}

					RowCell = Table->GetNextCellInColumn( RowCell.ToSharedRef() );
					IsFirstRow = false;
				}

				FPlatformApplicationMisc::ClipboardCopy( *CopiedText );
				Reply = FReply::Handled();
			}
		}
		else if( Key == EKeys::V && InKeyEvent.IsControlDown() ) 
		{
			if ( CurrentCell.IsValid() )
			{
				// Get string from clipboard
				FString Result;
				FPlatformApplicationMisc::ClipboardPaste( Result );

				Table->PasteTextAtCell( Result, CurrentCell.ToSharedRef() );

				Reply = FReply::Handled();
			}
		}
		else if ( Key == EKeys::A && InKeyEvent.IsControlDown() )
		{
			return SelectRange( Table->GetFirstCellInTable(), Table->GetLastCellInTable(), Table->GetCurrentCell() );
		}
		else if ( Key == EKeys::Home )
		{
			if ( CurrentCell.IsValid() && InKeyEvent.IsShiftDown() && InKeyEvent.IsControlDown() )
			{
				return SelectRange( Table->GetFirstCellInTable(), CurrentCell, CurrentCell );
			}
			else if ( InKeyEvent.IsControlDown() )
			{
				return MoveToCell( Table->GetFirstCellInTable() );
			}
			else if ( CurrentRow.IsValid() && CurrentCell.IsValid() && InKeyEvent.IsShiftDown() )
			{
				return SelectRange( Table->GetFirstCellInRow( CurrentRow.ToSharedRef() ), CurrentCell, CurrentCell );
			}
			else if ( CurrentRow.IsValid() )
			{
				return MoveToCell( Table->GetFirstCellInRow( CurrentRow.ToSharedRef() ) );
			}
		}
		else if ( Key == EKeys::End )
		{
			if ( CurrentCell.IsValid() && InKeyEvent.IsShiftDown() && InKeyEvent.IsControlDown() )
			{
				return SelectRange( CurrentCell, Table->GetLastCellInTable(), CurrentCell );
			}
			else if ( InKeyEvent.IsControlDown() )
			{
				return MoveToCell( Table->GetLastCellInTable() );
			}
			else if ( CurrentRow.IsValid() && CurrentCell.IsValid() && InKeyEvent.IsShiftDown() )
			{
				return SelectRange( CurrentCell, Table->GetLastCellInRow( CurrentRow.ToSharedRef() ), CurrentCell );
			}
			else if ( CurrentRow.IsValid() )
			{
				return MoveToCell( Table->GetLastCellInRow( CurrentRow.ToSharedRef() ) );
			}
		}
		else if ( Key == EKeys::Left )
		{
			if ( CurrentRow.IsValid() && CurrentCell.IsValid() && InKeyEvent.IsShiftDown() && InKeyEvent.IsControlDown() )
			{
				return SelectRange( Table->GetFirstCellInRow( CurrentRow.ToSharedRef() ), CurrentCell, CurrentCell );
			}
			else if ( InKeyEvent.IsControlDown() && CurrentRow.IsValid() )
			{
				return MoveToCell( Table->GetFirstCellInRow( CurrentRow.ToSharedRef() ) );
			}
			else if ( CurrentCell.IsValid() && InKeyEvent.IsShiftDown() )
			{
				if ( FirstCellInSelection.IsValid() && LastCellInSelection.IsValid() )
				{
					if ( FirstCellInSelection == CurrentCell || CurrentCell->GetColumn() == FirstCellInSelection->GetColumn() )
					{
						return SelectRange( FirstCellInSelection, Table->GetPreviousCellInRow( LastCellInSelection.ToSharedRef() ), CurrentCell );
					}
					else if ( LastCellInSelection == CurrentCell || CurrentCell->GetColumn() == LastCellInSelection->GetColumn() )
					{
						return SelectRange( Table->GetPreviousCellInRow( FirstCellInSelection.ToSharedRef() ), LastCellInSelection, CurrentCell );
					}
					else
					{
						return MoveToCell( Table->GetPreviousCellInRow( CurrentCell.ToSharedRef() ) );
					}
				}
				else
				{
					return MoveToCell( Table->GetPreviousCellInRow( CurrentCell.ToSharedRef() ) );
				}
			}
			else if ( CurrentCell.IsValid() )
			{
				return MoveToCell( Table->GetPreviousCellInRow( CurrentCell.ToSharedRef() ) );
			}
		}
		else if ( Key == EKeys::Right )
		{
			if ( CurrentRow.IsValid() && CurrentCell.IsValid() && InKeyEvent.IsShiftDown() && InKeyEvent.IsControlDown() )
			{
				return SelectRange( Table->GetLastCellInRow( CurrentRow.ToSharedRef() ), CurrentCell, CurrentCell );
			}
			else if ( InKeyEvent.IsControlDown() && CurrentRow.IsValid() )
			{
				return MoveToCell( Table->GetLastCellInRow( CurrentRow.ToSharedRef() ) );
			}
			else if ( CurrentCell.IsValid() && InKeyEvent.IsShiftDown() )
			{
				if ( FirstCellInSelection.IsValid() && LastCellInSelection.IsValid() )
				{
					if ( FirstCellInSelection == CurrentCell || CurrentCell->GetColumn() == FirstCellInSelection->GetColumn() )
					{
						return SelectRange( FirstCellInSelection, Table->GetNextCellInRow( LastCellInSelection.ToSharedRef() ), CurrentCell );
					}
					else if ( LastCellInSelection == CurrentCell || CurrentCell->GetColumn() == LastCellInSelection->GetColumn() )
					{
						return SelectRange( Table->GetNextCellInRow( FirstCellInSelection.ToSharedRef() ), LastCellInSelection, CurrentCell );
					}
					else
					{
						return MoveToCell( Table->GetNextCellInRow( CurrentCell.ToSharedRef() ) );
					}
				}
				else
				{
					return MoveToCell( Table->GetNextCellInRow( CurrentCell.ToSharedRef() ) );
				}
			}
			else if ( CurrentCell.IsValid() )
			{
				return MoveToCell( Table->GetNextCellInRow( CurrentCell.ToSharedRef() ) );
			}
		}
		else if ( Key == EKeys::Up )
		{
			if ( InKeyEvent.IsShiftDown() && InKeyEvent.IsControlDown() && CurrentColumn.IsValid() && CurrentCell.IsValid() )
			{
				return SelectRange( Table->GetFirstCellInColumn( CurrentColumn.ToSharedRef() ), CurrentCell, CurrentCell );
			}
			else if ( InKeyEvent.IsControlDown() && CurrentColumn.IsValid() )
			{
				return MoveToCell( Table->GetFirstCellInColumn( CurrentColumn.ToSharedRef() ) );
			}
			else if ( CurrentCell.IsValid() && InKeyEvent.IsShiftDown() )
			{
				if ( FirstCellInSelection.IsValid() && LastCellInSelection.IsValid() )
				{
					if ( FirstCellInSelection == CurrentCell || CurrentCell->GetRow() == FirstCellInSelection->GetRow() )
					{
						return SelectRange( FirstCellInSelection, Table->GetPreviousCellInColumn( LastCellInSelection.ToSharedRef() ), CurrentCell );
					}
					else if ( LastCellInSelection == CurrentCell || CurrentCell->GetRow() == LastCellInSelection->GetRow() )
					{
						return SelectRange( Table->GetPreviousCellInColumn( FirstCellInSelection.ToSharedRef() ), LastCellInSelection, CurrentCell );
					}
					else
					{
						return MoveToCell( Table->GetPreviousCellInColumn( CurrentCell.ToSharedRef() ) );
					}
				}
				else
				{
					return MoveToCell( Table->GetPreviousCellInColumn( CurrentCell.ToSharedRef() ) );
				}
			}
			else if ( CurrentCell.IsValid() )
			{
				return MoveToCell( Table->GetPreviousCellInColumn( CurrentCell.ToSharedRef() ) );
			}
		}
		else if ( Key == EKeys::Down )
		{
			if ( InKeyEvent.IsShiftDown() && InKeyEvent.IsControlDown() && CurrentColumn.IsValid() && CurrentCell.IsValid() )
			{
				return SelectRange( CurrentCell, Table->GetLastCellInColumn( CurrentColumn.ToSharedRef() ), CurrentCell );
			}
			else if ( InKeyEvent.IsControlDown() && CurrentColumn.IsValid() )
			{
				return MoveToCell( Table->GetLastCellInColumn( CurrentColumn.ToSharedRef() ) );
			}
			else if ( CurrentCell.IsValid() && InKeyEvent.IsShiftDown() )
			{
				if ( FirstCellInSelection.IsValid() && LastCellInSelection.IsValid() )
				{
					if ( FirstCellInSelection == CurrentCell || CurrentCell->GetRow() == FirstCellInSelection->GetRow() )
					{
						return SelectRange( FirstCellInSelection, Table->GetNextCellInColumn( LastCellInSelection.ToSharedRef() ), CurrentCell );
					}
					else if ( LastCellInSelection == CurrentCell || CurrentCell->GetRow() == LastCellInSelection->GetRow() )
					{
						return SelectRange( Table->GetNextCellInColumn( FirstCellInSelection.ToSharedRef() ), LastCellInSelection, CurrentCell );
					}
					else
					{
						return MoveToCell( Table->GetNextCellInColumn( CurrentCell.ToSharedRef() ) );
					}
				}
				else
				{
					return MoveToCell( Table->GetNextCellInColumn( CurrentCell.ToSharedRef() ) );
				}
			}
			else if ( CurrentCell.IsValid() )
			{
				return MoveToCell( Table->GetNextCellInColumn( CurrentCell.ToSharedRef() ) );
			}
		}
		else if ( Key == EKeys::Tab )
		{
			if ( CurrentCell.IsValid() )
			{
				if ( InKeyEvent.IsShiftDown() )
				{
					MoveToCell( Table->GetPreviousCellInRow( CurrentCell.ToSharedRef() ) );
				}
				else
				{
					MoveToCell( Table->GetNextCellInRow( CurrentCell.ToSharedRef() ) );
				}

				// We always handle the tab key if there is a current cell
				return FReply::Handled();
			}
		}
		else if ( Key == EKeys::SpaceBar )
		{
			// Don't allow the parent class to trigger selection via spacebar.
			return FReply::Handled();
		}
		else if ( Key == EKeys::Enter )
		{
			if ( CurrentCell.IsValid() )
			{
				return MoveToCell( Table->GetNextCellInColumn( CurrentCell.ToSharedRef() ) );
			}
		}

		return STreeView< TSharedRef< IPropertyTableRow > >::OnKeyDown( MyGeometry, InKeyEvent );
	}

	virtual FReply OnPreviewKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
	{
		const FKey Key = InKeyEvent.GetKey();
		if ( Key == EKeys::Enter && !InKeyEvent.IsShiftDown() )
		{
			const TSharedPtr< IPropertyTableCell > CurrentCell = Table->GetCurrentCell();
			if ( CurrentCell.IsValid() )
			{
				return MoveToCell( Table->GetNextCellInColumn( CurrentCell.ToSharedRef() ) );
			}
		}

		return STreeView< TSharedRef< IPropertyTableRow > >::OnPreviewKeyDown( MyGeometry, InKeyEvent );
	}

	FReply SelectRange( const TSharedPtr< IPropertyTableCell >& StartingCell, const TSharedPtr< IPropertyTableCell >& EndingCell, const TSharedPtr< IPropertyTableCell >& CellToFocus )
	{
		TSharedPtr< IPropertyTableCell > CurrentCell = Table->GetCurrentCell();

		if ( !StartingCell.IsValid() || !EndingCell.IsValid() )
		{
			return FReply::Unhandled();
		}

		const bool InEditMode = CurrentCell.IsValid() ? CurrentCell->InEditMode() : false;

		const int32 StartingCellRowIndex = Table->GetRows().Find( StartingCell->GetRow() );
		const int32 EndingCellRowIndex = Table->GetRows().Find( EndingCell->GetRow() );

		const int32 TopMostRowIndex = StartingCellRowIndex < EndingCellRowIndex ? StartingCellRowIndex : EndingCellRowIndex;
		const int32 BottomMostRowIndex = StartingCellRowIndex > EndingCellRowIndex ? StartingCellRowIndex : EndingCellRowIndex;

		const int32 StartingCellColumnIndex = Table->GetColumns().Find( StartingCell->GetColumn() );
		const int32 EndingCellColumnIndex = Table->GetColumns().Find( EndingCell->GetColumn() );

		const int32 LeftMostColumnIndex = StartingCellColumnIndex < EndingCellColumnIndex ? StartingCellColumnIndex : EndingCellColumnIndex;
		const int32 RightMostColumnIndex = StartingCellColumnIndex > EndingCellColumnIndex ? StartingCellColumnIndex : EndingCellColumnIndex;

		const TSharedRef< IPropertyTableCell > ActualStartingCell = Table->GetColumns()[ LeftMostColumnIndex ]->GetCell( Table->GetRows()[ TopMostRowIndex ] );
		const TSharedRef< IPropertyTableCell > ActualEndingCell = Table->GetColumns()[ RightMostColumnIndex ]->GetCell( Table->GetRows()[ BottomMostRowIndex ] );

		Table->SelectCellRange( ActualStartingCell, ActualEndingCell );
		
		if ( CellToFocus.IsValid() )
		{
			Table->SetCurrentCell( CellToFocus );

			if ( InEditMode )
			{
				CellToFocus->EnterEditMode();
			}
		}

		return FReply::Handled();
	}

	FReply MoveToCell( const TSharedPtr< IPropertyTableCell >& CellToFocus )
	{
		TSharedPtr< IPropertyTableCell > CurrentCell = Table->GetCurrentCell();

		if ( !CellToFocus.IsValid() || CellToFocus == CurrentCell )
		{
			return FReply::Unhandled();
		}

		bool InEditMode = CurrentCell.IsValid() ? CurrentCell->InEditMode() : false;
		TSet< TSharedRef< IPropertyTableCell > > CellsToSelect;
		CellsToSelect.Add( CellToFocus.ToSharedRef() );

		Table->SetSelectedCells( CellsToSelect );
		Table->SetCurrentCell( CellToFocus );

		// If we dont have a valid item for the row we want to move to, then it is outside the
		// scrolled area & we should scroll to view it.
		TSharedPtr<ITableRow> WidgetForItem = WidgetGenerator.GetWidgetForItem( CellToFocus->GetRow() );
		if ( !WidgetForItem.IsValid() )
		{
			RequestScrollIntoView( CellToFocus->GetRow() );
		}

		if ( InEditMode )
		{
			CellToFocus->EnterEditMode();
		}

		return FReply::Handled();
	}

	virtual FReply OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override
	{
		FReply Reply = FReply::Unhandled();

		TSharedPtr< IPropertyTableCell > CurrentCell = Table->GetCurrentCell();

		if ( CurrentCell.IsValid() )
		{
			FKey Key = InKeyEvent.GetKey();

			if ( Key == EKeys::F2 )
			{
				if ( !CurrentCell->InEditMode() )
				{
					CurrentCell->EnterEditMode();
				}

				return FReply::Handled();
			}
		}

		return Reply;
	}


protected:

	TSharedRef< ITableRow > GenerateRow( const TSharedRef< IPropertyTableRow > PropertyTableRow, const TSharedRef< STableViewBase >& OwnerTable )
	{
		return SNew( SPropertyTableRow, PropertyTableRow, HeaderRow.ToSharedRef(), OwnerTable )
			.Style( Style );
	}

	void OnSelectionChanged( const TSharedPtr< IPropertyTableRow > Row, const ESelectInfo::Type SelectionInfo )
	{
		if( bUpdatingSelection )
		{
			return;
		}

		bUpdatingSelection = true;
		TSet< TSharedRef< IPropertyTableRow > > SelectedRows;
		SelectedRows.Append( GetSelectedItems() );
		
		Table->SetSelectedRows( SelectedRows );

		TSharedPtr< IPropertyTableRow > CurrentRow = Table->GetCurrentRow();
		if ( CurrentRow.IsValid() )
		{
			if ( !IsItemVisible( CurrentRow.ToSharedRef() ) )
			{
				this->RequestScrollIntoView( CurrentRow.ToSharedRef() );
			}
		}

		bUpdatingSelection = false;
	}

	/**
	 *	Called whenever the table selection changes
	 */
	void UpdateSelection()
	{
		if( bUpdatingSelection )
		{
			return;
		}

		bUpdatingSelection = true;
		const auto& SelectedRows = Table->GetSelectedRows();
		ClearSelection();

		for( auto RowIter = SelectedRows.CreateConstIterator(); RowIter; ++RowIter )
		{
			SetItemSelection( *RowIter, true );
		}

		TSharedPtr< IPropertyTableRow > CurrentRow = Table->GetCurrentRow();
		if ( CurrentRow.IsValid() )
		{
			if ( !IsItemVisible( CurrentRow.ToSharedRef() ) )
			{
				this->RequestScrollIntoView( CurrentRow.ToSharedRef() );
			}
		}
		bUpdatingSelection = false;
	}

	void OnGetChildren( const TSharedRef< IPropertyTableRow > ParentPropertyTableRow, TArray< TSharedRef< IPropertyTableRow > >& OutChildren )
	{
		ParentPropertyTableRow->GetChildRows( OutChildren );
	}


private:

	void SyncBreadcrumbTrail()
	{
		if ( BreadcrumbTrail.IsValid() )
		{
			BreadcrumbTrail->ClearCrumbs( false );
			BreadcrumbTrail->PushCrumb( LOCTEXT("BreadcrumbRootDisplayName", "Root"), -1 );

			const TSharedPtr< FPropertyPath > RootPath = Table->GetRootPath();
			if ( RootPath.IsValid() )
			{
				for (int Index = 0; Index < RootPath->GetNumProperties(); Index++)
				{
					const FPropertyInfo& PropInfo = RootPath->GetPropertyInfo( Index );

					if ( PropInfo.ArrayIndex != INDEX_NONE )
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("ArrayIndex"), PropInfo.ArrayIndex);
						BreadcrumbTrail->PushCrumb(FText::Format(LOCTEXT("ArrayIndexWrapper", "[{ArrayIndex}]"), Args), Index);
					}
					else
					{
						BreadcrumbTrail->PushCrumb( FText::FromName(PropInfo.Property->GetFName()), Index );
					}
				}
			}
		}
	}

	void OnCrumbClicked( const int32& Item )
	{
		ClearSelection();
		
		const TSharedRef< FPropertyPath > RootPath = Table->GetRootPath();
		const int32 AmountToTrimRoot = ( RootPath->GetNumProperties() - 1 ) - Item;

		if ( Item == INDEX_NONE )
		{
			Table->SetRootPath( RootPath->TrimPath( RootPath->GetNumProperties() ) );
		}
		else if ( AmountToTrimRoot > 0 )
		{
			Table->SetRootPath( RootPath->TrimPath( AmountToTrimRoot ) );
		}
	}

	TSharedRef< SWidget > GetCrumbMenuContent( const int32& Item )
	{
		TSharedRef< FPropertyPath > RootPath = Table->GetRootPath();
		if ( Item == INDEX_NONE )
		{
			RootPath = RootPath->TrimPath( RootPath->GetNumProperties() );
		}
		else
		{
			RootPath = RootPath->TrimPath( ( RootPath->GetNumProperties() - 1 ) - Item );
		}

		TArray< FPropertyInfo > PathExtensions = Table->GetPossibleExtensionsForPath( RootPath );

		TMultiMap< UStruct*, FPropertyInfo > TypeToProperties;

		for( auto ExtensionIter = PathExtensions.CreateIterator(); ExtensionIter; ++ExtensionIter )
		{
			const FPropertyInfo& Extension = *ExtensionIter;
			if(FPropertyEditorPermissionList::Get().DoesPropertyPassFilter(Extension.Property->GetOwnerStruct(), Extension.Property->GetFName()))
			{
				TypeToProperties.Add( Extension.Property->GetOwnerStruct(), Extension );
			}
		}

		FMenuBuilder MenuBuilder( true, NULL );
		{
			TArray< UStruct* > Types;
			TypeToProperties.GetKeys( /*OUT*/ Types );

			for( auto TypeIter = Types.CreateConstIterator(); TypeIter; ++TypeIter )
			{
				MenuBuilder.BeginSection("PropertyTableCrumb", FText::FromName( (*TypeIter)->GetFName() ) );
				{
					for( auto ExtensionIter = TypeToProperties.CreateConstKeyIterator( *TypeIter ); ExtensionIter; ++ExtensionIter )
					{
						const FPropertyInfo& PropInfo = ExtensionIter.Value();
						const TWeakFieldPtr< const FProperty > Property = PropInfo.Property;
						const FText PropName = FText::FromString( UEditorEngine::GetFriendlyName(Property.Get()) );
						MenuBuilder.AddMenuEntry( PropName, PropName, FSlateIcon(), FUIAction( FExecuteAction::CreateSP( this, &SPropertyTable::SetRootPath, RootPath->ExtendPath( PropInfo ) ) ) );
					}
				}
				MenuBuilder.EndSection();
			}
		}

		return MenuBuilder.MakeWidget();
	}

	void SetRootPath( const TSharedRef< FPropertyPath > Path )
	{
		Table->SetRootPath( Path );
	}

public:
	//
	// Private Interface
	//
	// A low-level interface for use various widgets generated by ItemsWidgets(Lists, Trees, etc).
	// These handle selection, expansion, and other such properties common to ItemsWidgets.
	//

	virtual bool Private_UsesSelectorFocus() const override
	{
		return false;
	}

	virtual bool Private_HasSelectorFocus( const TSharedRef< IPropertyTableRow >& TheItem ) const override
	{
		return false;
	}

private:

	/**	Whether the view is currently updating the viewmodel selection */
	bool bUpdatingSelection;

	FName Style;

	TSharedPtr< IPropertyTable > Table;

	TSharedPtr< IPropertyTableUtilities > Utilities;

	TSharedPtr< SPropertyTableHeaderRow > HeaderRow;

	TSharedPtr< class SBreadcrumbTrail< int32 > > BreadcrumbTrail;
};

#undef LOCTEXT_NAMESPACE
