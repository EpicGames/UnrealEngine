// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyTable/SPropertyTableCell.h"

#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "Delegates/Delegate.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IPropertyTable.h"
#include "IPropertyTableCell.h"
#include "IPropertyTableCellPresenter.h"
#include "IPropertyTableRow.h"
#include "Input/Events.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateColor.h"
#include "Styling/WidgetStyle.h"
#include "Templates/TypeHash.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

class FSlateRect;
class SWidget;
class SWindow;
class UObject;
struct FPropertyChangedEvent;

void SPropertyTableCell::Construct( const FArguments& InArgs, const TSharedRef< class IPropertyTableCell >& InCell )
{
	Cell = InCell;
	Presenter = InArgs._Presenter;
	Style = InArgs._Style;

	// This draws a box around each column which looks like gridlines
	CellBackground = FAppStyle::GetBrush( Style, ".CellBorder" );

	SetContent( ConstructCellContents() );

	Cell->OnEnteredEditMode().AddSP( this, &SPropertyTableCell::EnteredEditMode );
	Cell->OnExitedEditMode().AddSP( this, &SPropertyTableCell::ExitedEditMode );

	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &SPropertyTableCell::OnCellValueChanged);
}

void SPropertyTableCell::SetContent( const TSharedRef< SWidget >& NewContents )
{
	TSharedRef<SWidget> Contents = NewContents;
	if (!Cell->IsValid())
	{
		Contents = ConstructInvalidPropertyWidget();
	}

	ChildSlot
	[
		Contents
	];
}

void SPropertyTableCell::OnCellValueChanged( UObject* Object, FPropertyChangedEvent& PropertyChangedEvent )
{
	if ( Cell->GetObject().Get() == Object )
	{
		if ( !Cell->InEditMode() )
		{
			SetContent( ConstructCellContents() );
		}
	}
}

TSharedRef< SWidget > SPropertyTableCell::ConstructCellContents()
{
	TSharedRef< SWidget > CellContents = SNullWidget::NullWidget;

	// If the cell doesnt pass the permission list, don't show any widget for it
	if(!Cell->PassesPermissionList())
	{
		return CellContents;
	}
	
	if ( Presenter.IsValid() )
	{
		if ( Cell->InEditMode() )
		{
			CellContents = ConstructEditModeCellWidget();
		}
		else
		{
			CellContents = Presenter->ConstructDisplayWidget();
		}
	}

	return CellContents;
}

void SPropertyTableCell::OnAnchorWindowClosed( const TSharedRef< SWindow >& WindowClosing )
{
	Cell->ExitEditMode();
}

void SPropertyTableCell::EnteredEditMode()
{
	if (Cell->IsValid())
	{
		// We delay the activation of editing mode till Tick due to mouse related input replies stomping on the focus
		RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SPropertyTableCell::TriggerEnterEditingMode));
	}
}

void SPropertyTableCell::ExitedEditMode()
{
	if ( Presenter.IsValid() )
	{
		SetContent( Presenter->ConstructDisplayWidget() );

		if ( DropDownAnchor.IsValid() )
		{
			FSlateApplication::Get().DismissAllMenus();
			DropDownAnchor = NULL;
		}
	}
}

EActiveTimerReturnType SPropertyTableCell::TriggerEnterEditingMode(double InCurrentTime, float InDeltaTime)
{
	if (Cell->GetTable()->GetCurrentCell() == Cell && Cell->InEditMode())
	{
		if (Presenter.IsValid())
		{
			SetContent(ConstructCellContents());

			if (DropDownAnchor.IsValid() && Presenter->RequiresDropDown())
			{
				DropDownAnchor->SetIsOpen(true, false);
			}

			FSlateApplication::Get().SetKeyboardFocus(Presenter->WidgetToFocusOnEdit(), EFocusCause::SetDirectly);
		}
		else
		{
			FSlateApplication::Get().SetKeyboardFocus(ChildSlot.GetChildAt(0), EFocusCause::SetDirectly);
		}
	}

	return EActiveTimerReturnType::Stop;
}

int32 SPropertyTableCell::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	if ( CellBackground && CellBackground->DrawAs != ESlateBrushDrawType::NoDrawType )
	{
		const FSlateBrush* Background = CellBackground;

		// We go through the row to grab the table because we are in OnPaint() and the column could be in the process of reconstruction
		if (Cell->GetRow()->GetTable()->GetCurrentCell() == Cell )
		{
			// The current cell has the blue selection color and a white border around it
			Background = FAppStyle::GetBrush( Style, ".CurrentCellBorder" );
		}
		else if ( Cell->GetRow()->GetTable()->GetSelectedCells().Contains( Cell.ToSharedRef() ) )
		{
			// Other selected cells have the blue selection color along with the gridlines
			Background = FAppStyle::GetBrush( Style, ".SelectedCellBorder" );
		}

		// If the cell is in edit mode, we give the presenter a chance to specify the background
		if(Cell->InEditMode() && Presenter)
		{
			if(const FSlateBrush* PresenterBackground = Presenter->GetEditModeCellBrush())
			{
				Background = PresenterBackground;
			}
		}
		

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			Background,
			ESlateDrawEffect::None,
			Background->GetTint( InWidgetStyle ) * InWidgetStyle.GetColorAndOpacityTint() 
			);
	}

	return SCompoundWidget::OnPaint( Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled );
}

FReply SPropertyTableCell::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	const TSharedRef< IPropertyTable > Table = Cell->GetTable();
	Table->SetLastClickedCell( Cell );

	return FReply::Unhandled();
}

FReply SPropertyTableCell::OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& MouseEvent )
{
	const TSharedRef< IPropertyTable > Table = Cell->GetTable();
	Table->SetLastClickedCell( Cell );

	return FReply::Unhandled();
}

TSharedRef< class SWidget > SPropertyTableCell::ConstructEditModeCellWidget()
{
	return
			SAssignNew( DropDownAnchor, SMenuAnchor )
			.Placement( MenuPlacement_ComboBox )
			.OnGetMenuContent( this, &SPropertyTableCell::ConstructEditModeDropDownWidget )
			.Content()
			[
				Presenter->ConstructEditModeCellWidget()
			];
}

TSharedRef< class SWidget > SPropertyTableCell::ConstructEditModeDropDownWidget()
{
	return Presenter->ConstructEditModeDropDownWidget();
}

TSharedRef<SWidget> SPropertyTableCell::ConstructInvalidPropertyWidget()
{
	return 
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Icons.Error"))
		]
		+SHorizontalBox::Slot()
		[
			SNew(STextBlock)
			.ColorAndOpacity(FLinearColor::Red)
			.Text(NSLOCTEXT("PropertyEditor", "InvalidTableCellProperty", "Failed to retrieve value"))
		];
}
