// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/PlatformMath.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
class IPropertyTableCellPresenter;
class SMenuAnchor;
class SWidget;
class SWindow;
class UObject;
struct FGeometry;
struct FPointerEvent;
struct FPropertyChangedEvent;
struct FSlateBrush;

class SPropertyTableCell : public SCompoundWidget
{
	public:

	SLATE_BEGIN_ARGS( SPropertyTableCell ) 
		: _Presenter( NULL )
		, _Style( TEXT("PropertyTable") )
	{}
		SLATE_ARGUMENT( TSharedPtr< IPropertyTableCellPresenter >, Presenter)
		SLATE_ARGUMENT( FName, Style )
	SLATE_END_ARGS()

	SPropertyTableCell()
		: DropDownAnchor( NULL )
		, Presenter( NULL )
		, Cell( NULL )
		, bEnterEditingMode( false )
		, Style()
	{ }

	void Construct( const FArguments& InArgs, const TSharedRef< class IPropertyTableCell >& InCell );

	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

	virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;

	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;


private:

	TSharedRef< SWidget > ConstructCellContents();

	void SetContent( const TSharedRef< SWidget >& NewContents );

	TSharedRef< class SWidget > ConstructEditModeCellWidget();

	TSharedRef< class SWidget > ConstructEditModeDropDownWidget();

	TSharedRef< class SWidget > ConstructInvalidPropertyWidget();

	void OnAnchorWindowClosed( const TSharedRef< SWindow >& WindowClosing );

	void EnteredEditMode();

	void ExitedEditMode();

	void OnCellValueChanged( UObject* Object, FPropertyChangedEvent& PropertyChangedEvent );

	/** One-off active timer to trigger entering the editing mode */
	EActiveTimerReturnType TriggerEnterEditingMode(double InCurrentTime, float InDeltaTime);

private:

	TSharedPtr< SMenuAnchor > DropDownAnchor;

	TSharedPtr< class IPropertyTableCellPresenter > Presenter;
	TSharedPtr< class IPropertyTableCell > Cell;
	bool bEnterEditingMode;
	FName Style;

	const FSlateBrush* CellBackground;
};
