// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/SWidget.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Textures/SlateIcon.h"
#include "Styling/AppStyle.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Views/SHeaderRow.h"
#include "IPropertyTable.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IPropertyTableColumn.h"
#include "UserInterface/PropertyTable/SColumnHeader.h"
#include "UserInterface/PropertyTable/ColumnWidgetFactory.h"

#define LOCTEXT_NAMESPACE "PropertyTableHeaderRow"

class SPropertyTableHeaderRow : public SHeaderRow
{
public:

	SLATE_BEGIN_ARGS( SPropertyTableHeaderRow )
		: _Style( TEXT("PropertyTable") )
		, _Customizations()
	{}
		SLATE_ARGUMENT( FName, Style )
		SLATE_ARGUMENT( TArray< TSharedRef< class IPropertyTableCustomColumn > >, Customizations )
	SLATE_END_ARGS()

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InViewModel			The UI logic not specific to slate
	 */
	void Construct( const FArguments& InArgs, const TSharedRef< class IPropertyTable >& InPropertyTable )
	{
		Style = InArgs._Style;

		SHeaderRow::FArguments HeaderRowArgs;
		HeaderRowArgs.Style( FAppStyle::Get(), FAppStyle::Join( Style, ".HeaderRow" ) );
		SHeaderRow::Construct( HeaderRowArgs );

		PropertyTable = InPropertyTable;

		ColumnFactory = MakeShareable( new ColumnWidgetFactory() );
		Customizations.Append( InArgs._Customizations );

		PropertyTable->OnColumnsChanged()->AddSP( this, &SPropertyTableHeaderRow::UpdateColumns );

		UpdateColumns();
	}

	void UpdateColumns()
	{
		ClearColumns();

		const TArray< TSharedRef< IPropertyTableColumn > > PropertyTableColumns = PropertyTable->GetColumns();

		ColumnHeaders.Empty();
		const TSharedRef< IPropertyTable > TableRef = PropertyTable.ToSharedRef();

		for( auto ColumnIter = PropertyTableColumns.CreateConstIterator(); ColumnIter; ++ColumnIter )
		{
			const TSharedRef< IPropertyTableColumn > Column = *ColumnIter;

			if ( !ColumnFactory->Supports( Column ) )
			{
				Column->SetHidden( true );
				continue;
			}

			Column->SetHidden( false );
			TSharedRef< SColumnHeader > ColumnHeader = ConstructColumnHeader( Column, TableRef, Style );
			const FName ColumnId = Column->GetId();
			ColumnHeaders.Add( ColumnId, ColumnHeader ); 

			if ( Column->GetSizeMode() == EPropertyTableColumnSizeMode::Fixed )
			{
				AddColumn( 
					SHeaderRow::Column( ColumnId )
					.SortMode( TableRef, &IPropertyTable::GetColumnSortMode, Column )
					.HeaderComboVisibility(EHeaderComboVisibility::Never)
					.SortPriority( TableRef, &IPropertyTable::GetColumnSortPriority, Column)
					.OnSort( TableRef, &IPropertyTable::SortByColumnWithId )
					.MenuContent()
					[
						GenerateColumnMenu( Column )
					]
					.FixedWidth( Column->GetWidth() )
					[
						ColumnHeader
					]
				);
			}
			else
			{
				TAttribute< EColumnSortMode::Type > SortMode = EColumnSortMode::None;
				TAttribute< EColumnSortPriority::Type > SortPriority = EColumnSortPriority::Max;
				FOnSortModeChanged OnSort;

				if ( Column->CanSortBy() )
				{
					SortMode = TAttribute< EColumnSortMode::Type >::Create( TAttribute< EColumnSortMode::Type >::FGetter::CreateSP( TableRef, &IPropertyTable::GetColumnSortMode, Column ) );
					SortPriority = TAttribute< EColumnSortPriority::Type >::Create(TAttribute< EColumnSortPriority::Type >::FGetter::CreateSP(TableRef, &IPropertyTable::GetColumnSortPriority, Column ) );
					OnSort = FOnSortModeChanged::CreateSP( TableRef, &IPropertyTable::SortByColumnWithId );
				}

				AddColumn(
					SHeaderRow::Column( ColumnId )
					.SortMode( SortMode )
					.HeaderComboVisibility(EHeaderComboVisibility::Never)
					.SortPriority( SortPriority )
					.OnSort( OnSort )
					.MenuContent()
					[
						GenerateColumnMenu( Column )
					]
					.FillWidth( Column, &IPropertyTableColumn::GetWidth )
					.OnWidthChanged( Column, &IPropertyTableColumn::SetWidth )
					[
						ColumnHeader
					]
				);
			}

			Column->OnFrozenStateChanged()->RemoveAll(this);
			Column->OnFrozenStateChanged()->AddSP(this, &SPropertyTableHeaderRow::RegenerateColumnMenu);
		}
	}

	TSharedPtr< SColumnHeader > Find( const FName& ColumnId )
	{
		const TSharedRef< SColumnHeader >* ColumnHeaderView = ColumnHeaders.Find( ColumnId );

		if( ColumnHeaderView == NULL )
		{
			return NULL;
		}

		return *ColumnHeaderView;
	}


private:

	TSharedRef< SColumnHeader > ConstructColumnHeader( const TSharedRef < class IPropertyTableColumn >& Column, const TSharedRef< class IPropertyTableUtilities >& Utilities, const FName& InStyle )
	{
		TSharedPtr< IPropertyTableCustomColumn > Customization;

		for( auto CustomizationIter = Customizations.CreateIterator(); CustomizationIter; ++CustomizationIter )
		{
			auto PotentialCustomization = *CustomizationIter;
			if ( PotentialCustomization->Supports( Column, Utilities ) )
			{
				Customization = PotentialCustomization;
				break;
			}
		}

		return ColumnFactory->CreateColumnHeaderWidget( Column, Utilities, Customization, InStyle );
	}

	TSharedRef< SWidget > GenerateColumnMenu( const TSharedRef< IPropertyTableColumn >& Column )
	{
		if ( Column->IsFrozen() )
		{
			return SNullWidget::NullWidget;
		}

		// Name column drop down menu
		const bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder( bShouldCloseWindowAfterMenuSelection, NULL );
		{
			{
				FUIAction Action(
					FExecuteAction::CreateSP(SharedThis(this), &SPropertyTableHeaderRow::RemoveColumn, Column),
					FCanExecuteAction()
				);

				MenuBuilder.AddMenuEntry( LOCTEXT("RemoveColumn_Name", "Remove"), LOCTEXT("RemoveColumn_ToolTip", "Removes the column from the table view"), FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::Button );
			}

			{
				FUIAction Action(
					FExecuteAction::CreateSP(SharedThis(this), &SPropertyTableHeaderRow::RemoveAllButThisColumn, Column),
					FCanExecuteAction()
				);

				MenuBuilder.AddMenuEntry(LOCTEXT("RemoveAllButThisColumn_Name", "Remove All But This"), LOCTEXT("RemoveAllButThisColumn_ToolTip", "Removes all the columns from the table view except this"), FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::Button);
			}
		}

		return MenuBuilder.MakeWidget();
	}

	void RegenerateColumnMenu( const TSharedRef< IPropertyTableColumn >& Column )
	{
		// Seems like a waste to update all the columns just because one changed,
		// but if I try to just remove the one and re-add it, UpdateColumns ends up getting called by RemoveColumn() anyway
		UpdateColumns();
	}

	void RemoveColumn( const TSharedRef< IPropertyTableColumn > Column )
	{
		PropertyTable->RemoveColumn( Column );
	}

	void RemoveAllButThisColumn(const TSharedRef<IPropertyTableColumn> ColumnToKeep)
	{
		TArray<TSharedRef<IPropertyTableColumn>> ExistingColumns = PropertyTable->GetColumns();
		for (TSharedRef<IPropertyTableColumn> ExistingColumn : ExistingColumns)
		{
			if (ExistingColumn->IsFrozen() || ExistingColumn == ColumnToKeep)
			{
				continue;
			}

			PropertyTable->RemoveColumn(ExistingColumn);
		}
	}

private:

	TSharedPtr< class IPropertyTable > PropertyTable;

	TSharedPtr< class ColumnWidgetFactory > ColumnFactory;

	TArray< TSharedRef< class IPropertyTableCustomColumn > > Customizations;

	TMap< FName, TSharedRef< class SColumnHeader > > ColumnHeaders;

	FName Style;
};

#undef LOCTEXT_NAMESPACE
