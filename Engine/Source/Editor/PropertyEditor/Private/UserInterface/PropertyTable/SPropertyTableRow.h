// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/AppStyle.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "IPropertyTable.h"
#include "UserInterface/PropertyTable/SPropertyTableHeaderRow.h"
#include "IPropertyTableCell.h"
#include "IPropertyTableRow.h"

typedef SMultiColumnTableRow< TSharedRef< class IPropertyTableRow > > SPropertyTableRowBase;

class SPropertyTableRow : public SPropertyTableRowBase
{
public:

	SLATE_BEGIN_ARGS( SPropertyTableRow )
		: _Style( "PropertyTable" )
	{}
		SLATE_ARGUMENT( FName, Style )
	SLATE_END_ARGS()

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InOwnerTableView	The owner of the row widget
	 */
	void Construct( const FArguments& InArgs, 
		const TSharedRef< class IPropertyTableRow >& InPropertyTableRow, 
		const TSharedRef< class SPropertyTableHeaderRow > InPropertyTableHeaderRow, 
		const TSharedRef< STableViewBase >& InOwnerTableView )
	{
		Row = InPropertyTableRow;
		HeaderRowWeakPtr = InPropertyTableHeaderRow;
		Row->OnRefresh()->AddSP( this, &SPropertyTableRow::Refresh );
		Style = InArgs._Style;

		SMultiColumnTableRow< TSharedRef< class IPropertyTableRow > >::Construct(
			FSuperRowType::FArguments()
			.Style( FAppStyle::Get(), "TableView.AlternatingRow" )
			.ShowSelection(false) // We don't show row selection on the property table - each selected cell is invidually shown
			, InOwnerTableView );
	}

	FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		PreviousCurrentCell = Row->GetTable()->GetCurrentCell();
		Row->GetTable()->SetCurrentCell( nullptr );

		return FReply::Unhandled();
	}

	FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		FReply Reply = FReply::Unhandled();

		const TSharedRef< IPropertyTable > Table = Row->GetTable();
		const EPropertyTableSelectionUnit::Type SelectionUnit = Table->GetSelectionUnit();

		Table->SetCurrentCell( PreviousCurrentCell );
		const TSharedPtr< IPropertyTableCell > CurrentCell = Table->GetCurrentCell();
		const TSharedPtr< IPropertyTableCell > LastClickedCell = Table->GetLastClickedCell();

		TSet< TSharedRef< IPropertyTableCell > > SelectedCells( Table->GetSelectedCells() );

		Reply = SPropertyTableRowBase::OnMouseButtonUp( MyGeometry, MouseEvent );

		if ( SelectionUnit == EPropertyTableSelectionUnit::Cell )
		{
			if ( LastClickedCell.IsValid() )
			{
				// The Row may not have any cells, so we'll detect an old LastClickedCell and clear
				if ( !Row->HasCells() || LastClickedCell->GetRow() != Row || !LastClickedCell->GetColumn()->CanSelectCells() )
				{
					Table->SetLastClickedCell( nullptr );
					Table->SetCurrentCell( nullptr );
				}
				else
				{
					if ( CurrentCell.IsValid() && LastClickedCell == CurrentCell && !MouseEvent.IsControlDown())
					{
						// Don't let the table enter edit mode if the ctrl key is down
						Table->GetLastClickedCell()->EnterEditMode();
					}
					else if ( MouseEvent.IsShiftDown() && CurrentCell.IsValid() && LastClickedCell != CurrentCell )
					{
						const int32 LastClickedRowIndex = Table->GetRows().Find( LastClickedCell->GetRow() );
						const int32 CurrentRowIndex = Table->GetRows().Find( CurrentCell->GetRow() );

						const int32 TopMostRowIndex = LastClickedRowIndex < CurrentRowIndex ? LastClickedRowIndex : CurrentRowIndex;
						const int32 BottomMostRowIndex = LastClickedRowIndex > CurrentRowIndex ? LastClickedRowIndex : CurrentRowIndex;

						const int32 LastClickedColumnIndex = Table->GetColumns().Find( LastClickedCell->GetColumn() );
						const int32 CurrentColumnIndex = Table->GetColumns().Find( CurrentCell->GetColumn() );

						const int32 LeftMostColumnIndex = LastClickedColumnIndex < CurrentColumnIndex ? LastClickedColumnIndex : CurrentColumnIndex;
						const int32 RightMostColumnIndex = LastClickedColumnIndex > CurrentColumnIndex ? LastClickedColumnIndex : CurrentColumnIndex;

						const TSharedRef< IPropertyTableCell > StartingCell = Table->GetColumns()[ LeftMostColumnIndex ]->GetCell( Table->GetRows()[ TopMostRowIndex ] );
						const TSharedRef< IPropertyTableCell > EndingCell = Table->GetColumns()[ RightMostColumnIndex ]->GetCell( Table->GetRows()[ BottomMostRowIndex ] );

						Table->SelectCellRange( StartingCell, EndingCell );
						Table->SetCurrentCell( CurrentCell );
					}
					else if (MouseEvent.IsControlDown())
					{
						if (SelectedCells.Contains(LastClickedCell.ToSharedRef()))
						{
							SelectedCells.Remove(LastClickedCell.ToSharedRef());
							Table->SetCurrentCell(nullptr);
						}
						else
						{
							SelectedCells.Add(LastClickedCell.ToSharedRef());
							Table->SetCurrentCell(LastClickedCell);
						}

						Table->SetSelectedCells(SelectedCells);
					}
					else
					{
						SelectedCells.Empty();
						SelectedCells.Add( LastClickedCell.ToSharedRef() );

						Table->SetSelectedCells( SelectedCells );
						Table->SetCurrentCell( LastClickedCell );
					}
				}
			}
		}
		else if ( SelectionUnit == EPropertyTableSelectionUnit::Row )
		{
			if ( !Table->GetSelectedRows().Contains( Row.ToSharedRef() ) )
			{
				Table->SetCurrentCell( nullptr );
			}
		}
		
		return Reply;
	}

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override
	{
		Row->Tick();
	}


protected:

	void Refresh()
	{
		ClearCellCache();
		GenerateColumns( HeaderRowWeakPtr.Pin().ToSharedRef() );
	}

	virtual TSharedRef< SWidget > GenerateWidgetForColumn( const FName& ColumnId ) override
	{
		TSharedPtr< SPropertyTableHeaderRow > PropertyTableHeaderRow = HeaderRowWeakPtr.Pin();
		if ( !PropertyTableHeaderRow.IsValid() )
		{
			return SNullWidget::NullWidget;
		}

		TSharedPtr< SColumnHeader > ColumnHeader = PropertyTableHeaderRow->Find( ColumnId );

		if( !ColumnHeader.IsValid() )
		{
			return SNew( STextBlock ).Text( NSLOCTEXT("PropertyTable", "UnknownColumnId", "ERROR INVALID COLUMN ID") );
		}

		TSharedRef< SWidget > CellWidget = ColumnHeader->GenerateCell( Row.ToSharedRef() );

		return CellWidget;
	}


private:

	TSharedPtr< class IPropertyTableRow > Row;
	TSharedPtr< class IPropertyTableCell > PreviousCurrentCell;
	TWeakPtr< class SPropertyTableHeaderRow > HeaderRowWeakPtr;
	FName Style;
};
