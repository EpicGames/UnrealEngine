// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "IPropertyTable.h"
#include "Widgets/Images/SImage.h"
#include "Styling/AppStyle.h"
#include "Widgets/SOverlay.h"
#include "PropertyEditorHelpers.h"
#include "IPropertyTableCell.h"
#include "Widgets/Layout/SBox.h"

class SRowHeaderCell : public SCompoundWidget
{
	public:

	SLATE_BEGIN_ARGS( SRowHeaderCell )
		: _Style( TEXT("PropertyTable") )
	{}
		SLATE_ARGUMENT( FName, Style )
	SLATE_END_ARGS()

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InViewModel			The UI logic not specific to slate
	 */
	void Construct( const FArguments& InArgs, const TSharedRef< class IPropertyTableCell >& InCell, const TSharedPtr< FPropertyEditor >& PropertyEditor )
	{
		Style = InArgs._Style;
		Cell = InCell;

		// This draws a box around the cell which looks like gridlines on the table
		CellBackground = FAppStyle::GetBrush( Style, ".CellBorder" );
		
		TArray< EPropertyButton::Type > RequiredButtons;
		if ( PropertyEditor.IsValid() )
		{
			PropertyEditorHelpers::GetRequiredPropertyButtons( PropertyEditor->GetPropertyNode(), RequiredButtons );
		}

		TSharedRef< SWidget > Content = 
			SNew(SImage)
			.Image( FAppStyle::GetBrush("Icons.DirtyBadge") )
			.DesiredSizeOverride(FVector2D{ 16.0f })
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Visibility( this, &SRowHeaderCell::GetDirtyImageVisibility );

		if ( RequiredButtons.Contains( EPropertyButton::Insert_Delete_Duplicate ) )
		{
			Editor = PropertyEditor;
			Content =
				SNew( SOverlay )
				+SOverlay::Slot()
				[
					Content
				]
				
				+SOverlay::Slot()
				[
					PropertyEditorHelpers::MakePropertyButton( EPropertyButton::Insert_Delete_Duplicate, PropertyEditor.ToSharedRef() )
				];
		}

		ChildSlot
		[
			SNew( SBox )
			.HeightOverride( 20.0f )
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				Content
			]
		];
	}

	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		const TSharedRef< IPropertyTable > Table = Cell->GetTable();
		Table->SetLastClickedCell( Cell );

		return FReply::Unhandled();
	}

	virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& MouseEvent ) override
	{
		const TSharedRef< IPropertyTable > Table = Cell->GetTable();
		Table->SetLastClickedCell( Cell );

		return FReply::Unhandled();
	}

	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override
	{
		if ( CellBackground && CellBackground->DrawAs != ESlateBrushDrawType::NoDrawType )
		{
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(),
				CellBackground,
				ESlateDrawEffect::None,
				CellBackground->GetTint( InWidgetStyle ) * InWidgetStyle.GetColorAndOpacityTint() 
				);
		}

		return SCompoundWidget::OnPaint( Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled );
	}



private:

	const FSlateBrush* GetBorder() const 
	{
		return FAppStyle::GetBrush( Style, ".RowHeader.Background" );
	}

	EVisibility GetDirtyImageVisibility() const
	{
		const TWeakObjectPtr< UObject > Object = Cell->GetObject();

		if ( !Object.IsValid() )
		{
			return EVisibility::Hidden;
		}

		UPackage* Package = Object->GetOutermost();
		return Package != NULL && Package->IsDirty() ? EVisibility::Visible : EVisibility::Hidden;
	}


private:

	TSharedPtr< IPropertyTableCell > Cell;
	TSharedPtr< FPropertyEditor > Editor;
	FName Style;
	const FSlateBrush* CellBackground = nullptr;
};
