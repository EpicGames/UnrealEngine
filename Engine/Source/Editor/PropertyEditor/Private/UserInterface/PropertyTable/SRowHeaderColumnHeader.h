// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Presentation/PropertyEditor/PropertyEditor.h"
#include "UserInterface/PropertyTable/SColumnHeader.h"
#include "Widgets/Layout/SSpacer.h"
#include "IPropertyTableCell.h"
#include "UserInterface/PropertyTable/SRowHeaderCell.h"

class SRowHeaderColumnHeader : public SColumnHeader
{
	public:

	SLATE_BEGIN_ARGS( SRowHeaderColumnHeader )
		: _Style( TEXT("PropertyTable") )
	{}
		SLATE_ARGUMENT( FName, Style )
	SLATE_END_ARGS()

	~SRowHeaderColumnHeader() = default;

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InViewModel			The UI logic not specific to slate
	 */
	void Construct( const FArguments& InArgs, const TSharedRef< class IPropertyTableColumn >& InPropertyTableColumn, const TSharedRef< class IPropertyTableUtilities >& InPropertyUtilities )
	{
		Style = InArgs._Style;

		SColumnHeader::FArguments ColumnArgs;
		ColumnArgs.Style( Style );

		SColumnHeader::Construct( ColumnArgs, InPropertyTableColumn, InPropertyUtilities );

		ChildSlot
		[
			SNew( SImage )
			.DesiredSizeOverride(FVector2D{16.0f})
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::GetBrush("Icons.Save"))
		];
	}

	virtual TSharedRef< SWidget > GenerateCell( const TSharedRef< class IPropertyTableRow >& PropertyTableRow ) override
	{
		TSharedRef< IPropertyTableCell > Cell = Column->GetCell( PropertyTableRow );
		TSharedPtr< FPropertyNode > Node = Cell->GetNode();

		TSharedPtr< FPropertyEditor > Editor;
		if ( Node.IsValid() )
		{
			Editor = FPropertyEditor::Create( Node.ToSharedRef(), Utilities.ToSharedRef() );
		}
		
		return SNew( SRowHeaderCell, Cell, Editor )
			.Style( Style );
	}


private:

	FName Style;
};
