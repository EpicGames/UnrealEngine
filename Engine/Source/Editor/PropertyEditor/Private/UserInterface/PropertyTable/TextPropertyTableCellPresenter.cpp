// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/PropertyTable/TextPropertyTableCellPresenter.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBorder.h"
#include "IPropertyTable.h"
#include "IPropertyTableCell.h"
#include "IPropertyTableUtilities.h"
#include "PropertyEditorHelpers.h"
#include "SResetToDefaultPropertyEditor.h"
#include "Styling/StyleColors.h"

#include "UserInterface/PropertyEditor/SPropertyEditor.h"
#include "UserInterface/PropertyEditor/SPropertyEditorNumeric.h"
#include "UserInterface/PropertyEditor/SPropertyEditorCombo.h"
#include "UserInterface/PropertyEditor/SPropertyEditorEditInline.h"
#include "UserInterface/PropertyEditor/SPropertyEditorText.h"
#include "UserInterface/PropertyEditor/SPropertyEditorBool.h"
#include "UserInterface/PropertyEditor/SPropertyEditorColor.h"
#include "UserInterface/PropertyEditor/SPropertyEditorDateTime.h"

FTextPropertyTableCellPresenter::FTextPropertyTableCellPresenter( const TSharedRef< class FPropertyEditor >& InPropertyEditor, const TSharedRef< class IPropertyTableUtilities >& InPropertyUtilities, FSlateFontInfo InFont /*= FAppStyle::GetFontStyle( PropertyTableConstants::NormalFontStyle ) */, const TSharedPtr< IPropertyTableCell >& InCell /*= nullptr*/ )
	: PropertyWidget( NULL )
	, PropertyEditor( InPropertyEditor )
	, PropertyUtilities( InPropertyUtilities )
	, Cell(InCell)
	, HasReadOnlyEditingWidget( false )
	, Font(InFont)
{
	HasReadOnlyEditingWidget = CalculateIfUsingReadOnlyEditingWidget();
}

TSharedRef< class SWidget > FTextPropertyTableCellPresenter::ConstructDisplayWidget()
{
	TSharedRef<SHorizontalBox> HorizontalBox = SNew( SHorizontalBox )
		+SHorizontalBox::Slot()
		.FillWidth( 1.0 )
		.VAlign( VAlign_Center )
		.Padding( FMargin( 2, 0, 0, 0 ) )
		[
			SNew( STextBlock )
			.Text( PropertyEditor->GetValueAsText() )
			.ColorAndOpacity(this, &FTextPropertyTableCellPresenter::GetTextForegroundColor)
			.ToolTipText( PropertyEditor->GetToolTipText() )
			.Font( Font )
		];

	if (!PropertyEditor->GetPropertyHandle()->HasMetaData(TEXT("NoResetToDefault")))
	{
		HorizontalBox->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(0, 0, 2, 0))
		[
			SNew(SResetToDefaultPropertyEditor, PropertyEditor->GetPropertyHandle())
		];
	}
	
	return HorizontalBox;
}

bool FTextPropertyTableCellPresenter::RequiresDropDown()
{
	// Don't create Anchor
	TArray<EPropertyButton::Type> OutRequiredButtons;
	PropertyEditorHelpers::GetRequiredPropertyButtons( PropertyEditor->GetPropertyNode(), OutRequiredButtons );
	return OutRequiredButtons.Num() > 0;
}

TSharedRef< class SWidget > FTextPropertyTableCellPresenter::ConstructEditModeDropDownWidget()
{
	TSharedRef< SWidget > Result = SNullWidget::NullWidget;

	TArray< TSharedRef<SWidget> > RequiredButtons;
	PropertyEditorHelpers::MakeRequiredPropertyButtons( PropertyEditor, /*OUT*/RequiredButtons );

	if ( RequiredButtons.Num() > 0 )
	{
		TSharedPtr< SHorizontalBox > ButtonBox;
		Result = SNew( SBorder )
			.BorderImage(FAppStyle::GetBrush("PropertyTable.Cell.DropDown.Background"))
			.Padding( 0 )
			.Content()
			[
				SAssignNew( ButtonBox, SHorizontalBox )
			];

		for( int32 ButtonIndex = 0; ButtonIndex < RequiredButtons.Num(); ++ButtonIndex )
		{
			ButtonBox->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding( 2.0f, 1.0f )
				[ 
					RequiredButtons[ButtonIndex]
				];
		}
	}

	return Result;
}

TSharedRef<SWidget> FTextPropertyTableCellPresenter::ConstructEditModeCellWidget()
{
	HasReadOnlyEditingWidget = false;
	const FProperty* const Property = PropertyEditor->GetProperty();

	if( Property )
	{
		// ORDER MATTERS: first widget type to support the property node wins!
		if ( SPropertyEditorNumeric<float>::Supports(PropertyEditor) )
		{
			PropertyWidget = SNew( SPropertyEditorNumeric<float>, PropertyEditor )
				.Font( Font );
		}
		else if (SPropertyEditorNumeric<int8>::Supports(PropertyEditor))
		{
			PropertyWidget = SNew(SPropertyEditorNumeric<int8>, PropertyEditor)
				.Font(Font);
		}
		else if (SPropertyEditorNumeric<int16>::Supports(PropertyEditor))
		{
			PropertyWidget = SNew(SPropertyEditorNumeric<int16>, PropertyEditor)
				.Font(Font);
		}
		else if ( SPropertyEditorNumeric<int32>::Supports(PropertyEditor) )
		{
			PropertyWidget = SNew( SPropertyEditorNumeric<int32>, PropertyEditor )
				.Font( Font );
		}
		else if (SPropertyEditorNumeric<int64>::Supports(PropertyEditor))
		{
			PropertyWidget = SNew(SPropertyEditorNumeric<int64>, PropertyEditor)
				.Font(Font);
		}
		else if ( SPropertyEditorNumeric<uint8>::Supports(PropertyEditor) )
		{
			PropertyWidget = SNew( SPropertyEditorNumeric<uint8>, PropertyEditor )
				.Font( Font );
		}
		else if (SPropertyEditorNumeric<uint16>::Supports(PropertyEditor))
		{
			PropertyWidget = SNew(SPropertyEditorNumeric<uint16>, PropertyEditor)
				.Font(Font);
		}
		else if (SPropertyEditorNumeric<uint32>::Supports(PropertyEditor))
		{
			PropertyWidget = SNew(SPropertyEditorNumeric<uint32>, PropertyEditor)
				.Font(Font);
		}
		else if (SPropertyEditorNumeric<uint64>::Supports(PropertyEditor))
		{
			PropertyWidget = SNew(SPropertyEditorNumeric<uint64>, PropertyEditor)
				.Font(Font);
		}
		else if ( SPropertyEditorCombo::Supports(PropertyEditor) )
		{
			FPropertyComboBoxArgs ComboArgs;
			ComboArgs.Font = Font;

			PropertyWidget = SNew( SPropertyEditorCombo, PropertyEditor )
				.ComboArgs( ComboArgs );
		}
		else if ( SPropertyEditorEditInline::Supports(PropertyEditor) )
		{
			PropertyWidget = SNew( SPropertyEditorEditInline, PropertyEditor )
				.Font( Font );
		}
		else if ( SPropertyEditorText::Supports(PropertyEditor) )
		{
			PropertyWidget = SNew( SPropertyEditorText, PropertyEditor )
				.Font( Font );
		}
		else if ( SPropertyEditorBool::Supports(PropertyEditor) )
		{
			PropertyWidget = SNew( SPropertyEditorBool, PropertyEditor );
		}
		else if ( SPropertyEditorColor::Supports(PropertyEditor) )
		{
			PropertyWidget = SNew( SPropertyEditorColor, PropertyEditor, PropertyUtilities );
		}
		else if ( SPropertyEditorDateTime::Supports(PropertyEditor) )
		{
			PropertyWidget = SNew( SPropertyEditorDateTime, PropertyEditor )
				.Font( Font );
		}
	}

	if( !PropertyWidget.IsValid() )
	{
		PropertyWidget = SNew( SPropertyEditor, PropertyEditor )
			.Font( Font );
		HasReadOnlyEditingWidget = true;
	}

	PropertyWidget->SetToolTipText( PropertyEditor->GetToolTipText() );

	return PropertyWidget.ToSharedRef();
}

bool FTextPropertyTableCellPresenter::CalculateIfUsingReadOnlyEditingWidget() const
{
	const FProperty* const Property = PropertyEditor->GetProperty();

	if( Property )
	{
		// ORDER MATTERS: first widget type to support the property node wins!
		if (SPropertyEditorNumeric<float>::Supports(PropertyEditor) || 
			SPropertyEditorNumeric<int32>::Supports(PropertyEditor) ||
			SPropertyEditorNumeric<uint8>::Supports(PropertyEditor) ||
			SPropertyEditorCombo::Supports(PropertyEditor) || 
			SPropertyEditorEditInline::Supports(PropertyEditor) || 
			SPropertyEditorText::Supports(PropertyEditor) || 
			SPropertyEditorBool::Supports(PropertyEditor) || 
			SPropertyEditorColor::Supports(PropertyEditor) ||
			SPropertyEditorDateTime::Supports(PropertyEditor))
		{
			return false;
		}
	}

	return true;
}

TSharedRef< class SWidget > FTextPropertyTableCellPresenter::WidgetToFocusOnEdit()
{
	return PropertyWidget.ToSharedRef();
}

FString FTextPropertyTableCellPresenter::GetValueAsString()
{
	return PropertyEditor->GetValueAsString();
}

FText FTextPropertyTableCellPresenter::GetValueAsText()
{
	return PropertyEditor->GetValueAsText();
}

FSlateColor FTextPropertyTableCellPresenter::GetTextForegroundColor() const
{
	if(Cell && Cell->GetTable()->GetSelectedCells().Contains( Cell.ToSharedRef() ))
	{
		return FStyleColors::White;
	}

	return FSlateColor::UseForeground();
}
