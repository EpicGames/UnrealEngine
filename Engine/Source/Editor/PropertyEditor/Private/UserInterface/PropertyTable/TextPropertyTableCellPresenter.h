// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/SWidget.h"
#include "Styling/AppStyle.h"
#include "IPropertyTableCellPresenter.h"
#include "UserInterface/PropertyTable/PropertyTableConstants.h"

class IPropertyTableCell;

class FTextPropertyTableCellPresenter : public TSharedFromThis< FTextPropertyTableCellPresenter >, public IPropertyTableCellPresenter
{
public:

	FTextPropertyTableCellPresenter( const TSharedRef< class FPropertyEditor >& InPropertyEditor, const TSharedRef< class IPropertyTableUtilities >& InPropertyUtilities, FSlateFontInfo InFont = FAppStyle::GetFontStyle( PropertyTableConstants::NormalFontStyle ), const TSharedPtr< IPropertyTableCell >& InCell = nullptr );

	virtual ~FTextPropertyTableCellPresenter() = default;

	virtual TSharedRef< class SWidget > ConstructDisplayWidget() override;

	virtual bool RequiresDropDown() override;
	
	virtual const FSlateBrush* GetEditModeCellBrush() override { return FAppStyle::GetBrush("NoBrush"); }

	virtual TSharedRef< class SWidget > ConstructEditModeCellWidget() override;

	virtual TSharedRef< class SWidget > ConstructEditModeDropDownWidget() override;

	virtual TSharedRef< class SWidget > WidgetToFocusOnEdit() override;

	virtual FString GetValueAsString() override;

	virtual FText GetValueAsText() override;

	virtual bool HasReadOnlyEditMode() override { return HasReadOnlyEditingWidget; }


private:

	bool CalculateIfUsingReadOnlyEditingWidget() const;

	FSlateColor GetTextForegroundColor() const;

private:

	TSharedPtr< class SWidget > PropertyWidget; 

	TSharedRef< class FPropertyEditor > PropertyEditor;
	TSharedRef< class IPropertyTableUtilities > PropertyUtilities;
	
	TSharedPtr< IPropertyTableCell > Cell;

	bool HasReadOnlyEditingWidget;
	FSlateFontInfo Font;
};
