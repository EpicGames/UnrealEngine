// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace PropertyTreeConstants
{
	const FName ColumnId_Name( TEXT("Property") );
	const FName ColumnId_Property( TEXT("Value") );

	const FText ColumnText_Name( NSLOCTEXT("PropertyEditorWidgets", "Property", "Property") );
	const FText ColumnText_Property( NSLOCTEXT("PropertyEditorWidgets", "Value", "Value") );
}


