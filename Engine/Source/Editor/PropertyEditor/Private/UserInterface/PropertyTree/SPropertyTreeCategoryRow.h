// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/Margin.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Styling/AppStyle.h"
#include "UserInterface/PropertyEditor/PropertyEditorConstants.h"

class FPropertyNode;

class SPropertyTreeCategoryRow : public STableRow< TSharedPtr< class FPropertyNode* > >
{
public:

	SLATE_BEGIN_ARGS( SPropertyTreeCategoryRow )
		: _DisplayName()
	{}
		SLATE_ARGUMENT( FText, DisplayName )

	SLATE_END_ARGS()


	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable )
	{
		STableRow< TSharedPtr< class FPropertyNode* > >::Construct(
			STableRow< TSharedPtr< class FPropertyNode* > >::FArguments()
				.Padding(3.0f)
				[
					SNew( STextBlock )
					.Text( InArgs._DisplayName )
					.Font( FAppStyle::GetFontStyle( PropertyEditorConstants::CategoryFontStyle ) )
				]
			, InOwnerTable );
	}

	// We override ConstructChildren because we want a border around the row to add some spacing between headers and have the expander spaced differently
	virtual void ConstructChildren( ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent ) override
	{
		this->Content = InContent;
		InnerContentSlot = nullptr;

		SHorizontalBox::FSlot* InnerContentSlotNativePtr = nullptr;

		this->ChildSlot
		[
			// This border determines how far the category header will be indented based on its hierarchy
			SNew(SBorder)
			.Padding(this, &SPropertyTreeCategoryRow::GetBorderPadding)
			.BorderImage(FAppStyle::GetBrush("Brushes.Panel"))
			[
				// This border draws a grind line after the header for when two headers are next to each other
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("DetailsView.GridLine"))
				.Padding(FMargin(0,0,0,1))
				[
					// This border is what contains the actual category row color and contents
					SNew(SBorder)
					.BorderImage(this, &SPropertyTreeCategoryRow::GetInnerBackgroundImage)
					.Padding(FMargin(5,1,5,0))
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Fill)
						.Padding(FMargin(4, 0, 0, 0))
						[
							SAssignNew(ExpanderArrowWidget, SExpanderArrow, SharedThis(this) )
							.StyleSet(ExpanderStyleSet)
							// We set the indent amount to 0 as we indent the widget ourselves so that we can also indent the border around it
							.IndentAmount(0) 
						]

						+ SHorizontalBox::Slot()
						.FillWidth(1)
						.Expose( InnerContentSlotNativePtr )
						.Padding( InPadding )
						[
							InContent
						]
					]
				]
			]
		];

		InnerContentSlot = InnerContentSlotNativePtr;
	}


	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			ToggleExpansion();
		}

		return STableRow< TSharedPtr< class FPropertyNode* > >::OnMouseButtonDown(MyGeometry, MouseEvent);
	}

protected:

	const FSlateBrush* GetInnerBackgroundImage() const
	{
		if(IsHovered())
		{
			return FAppStyle::GetBrush("Brushes.Dropdown");
		}

		return FAppStyle::GetBrush("Brushes.Header");
	}

	/** @return the margin corresponding to how far this item is indented */
	FMargin GetBorderPadding() const
	{
		const int32 NestingDepth = GetIndentLevel();
		const float Indent = 20.0f;
		return FMargin( NestingDepth * Indent, 0,0,0 );
	}

};
