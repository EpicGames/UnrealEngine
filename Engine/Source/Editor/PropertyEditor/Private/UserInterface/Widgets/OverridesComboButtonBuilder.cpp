//  Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/Widgets/OverridesComboButtonBuilder.h"

#include "Brushes/SlateRoundedBoxBrush.h"
#include "DetailsViewStyle.h"
#include "Styling/SlateBrush.h"
#include "Widgets/Images/SImage.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SBox.h"

/*
 * We use a custom widget that acts like a combo button, an image that pops up a menu on clicked so that we can have customized hover behavior
 * to change the foreground color and not show the button's background highlight
 */
class SOverrideWidget : public SImage
{
public:
	
	SLATE_BEGIN_ARGS(SOverrideWidget) {}
	
	SLATE_ARGUMENT( const FSlateBrush*, Image )
	SLATE_ARGUMENT( const FSlateBrush*, HoveredImage )
	SLATE_EVENT( FOnGetContent, OnGetMenuContent )

	SLATE_END_ARGS()

	const FSlateBrush* GetBrush() const
	{
		return IsHovered() ? HoveredImage : Image;
	}

	FSlateColor GetColor() const
	{
		return IsHovered() ? FStyleColors::White : FStyleColors::Foreground;
	}
	
	FReply HandleClick()
	{
		// Create the overrides menu at the cursor location
		if (OnGetMenuContent.IsBound())
		{
			FSlateApplication::Get().PushMenu(
				SharedThis(this),
				FWidgetPath(),
				OnGetMenuContent.Execute(),
				FSlateApplication::Get().GetCursorPos(),
				FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup));

			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	/** Called when the mouse button is pressed down on this widget */
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
		{
			return FReply::Unhandled();
		}

		return HandleClick();
	}
	
	/** Construct this widget */
	void Construct(const FArguments& InArgs)
	{
		Image = InArgs._Image;
		HoveredImage = InArgs._HoveredImage;
		OnGetMenuContent = InArgs._OnGetMenuContent;
		
		SImage::Construct(
		SImage::FArguments()
			.ColorAndOpacity(this, &SOverrideWidget::GetColor)
			.Image(this, &SOverrideWidget::GetBrush)
		);
	}
private:
	const FSlateBrush* Image = nullptr;
	const FSlateBrush* HoveredImage = nullptr;
	FOnGetContent OnGetMenuContent;
};



FOverridesComboButtonBuilder::FOverridesComboButtonBuilder(
	TSharedRef<FDetailsDisplayManager> InDetailsDisplayManager,
	bool bInIsCategoryOverridesComboButton,
	TWeakObjectPtr<UObject> InObject ):
	FPropertyUpdatedWidgetBuilder(),
	DisplayManager(InDetailsDisplayManager),
	bIsCategoryOverridesComboButton(bInIsCategoryOverridesComboButton),
    Object(InObject),
    EditPropertyChain(nullptr)
{
}

FOverridesComboButtonBuilder& FOverridesComboButtonBuilder::Set_OnGetContent(FOnGetContent InOnGetContent)
{
	OnGetContent.Unbind();
	OnGetContent = InOnGetContent;
	return *this;
}

TSharedPtr<SWidget> FOverridesComboButtonBuilder::GenerateWidget()
{
	const TArray< TSharedRef< const FOverridesWidgetStyleKey >> OverridesWidgetKeys = FOverridesWidgetStyleKeys::GetKeys();
	
	if (!Object.IsValid() || OverridesWidgetKeys.IsEmpty())
	{
		return SNullWidget::NullWidget;
	}

	TSharedRef<SHorizontalBox> Box = SNew(SHorizontalBox).Visibility(IsVisible);
	
	for (const TSharedRef< const FOverridesWidgetStyleKey >& Key : OverridesWidgetKeys )
	{
		if (Key->bCanBeVisible)
		{
			Box->AddSlot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SOverrideWidget)
					.Image(&Key->GetConstStyleBrush())
					.HoveredImage(&Key->GetConstStyleBrushHovered())
					.Visibility(Key->GetVisibilityAttribute(EditPropertyChain, Object))
					.ToolTipText(Key->GetToolTipText(bIsCategory))
					.OnGetMenuContent(OnGetContent)
				];
		}
	}

	// We use an SBox to force our size to fill the entire slot, otherwise the button is not centered on the column
	return SNew(SBox)
			.WidthOverride(20.0f)
			.HeightOverride(20.0f)
			[
				Box
			];
}

TSharedRef<SWidget> FOverridesComboButtonBuilder::operator*()
{
	return GenerateWidget().ToSharedRef();
}

FOverridesComboButtonBuilder::~FOverridesComboButtonBuilder()
{
	OnGetContent.Unbind();
}

void FOverridesComboButtonBuilder:: SetEditPropertyChain(TSharedRef<FEditPropertyChain>& InEditPropertyChain)
{
	EditPropertyChain = InEditPropertyChain;	
	bIsCategory = false;
}

TSharedPtr<FEditPropertyChain> FOverridesComboButtonBuilder::GetEditPropertyChain() const
{
	return EditPropertyChain;
}
