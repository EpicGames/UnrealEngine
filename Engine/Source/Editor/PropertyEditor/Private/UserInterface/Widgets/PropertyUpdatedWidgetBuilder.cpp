// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserInterface/Widgets/PropertyUpdatedWidgetBuilder.h"

FPropertyUpdatedWidgetBuilder::FPropertyUpdatedWidgetBuilder() : FToolElementRegistrationArgs("FPropertyUpdatedWidget")
{
}

FPropertyUpdatedWidgetBuilder& FPropertyUpdatedWidgetBuilder::Bind_IsVisible(TAttribute<EVisibility> InIsVisible)
{
	IsVisible = InIsVisible;
	return *this;
}

FPropertyUpdatedWidgetBuilder& FPropertyUpdatedWidgetBuilder::Set_ResetToDefault(FResetToDefault InResetToDefault)
{
	ResetToDefault.Unbind();
	ResetToDefault = InResetToDefault;
	return *this;
}

FPropertyUpdatedWidgetBuilder::FResetToDefault FPropertyUpdatedWidgetBuilder::Get_ResetToDefault()
{
	return ResetToDefault;
}

void FPropertyUpdatedWidgetBuilder::Bind_IsRowHovered(TAttribute<bool> InIsRowHoveredAttr)
{
	IsRowHoveredAttr = InIsRowHoveredAttr;
}
