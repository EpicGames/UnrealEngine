// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class PropertyEditor : ModuleRules
{
	public PropertyEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"EditorConfig",
				"EditorFramework",
				"UnrealEd",
                "ActorPickerMode",
                "SceneDepthPickerMode",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"MainFrame",
               	"AssetRegistry",
                "AssetTools",
				"ClassViewer",
				"StructViewer",
				"ContentBrowser",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
                "AppFramework",
				"AssetDefinition",
				"Core",
				"CoreUObject",
				"ApplicationCore",
				"EditorStyle",
				"Engine",
				"InputCore",
				"Slate",
				"SlateCore",
				"EditorWidgets",
				"Documentation",
				"ConfigEditor",
                "SceneOutliner",
				"DesktopPlatform",
				"Projects",
				"PropertyPath",
				"ToolWidgets",
				"WidgetRegistration",
				"Json",
				"ToolMenus"
			}
        );

		if (Target.WithAutomationTests)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"CQTest",
				}
			);
		}

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
                "AssetRegistry",
                "AssetTools",
				"ClassViewer",
				"StructViewer",
				"ContentBrowser",
				"MainFrame",
			}
		);
	}
}
