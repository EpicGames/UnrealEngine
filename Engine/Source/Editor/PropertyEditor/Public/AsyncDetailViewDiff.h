// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "AsyncTreeDifferences.h"
#include "DiffUtils.h"
#include "IDetailsView.h"

#define UE_API PROPERTYEDITOR_API

template<>
class TTreeDiffSpecification<TWeakPtr<FDetailTreeNode>>
{
public:
	virtual ~TTreeDiffSpecification() = default;
	/**
	 * determine whether the values stored in two nodes are equal.
	 * @param TreeNodeA node from the first user provided tree (guaranteed not to be null)
	 * @param TreeNodeB node from the second user provided tree (guaranteed not to be null)
	 */
	UE_API virtual bool AreValuesEqual(const TWeakPtr<FDetailTreeNode>& TreeNodeA, const TWeakPtr<FDetailTreeNode>& TreeNodeB) const;
	UE_DEPRECATED(5.6, "Output parameters wasn't generic and is no longer used.")
	UE_API virtual bool AreValuesEqual(const TWeakPtr<FDetailTreeNode>& TreeNodeA, const TWeakPtr<FDetailTreeNode>& TreeNodeB, TArray<FPropertySoftPath>*) const final;

	/**
	 * determine whether two nodes occupy the same space in their trees
	 * for example if you have a tree key/value pairs, AreMatching should compare the keys while AreValuesEqual
	 * should compare the values
	 * @param TreeNodeA node from the first user provided tree (guaranteed not to be null)
	 * @param TreeNodeB node from the second user provided tree (guaranteed not to be null)
	 */
	UE_API virtual bool AreMatching(const TWeakPtr<FDetailTreeNode>& TreeNodeA, const TWeakPtr<FDetailTreeNode>& TreeNodeB) const;
	UE_DEPRECATED(5.6, "Output parameters wasn't generic and is no longer used.")
	UE_API virtual bool AreMatching(const TWeakPtr<FDetailTreeNode>& TreeNodeA, const TWeakPtr<FDetailTreeNode>& TreeNodeB, TArray<FPropertySoftPath>*) const final;

	/**
	 * retrieves an array of children nodes from the parent node
	 * @param InParent node from one of the two user provided trees (guaranteed not to be null)
	 * @param[out] OutChildren to be filled with the children of parent
	 */
	UE_API virtual void GetChildren(const TWeakPtr<FDetailTreeNode>& InParent, TArray<TWeakPtr<FDetailTreeNode>>& OutChildren) const;

	/**
	 * return true for nodes that match using AreValuesEqual first, and pair up by position second
	 * this is useful for arrays since we often want to keep elements with the same data paired while diffing other elements in order
	 * @param TreeNode node from one of the two user provided trees (guaranteed not to be null)
	*/
	UE_API virtual bool ShouldMatchByValue(const TWeakPtr<FDetailTreeNode>& TreeNode) const;

	/**
	 * return true if TreeNode is considered equal when all it's children are equal.
	 * This avoids an unnecessary call to AreValuesEqual
	*/
	UE_API virtual bool ShouldInheritEqualFromChildren(const TWeakPtr<FDetailTreeNode>& TreeNodeA, const TWeakPtr<FDetailTreeNode>& TreeNodeB) const;
};

// asynchronously update a difference tree with changes to details views
// note: users are expected to call TAsyncTreeDifferences::Tick or TAsyncTreeDifferences::FlushQueue to get accurate data
class FAsyncDetailViewDiff : public TAsyncTreeDifferences<TWeakPtr<FDetailTreeNode>>
{
public:
	UE_API FAsyncDetailViewDiff(TSharedRef<IDetailsView> LeftView, TSharedRef<IDetailsView> RightView);

	// Note: by default this list is not necessarily accurate or exhaustive.
	// if you need a perfectly accurate list, call FlushQueue() first
	UE_API void GetPropertyDifferences(TArray<FSingleObjectDiffEntry>& OutDiffEntries) const;

	// execute a method for each node associated with a row in at least one details view
	// method provides current left and right row number
	// returns the number of rows in each details view
	UE_API TPair<int32, int32> ForEachRow(const TFunction<ETreeTraverseControl(const TUniquePtr<DiffNodeType>&, int32, int32)>& Method) const;

	// generates scroll data used by LinkableScrollbar
	UE_API TArray<FVector2f> GenerateScrollSyncRate() const;

private:
	static UE_API TAttribute<TArray<TWeakPtr<FDetailTreeNode>>> RootNodesAttribute(TWeakPtr<IDetailsView> DetailsView);
	TWeakPtr<IDetailsView> LeftView;
	TWeakPtr<IDetailsView> RightView;
};

#undef UE_API
