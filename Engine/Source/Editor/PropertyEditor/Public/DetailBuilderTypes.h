// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/NameTypes.h"
#include "Misc/Optional.h"

/**
 * Parameters required for specifying behavior when adding external properties from detail customizations
 */
struct FAddPropertyParams
{
	FAddPropertyParams()
		: bForceShowProperty(false)
		, bHideRootObjectNode(false)
	{}


	/**
	 * Forcibly show the property, even if it does not have CPF_Edit
	 */
	FAddPropertyParams& ForceShowProperty()
	{
		bForceShowProperty = true;
		return *this;
	}


	/**
	 * Override whether the property node should allow children or not. If not overridden the default is implementation defined.
	 */
	FAddPropertyParams& AllowChildren(bool bAllowChildren)
	{
		bAllowChildrenOverride = bAllowChildren;
		return *this;
	}


	/**
	 * Override whether the property node should create category nodes or not. If not overridden the default is implementation defined.
	 */
	FAddPropertyParams& CreateCategoryNodes(bool bCreateCategoryNodes)
	{
		bCreateCategoryNodesOverride = bCreateCategoryNodes;
		return *this;
	}


	/**
	 * Set a unique name for this property, allowing it to correctly save expansion states and other persistent UI state
	 */
	FAddPropertyParams& UniqueId(FName InUniqueId)
	{
		UniqueIdName = InUniqueId;
		return *this;
	}


	/**
	 * Override whether the root object node should be shown for external object properties. This only applies when adding an entire uobject's properties to the details panel rather than an individual row or struct
	 */
	FAddPropertyParams& HideRootObjectNode(bool bInHideRootObjectNode)
	{
		bHideRootObjectNode = bInHideRootObjectNode;
		return *this;
	}

	/**
	 * Check whether to forcibly show the property, even if it does not have CPF_Edit
	 */
	bool ShouldForcePropertyVisible() const
	{
		return bForceShowProperty;
	}

	bool ShouldHideRootObjectNode() const { return bHideRootObjectNode; }

	/**
	 * Conditionally overwrites the specified boolean with a value specifying whether to allow child properties or not.
	 * Only overrides the value if the calling code specified an override
	 */
	void OverrideAllowChildren(bool& OutAllowChildren) const
	{
		if (bAllowChildrenOverride.IsSet())
		{
			OutAllowChildren = bAllowChildrenOverride.GetValue();
		}
	}


	/**
	 * Conditionally overwrites the specified boolean with a value specifying whether to create category nodes or not
	 * Only overrides the value if the calling code specified an override
	 */
	void OverrideCreateCategoryNodes(bool& OutCreateCategoryNodes) const
	{
		if (bCreateCategoryNodesOverride.IsSet())
		{
			OutCreateCategoryNodes = bCreateCategoryNodesOverride.GetValue();
		}
	}


	/**
	 * Get this property's unique ID name
	 */
	FName GetUniqueId() const
	{
		return UniqueIdName;
	}

private:

	/** When true, the property will be forcefully shown, even if it does not have CPF_Edit. When false the property will only be created if it has CPF_Edit. */
	bool bForceShowProperty : 1;

	/** When true any root ObjectPropertyNode generated from an external UObject being added will be hidden from view and only its children are shown */
	bool bHideRootObjectNode : 1;

	/** Tristate override for allowing children - if value is true: Allow Children, false: Disallow Children, unset: no override */
	TOptional<bool> bAllowChildrenOverride;

	/** Tristate override for allowing children - if value is true: create category nodes, false: do not create category nodes, unset: no override */
	TOptional<bool> bCreateCategoryNodesOverride;

	/** Unique ID name that is used for saving persistent UI state such as expansion */
	FName UniqueIdName;
};