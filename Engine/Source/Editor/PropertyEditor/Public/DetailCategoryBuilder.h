// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Framework/SlateDelegates.h"
#include "UObject/StructOnScope.h"
#include "PropertyHandle.h"
#include "IDetailCustomNodeBuilder.h"
#include "DetailBuilderTypes.h"
#include "PropertyEditorCopyPaste.h"

class FDetailWidgetRow;
class IDetailGroup;
class IDetailPropertyRow;

/** Interface for the various types of rows that can be created through a detail layout customization */
class IDetailLayoutRow
{
public:
	virtual FName GetRowName() const = 0;
	virtual TOptional<FResetToDefaultOverride> GetCustomResetToDefault() const = 0;
};

/** The location of a property within a category */
namespace EPropertyLocation
{
	enum Type
	{
		/** The property appears in the simple area of a category (displayed by default) */
		Common,
		/** The property appears in the advanced area of a category(collapsed by default) */
		Advanced,
		/** The property appears in the default way specified in the UPROPERTY class flag */
		Default,
	};
}

/**
 * Represents a category for the a details view
 */
class IDetailCategoryBuilder
{
public:
	virtual ~IDetailCategoryBuilder() = default;
	/**
	 * Whether or not the category should be initially collapsed
	 */
	virtual IDetailCategoryBuilder& InitiallyCollapsed(bool bShouldBeInitiallyCollapsed) = 0;

	/**
	 * Adds a delegate to the category which is called when the category expansion state changes
	 */
	virtual IDetailCategoryBuilder& OnExpansionChanged(FOnBooleanValueChanged InOnExpansionChanged) = 0;

	/**
	 * Sets whether or not the expansion state should be restored between sessions
	 */
	virtual IDetailCategoryBuilder& RestoreExpansionState(bool bRestore) = 0;

	/**
	 * Adds header content to the category
	 *
	 * @param InHeaderContent	The header content widget
	 * @param bWholeRowContent	The header content should span the whole row (hides category name)
	 */
	virtual IDetailCategoryBuilder& HeaderContent(TSharedRef<SWidget> InHeaderContent, bool bWholeRowContent = false) = 0;

	/**
	 * Gets the current sort order of the category
	 */
	virtual int32 GetSortOrder() const = 0;

	/**
	 * Sets the sort order of the category
	 */
	virtual void SetSortOrder(int32 InSortOrder) = 0;

	/**
	 * Adds a property, shown in the default way to the category
	 *
	 * @param Path						The path to the property.  Can be just a name of the property or a path in the format outer.outer.value[optional_index_for_static_arrays]
	 * @param ClassOutermost			Optional outer class if accessing a property outside of the current class being customized
	 * @param InstanceName				Optional instance name if multiple FProperty's of the same type exist. such as two identical structs, the instance name is one of the struct variable names)
	 * @param Location				The location within the category where the property is shown
	 * See IDetailCategoryBuilder::GetProperty for clarification of parameters
	 */
	virtual IDetailPropertyRow& AddProperty(FName PropertyPath, UClass* ClassOutermost = nullptr, FName InstanceName = NAME_None, EPropertyLocation::Type Location = EPropertyLocation::Default) = 0;

	/**
	 * Adds a property, shown in the default way to the category
	 *
	 * @param PropertyHandle		The handle to the property to add
	 * @param Location				The location within the category where the property is shown
	 */
	virtual IDetailPropertyRow& AddProperty(TSharedPtr<IPropertyHandle> PropertyHandle, EPropertyLocation::Type Location = EPropertyLocation::Default) = 0;

	/**
	 * Adds an entire external object to this category.  The object should not be one of the objects already visible by this details panel
	 *
	 * @param Objects		List of objects that contain the property
	 * @param Location		The location within the category where the property is shown
	 * @return A property row for customizing the property or NULL if the property could not be found
	 */
	virtual IDetailPropertyRow* AddExternalObjects(const TArray<UObject*>& Objects, EPropertyLocation::Type Location = EPropertyLocation::Default, const FAddPropertyParams& Params = FAddPropertyParams()) = 0;

	/**
	 * Adds an external object property that is not a property on the object(s) being customized
	 *
	 * @param Objects		List of objects that contain the property
	 * @param PropertyName	Optional name of the property to view.  If this is empty, the entire set of properties on the objects are added
	 * @param Location		The location within the category where the property is shown
	 * @return A property row for customizing the property or NULL if the property could not be found
	 */
	virtual IDetailPropertyRow* AddExternalObjectProperty(const TArray<UObject*>& Objects, FName PropertyName, EPropertyLocation::Type Location = EPropertyLocation::Default, const FAddPropertyParams& Params = FAddPropertyParams()) = 0;

	/**
	 * Adds an external standalone structure to this details panel
	 *
	 * @param StructData		Struct data to find the property within
	 * @param Location			The location within the category where the property is shown
	 * @return A property row for customizing the property or NULL if the property could not be found
	 */
	virtual IDetailPropertyRow* AddExternalStructure(TSharedPtr<FStructOnScope> StructData, EPropertyLocation::Type Location = EPropertyLocation::Default) = 0;

	/**
	 * Adds an external property, that is contained within a UStruct, that is not a property on the object(s) being customized
	 *
	 * @param StructData		Struct data to find the property within
	 * @param PropertyName		The name of the property to view.  If this is empty the entire structure is added
	 * @param Location			The location within the category where the property is shown
	 * @return A property row for customizing the property or NULL if the property could not be found
	 */
	virtual IDetailPropertyRow* AddExternalStructureProperty(TSharedPtr<FStructOnScope> StructData, FName PropertyName, EPropertyLocation::Type Location = EPropertyLocation::Default, const FAddPropertyParams& Params = FAddPropertyParams()) = 0;
	virtual IDetailPropertyRow* AddExternalStructureProperty(TSharedPtr<IStructureDataProvider> StructData, FName PropertyName, EPropertyLocation::Type Location = EPropertyLocation::Default, const FAddPropertyParams& Params = FAddPropertyParams()) = 0;

	/**
	 * Adds all properties for the specified external structure to this category
	 *
	 * @param StructData		External struct data to add to the root
	 * @param Location			The location within the category where the properties are to be shown
	 * @param PropertiesRow		Optional pointer to an array that will contains the IDetailPropertyRow of the properties added
	 * @return Array of all properties added to this category
	 */
	virtual TArray<TSharedPtr<IPropertyHandle>> AddAllExternalStructureProperties(TSharedRef<FStructOnScope> StructData, EPropertyLocation::Type Location = EPropertyLocation::Default, TArray<IDetailPropertyRow*>* OutPropertiesRow = nullptr) = 0;
	virtual TArray<TSharedPtr<IPropertyHandle>> AddAllExternalStructureProperties(TSharedPtr<IStructureDataProvider> StructProvider, EPropertyLocation::Type Location = EPropertyLocation::Default, TArray<IDetailPropertyRow*>* OutPropertiesRow = nullptr) = 0;

	/**
	 * Adds a custom widget row to the category
	 *
	 * @param FilterString	 A string which is used to filter this custom row when a user types into the details panel search box
	 * @param bForAdvanced	 Whether the widget should appear in the advanced section
	 */
	virtual FDetailWidgetRow& AddCustomRow(const FText& FilterString, bool bForAdvanced = false) = 0;

	/**
	 * Adds a custom builder to the category (for more complicated layouts)
	 *
	 * @param InCustomBuilder				The custom builder to add
	 * @param bForAdvanced					true if this builder is for the advanced section of the category
	 */
	virtual void AddCustomBuilder(TSharedRef<IDetailCustomNodeBuilder> InCustomBuilder, bool bForAdvanced = false) = 0;

	/**
	 * Adds a group to the category
	 *
	 * @param GroupName	The name of the group
	 * @param LocalizedDisplayName	The display name of the group
	 * @param true if the group should appear in the advanced section of the category
	 * @param true if the group should start expanded
	 */
	virtual IDetailGroup& AddGroup(FName GroupName, const FText& LocalizedDisplayName, bool bForAdvanced = false, bool bStartExpanded = false) = 0;

	/**
	 * Gets the default properties of this category
	 *
	 * @param OutAllProperties		The list of property handles that will be populated
	 * @param bSimpleProperties		true to get simple properties
	 * @param bAdvancedProperties	true to get advanded properties
	 */
	virtual void GetDefaultProperties(TArray<TSharedRef<IPropertyHandle> >& OutAllProperties, bool bSimpleProperties = true, bool bAdvancedProperties = true) = 0;

	/**
	 * @return The parent layout builder of this class
	 */
	virtual class IDetailLayoutBuilder& GetParentLayout() const = 0;

	/**
	 * @return true if the parent layout is valid or has been destroyed by a refresh.
	 */
	virtual bool IsParentLayoutValid() const = 0;

	/**
	 * @return The localized display name of the category
	 */
	virtual const FText& GetDisplayName() const = 0;

	/**
	 * Set a new display name for this category.
	 */
	virtual void SetDisplayName(const FText& DisplayName) = 0;

	/**
	 * Gets the localized tooltip for this category.
	 */
	virtual const TOptional<FText>& GetToolTip() const = 0;

	/**
	 * Sets the localized tooltip for this category.
	 */
	virtual void SetToolTip(const FText& ToolTip) = 0;

	/**
	 * Sets whether or not this category is hidden or shown
	 * This is designed to be used for dynamic category visibility after construction of the category
	 */
	virtual void SetCategoryVisibility(bool bVisible) = 0;

	/**
	* Sets whether or not this category should show its advanced properties.
	* This is designed to be used for dynamic display of advanced properties.
	*/
	virtual void SetShowAdvanced(bool bShowAdvanced) = 0;

	/** Add a property, but force it to behave as a normal, peer reference regardless of CPF_InstancedReference */
	virtual void AddPropertyDisableInstancedReference(TSharedPtr<IPropertyHandle> PropertyHandle) = 0;

	/** Optional PasteFromText delegate for this category */
	virtual TSharedPtr<FOnPasteFromText> OnPasteFromText() const { return nullptr; }

	/** Optional CopyToText delegate for this category */
	virtual TSharedPtr<FOnCopyToText> OnCopyToText() const { return nullptr; }

	/**
	 * If true, this Category should have no UProperty data associated with it, and will be shown as an empty stub
	 * with no expansion arrow
	 */
	virtual bool IsEmpty() const { return false; }

	/**
	 * Sets whether this Category is "Empty" ~ that is, should have no UProperty data associated with it, and will be shown
	 * as an empty stub with no expansion arrow
	 */
	virtual void SetIsEmpty(bool bInIsEmpty) { }

	/**
	 * If valid, the color that represented the category. It can be used to visually identify the category quickly.
	 */
	virtual TOptional<FLinearColor> GetColor() const = 0;

	/**
	 * Set the color that represents the category.
	 */
	virtual void SetColor(FLinearColor InColor) = 0;

	/**
 	 * Clear the color that represents the category.
 	 */
	virtual void ClearColor() = 0;
};
