// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Misc/Attribute.h"
#include "Widgets/Layout/SSplitter.h"

class FDetailColumnSizeData
{
public:

	FDetailColumnSizeData()
	{
		ValueColumnWidthValue = 0.7f;
		RightColumnWidthValue = 0;
		HoveredSplitterIndexValue = INDEX_NONE;

		NameColumnWidth.BindRaw(this, &FDetailColumnSizeData::GetNameColumnWidthValue);
		ValueColumnWidth.BindRaw(this, &FDetailColumnSizeData::GetValueColumnWidthValue);
		WholeRowColumnWidth.BindRaw(this, &FDetailColumnSizeData::GetWholeRowColumnWidthValue);
		RightColumnWidth.BindRaw(this, &FDetailColumnSizeData::GetRightColumnWidthValue);
		HoveredSplitterIndex.BindRaw(this, &FDetailColumnSizeData::GetHoveredSplitterIndexValue);
		OnValueColumnResized.BindRaw(this, &FDetailColumnSizeData::SetValueColumnWidth);
		OnRightColumnResized.BindRaw(this, &FDetailColumnSizeData::OnSetRightColumnWidth);
		OnSplitterHandleHovered.BindRaw(this, &FDetailColumnSizeData::OnSetHoveredSplitterIndex);

		// these are intentionally left no-op, since the widths are derived from the other column width
		OnWholeRowColumnResized.BindLambda([](float) {});
		OnNameColumnResized.BindLambda([](float) {});
	}

	TAttribute<float> GetNameColumnWidth() const		{ return NameColumnWidth; }
	TAttribute<float> GetValueColumnWidth() const		{ return ValueColumnWidth; }
	TAttribute<float> GetWholeRowColumnWidth() const	{ return WholeRowColumnWidth; }
	TAttribute<float> GetRightColumnWidth() const		{ return RightColumnWidth; }
	TAttribute<float> GetRightColumnMinWidth() const	{ return RightColumnMinWidth; }
	TAttribute<int> GetHoveredSplitterIndex() const		{ return HoveredSplitterIndex; }

	const SSplitter::FOnSlotResized& GetOnNameColumnResized() const		{ return OnNameColumnResized; }
	const SSplitter::FOnSlotResized& GetOnValueColumnResized() const	{ return OnValueColumnResized; }
	const SSplitter::FOnSlotResized& GetOnWholeRowColumnResized() const { return OnWholeRowColumnResized; }
	const SSplitter::FOnSlotResized& GetOnRightColumnResized() const	{ return OnRightColumnResized; }
	const SSplitter::FOnHandleHovered& GetOnSplitterHandleHovered()	const { return OnSplitterHandleHovered; }

	void SetValueColumnWidth(float NewWidth)
	{
		ensure(NewWidth <= 1.0f);
		ValueColumnWidthValue = NewWidth;
	}

	void SetRightColumnMinWidth(float NewMinWidth)
	{
		RightColumnMinWidth = NewMinWidth;
	}

private:
	float ValueColumnWidthValue;
	float RightColumnWidthValue;
	int HoveredSplitterIndexValue;

	TAttribute<float> NameColumnWidth;
	TAttribute<float> ValueColumnWidth;
	TAttribute<float> WholeRowColumnWidth;
	TAttribute<float> RightColumnWidth;
	TAttribute<float> RightColumnMinWidth;
	TAttribute<int32> HoveredSplitterIndex;
	SSplitter::FOnSlotResized OnNameColumnResized;
	SSplitter::FOnSlotResized OnValueColumnResized;
	SSplitter::FOnSlotResized OnWholeRowColumnResized;
	SSplitter::FOnSlotResized OnRightColumnResized;
	SSplitter::FOnHandleHovered OnSplitterHandleHovered;

	float GetNameColumnWidthValue() const { return 1.0f - (ValueColumnWidthValue + RightColumnWidthValue); }
	float GetValueColumnWidthValue() const { return ValueColumnWidthValue; }
	float GetWholeRowColumnWidthValue() const { return 1.0f - RightColumnWidthValue; }
	int32 GetHoveredSplitterIndexValue() const { return HoveredSplitterIndexValue; }
	float GetRightColumnWidthValue() const { return RightColumnWidthValue; }

	void OnSetRightColumnWidth(float NewWidth)
	{
		ensure(NewWidth <= 1.0f);
		RightColumnWidthValue = NewWidth;
	}

	void OnSetHoveredSplitterIndex(int32 HoveredIndex)
	{
		HoveredSplitterIndexValue = HoveredIndex;
	}
};
