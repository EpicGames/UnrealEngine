// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/AppStyle.h"
#include "PropertyHandle.h"
#include "IDetailPropertyRow.h"
#include "Styling/AppStyle.h"

class IDetailCategoryBuilder;
class IDetailsView;
class IPropertyUtilities;

namespace ECategoryPriority
{
	enum Type
	{

		// Highest sort priority
		Variable = 0,
		Transform,
		Important,
		TypeSpecific,
		Default,
		// Lowest sort priority
		Uncommon,
	};
}

/**
 * The builder for laying custom details
 */
class IDetailLayoutBuilder
{
	
public:
	using FOnCategorySortOrderFunction = TFunction<void(const TMap<FName, IDetailCategoryBuilder*>& /* AllCategoryMap */)>;

	virtual ~IDetailLayoutBuilder()= default;

	/**
	 * @return the font used for properties and details
	 */ 
	static FSlateFontInfo GetDetailFont() { return FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont") ); }

	/**
	 * @return the bold font used for properties and details
	 */ 
	static FSlateFontInfo GetDetailFontBold() { return FAppStyle::GetFontStyle( TEXT("PropertyWindow.BoldFont") ); }
	
	/**
	 * @return the italic font used for properties and details
	 */ 
	static FSlateFontInfo GetDetailFontItalic() { return FAppStyle::GetFontStyle( TEXT("PropertyWindow.ItalicFont") ); }
	
	/**
	 * @return the parent detail view for this layout builder
	 */
	virtual TSharedPtr<const IDetailsView> GetDetailsViewSharedPtr() const = 0;

	UE_DEPRECATED(5.5, "Use GetDetailsViewSharedPtr() instead.")
	const IDetailsView* GetDetailsView() const
	{
		return GetDetailsViewSharedPtr().Get();
	}

	/**
	 * @return the parent detail view for this layout builder
	 */
	virtual TSharedPtr<IDetailsView> GetDetailsViewSharedPtr() = 0;

	UE_DEPRECATED(5.5, "Use GetDetailsViewSharedPtr() instead.")
	IDetailsView* GetDetailsView()
	{
		return GetDetailsViewSharedPtr().Get();
	}

	/**
	 * @return The base class of the objects being customized in this detail layout
	 */
	virtual UClass* GetBaseClass() const = 0;
	
	/**
	 * Get the root objects observed by this layout.  
	 * This is not guaranteed to be the same as the objects customized by this builder.  See GetObjectsBeingCustomized for that.
	 */
	virtual const TArray< TWeakObjectPtr<UObject> >& GetSelectedObjects() const = 0;

	/**
	 * Get the root objects (of ObjectType) observed by this layout.  
	 * This is not guaranteed to be the same as the objects customized by this builder.  See GetObjectsBeingCustomized for that.
	 */
	template <typename ObjectType = UObject>
	TArray< TWeakObjectPtr<ObjectType> > GetSelectedObjectsOfType() const;

	/**
	 * Gets the current object(s) being customized by this builder
	 *
	 * If this is a sub-object customization it will return those sub objects.  Otherwise the root objects will be returned.
	 */
	virtual void GetObjectsBeingCustomized( TArray< TWeakObjectPtr<UObject> >& OutObjects ) const = 0;

	/**
	 * Gets the current object(s) being customized by this builder of ObjectType 
	 *
	 * If this is a sub-object customization it will return those sub objects.  Otherwise the root objects will be returned.
	 * @return true if one or more objects of ObjectType were found.
	 */
	template <typename ObjectType = UObject>
	TArray< TWeakObjectPtr<ObjectType> > GetObjectsOfTypeBeingCustomized() const;

	/**
	 * Gets the current struct(s) being customized by this builder
	 *
	 * If this is a sub-struct customization it will return those sub struct.  Otherwise the root struct will be returned.
	 */
	virtual void GetStructsBeingCustomized( TArray< TSharedPtr<FStructOnScope> >& OutStructs ) const = 0;

	/**
	 *	@return the utilities various widgets need access to certain features of PropertyDetails
	 */
	virtual TSharedRef<IPropertyUtilities> GetPropertyUtilities() const = 0; 


	/**
	 * Edits an existing category or creates a new one
	 * 
	 * @param CategoryName				The name of the category
	 * @param NewLocalizedDisplayName	The new display name of the category (optional)
	 * @param CategoryType				Category type to define sort order.  Category display order is sorted by this type (optional)
	 */
	virtual IDetailCategoryBuilder& EditCategory(FName CategoryName, const FText& NewLocalizedDisplayName = FText::GetEmpty(), ECategoryPriority::Type CategoryType = ECategoryPriority::Default) = 0;

	/**
	* Edits an existing category or creates a new one
	* If CategoryName is NAME_None, will enable access to properties without categories
	* 
	* @param CategoryName				The name of the category
	* @param NewLocalizedDisplayName	The new display name of the category (optional)
	* @param CategoryType				Category type to define sort order.  Category display order is sorted by this type (optional)
	*/
	virtual IDetailCategoryBuilder& EditCategoryAllowNone(FName CategoryName, const FText& NewLocalizedDisplayName = FText::GetEmpty(), ECategoryPriority::Type CategoryType = ECategoryPriority::Default) = 0;

	/**
	 * Gets the current set of existing category names. This includes both categories derived from properties and categories added via EditCategory.
	 * @param	OutCategoryNames	 The array of category names
	 */
	virtual void GetCategoryNames(TArray<FName>& OutCategoryNames) const = 0;

	/**
	 * Adds sort algorythm which overrides standard algorythm with that provided by the caller.
	 * Function called on each category after all categories have been added, and provides caller
	 * with ability to override sort order.
	 * @param 	SortFunction Function called on final pass of detail panel build, which provides map of all categories to set final sort order on.
	 */
	virtual void SortCategories(const FOnCategorySortOrderFunction& SortFunction) = 0;

	/** 
	 * Adds the property to its given category automatically. Useful in detail customizations which want to preserve categories.
	 * @param InPropertyHandle			The handle to the property that you want to add to its own category.
	 * @return the property row with which the property was added.
	 */
	virtual IDetailPropertyRow& AddPropertyToCategory(TSharedPtr<IPropertyHandle> InPropertyHandle) = 0;

	/**
	 * Adds a custom row to the property's category automatically. Useful in detail customizations which want to preserve categories.
	 * @param InPropertyHandle			The handle to the property that you want to add to its own category.
	 * @param InCustomSearchString		A string which is used to filter this custom row when a user types into the details panel search box.
	 * @return the detail widget that can be further customized.
	 */
	virtual FDetailWidgetRow& AddCustomRowToCategory(TSharedPtr<IPropertyHandle> InPropertyHandle, const FText& InCustomSearchString, bool bForAdvanced = false) = 0;

	/**
	 * Adds an external object's property to this details panel's PropertyMap.
	 * Allows getting the property handle for the property without having to generate a row widget.
	 *
	 * @param Objects		List of objects that contain the property.
	 * @param PropertyName	Name of the property to generate a node from.
	 * @return The property handle created tied to generated property node.
	 */	
	virtual TSharedPtr<IPropertyHandle> AddObjectPropertyData(TConstArrayView<UObject*> Objects, FName PropertyName) = 0;

	/**
	 * Adds an external structure's property data to this details panel's PropertyMap.
	 * Allows getting the property handle for the property without having to generate a row widget.
	 *
	 * @param StructData    Struct data to find the property within.
	 * @param PropertyName	Name of the property to generate a node from.
	 * @return			    The property handle tied to the generated property node.
	 */
	virtual TSharedPtr<IPropertyHandle> AddStructurePropertyData(const TSharedPtr<FStructOnScope>& StructData, FName PropertyName) = 0;

	/**
	 * Allows for the customization of a property row for a property that already exists on a class being edited in the details panel
	 * The property will remain in the default location but the widget or other attributes for the property can be changed 
	 * Note This cannot be used to customize other customizations

	 * @param InPropertyHandle	The handle to the property that you want to edit
	 * @return					The property row to edit or nullptr if the property row does not exist
	 */
	virtual IDetailPropertyRow* EditDefaultProperty(TSharedPtr<IPropertyHandle> InPropertyHandle) = 0;

	/**
	 * Get the property row from the root of the details panel after it's been constructed, so this will work with default or custom 
	 * properties
	 * @param InPropertyHandle	The handle to the property that you want to edit
	 * @return					The property row to edit or nullptr if the property row does not exist, which may happen if not 
	 * constructed yet
	 */
	virtual IDetailPropertyRow* EditPropertyFromRoot(TSharedPtr<IPropertyHandle> InPropertyHandle) = 0;

	/**
	 * @return true if the category contains child rows. 
	 */
	virtual bool DoesCategoryHaveGeneratedChildren(FName CategoryName) = 0;

	/**
	 * Hides an entire category
	 *
	 * @param CategoryName	The name of the category to hide
	 */
	virtual void HideCategory( FName CategoryName ) = 0;
	
	/**
	 * Gets a handle to a property which can be used to read and write the property value and identify the property in other detail customization interfaces.
	 * Instructions
	 *
	 * @param Path	The path to the property.  Can be just a name of the property or a path in the format outer.outer.value[optional_index_for_static_arrays]
	 * @param ClassOutermost	Optional outer class if accessing a property outside of the current class being customized
	 * @param InstanceName		Optional instance name if multiple FProperty's of the same type exist. such as two identical structs, the instance name is one of the struct variable names)
	    Examples:

		struct MyStruct
		{ 
			int32 StaticArray[3];
			float FloatVar;
		}

		class MyActor
		{ 
			MyStruct Struct1;
			MyStruct Struct2;
			float MyFloat
		}
		
		To access StaticArray at index 2 from Struct2 in MyActor, your path would be MyStruct.StaticArray[2]" and your instance name is "Struct2"
		To access MyFloat in MyActor you can just pass in "MyFloat" because the name of the property is unambiguous
	 */
	virtual TSharedRef<IPropertyHandle> GetProperty( const FName PropertyPath, const UStruct* ClassOutermost = NULL, FName InstanceName = NAME_None ) const = 0;

	/**
	 * Gets the top level property, for showing the warning for experimental or early access class
	 * 
	 * @return the top level property name
	 */

	virtual FName GetTopLevelProperty() = 0;

	/**
	 * Hides a property from view 
	 *
	 * @param PropertyHandle	The handle of the property to hide from view
	 */
	virtual void HideProperty( const TSharedPtr<IPropertyHandle> PropertyHandle ) = 0;

	/**
	 * Hides a property from view
	 *
	 * @param Path						The path to the property.  Can be just a name of the property or a path in the format outer.outer.value[optional_index_for_static_arrays]
	 * @param NewLocalizedDisplayName	Optional display name to show instead of the default name
	 * @param ClassOutermost			Optional outer class if accessing a property outside of the current class being customized
	 * @param InstanceName				Optional instance name if multiple FProperty's of the same type exist. such as two identical structs, the instance name is one of the struct variable names)
	 * See IDetailCategoryBuilder::GetProperty for clarification of parameters
	 */
	virtual void HideProperty( FName PropertyPath, const UStruct* ClassOutermost = NULL, FName InstanceName = NAME_None ) = 0;

	/**
	 * Refreshes the details view and regenerates all the customized layouts
	 * Use only when you need to remove or add complicated dynamic items
	 */
	virtual void ForceRefreshDetails() = 0;

	/**
	 * Gets the thumbnail pool that should be used for rendering thumbnails in the details view                   
	 */
	virtual TSharedPtr<class FAssetThumbnailPool> GetThumbnailPool() const = 0;

	/**
	 * @return true if the property should be visible in the details panel or false if the specific details panel is not showing this property
	 */
	virtual bool IsPropertyVisible( TSharedRef<IPropertyHandle> PropertyHandle ) const = 0;

	/**
	 * @return true if the property should be visible in the details panel or false if the specific details panel is not showing this property
	 */
	virtual bool IsPropertyVisible( const struct FPropertyAndParent& PropertyAndParent ) const = 0;

	/**
	 * @return True if an object in the builder is a class default object
	 */
	virtual bool HasClassDefaultObject() const = 0;

	/**
	 * Registers a custom detail layout delegate for a specific type in this layout only
	 *
	 * @param PropertyTypeName	The type the custom detail layout is for
	 * @param DetailLayoutDelegate	The delegate to call when querying for custom detail layouts for the classes properties
	 */
	virtual void RegisterInstancedCustomPropertyTypeLayout(FName PropertyTypeName, FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate, TSharedPtr<IPropertyTypeIdentifier> Identifier = nullptr) = 0;

	/**
	 * This function sets property paths to generate PropertyNodes.This improves the performance for cases where PropertyView is only showing a few properties of the object by not generating all other PropertyNodes
	 *
	 * @param InPropertyGenerationAllowListPaths Set of the property paths
	 */
	virtual void SetPropertyGenerationAllowListPaths(const TSet<FString>& InPropertyGenerationAllowListPaths) = 0;

	/**
	 * @return True if the property path is contained within our allowed paths
	 */
	virtual bool IsPropertyPathAllowed(const FString& InPath) const = 0;

	/**
	 * Force a property to behave as a normal, peer reference regardless of CPF_InstancedReference
	 */
	virtual void DisableInstancedReference(TSharedRef<IPropertyHandle> PropertyHandle) const = 0;
};

template <typename ObjectType>
TArray<TWeakObjectPtr<ObjectType>> IDetailLayoutBuilder::GetSelectedObjectsOfType() const
{
	TArray<TWeakObjectPtr<UObject>> SelectedObjects = GetSelectedObjects();
	TArray<TWeakObjectPtr<ObjectType>> SelectedObjectsOfType;
	Algo::TransformIf(
		SelectedObjects,
		SelectedObjectsOfType,
		[](const TWeakObjectPtr<UObject>& InObj)
		{
			return InObj.IsValid() && InObj->IsA(ObjectType::StaticClass());			
		},
		[](const TWeakObjectPtr<UObject>& InObj)
		{
			return Cast<ObjectType>(InObj);			
		});
	
	return SelectedObjectsOfType;
}

template <typename ObjectType>
TArray<TWeakObjectPtr<ObjectType>> IDetailLayoutBuilder::GetObjectsOfTypeBeingCustomized() const
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	GetObjectsBeingCustomized(ObjectsBeingCustomized);

	TArray<TWeakObjectPtr<ObjectType>> ObjectsOfTypeBeingCustomized;

	Algo::TransformIf(
		ObjectsBeingCustomized,
		ObjectsOfTypeBeingCustomized,
		[](const TWeakObjectPtr<UObject>& InObj)
		{
			return InObj.IsValid() && InObj->IsA(ObjectType::StaticClass());			
		},
		[](const TWeakObjectPtr<UObject>& InObj)
		{
			return Cast<ObjectType>(InObj);			
		});

	return ObjectsOfTypeBeingCustomized;
}
