// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "DetailRowMenuContext.generated.h"

class IDetailsView;
class IPropertyHandle;

UCLASS(MinimalAPI)
class UDetailRowMenuContext : public UObject
{
	GENERATED_BODY()

public:
	/** Optionally invoke to refresh the widget. */
	TMulticastDelegate<void()>& ForceRefreshWidget() { return ForceRefreshWidgetDelegate; } 
	
	/** PropertyHandles associated with the Row. */
	TArray<TSharedPtr<IPropertyHandle>> PropertyHandles;

	/** Containing DetailsView. */
	TWeakPtr<IDetailsView> DetailsView;
	
private:
	/** Optionally invoke to refresh the widget. */
	TMulticastDelegate<void()> ForceRefreshWidgetDelegate;
};
