// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyUtilities.h"
#include "IDetailTreeNode.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyPath.h"
#include "Misc/Attribute.h"

class FComplexPropertyNode;
class FDetailCategoryImpl;
class FDetailTreeNode;
class FDetailWidgetRow;
class FPropertyNode;
class IDetailPropertyRow;
class IDetailsViewPrivate;
class ITableRow;
class STableViewBase;

struct FDetailFilter;

typedef TArray<TSharedRef<FDetailTreeNode>> FDetailNodeList;

enum class ENodeVisibility : uint8
{
	// Hidden but can be visible if parent is visible due to filtering
	HiddenDueToFiltering,
	// Never visible no matter what
	ForcedHidden,
	// Always visible
	Visible,
};

class FDetailTreeNode : public IDetailTreeNode
{
public:

	/** IDetailTreeNode interface */
	virtual FNodeWidgets CreateNodeWidgets() const;
	virtual void GetChildren(TArray<TSharedRef<IDetailTreeNode>>& OutChildren, const bool& bInIgnoreVisibility = false);
	virtual TSharedPtr<IDetailPropertyRow> GetRow() const override { return nullptr; }
	virtual void GetFilterStrings(TArray<FString>& OutFilterStrings) const override { };
	virtual bool GetInitiallyCollapsed() const override { return false; }

	/** @return The details view that this node is in */
	virtual TSharedPtr<IDetailsViewPrivate> GetDetailsViewSharedPtr() const = 0;

	UE_DEPRECATED(5.5, "Use GetDetailsViewSharedPtr() instead.")
	IDetailsViewPrivate* GetDetailsView() const
	{
		return GetDetailsViewSharedPtr().Get();
	}

	/**
	 * Generates the widget representing this node
	 *
	 * @param OwnerTable		The table owner of the widget being generated
	 * @param PropertyUtilities	Property utilities to help generate widgets
	 */
	virtual TSharedRef< ITableRow > GenerateWidgetForTableView(const TSharedRef<STableViewBase>& OwnerTable, bool bAllowFavoriteSystem) = 0;

	virtual bool GenerateStandaloneWidget(FDetailWidgetRow& OutRow) const = 0;

	/**
	 * Filters this nodes visibility based on the provided filter strings
	 */
	virtual void FilterNode(const FDetailFilter& InFilter) = 0;

	/**
	 * Gets child tree nodes
	 *
	 * @param OutChildren			The array to add children to
	 * @param bInIgnoreVisibility	Ignore the child node visibility
	 */
	virtual void GetChildren(FDetailNodeList& OutChildren, const bool& bInIgnoreVisibility = false) = 0;

	/**
	 * @ The parent DetailTreeNode of this node, or nullptr if this is a root node (or not yet added to a tree)
	 */
	TWeakPtr<FDetailTreeNode> GetParentNode() const { return ParentNode; }

	/**
	 * Sets the parent node. The parent node's GetChildren() call should include this node.
	 *
	 * @param The new parent for this node
	 */
	void SetParentNode(TWeakPtr<FDetailTreeNode> InParentNode) { ParentNode = InParentNode; }

	/**
	 * Finds the immediate UScriptStruct or UClass containing this node, which is found through a series of checks:
	 * 1. Get the corresponding property node and find the nearest object or struct property containing the property
	 * 2. If that fails, each parent node is checked with the same criteria as above
	 * 3. If that fails, the base structure of the detail layout containing this node is used
	 * 4. If that fails, the class of the first selected object in the details view is used
	 *
	 * @return The best matching UStruct that was responsible for creating this node
	 */
	const UStruct* GetParentBaseStructure() const;

	/**
	 * A helper function for GetParentBaseStructure that gets the parent specifically for a property node instead of a detail node
	 */
	static const UStruct* GetPropertyNodeBaseStructure(const FPropertyNode* PropertyNode);

	/**
	 * Called when the item is expanded in the tree
	 */
	virtual void OnItemExpansionChanged(bool bIsExpanded, bool bShouldSaveState) = 0;

	/**
	 * @return Whether or not the tree node should be expanded
	 */
	virtual bool ShouldBeExpanded() const = 0;

	/**
	 * @return the visibility of this node in the tree
	 */
	virtual ENodeVisibility GetVisibility() const = 0;

	/**
	 * updates the visibility of this node if necessary
	 */
	virtual void RefreshVisibility() {}

	/**
	 * Called each frame if the node requests that it should be ticked
	 */
	virtual void Tick(float DeltaTime) = 0;

	/**
	 * @return true to ignore this node for visibility in the tree and only examine children
	 */
	virtual bool ShouldShowOnlyChildren() const = 0;

	/**
	 * The identifier name of the node
	 */
	virtual FName GetNodeName() const = 0;

	/**
	 * @return The category node that this node is nested in, if any:
	 */
	virtual TSharedPtr<FDetailCategoryImpl> GetParentCategory() const { return TSharedPtr<FDetailCategoryImpl>(); }

	/**
	 * @return The property path that this node is associate with, if any:
	 */
	virtual FPropertyPath GetPropertyPath() const { return FPropertyPath(); }

	/**
	 * Called when the node should appear 'highlighted' to draw the users attention to it
	 */
	virtual void SetIsHighlighted(bool bInIsHighlighted) {}

	/**
	 * @return true if the node has been highlighted
	 */
	virtual bool IsHighlighted() const { return false; }

	/**
	 * @return true if this is a leaf node:
	 */
	virtual bool IsLeaf() { return false; }

	/**
	 * @return TAttribute indicating whether editing is enabled or whether the property is readonly:
	 */
	virtual TAttribute<bool> IsPropertyEditingEnabled() const { return false; }

	/** 
	 * Gets the property node associated with this node.  Not all nodes have properties so this will fail for anything other than a property row or for property rows that have complex customizations that ignore the property
	 */
	virtual TSharedPtr<FPropertyNode> GetPropertyNode() const { return nullptr; }
	
	/** 
	 * In certain circumstances a single DetailTreeNode can have multiple property nodes. This will get all of them.
	 */
	virtual void GetAllPropertyNodes(TArray<TSharedRef<FPropertyNode>>& OutNodes) const {}

	/**
	 * Gets the external property node associated with this node.  This will return nullptr for all rows expect property rows which were generated from an external root.
	 */
	virtual TSharedPtr<FComplexPropertyNode> GetExternalRootPropertyNode() const { return nullptr; }

private:

	/** The parent whose GetChildren() call will return this node. This can be null for root nodes or nodes not added to the tree yet. */
	TWeakPtr<FDetailTreeNode> ParentNode = nullptr;
};

