// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailPropertyRow.h"
#include "PropertyHandle.h"
#include "Framework/Commands/UIAction.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SSpacer.h"
#include "DetailCategoryBuilder.h"
#include "PropertyEditorCopyPaste.h"

class FDetailWidgetRow;
class FResetToDefaultOverride;
class IDetailDragDropHandler;

/** Widget declaration for custom widgets in a widget row */
class FDetailWidgetDecl
{
public:
	
	FDetailWidgetDecl( class FDetailWidgetRow& InParentDecl, float InMinWidth, float InMaxWidth, EHorizontalAlignment InHAlign, EVerticalAlignment InVAlign )
		: Widget( SNew( SInvalidDetailWidget ) )
		, HorizontalAlignment( InHAlign )
		, VerticalAlignment( InVAlign )
		, MinWidth( InMinWidth )
		, MaxWidth( InMaxWidth )
		, ParentDecl( &InParentDecl )
	{
	}

	FDetailWidgetDecl( class FDetailWidgetRow& InParentDecl, const FDetailWidgetDecl& Other )
		: Widget( Other.Widget )
		, HorizontalAlignment( Other.HorizontalAlignment )
		, VerticalAlignment( Other.VerticalAlignment )
		, MinWidth( Other.MinWidth )
		, MaxWidth( Other.MaxWidth )
		, ParentDecl( &InParentDecl )
	{
	}

	FDetailWidgetRow& operator[]( TSharedRef<SWidget> InWidget )
	{
		Widget = InWidget;
		return *ParentDecl;
	}

	FDetailWidgetDecl& VAlign( EVerticalAlignment InAlignment )
	{
		VerticalAlignment = InAlignment;
		return *this;
	}

	FDetailWidgetDecl& HAlign( EHorizontalAlignment InAlignment )
	{
		HorizontalAlignment = InAlignment;
		return *this;
	}

	FDetailWidgetDecl& MinDesiredWidth( TOptional<float> InMinWidth )
	{
		MinWidth = InMinWidth;
		return *this;
	}

	FDetailWidgetDecl& MaxDesiredWidth( TOptional<float> InMaxWidth )
	{
		MaxWidth = InMaxWidth;
		return *this;
	}

private:
	class SInvalidDetailWidget : public SSpacer
	{
		SLATE_BEGIN_ARGS( SInvalidDetailWidget )
		{}
		SLATE_END_ARGS()

		void Construct( const FArguments& InArgs )
		{
			SetVisibility(EVisibility::Collapsed);
		}

	};
public:
	TSharedRef<SWidget> Widget;
	EHorizontalAlignment HorizontalAlignment;
	EVerticalAlignment VerticalAlignment;
	TOptional<float> MinWidth;
	TOptional<float> MaxWidth;
private:
	class FDetailWidgetRow* ParentDecl;
};


static FName InvalidDetailWidgetName = TEXT("SInvalidDetailWidget");

/**
 * Represents a single row of custom widgets in a details panel 
 */
class FDetailWidgetRow : public IDetailLayoutRow
{
public:
	PROPERTYEDITOR_API const static float DefaultValueMinWidth;
	PROPERTYEDITOR_API const static float DefaultValueMaxWidth;

	FDetailWidgetRow()
		: NameWidget( *this, 0.0f, 0.0f, HAlign_Left, VAlign_Center )
		, ValueWidget( *this, DefaultValueMinWidth, DefaultValueMaxWidth, HAlign_Left, VAlign_Center )
		, ExtensionWidget( *this, 0.0f, 0.0f, HAlign_Right, VAlign_Center)
		, ResetToDefaultWidget( *this, 0.0f, 0.0f, HAlign_Center, VAlign_Center)
		, WholeRowWidget( *this, 0.0f, 0.0f, HAlign_Fill, VAlign_Fill )
		, VisibilityAttr( EVisibility::Visible )
		, FilterTextString()
		, CopyMenuAction()
		, PasteMenuAction()
		, RowTagName()
	{
	}
	
	FDetailWidgetRow& operator=(const FDetailWidgetRow& Other)
	{
		NameWidget = FDetailWidgetDecl(*this, Other.NameWidget);
		ValueWidget = FDetailWidgetDecl(*this, Other.ValueWidget);
		ExtensionWidget = FDetailWidgetDecl(*this, Other.ExtensionWidget);
		ResetToDefaultWidget = FDetailWidgetDecl(*this, Other.ResetToDefaultWidget);
		WholeRowWidget = FDetailWidgetDecl(*this, Other.WholeRowWidget);
		VisibilityAttr = Other.VisibilityAttr;
		IsEnabledAttr = Other.IsEnabledAttr;
		IsValueEnabledAttr = Other.IsValueEnabledAttr;
		FilterTextString = Other.FilterTextString;
		CopyMenuAction = Other.CopyMenuAction;
		PasteMenuAction = Other.PasteMenuAction;
		CustomMenuItems = Other.CustomMenuItems;
		RowTagName = Other.RowTagName;
		CustomResetToDefault = Other.CustomResetToDefault;
		EditConditionValue = Other.EditConditionValue;
		OnEditConditionValueChanged = Other.OnEditConditionValueChanged;
		CustomDragDropHandler = Other.CustomDragDropHandler;
		PropertyHandles = Other.PropertyHandles;
		return *this;
	}

	virtual ~FDetailWidgetRow() = default;

	/** IDetailLayoutRow interface */
	virtual FName GetRowName() const override { return RowTagName; }
	virtual TOptional<FResetToDefaultOverride> GetCustomResetToDefault() const override { return CustomResetToDefault; }

	/**
	 * Assigns content to the entire row
	 */
	FDetailWidgetRow& operator[]( TSharedRef<SWidget> InWidget )
	{
		WholeRowWidget.Widget = InWidget;
		return *this;
	}

	/**
	* Assigns content to the whole slot, this is an explicit alternative to using []
	*/
	FDetailWidgetDecl& WholeRowContent()
	{
		return WholeRowWidget;
	}

	/**
	 * Assigns content to just the name slot
	 */
	FDetailWidgetDecl& NameContent()
	{
		return NameWidget;
	}

	/**
	 * Assigns content to the value slot
	 */
	FDetailWidgetDecl& ValueContent()
	{
		return ValueWidget;
	}

	/**
     * Assigns content to the extension (right) slot
	 */
	FDetailWidgetDecl& ExtensionContent()
	{
		return ExtensionWidget;
	}

	/**
	 * Assigns content to the reset to default (right-most) slot
	 */
	FDetailWidgetDecl& ResetToDefaultContent()
	{
		return ResetToDefaultWidget;
	}

	/**
	 * Sets a string which should be used to filter the content when a user searches
	 */
	FDetailWidgetRow& FilterString( const FText& InFilterString )
	{
		FilterTextString = InFilterString;
		return *this;
	}

	/**
	 * Sets the visibility of the entire row
	 */
	FDetailWidgetRow& Visibility( const TAttribute<EVisibility>& InVisibility )
	{
		VisibilityAttr = InVisibility;
		return *this;
	}

	/**
	 * Sets the enabled state of the entire row
	 */
	FDetailWidgetRow& IsEnabled( const TAttribute<bool>& InIsEnabled )
	{
		IsEnabledAttr = InIsEnabled;
		return *this;
	}

    /**
     * Sets the enabled state of the value widget only
     */
	FDetailWidgetRow& IsValueEnabled( const TAttribute<bool>& InIsEnabled )
	{
		IsValueEnabledAttr = InIsEnabled;
		return *this;
	}

	/**
	 * Sets a custom copy action to take when copying the data from this row
	 */
	FDetailWidgetRow& CopyAction( const FUIAction& InCopyAction )
	{
		CopyMenuAction = InCopyAction;
		return *this;
	}

	/**
	 * Sets a custom paste action to take when copying the data from this row
	 */
	FDetailWidgetRow& PasteAction( const FUIAction& InPasteAction  )
	{
		PasteMenuAction = InPasteAction;
		return *this;
	}

	/**
	* Add a custom action to the row context menu
	*/
	FDetailWidgetRow& AddCustomContextMenuAction(const FUIAction& Action, const FText& Name, const FText& Tooltip = FText(), const FSlateIcon& SlateIcon = FSlateIcon())
	{
		CustomMenuItems.Add(FCustomMenuData(Action, Name, Tooltip, SlateIcon));
		return *this;
	}

	bool HasNameContent() const 
	{
		return NameWidget.Widget->GetType() != InvalidDetailWidgetName;
	}

	bool HasValueContent() const
	{
		return ValueWidget.Widget->GetType() != InvalidDetailWidgetName;
	}

	bool HasExtensionContent() const
	{
		return ExtensionWidget.Widget->GetType() != InvalidDetailWidgetName;
	}

	bool HasResetToDefaultContent() const
	{
		return ResetToDefaultWidget.Widget->GetType() != InvalidDetailWidgetName;
	}
	
	/**
	 * @return true if the row has columns, false if it spans the entire row
	 */
	bool HasColumns() const
	{
		return HasNameContent() || HasValueContent();
	}

	/**
	 * @return true of the row has any content
	 */
	bool HasAnyContent() const
	{
		return WholeRowWidget.Widget->GetType() != InvalidDetailWidgetName || HasColumns() || HasResetToDefaultContent();
	}

	/** @return true if a custom copy/paste is bound on this row */
	bool IsCopyPasteBound() const
	{
		return CopyMenuAction.ExecuteAction.IsBound() && PasteMenuAction.ExecuteAction.IsBound();
	}

	/** @return true if a custom paste text is bound on this row */
	bool IsPasteFromTextBound() const
	{
		return OnPasteFromTextDelegate.IsValid() && OnPasteFromTextDelegate.Pin()->IsBoundToObject(this);
	}
	
	/** @return true if a custom copy text is bound on this row */
	bool IsCopyToTextBound() const
	{
		return OnCopyToTextDelegate.IsValid() && OnCopyToTextDelegate.Pin()->IsBoundToObject(this);
	}

	/**
	* Sets a tag which can be used to identify this row 
	*/
	FDetailWidgetRow& RowTag(const FName& InRowTagName)
	{
		RowTagName = InRowTagName;
		return *this;
	}

	/**
	* Sets flag to indicate if property value differs from the default
	*/
	FDetailWidgetRow& OverrideResetToDefault(const FResetToDefaultOverride& InResetToDefaultOverride)
	{
		CustomResetToDefault = InResetToDefaultOverride;
		return *this;
	}

	/** 
	 * Override the edit condition.
	 */ 
	FDetailWidgetRow& EditCondition(TAttribute<bool> InEditConditionValue, FOnBooleanValueChanged InOnEditConditionValueChanged)
	{
		EditConditionValue = InEditConditionValue;
		OnEditConditionValueChanged = InOnEditConditionValueChanged;
		return *this;
	}
	
	/**
	 * Sets a handler for the row to be a source or target of drag-and-drop operations.
	 */
	FDetailWidgetRow& DragDropHandler(TSharedPtr<IDetailDragDropHandler> InDragDropHandler)
	{
		CustomDragDropHandler = InDragDropHandler;
		return *this;
	}

	/**
	* Used to provide all the property handles this WidgetRow represent
	*/
	FDetailWidgetRow& PropertyHandleList(const TArray<TSharedPtr<IPropertyHandle>>& InPropertyHandles)
	{
		PropertyHandles = InPropertyHandles;
		return *this;
	}

	/**
	* Return all the property handles this WidgetRow represent
	*/
	const TArray<TSharedPtr<IPropertyHandle>>& GetPropertyHandles() const { return PropertyHandles;  }

	/**
	 * Sets whether or not this property should auto-expand
	 *
	 * @param bForceExpansion	true to force the property to be expanded
	 */
	FDetailWidgetRow& ShouldAutoExpand(bool bForceExpansion = true)
	{
		ForceAutoExpansion = bForceExpansion;
		return *this;
	}

public:
	/** Name column content */
	FDetailWidgetDecl NameWidget;
	/** Value column content */
	FDetailWidgetDecl ValueWidget;
	/** Extension (right) column content */
	FDetailWidgetDecl ExtensionWidget;
	/** Reset to default (right-most) column content */
	FDetailWidgetDecl ResetToDefaultWidget;
	/** Whole row content */
	FDetailWidgetDecl WholeRowWidget;
	/** Visibility of the row */
	TAttribute<EVisibility> VisibilityAttr;
	/** IsEnabled of the row */
	TAttribute<bool> IsEnabledAttr;
    /** IsEnabled of the value widget only */
	TAttribute<bool> IsValueEnabledAttr;
	/** String to filter with */
	FText FilterTextString;
	/** Action for copying data on this row */
	FUIAction CopyMenuAction;
	/** Action for pasting data on this row */
	FUIAction PasteMenuAction;	
	/** Delegate for pasting (optionally tagged) text on this row */
	TWeakPtr<FOnPasteFromText> OnPasteFromTextDelegate;
	/** Optional Delegate for copying text on this row (default property handle copy used if unbound) */
	TWeakPtr<FOnCopyToText> OnCopyToTextDelegate;

	struct FCustomMenuData
	{
		FCustomMenuData(const FUIAction& InAction, const FText& InName, const FText& InTooltip, const FSlateIcon& InSlateIcon)
			: Action(InAction)
			, Name(InName)
			, Tooltip(InTooltip)
			, SlateIcon(InSlateIcon)
		{}

		FCustomMenuData(const FName InEntryName, const FUIAction& InAction, const FText& InName, const FText& InTooltip, const FSlateIcon& InSlateIcon)
			: EntryName(InEntryName)
			, Action(InAction)
			, Name(InName)
			, Tooltip(InTooltip)
			, SlateIcon(InSlateIcon)
		{}

		const FName EntryName;
		const FUIAction Action;
		const FText Name;
		const FText Tooltip;
		const FSlateIcon SlateIcon;

		/** Returns the EntryName if specified, otherwise derive from Name. */
		const FName GetEntryName() const;
	};
	/** Custom Action on this row */
	TArray<FCustomMenuData> CustomMenuItems;
	/* Tag to identify this row */
	FName RowTagName;
	/** Custom reset to default handler */
	TOptional<FResetToDefaultOverride> CustomResetToDefault;
	/** Custom edit condition value. */
	TAttribute<bool> EditConditionValue;
	/** Custom edit condition value changed handler. */
	FOnBooleanValueChanged OnEditConditionValueChanged;
	/** Custom handler for drag-and-drop of the row */
	TSharedPtr<IDetailDragDropHandler> CustomDragDropHandler;
	/* All property handle that this custom widget represent */
	TArray<TSharedPtr<IPropertyHandle>> PropertyHandles;
	/** True to force auto-expansion */
	TOptional<bool> ForceAutoExpansion;
};
