//  Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DetailsViewStyleKey.h"
#include "Framework/Commands/UIAction.h"
#include "Templates/SharedPointer.h"
#include "UserInterface/Widgets/PropertyUpdatedWidgetBuilder.h"
#include "Widgets/SWidget.h"

class FPropertyPath;
DECLARE_DELEGATE(FOnDetailsNeedsUpdate)

class FDetailsViewStyle;
class FComplexPropertyNode;
class FDetailLayoutBuilderImpl;

struct FConstructPropertyUpdatedWidgetBuilderArgs;

/** An @code FDetailsDisplayManager @endcode provides an API to tweak various settings of your details view, and
 * provides some utility methods to work with Details.  */
class FDetailsDisplayManager : public TSharedFromThis<FDetailsDisplayManager>
{
public:
	DECLARE_DELEGATE(FResetToDefault);
	
	FOnDetailsNeedsUpdate OnDetailsNeedsUpdate;

	PROPERTYEDITOR_API FDetailsDisplayManager();

	PROPERTYEDITOR_API virtual ~FDetailsDisplayManager();

	PROPERTYEDITOR_API virtual TOptional<bool> OverrideCreateCategoryNodes() const;

	/**
	 * Returns a boolean indicating if the Component Editor should be hidden 
	 */
	PROPERTYEDITOR_API virtual bool ShouldHideComponentEditor();

	/**
	 * Returns a boolean indicating whether the sub-object editor should show regardless of
	 * any object filter preference settings this would override any value retrieved from ShouldHideComponentEditor.
	 */
	PROPERTYEDITOR_API static bool GetForceShowSubObjectEditor();

	/**
	 * Returns a @code bool @endcode indicating whether this @code DetailsViewObjectFilter @endcode instance
	 * should show a category menu
	 */
	PROPERTYEDITOR_API virtual bool ShouldShowCategoryMenu();

	/**
	 * Gets the category menu SWidget and returns a shared pointer to it
	 *
	 *  @param InCategoryObjectName the name of the category
	 */
	PROPERTYEDITOR_API virtual TSharedPtr<SWidget> GetCategoryMenu(FName InCategoryObjectName);

	/**
	 * Updates the current details view
	 */
	PROPERTYEDITOR_API void UpdateView() const;

	/**
	 * Returns the @code FDetailsViewStyleKey @endcode that is the Key to the current FDetailsViewStyle style
	 */
	PROPERTYEDITOR_API virtual const FDetailsViewStyleKey& GetDetailsViewStyleKey() const;

	/** sets whether the currently active category is an Outer category*/
	void SetIsOuterCategory(bool bInIsOuterCategory);

	/**
	 * Returns the @code FDetailsViewStyle @endcode that is the current FDetailsViewStyle style
	 */
	PROPERTYEDITOR_API const FDetailsViewStyle* GetDetailsViewStyle() const;
	
	    /**
		* Returns a bool indicating whether or not the scrollbar is needed on the details view. Note that the "needed"
		* here means that in this value the work has been done to figure out if the scrollbar should show, and
		* anything can query this to see if it needs to alter the display accordingly
		*/
	PROPERTYEDITOR_API virtual bool GetIsScrollBarNeeded() const;
	
	/**
	* Set a bool indicating whether or not the scrollbar is needed on the details view. Note that the "needed"
	* here means that in this value the work has been done to figure out if the scrollbar should show, and
	* anything can query this to see if it needs to alter the display accordingly
	*
	* @param bInIsScrollBarNeeded a bool indicating whether or not the scrollbar is Needed on the details view
	*/
	PROPERTYEDITOR_API virtual void SetIsScrollBarNeeded(bool bInIsScrollBarNeeded);
	
	/**
	 * Returns the FMargin which provides the padding around the whole details view table
	 */
	FMargin GetTablePadding() const;

	/**
	 * Returns whether or not this display manager provides a widget to overlay over the detail tree.  This can be used to facilitate interactions like drag and drop.
	 */
	virtual bool SupportsDetailTreeOverlay() const { return false; }

	/**
	 * Returns a widget which will be put in an overlay slot which is over the detail tree portion of the widget.
	 * @note Implementations must manage hit test visibility of the supplied widget carefully to prevent blocking UI interactions with the detail tree widgets.
	 */
	PROPERTYEDITOR_API virtual TSharedRef<SWidget> ConstructDetailTreeOverlay();

	/**
	 * If TSharedRef<FComplexPropertyNode> Node has a valid UObject instance, add an empty properties
	 * Category to the DetailLayoutBuilder
	 *
	 * @param Node the FComplexPropertyNode that wil be added as a property to the new Category
	 * @param DetailLayoutBuilder The FDetailLayoutBuilderImpl that will have a stub category added to it
	 *
	 * @return true if the Category was successfully added to FDetailLayoutBuilderImpl& DetailLayoutBuilder,
	 * else it returns false
	 */	
	PROPERTYEDITOR_API virtual bool AddEmptyCategoryToDetailLayoutIfNeeded(TSharedRef<FComplexPropertyNode> Node, TSharedRef<FDetailLayoutBuilderImpl> DetailLayoutBuilder);

	/**
	* Returns a widget which will show in place of the reset to default button, or a nullptr if the default reset button should be used
	*
	* @param ResetToDefault the delegate which should be called to reset the row to default
	* @param bIsCategoryUpdateWidget if true this widget builder is for a Category instead of a property within
	* @param InCategoryObjectName  the name of the UObject associated with the Category for the widget builder, if one exists, else it is NAME_NONE
	*/
	UE_DEPRECATED(5.5, "Use ConstructPropertyUpdatedWidgetBuilder")
	PROPERTYEDITOR_API virtual TSharedPtr<FPropertyUpdatedWidgetBuilder> GetPropertyUpdatedWidget(FResetToDefault ResetToDefault, bool bIsCategoryUpdateWidget = false, FName InCategoryObjectName = NAME_None);

	/**
	* Returns a widget which will show in place of the reset to default button, or a nullptr if the default reset button should be used
	*
	* @param ResetToDefault the delegate which should be called to reset the row to default
	* @param InEditorPropertyChain the FEditorPropertyChain for the FPropertyNode whose state is visualized by this  property updated widget 
	* @param InCategoryObjectName the name of the UObject for which the Category is displayed, if one is associated with the Category
	*/
	UE_DEPRECATED(5.5, "Use ConstructPropertyUpdatedWidgetBuilder")
	PROPERTYEDITOR_API virtual TSharedPtr<FPropertyUpdatedWidgetBuilder> GetPropertyUpdatedWidget(FResetToDefault ResetToDefault, TSharedRef<FEditPropertyChain> InEditorPropertyChain, FName InCategoryObjectName);

	/**
	 * If returning true, the row widgets (ie. SDetailSingleItemRow, SDetailCategoryTableRow) will call ConstructPropertyUpdatedWidgetBuilder.
	 * Enables potentially expensive setup to be elided
	 */
	UE_DEPRECATED(5.5, "Experimental API")
	PROPERTYEDITOR_API virtual bool CanConstructPropertyUpdatedWidgetBuilder() const;

	/**
	 * Returns a builder class which will generate a widget in place of the Extension widgets on an item row.
	 * For non-item rows (ie. category), a widget will be placed in the same location
	 */
	UE_DEPRECATED(5.5, "Experimental API")
	PROPERTYEDITOR_API virtual TSharedPtr<FPropertyUpdatedWidgetBuilder> ConstructPropertyUpdatedWidgetBuilder(const FConstructPropertyUpdatedWidgetBuilderArgs& Args);

	void UpdatePropertyForCategory(FName InCategoryObjectName, FProperty* Property, bool bAddProperty);

	bool GetCategoryHasAnyUpdatedProperties(FName InCategoryObjectName) const;

	/**
	* Returns true if the specified UObject is a Root Node Object and should show an empty Category stub even if it
	* has no UProperty Data, else it returns false
	*
	* @param InNode the @code UObject* @endcode which will be tested to see if it needs an empty category stub
	*/
	PROPERTYEDITOR_API virtual bool ShowEmptyCategoryIfRootUObjectHasNoPropertyData(UObject* InNode) const;

protected:

	
	/**
	 * The primary style key for the details view. 
	 */
	FDetailsViewStyleKey PrimaryStyleKey;

	/**
	 * A bool indicating whether or not the currently active category is an outer category
	 */
	bool bIsOuterCategory;

	/**
	* a bool indicating whether or not the scrollbar is needed on the details view. Note that the "needed"
	* here means that in this value the work has been done to figure out if the scrollbar should show 
	 */
	bool bIsScrollBarNeeded = false;

	/** A map of category object name to a set of properties that has been updated for it */
	TMap<FName, TSet<FProperty*>> CategoryNameToUpdatePropertySetMap;
};

struct FConstructPropertyUpdatedWidgetBuilderArgs
{
	~FConstructPropertyUpdatedWidgetBuilderArgs();
	
	FExecuteAction ResetToDefaultAction;
	TSharedPtr<FPropertyPath> PropertyPath;
	FName Category = NAME_None;
	FExecuteAction InvalidateCachedState;
	TArray<TWeakObjectPtr<UObject>>* Objects = nullptr;
};
