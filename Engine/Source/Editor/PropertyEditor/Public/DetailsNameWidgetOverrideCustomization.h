// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FPropertyPath;
class SWidget;

/**
 * A customization applied to the whole details panel which can update any Name widget
 */
class FDetailsNameWidgetOverrideCustomization : public TSharedFromThis<FDetailsNameWidgetOverrideCustomization>
{
public:
	/**
	 * default constructor
	 */
	PROPERTYEDITOR_API explicit FDetailsNameWidgetOverrideCustomization();

	/**
	 * destructor
	 */
	virtual ~FDetailsNameWidgetOverrideCustomization() = default;

	/**
	 * @param InnerNameContent the unaltered content of the Name widget. It shows what would
	 * be there normally with no modification by this customization
	 * @param Path the FPropertyPath for the current property
	 * @return the new TSharedRef<SWidget> for the Name widget if one is provided, else it returns
	 * the default Name widget for the current property path
	 */
	 PROPERTYEDITOR_API virtual TSharedRef<SWidget> CustomizeName( TSharedRef<SWidget> InnerNameContent,  FPropertyPath& Path);
};
