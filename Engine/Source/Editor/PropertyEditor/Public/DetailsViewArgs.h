// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DetailsViewStyleKey.h"
#include "Widgets/Layout/SScrollBar.h"

class FDetailsNameWidgetOverrideCustomization;
class FDetailsViewObjectFilter;
class FExtender;
class FNotifyHook;
class FTabManager;
class FUICommandList;
class IClassViewerFilter;
class FDetailsViewStyleKey;
class FDetailsViewStyleKeys;
class IPropertyHandle;

enum class EEditDefaultsOnlyNodeVisibility : uint8
{
	/** Always show nodes that have the EditDefaultsOnly aka CPF_DisableEditOnInstance flag. */
	Show,
	/** Always hide nodes that have the EditDefaultsOnly aka CPF_DisableEditOnInstance flag. */
	Hide,
	/** Let the details panel control it. If the CDO is selected EditDefaultsOnly nodes will be visible, otherwise false. */
	Automatic,
};

/**
 * Init params for a details view widget
 */
struct FDetailsViewArgs
{
	enum ENameAreaSettings
	{
		/** The name area should never be displayed */
		HideNameArea = 0,
		/** All object types use the name area */
		ObjectsUseNameArea = 1<<0,
		/** Actors prioritize using the name area */
		ActorsUseNameArea = 1<<1,
		/** Components and actors prioritize using the name area. Components will display their actor owner as the name */
		ComponentsAndActorsUseNameArea = 1<<2 | ActorsUseNameArea,
	};

	/** Enum to set expansion persistence. Used to support saving expansion per details view. */
	enum class EExpansionPersistenceMethod : uint8
	{
		/** Expansion will use default persistence (Global shared per struct type) */
		Default = 0,
		/** 
		 * Expansion will be stored transiently per details view identifier and persist in a single editor session.
		 * The Expansion will be written to EditorPerProjectUserSettings.ini, but will be cleared without read on editor startup.
		 */
		TransientPerViewIdentifier = 1,
		/** Expansion will be stored per details view identifier and persist on disk. */
		PersistentPerViewIdentifier = 2,
	};

	/** Controls how CPF_DisableEditOnInstance nodes will be treated */
	EEditDefaultsOnlyNodeVisibility DefaultsOnlyVisibility;
	/** The command list from the host of the details view, allowing child widgets to bind actions with a bound chord */
	TSharedPtr<FUICommandList> HostCommandList;
	/** The tab manager from the host of the details view, allowing child widgets to spawn tabs */
	TSharedPtr<FTabManager> HostTabManager;
	/** Optional object filter to use for more complex handling of what a details panel is viewing. */
	TSharedPtr<FDetailsViewObjectFilter> ObjectFilter;
	/** Optional custom filter(s) to apply to the class viewer widget for class object property values. */
	TArray<TSharedRef<IClassViewerFilter>> ClassViewerFilters;
	/** Externally constructed Scrollbar (internally constructed if not provided) */
	TSharedPtr<SScrollBar> ExternalScrollbar;
	/** whether to display scrollbar on the left or the right of the panel */
	EHorizontalAlignment ScrollbarAlignment;
	/** A @code TSharedPtr<FDetailsViewStyleKey> @endcode which provides the key to the Details View's style */
	TSharedPtr<FDetailsViewStyleKey> StyleKey;
	/** A Customization that can be applied to all name widgets */
	TSharedPtr<FDetailsNameWidgetOverrideCustomization> DetailsNameWidgetOverrideCustomization;
	/** If bShowOptions menu is available, adds support for user defined menu options*/
	TSharedPtr<FExtender> OptionsExtender;
	/** Identifier for this details view; NAME_None if this view is anonymous */
	FName ViewIdentifier;
	/** Persistence method for expansion in this details view */
	EExpansionPersistenceMethod ExpansionPersistenceMethod;
	/** Notify hook to call when properties are changed */
	FNotifyHook* NotifyHook;
	/** Settings for displaying the name area (@see ENameAreaSettings) */
	int32 NameAreaSettings;
	/** The default value column width, as a percentage, 0-1. */
	float ColumnWidth;
	/** The minimum width of the right column in Slate units. */
	float RightColumnMinWidth;
	/** True if the viewed objects updates from editor selection */
	bool bUpdatesFromSelection : 1;
	/** True if this property view can be locked */
	bool bLockable : 1;
	/** True if we allow searching */
	bool bAllowSearch : 1;
	/** True if you want to not show the tip when no objects are selected (should only be used if viewing actors properties or bObjectsUseNameArea is true ) */
	bool bHideSelectionTip : 1;
	/** True if you want the search box to have initial keyboard focus */
	bool bSearchInitialKeyFocus : 1;
	/** True if the 'Open Selection in Property Matrix' button should be shown */
	bool bShowPropertyMatrixButton : 1;
	/** Allow options to be changed */
	bool bShowOptions : 1;
	/** Allow headers to be hidden */
	bool bShowHeaders : 1;
	/** True if you want to show the object label */
	bool bShowObjectLabel : 1;
	UE_DEPRECATED(5.0, "bShowActorLabel has been renamed bShowObjectLabel")
	bool bShowActorLabel : 1;
	/** True if you want to show the 'Show Only Modified Properties' option. Only valid in conjunction with bShowOptions */
	bool bShowModifiedPropertiesOption : 1;
	/** True if you want to show the 'Show Only Differing Properties' option. Only valid in conjunction with bShowOptions. */
	bool bShowDifferingPropertiesOption : 1;
	/** True if you want to show the 'Show Hidden Properties While Playing' option. Only valid in conjunction with bShowOptions. */
	bool bShowHiddenPropertiesWhilePlayingOption : 1;	
	/** If true, the name area will be created but will not be displayed so it can be placed in a custom location.  */
	bool bCustomNameAreaLocation : 1;
	/** If true, the filter area will be created but will not be displayed so it can be placed in a custom location.  */
	bool bCustomFilterAreaLocation : 1;
	/** If false, the current properties editor will never display the favorite system */
	bool bAllowFavoriteSystem : 1;
	/** If true the details panel will assume each object passed in through SetObjects will be a unique object shown in the tree and not combined with other objects */
	bool bAllowMultipleTopLevelObjects : 1;
	/** If false, the details panel's scrollbar will always be hidden. Useful when embedding details panels in widgets that either grow to accommodate them, or with scrollbars of their own. */
	bool bShowScrollBar : 1;
	/** If true, all properties will be visible, not just those with CPF_Edit */
	bool bForceHiddenPropertyVisibility : 1;
	/** True if you want to show the 'Show Only Keyable Properties'. Only valid in conjunction with bShowOptions */
	bool bShowKeyablePropertiesOption : 1;
	/** True if you want to show the 'Show Only Animated Properties'. Only valid in conjunction with bShowOptions */
	bool bShowAnimatedPropertiesOption: 1;
	/** True if you want to show a custom filter. */
	bool bShowCustomFilterOption : 1;
	/** True if the section selector should be shown. */
	bool bShowSectionSelector : 1;
	/** If true, loose properties should be shown. */
	bool bShowLooseProperties : 1;
	
	/** Delegate that determines whether a property should be forced hidden - evaluated on panel refresh */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FShouldForceHideProperty, const TSharedRef<IPropertyHandle>&);
	FShouldForceHideProperty ShouldForceHideProperty;

	/** Resolve any UObject* to their IDO counterparts if they exist */
	TOptional<bool> bResolveInstanceDataObjects;

	/** Initial sections to be shown enabled in the filter */
	TSet<FName> InitialSections;

	/** Initial categories to be shown enabled in the filter */
	TSet<FName> InitialCategories;

public:

	FDetailsViewArgs()
		: DefaultsOnlyVisibility(EEditDefaultsOnlyNodeVisibility::Show)
		, ScrollbarAlignment(HAlign_Right)
	    , DetailsNameWidgetOverrideCustomization(nullptr)
		, ViewIdentifier(NAME_None)
		, ExpansionPersistenceMethod(EExpansionPersistenceMethod::Default)
		, NotifyHook(nullptr)
		, NameAreaSettings(ActorsUseNameArea)
		, ColumnWidth(.6f)
		, RightColumnMinWidth(22)
		, bUpdatesFromSelection(false)
		, bLockable(false)
		, bAllowSearch(true)
		, bHideSelectionTip(false)
		, bSearchInitialKeyFocus(false)
		, bShowPropertyMatrixButton(true)
		, bShowOptions(true)
		, bShowHeaders(true)
		, bShowObjectLabel(true)
		, bShowModifiedPropertiesOption(true)
		, bShowDifferingPropertiesOption(false)
		, bShowHiddenPropertiesWhilePlayingOption(true)
		, bCustomNameAreaLocation(false)
		, bCustomFilterAreaLocation(false)
		, bAllowFavoriteSystem(false)	
		, bAllowMultipleTopLevelObjects(false)
		, bShowScrollBar(true)
		, bForceHiddenPropertyVisibility(false)
		, bShowKeyablePropertiesOption(true)
		, bShowAnimatedPropertiesOption(true)
		, bShowCustomFilterOption(false)
		, bShowSectionSelector(false)
		, bShowLooseProperties(false)
	{
	}

	/** Default constructor */
	UE_DEPRECATED(5.0, "This constructor is deprecated, please create an empty FDetailsViewArgs and explicitly set any values that you wish to change.")
	FDetailsViewArgs( const bool InUpdateFromSelection
					, const bool InLockable = false
					, const bool InAllowSearch = true
					, const ENameAreaSettings InNameAreaSettings = ActorsUseNameArea
					, const bool InHideSelectionTip = false
					, FNotifyHook* InNotifyHook = NULL
					, const bool InSearchInitialKeyFocus = false
					, FName InViewIdentifier = NAME_None )
		: FDetailsViewArgs()
	{
		bUpdatesFromSelection = InUpdateFromSelection;
		bLockable = InLockable;
		bAllowSearch = InAllowSearch;
		NameAreaSettings = InNameAreaSettings;
		bHideSelectionTip = InHideSelectionTip;
		NotifyHook = InNotifyHook;
		bSearchInitialKeyFocus = InSearchInitialKeyFocus;
		ViewIdentifier = InViewIdentifier;
	}

	FDetailsViewArgs(const FDetailsViewArgs&);
	FDetailsViewArgs& operator=(const FDetailsViewArgs&);
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
inline FDetailsViewArgs::FDetailsViewArgs(const FDetailsViewArgs&) = default;
inline FDetailsViewArgs& FDetailsViewArgs::operator=(const FDetailsViewArgs&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
