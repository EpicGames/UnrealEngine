// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DetailsViewStyleKey.h"
#include "Widgets/SWidget.h"
#include "Templates/SharedPointer.h"
#include "DetailsDisplayManager.h"

DECLARE_DELEGATE(FOnUpdateFilteredObjects)

/**
 * An object root is a collection of UObjects that represent a top level set of properties in a details panel
 * When there are multiple objects in the root, the common base class for all those objects are found and the properties on that common base class are displayed
 * When a user edits one of those properties, it will propagate the result to all objects in the root.  
 * If multiple differing values are found on a single property, the details panel UI displays "multiple values" 
 */
struct FDetailsViewObjectRoot
{
	FDetailsViewObjectRoot()
	{}

	FDetailsViewObjectRoot(UObject* InObject)
	{
		Objects.Add(InObject);
	}

	FDetailsViewObjectRoot(TArray<UObject*> InObjects)
		: Objects(MoveTemp(InObjects))
	{}

	TArray<UObject*> Objects;
};

/**
 * An object filter determines the root objects that should be displayed from a set of given source objects passed to the details panel.
 * It can also be used to convey characteristics of a details view which may alter depending upon the type of objects
 * that have been filtered.
 */
class FDetailsViewObjectFilter
{
public:

	explicit FDetailsViewObjectFilter()
	{
		DisplayManager = MakeShared<FDetailsDisplayManager>();
	}

	virtual ~FDetailsViewObjectFilter() = default;

	/**
	 * Given a const TArray<UObject*>& SourceObjects, it fills a TArray<FDetailsViewObjectRoot> with the objects
	 * that we need as details objects roots. This may be the same as the original SourceObjects, or it may be
	 * some subset of them, or some of their Sub-objects
	 *
	 * @param SourceObjects the array of objects acting as the Source of the root objects for the details view
	 * @return the TArray<FDetailsViewObjectRoot> with the objects that will act as root objects for the details view 
	 */
	virtual TArray<FDetailsViewObjectRoot> FilterObjects(const TArray<UObject*>& SourceObjects) = 0;

	/** Updates the view of anything being filtered by this filter */
	virtual void UpdateFilterView() const
	{
	}

	/**
	* Returns a @code TSharedPtr @endcode to the @code FDetailsDisplayManager @endcode
	*/
	virtual TSharedPtr<FDetailsDisplayManager> GetDisplayManager()
	{
		return DisplayManager;
	}


protected:
	/**
	 * The @code DetailsDisplayManager @endcode which provides an API to manage some of the characteristics of the
	 * details display
	 */
	TSharedPtr<FDetailsDisplayManager> DisplayManager;
	
};
 
