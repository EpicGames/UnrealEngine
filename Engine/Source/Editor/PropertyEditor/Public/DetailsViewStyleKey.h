//  Copyright Epic Games, Inc. All Rights Reserved.


#pragma once
#include "UObject/NameTypes.h"


/**
 * A class which provides a strongly typed Key object for @code FDetailsViewStyleKey @endcode instances
 */
class FDetailsViewStyleKey
{
public:
	/**
	 * Default constructor for @code FDetailsViewStyleKey @endcode instances
	 */ 
	PROPERTYEDITOR_API FDetailsViewStyleKey();

	/**
	 * Copy Constructor for @code FDetailsViewStyleKey @endcode instances
	 */
	PROPERTYEDITOR_API FDetailsViewStyleKey(FDetailsViewStyleKey& Key);

	/**
	 * const Copy Constructor for @code FDetailsViewStyleKey @endcode instances
	 */
	PROPERTYEDITOR_API FDetailsViewStyleKey(const FDetailsViewStyleKey& Key);

	/**
	 * Assignment operator for @code FDetailsViewStyleKey @endcode instances.
	 *
	 * @param OtherStyleKey the key for the @code FDetailsViewStyleKey @endcode with which to initialize this @code FDetailsViewStyleKey @endcode
	 */
	PROPERTYEDITOR_API FDetailsViewStyleKey& operator=(FDetailsViewStyleKey& OtherStyleKey);

	/**
	 * const assignment operator @code FDetailsViewStyleKey @endcode instances.
	 *
	 * @param OtherStyleKey the key for the @code FDetailsViewStyleKey @endcode with which to initialize this @code FDetailsViewStyleKey @endcode
	 */
	PROPERTYEDITOR_API FDetailsViewStyleKey& operator=(const FDetailsViewStyleKey& OtherStyleKey);

	PROPERTYEDITOR_API bool operator==(const FDetailsViewStyleKey& OtherStyleKey) const;

	/**
	 * Returns the FName Name of this @code FDetailsViewStyleKey @endcode 
	 */
	PROPERTYEDITOR_API FName GetName() const;

	/**
	 * A constructor for @code FDetailsViewStyleKey @endcode instances which takes an FName InName to initialize the leu
	 */
	PROPERTYEDITOR_API FDetailsViewStyleKey(FName InName);

private:
	FName Name;

};

/**
 * A Class which provides keys for @code FDetailsViewStyleKey @endcode instances
 */
class FDetailsViewStyleKeys 
{
public:

	/**
	 * Returns The Classic Details View style used in Unreal Engine 5.0 and prior, which has Categories displayed with no
	 * padding between them and no built in horizontal space 
	 */
	static PROPERTYEDITOR_API const FDetailsViewStyleKey& Classic();
	
	/**
	* Returns A Details View style which presents with the top level Categories in a card-like fashion, with space between
	* and around them.
	*/
	static PROPERTYEDITOR_API const FDetailsViewStyleKey& Card();

	/**
	 * The @code Default @endcode Details View style should be used when an object has no interest in displaying its own custom
	 * Details View and wants to default to whatever is the Primary Details View.
	 */
	static PROPERTYEDITOR_API const FDetailsViewStyleKey& Default();

	/**
	* Returns true if the @code const FDetailsViewStyleKey& @endcode is the @code Default @endcode Key, else it returns false.
	*
	* @param StyleKey a FDetailsViewStyleKey& to check whether it is the default 
	*/
	static bool PROPERTYEDITOR_API IsDefault(const FDetailsViewStyleKey& StyleKey);
};


