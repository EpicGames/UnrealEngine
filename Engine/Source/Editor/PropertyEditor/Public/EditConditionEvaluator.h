// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FProperty;
class UScriptStruct;

namespace EditConditionEvaluator
{
	/**
	 * Evaluates the EditCondition metadata of a property against a live struct instance.
	 *
	 * Returns true  if the condition is absent or evaluates to true (property is editable).
	 * Returns true  (fail open) if the condition cannot be parsed or evaluated, so callers
	 *               never incorrectly suppress a property due to an unrecognised expression.
	 * Returns false if the condition evaluates to false (property is not editable).
	 *
	 * Supports all condition types handled by the details panel:
	 *   bool / negation        bEnable, !bEnable
	 *   enum equality          DataType == EFoo::Bar, DataType != EFoo::Bar
	 *   numeric comparisons    Count > 0, Scale <= 1.0
	 *   logical operators      bA && DataType == EFoo::Bar, bA || bB
	 *
	 * Function-call conditions (e.g. "SomeFunction()") are not supported and fail open.
	 */
	UE_EXPERIMENTAL(5.8, "The behavior of IsPropertyEditable may be subject to change in the future")
	PROPERTYEDITOR_API bool IsPropertyEditable(
		const FProperty* Property,
		const UScriptStruct* Struct,
		const void* Memory);

} // namespace EditConditionEvaluator
