// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DetailBuilderTypes.h"

class FDetailWidgetRow;
class FStructOnScope;
class IDetailCategoryBuilder;
class IDetailCustomNodeBuilder;
class IDetailGroup;
class IDetailPropertyRow;
class IPropertyHandle;
class IStructureDataProvider;
class SWidget;

/**
 * Builder for adding children to a detail customization
 */
class IDetailChildrenBuilder
{
public:
	virtual ~IDetailChildrenBuilder() = default;


	/**
	 * Adds a custom builder as a child
	 *
	 * @param InCustomBuilder		The custom builder to add
	 */
	virtual IDetailChildrenBuilder& AddCustomBuilder(TSharedRef<IDetailCustomNodeBuilder> InCustomBuilder) = 0;

	/**
	 * Adds a group to the category
	 *
	 * @param GroupName	The name of the group
	 * @param LocalizedDisplayName	The display name of the group
	 * @param bStartExpanded Set to true to force the category to expand by default
	 * @param true if the group should appear in the advanced section of the category
	 */
	virtual IDetailGroup& AddGroup(FName GroupName, const FText& LocalizedDisplayName, const bool bStartExpanded=false) = 0;

	/**
	 * Get an existing group in the category
	 *
	 * @param GroupName	The name of the group
	 * @return a pointer to the group if it exists, otherwise nullptr
	 */
	virtual IDetailGroup* GetGroup(FName GroupName) = 0;

	/**
	 * Adds new custom content as a child to the struct
	 *
	 * @param SearchString	Search string that will be matched when users search in the details panel.  If the search string doesnt match what the user types, this row will be hidden
	 * @return A row that accepts widgets
	 */

	virtual FDetailWidgetRow& AddCustomRow(const FText& SearchString) = 0;

	/**
	 * Adds a property to the struct
	 *
	 * @param PropertyHandle	The handle to the property to add
	 * @return An interface to the property row that can be used to customize the appearance of the property
	 */
	virtual IDetailPropertyRow& AddProperty(TSharedRef<IPropertyHandle> PropertyHandle) = 0;

	/**
	 * Adds a set of objects to as a child.  Similar to details panels, all objects will be visible in the details panel as set of properties from the common base class from the list of objects
	 *
	 * @param  Objects			The objects to add
	 * @param  UniqueIdName		Optional identifier that uniquely identifies this object among other objects of the same type.  If this is empty, saving and restoring expansion state of this object may not work
	 * @return The header row generated for this set of objects by the details panel
	 */
	virtual IDetailPropertyRow* AddExternalObjects(const TArray<UObject*>& Objects, FName UniqueIdName = NAME_None) = 0;

	/**
	 * Adds a set of objects to as a child.  Similar to details panels, all objects will be visible in the details panel as set of properties from the common base class from the list of objects
	 *
	 * @param  Objects			The objects to add
	 * @param  Params			Optional parameters for customizing the display of the property rows.
	 * @return The header row generated for this set of objects by the details panel
	 */
	virtual IDetailPropertyRow* AddExternalObjects(const TArray<UObject*>& Objects, const FAddPropertyParams& Params = FAddPropertyParams()) = 0;

	/**
	 * Adds a set of objects to as a child.  Similar to details panels, all objects will be visible in the details panel as set of properties from the common base class from the list of objects
	 *
	 * @param  Objects			The objects to add
	 * @param  PropertyName		Name of a property inside the object(s) to add.
	 * @param  Params			Optional parameters for customizing the display of the property rows.
	 * @return The header row generated for this set of objects by the details panel
	 */
	virtual IDetailPropertyRow* AddExternalObjectProperty(const TArray<UObject*>& Objects, FName PropertyName, const FAddPropertyParams& Params = FAddPropertyParams()) = 0;


	/**
	 * Adds a property from a custom structure as a child
	 *
	 * @param ChildStructure	The structure to add
	 * @param PropertyName		Optional name of a property inside the Child structure to add.  If this is empty, the entire structure will be added
	 * @param Params		    Optional identifier that uniquely identifies this structure among other structures of the same type.  If this is empty, saving and restoring expansion state of this structure may not work
	 */
	virtual IDetailPropertyRow* AddExternalStructureProperty(TSharedRef<FStructOnScope> ChildStructure, FName PropertyName, const FAddPropertyParams& Params = FAddPropertyParams()) = 0;
	virtual IDetailPropertyRow* AddExternalStructureProperty(TSharedPtr<IStructureDataProvider> ChildStructure, FName PropertyName, const FAddPropertyParams& Params = FAddPropertyParams()) = 0;
	virtual IDetailPropertyRow* AddChildStructureProperty(TSharedRef<IPropertyHandle> PropertyHandle, TSharedPtr<IStructureDataProvider> ChildStructure, FName PropertyName, const FAddPropertyParams& Params = FAddPropertyParams(), const FText& DisplayNameOverride = FText()) = 0;

	/**
	 * Adds a custom structure as a child
	 *
	 * @param ChildStructure	The structure to add
	 * @param UniqueIdName		Optional identifier that uniquely identifies this structure among other structures of the same type.  If this is empty, saving and restoring expansion state of this structure may not work
	 */
	virtual IDetailPropertyRow* AddExternalStructure(TSharedRef<FStructOnScope> ChildStructure, FName UniqueIdName = NAME_None) = 0;
	virtual IDetailPropertyRow* AddExternalStructure(TSharedPtr<IStructureDataProvider> ChildStructure, FName UniqueIdName = NAME_None) = 0;
	virtual IDetailPropertyRow* AddChildStructure(TSharedRef<IPropertyHandle> PropertyHandle, TSharedPtr<IStructureDataProvider> ChildStructure, FName UniqueIdName = NAME_None, const FText& DisplayNameOverride = FText()) = 0;

	/**
	 * Adds all the properties of an external structure as a children
	 *
	 * @param ChildStructure	The structure containing the properties to add
	 * @return An array of interfaces to the properties that were added
	 */
	virtual TArray<TSharedPtr<IPropertyHandle>> AddAllExternalStructureProperties(TSharedRef<FStructOnScope> ChildStructure) = 0;

	virtual TArray<TSharedPtr<IPropertyHandle>> AddAllExternalStructureProperties(TSharedPtr<IStructureDataProvider> ChildStructure) = 0;
	
	/**
	 * Generates a value widget from a customized struct
	 * If the customized struct has no value widget an empty widget will be returned
	 *
	 * @param StructPropertyHandle	The handle to the struct property to generate the value widget from
	 */
	virtual TSharedRef<SWidget> GenerateStructValueWidget(TSharedRef<IPropertyHandle> StructPropertyHandle) = 0;

	/**
	 * @return the parent category on the customized object that this children is in.
	 */
	virtual IDetailCategoryBuilder& GetParentCategory() const = 0;

	/**
	* @return the parent group on the customized object that this children is in (if there is one)
	*/
	virtual IDetailGroup* GetParentGroup() const = 0;
};
