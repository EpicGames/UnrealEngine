// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;

DECLARE_DELEGATE_OneParam(FOnToggleNodeExpansion, bool)
/**
 * A custom node that can be given to a details panel category to customize widgets
 */
class IDetailCustomNodeBuilder
{
public:
	virtual ~IDetailCustomNodeBuilder()= default;

	/**
	 * Sets a delegate that should be used when the custom node needs to rebuild children
	 *
	 * @param A delegate to invoke when the tree should rebuild this nodes children
	 */
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) {}

	/**
	 * Sets a delegate that should be used when the custom node wants to expand
	 *
	 * @param A delegate to invoke when the tree should expand
	 */
	virtual void SetOnToggleExpansion(FOnToggleNodeExpansion InOnToggleExpansion) {}

	/**
	 * Called to generate content in the header of this node ( the actual node content ). 
	 * Only called if HasHeaderRow is true
	 *
	 * @param NodeRow The row to put content in
	 */
	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) = 0;

	/**
	 * Called to generate child content of this node
	 *
	 * @param ChildrenBuilder The builder to add child rows to.
	 */
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) {}

	/**
	 * Called each tick if RequiresTick is true
	 */
	virtual void Tick(float DeltaTime) {}

	/**
	 * @return true if this node requires tick, false otherwise
	 */
	virtual bool RequiresTick() const { return false; }

	/**
	 * @return true if this node should be collapsed in the tree
	 */
	virtual bool InitiallyCollapsed() const { return true; }

	/**
	 * @return The name of this custom builder.  This is used as an identifier to save expansion state if needed
	 */
	virtual FName GetName() const = 0;

	/**
	 * @return The property associated with this builder (if any).
	 */
	virtual TSharedPtr<IPropertyHandle> GetPropertyHandle() const { return nullptr; }
};
