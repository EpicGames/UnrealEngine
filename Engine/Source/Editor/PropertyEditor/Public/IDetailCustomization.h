// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class IDetailLayoutBuilder;

/** 
 * Interface for any class that lays out details for a specific class
 */
class IDetailCustomization : public TSharedFromThis<IDetailCustomization>
{
public:
	virtual ~IDetailCustomization() = default;

	/** Called when no longer used and will be deleted */
	virtual void PendingDelete() {}

	/** Called when details should be customized */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) = 0;

	/** 
	 * Called when details should be customized, this version allows the customization to store a weak ptr to the layout builder.
	 * This allows property changes to trigger a force refresh of the detail panel.
	 */
	virtual void CustomizeDetails( const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder )  { CustomizeDetails(*DetailBuilder); }

};
