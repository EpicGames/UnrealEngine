// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Views/STableRow.h"

class FDragDropEvent;
class FDragDropOperation;

/** Specifies how a widget row can initiate a drag-and-drop (be dragged) and/or act as the target of a drag-and-drop (be dropped onto). */
class IDetailDragDropHandler
{
public:
	virtual ~IDetailDragDropHandler() = default;

	/** Create a drag operation with the current widget row as the source. */
	virtual TSharedPtr<FDragDropOperation> CreateDragDropOperation() const = 0;

	/** Accepts an arbitrary drag-and-drop (doesn't need to be from another widget row) with the current widget row as the target. */
	virtual bool AcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const = 0;

	/**
	 * Determines whether the drag-and-drop event contains data that can be accepted by the current widget row as the target.
	 * The handler can return a different drop zone if desired, or no drop zone if it can't accept the drop.
	 */
	virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const = 0;

	/** Determines whether the reorder handle widget is added automatically. */
	virtual bool UseHandleWidget() const { return true; }
};
