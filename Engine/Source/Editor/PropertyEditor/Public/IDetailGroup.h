// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailCategoryBuilder.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class IDetailPropertyRow;
class IPropertyHandle;

/**
 * How the group should be formatted - either as a minor group, or identically to a major subcategory.
 */
enum class EDetailGroupDisplayMode
{
	Group,
	Category
};

/**
 * A group in the details panel that can have children
 */
class IDetailGroup : public IDetailLayoutRow
{
public:
	virtual ~IDetailGroup() = default;

	/**
	 * Makes a custom row for the groups header
	 *
	 * @return a new row for customizing the header
	 */
	virtual FDetailWidgetRow& HeaderRow() = 0;

	/**
	 * Adds a property as the groups header      
	 */
	virtual IDetailPropertyRow& HeaderProperty( TSharedRef<IPropertyHandle> PropertyHandle ) = 0;

	/** 
	 * Adds a new row for custom widgets *
	 * 
	 * @return a new row for adding widgets
	 */
	virtual FDetailWidgetRow& AddWidgetRow() = 0;

	/**
	 * Adds a new row for a property
	 *
	 * @return an interface for customizing the appearance of the property row
	 */
	virtual IDetailPropertyRow& AddPropertyRow(TSharedRef<IPropertyHandle> PropertyHandle) = 0;

	/**
	 * Adds a new row for an external struct property
	 *
	 * @return an interface for customizing the appearance of the property row
	 */
	virtual IDetailPropertyRow& AddExternalStructureProperty(TSharedPtr<IStructureDataProvider> StructDataProvider, FName PropertyName, EPropertyLocation::Type Location, const FAddPropertyParams& Params) = 0;

	/**
	 * Adds a new row for an external property
	 *
	 * @return an interface for customizing the appearance of the property row
	 */
	virtual IDetailPropertyRow& AddExternalObjectProperty(const TArray<UObject*>& Objects, FName PropertyName, EPropertyLocation::Type Location, const FAddPropertyParams& Params) = 0;

	/**
	 * Adds a group to the category
	 *
	 * @param NewGroupName	The name of the group
	 * @param LocalizedDisplayName	The display name of the group
	 * @param true if the group should start expanded
	 */
	virtual IDetailGroup& AddGroup(FName NewGroupName, const FText& InLocalizedDisplayName, bool bInStartExpanded = false) = 0;

	/**
	 * Gets the localized tooltip for this group.
	 */
	virtual const TOptional<FText>& GetToolTip() const = 0;

	/**
	 * Sets the localized tooltip for this group.
	 */
	virtual void SetToolTip(const FText& ToolTip) = 0;

	/**
	 * Toggles expansion on the group
	 *
	 * @param bExpand	true to expand the group, false to collapse
	 */
	virtual void ToggleExpansion( bool bExpand ) = 0;

	/**
	 * Gets the current state of expansion for the group
	 */
	virtual bool GetExpansionState() const = 0;

	/**
	* Permit resetting the properties in this group
	*/
	virtual void EnableReset(bool InValue) = 0;

	/**
	* Delegate called when user press the Group Reset ui
	*/
	DECLARE_MULTICAST_DELEGATE(FDetailGroupReset);

	/**
	* Return the delegate called when user press the Group Reset ui
	*/
	virtual FDetailGroupReset& GetOnDetailGroupReset() = 0;

	/**
	* Return the name associated with this group
	*/
	virtual FName GetGroupName() const = 0;

	/**
	 * Returns the number of child rows under this group
	 */
	virtual int32 GetNumChildren() const = 0;

	/**
	* Return the property row associated with the specified property handle
	*/
	virtual TSharedPtr<IDetailPropertyRow> FindPropertyRow(TSharedRef<IPropertyHandle> PropertyHandle) const = 0;

	/** Specify whether this group should be formatted to display like a subcategory */
	virtual void SetDisplayMode(EDetailGroupDisplayMode Mode) = 0;

	/** Return an optional PasteFromText delegate */
	virtual TSharedPtr<FOnPasteFromText> OnPasteFromText() const { return nullptr; }
	
	/** Return an optional CopyToText delegate */
	virtual TSharedPtr<FOnCopyToText> OnCopyToText() const { return nullptr; }

};
