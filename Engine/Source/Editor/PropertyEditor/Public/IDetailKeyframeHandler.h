// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IPropertyHandle;

/**
 * Keyed Status of a Property, ordered by their precedence
 * E.g. if a property is keyed in current and another frame, Keyed In Frame should take precedence
 */
enum class EPropertyKeyedStatus : uint8
{
	/** Property not keyed, or not animated in current time */
	NotKeyed,

	/** Property is animated in the current time, but there's no key in current frame */
	KeyedInOtherFrame,

	/** For Top-level properties only -- some but not all the properties of the struct are keyed in Current Frame */
	PartiallyKeyed,

	/** Property (and all its sub-properties) are keyed in current frame */
	KeyedInFrame,
};

class IDetailKeyframeHandler
{
public:
	virtual ~IDetailKeyframeHandler()= default;

	virtual bool IsPropertyKeyable(const UClass* InObjectClass, const IPropertyHandle& PropertyHandle) const = 0;

	virtual bool IsPropertyKeyingEnabled() const = 0;

	virtual void OnKeyPropertyClicked(const IPropertyHandle& KeyedPropertyHandle) = 0;

	virtual bool IsPropertyAnimated(const IPropertyHandle& PropertyHandle, UObject* ParentObject) const = 0;

	virtual EPropertyKeyedStatus GetPropertyKeyedStatus(const IPropertyHandle& PropertyHandle) const { return EPropertyKeyedStatus::NotKeyed; }
};
