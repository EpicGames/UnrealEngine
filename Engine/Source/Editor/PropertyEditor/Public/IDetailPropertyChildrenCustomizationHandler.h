// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class IDetailChildrenBuilder;
class IPropertyHandle;

/* Customizes properties children in a detail view. */
class IDetailPropertyChildrenCustomizationHandler
{
public:
	virtual ~IDetailPropertyChildrenCustomizationHandler() = default;

	/* Returns true if the handler should override provided property's children. */
	virtual bool ShouldCustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle)  = 0;

	/* Gives a chance to customize provided property's children. */
	virtual void CustomizeChildren(IDetailChildrenBuilder& ChildrenBuilder, TSharedPtr<IPropertyHandle> PropertyHandle) {}
};
