// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DetailWidgetRow.h"
#include "Widgets/SWidget.h"

class IPropertyHandle;
class IDetailLayoutBuilder;

class IDetailPropertyExtensionHandler
{
public:
	virtual ~IDetailPropertyExtensionHandler()= default;

	virtual bool IsPropertyExtendable(const UClass* InObjectClass, const IPropertyHandle& PropertyHandle) const = 0;

	UE_DEPRECATED(4.24, "Please use ExtendWidgetRow")
	virtual TSharedRef<SWidget> GenerateExtensionWidget(const UClass* InObjectClass, TSharedPtr<IPropertyHandle> PropertyHandle) 
	{ 
		return SNullWidget::NullWidget;
	}

	UE_DEPRECATED(5.0, "Please use ExtendWidgetRow")
	virtual TSharedRef<SWidget> GenerateExtensionWidget(const IDetailLayoutBuilder& InDetailBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> PropertyHandle)
	{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Call old deprecated path for back-compat
		return GenerateExtensionWidget(InObjectClass, PropertyHandle);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Gives the extension handler a chance to add extension widgets to the widget row.
	 * Typically, an extension handler can do so by using InWidgetRow.ExtensionContent(),
	 * but it can also modify other aspects of the widget row as well.
	 */
	virtual void ExtendWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> PropertyHandle)
	{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Call old deprecated path for back-compat
		TSharedRef<SWidget> ExtensionWidget = GenerateExtensionWidget(InDetailBuilder, InObjectClass, PropertyHandle);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (ExtensionWidget != SNullWidget::NullWidget)
		{
			InWidgetRow.ExtensionContent()[
				ExtensionWidget
			];
		}
	}
};
