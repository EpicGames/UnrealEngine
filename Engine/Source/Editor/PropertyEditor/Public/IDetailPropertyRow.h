// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/SlateDelegates.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"

class FDetailWidgetRow;
class FDetailWidgetDecl;
class IDetailDragDropHandler;

DECLARE_DELEGATE_RetVal_OneParam(bool, FIsResetToDefaultVisible, TSharedPtr<IPropertyHandle> /* PropertyHandle */);
DECLARE_DELEGATE_OneParam(FResetToDefaultHandler, TSharedPtr<IPropertyHandle> /* PropertyHandle*/);

enum class ECustomEditConditionMode : uint8
{
	/** The custom edit condition overrides the edit condition set directly on the property (if any). Only the custom edit condition must evaluate to true for the property to be editable. */
	Override,
	/** The custom edit condition merges with the existing edit condition set directly on the property (if any). Both edit conditions must evaluate to true for the property to be editable. */
	Merge,
};

/**
 * Structure describing the delegates needed to override the behavior of reset to default in detail properties
 */
class FResetToDefaultOverride
{
public:
	/** Creates a FResetToDefaultOverride in which the the reset to default is always visible */
	static FResetToDefaultOverride Create(FResetToDefaultHandler InResetToDefaultClicked, const bool InPropagateToChildren = false)
	{
		FResetToDefaultOverride ResetToDefault;
		ResetToDefault.bForceShow = true;
		ResetToDefault.OnClickedPropertyDelegate = InResetToDefaultClicked;
		ResetToDefault.bPropagateToChildren = InPropagateToChildren;
		ResetToDefault.bForceHide = false;
		return ResetToDefault;
	}

	/** Creates a FResetToDefaultOverride from visibility and click handler callback delegates */
	static FResetToDefaultOverride Create(FIsResetToDefaultVisible InIsResetToDefaultVisible, FResetToDefaultHandler InResetToDefaultClicked, const bool InPropagateToChildren = false)
	{
		FResetToDefaultOverride ResetToDefault;
		ResetToDefault.bForceShow = false;
		ResetToDefault.IsVisiblePropertyDelegate = InIsResetToDefaultVisible;
		ResetToDefault.OnClickedPropertyDelegate = InResetToDefaultClicked;
		ResetToDefault.bPropagateToChildren = InPropagateToChildren;
		ResetToDefault.bForceHide = false;
		return ResetToDefault;
	} 

	/** Create a FResetToDefaultOverride from a visibility attribute. */
	static FResetToDefaultOverride Create(TAttribute<bool> InIsResetToDefaultVisible, const bool InPropagateToChildren = false)
	{
		FResetToDefaultOverride ResetToDefault;
		ResetToDefault.bForceShow = false;
		ResetToDefault.IsVisibleAttribute = InIsResetToDefaultVisible;
		ResetToDefault.bPropagateToChildren = InPropagateToChildren;
		ResetToDefault.bForceHide = false;
		return ResetToDefault;
	}

	/** Create a FResetToDefaultOverride from a visibility attribute and a simple delegate. */
	static FResetToDefaultOverride Create(TAttribute<bool> InIsResetToDefaultVisible, FSimpleDelegate InResetToDefaultClicked, const bool InPropagateToChildren = false)
	{
		FResetToDefaultOverride ResetToDefault;
		ResetToDefault.bForceShow = false;
		ResetToDefault.IsVisibleAttribute = InIsResetToDefaultVisible;
		ResetToDefault.OnClickedDelegate = InResetToDefaultClicked;
		ResetToDefault.bPropagateToChildren = InPropagateToChildren;
		ResetToDefault.bForceHide = false;
		return ResetToDefault;
	}

	/** Create a FResetToDefaultOverride from a simple delegate. */
	static FResetToDefaultOverride Create(FSimpleDelegate InResetToDefaultClicked, const bool InPropagateToChildren = false)
	{
		FResetToDefaultOverride ResetToDefault;
		ResetToDefault.bForceShow = true;
		ResetToDefault.OnClickedDelegate = InResetToDefaultClicked;
		ResetToDefault.bPropagateToChildren = InPropagateToChildren;
		ResetToDefault.bForceHide = false;
		return ResetToDefault;
	}

	/** Creates a FResetToDefaultOverride in which reset to default is never visible */
	static FResetToDefaultOverride Hide(const bool InPropagateToChildren = false)
	{
		FResetToDefaultOverride HideResetToDefault;
		HideResetToDefault.bForceShow = false;
		HideResetToDefault.bForceHide = true;
		HideResetToDefault.bPropagateToChildren = InPropagateToChildren;
		return HideResetToDefault;
	}

	/** Called by the UI to show/hide the reset widgets */
	bool IsResetToDefaultVisible(TSharedPtr<IPropertyHandle> Property) const
	{
		if (bForceShow)
		{
			return true;
		}

		if (bForceHide)
		{
			return false;
		}

		if (IsVisiblePropertyDelegate.IsBound())
		{
			return IsVisiblePropertyDelegate.Execute(Property);
		}

		if (IsVisibleAttribute.IsSet())
		{
			return IsVisibleAttribute.Get();
		}
		
		return false;
	}

	/** Does this have a custom reset to default handler? */
	bool HasResetToDefaultHandler() const
	{
		return OnClickedPropertyDelegate.IsBound() || OnClickedDelegate.IsBound();
	}

	/** Called by the property editor to actually reset the property to default */
	void OnResetToDefaultClicked(TSharedPtr<IPropertyHandle> PropertyHandle) const
	{
		if (PropertyHandle.IsValid() && OnClickedPropertyDelegate.IsBound())
		{
			PropertyHandle->ExecuteCustomResetToDefault(*this);
		} 
		else
		{
			OnClickedDelegate.ExecuteIfBound();
		}
	}

	/** Get the actual delegate bound to this reset to default handler. */
	FResetToDefaultHandler GetPropertyResetToDefaultDelegate() const { return OnClickedPropertyDelegate; }

	/** Called by properties to determine whether this override should set on their children */
	bool PropagatesToChildren() const
	{
		return bPropagateToChildren;
	}

private:
	/** Callback to indicate whether or not reset to default is visible */
	FIsResetToDefaultVisible IsVisiblePropertyDelegate;

	/** Delegate called when reset to default is clicked */
	FResetToDefaultHandler OnClickedPropertyDelegate;

	/** Attribute to determine whether or not reset to default is visible */
	TAttribute<bool> IsVisibleAttribute; 

	/** Delegate called when reset to default is clicked */
	FSimpleDelegate OnClickedDelegate;

	/** Should properties pass this on to their children? */
	bool bPropagateToChildren;

	/** Ignore the visibility delegate and always show the reset to default widgets? */
	bool bForceShow;

	/** Ignore the visibility delegate and never show the reset to default widgets? */
	bool bForceHide;
};

/**
 * A single row for a property in a details panel                                                              
 */
class IDetailPropertyRow
{
public:
	virtual ~IDetailPropertyRow()= default;

	/** @return the property handle for the property on this row */
	virtual TSharedPtr<IPropertyHandle> GetPropertyHandle() const = 0;

	/**
	 * Sets the localized display name of the property
	 *
	 * @param InDisplayName	The name of the property
	 */
	virtual IDetailPropertyRow& DisplayName( const FText& InDisplayName ) = 0;

	/**
	 * Sets the localized tooltip of the property
	 *
	 * @param InToolTip	The tooltip of the property
	 */
	virtual IDetailPropertyRow& ToolTip( const FText& InToolTip ) = 0;

	/**
	 * Sets whether or not we show the default property editing buttons for this property
	 *
	 * @param bShowPropertyButtons	if true property buttons for this property will be shown.  
	 */
	virtual IDetailPropertyRow& ShowPropertyButtons( bool bShowPropertyButtons ) = 0;

	/**
	 * Sets an edit condition for this property.  If the edit condition fails, the property is not editable
	 * This will add a checkbox before the name of the property that users can click to toggle the edit condition
	 * Properties with built in edit conditions will override this automatically.  If the 
	 *
	 * @param EditConditionValue			Attribute for the value of the edit condition (true indicates that the edit condition passes)
	 * @param OnEditConditionValueChanged	Delegate called when the edit condition value changes
	 * @param EditConditionMode				Should this custom edit condition override or merge with the existing edit condition of the property (if any)?
	 */
	virtual IDetailPropertyRow& EditCondition( TAttribute<bool> EditConditionValue, FOnBooleanValueChanged OnEditConditionValueChanged, ECustomEditConditionMode EditConditionMode = ECustomEditConditionMode::Override ) = 0;

	/**
	 * Sets whether or not the edit condition for this property should affect its visibility.  If the edit condition fails, the property will be hidden outright.
	 * 
	 * @param bEditConditionHidesValue		if true the property be shown/hidden based on the edit condition
	 */
	virtual IDetailPropertyRow& EditConditionHides( bool bEditConditionHidesValue ) = 0;

	/**
	 * Sets whether or not this property is enabled
	 *
	 * @param InIsEnabled	Attribute for the enabled state of the property (true to enable the property)
	 */
	virtual IDetailPropertyRow& IsEnabled( TAttribute<bool> InIsEnabled ) = 0;
	

	/**
	 * Sets whether or not this property should auto-expand
	 *
	 * @param bForceExpansion	true to force the property to be expanded
	 */
	virtual IDetailPropertyRow& ShouldAutoExpand(bool bForceExpansion = true) = 0;

	/**
	 * Sets the visibility of this property
	 *
	 * @param Visibility	Attribute for the visibility of this property
	 */
	virtual IDetailPropertyRow& Visibility( TAttribute<EVisibility> Visibility ) = 0;

	/**
	 * Overrides the behavior of reset to default
	 *
	 * @param ResetToDefault	Contains the delegates needed to override the behavior of reset to default
	 */
	virtual IDetailPropertyRow& OverrideResetToDefault(const FResetToDefaultOverride& ResetToDefault) = 0;

	/**
	 * Sets a handler for this property row to be a source or target of drag-and-drop operations
	 * 
	 * @param InDragDropHandler	Handler used when starting a drag or accepting a drop operation
	 */
	virtual IDetailPropertyRow& DragDropHandler(TSharedPtr<IDetailDragDropHandler> InDragDropHandler) = 0;

	/**
	 * Returns the property row expansion state
	 *
	 * @return Will return true if the row is expanded, false if not
	 */
	virtual bool IsExpanded() const = 0;

	/**
	 * Returns the name and value widget of this property row.  You can use this widget to apply further customization to existing widgets (by using this  with CustomWidget)
	 *
	 * @param OutNameWidget		The default name widget
	 * @param OutValueWidget	The default value widget
	 */
	virtual void GetDefaultWidgets( TSharedPtr<SWidget>& OutNameWidget, TSharedPtr<SWidget>& OutValueWidget, bool bAddWidgetDecoration = false ) = 0;

	/**
	 * Returns the name and value widget of this property row.  You can use this widget to apply further customization to existing widgets (by using this  with CustomWidget)
	 *
	 * @param OutNameWidget		The default name widget
	 * @param OutValueWidget	The default value widget
	 * @param OutCustomRow		The default widget row
	 */
	virtual void GetDefaultWidgets( TSharedPtr<SWidget>& OutNameWidget, TSharedPtr<SWidget>& OutValueWidget, FDetailWidgetRow& Row, bool bAddWidgetDecoration = false ) = 0;

	/**
	 * Overrides the property widget. Destroys any existing custom property widgets.
	 *
	 * @param bShowChildren	Whether or not to still show any children of this property
	 * @return a row for the property that custom widgets can be added to
	 */
	virtual FDetailWidgetRow& CustomWidget( bool bShowChildren = false ) = 0;

	/**
	 * Gives a non-owning pointer to name widget on existing custom property widget if it exists.
	 *
	 * @return	Pointer to name widget if custom property widget already exists, nullptr otherwise
	 */
	virtual FDetailWidgetDecl* CustomNameWidget() = 0;

	/**
	 * Gives a non-owning pointer to value widget on existing custom property widget if it exists.
	 *
	 * @return	Pointer to value widget if custom property widget already exists, nullptr otherwise
	 */
	virtual FDetailWidgetDecl* CustomValueWidget() = 0;

	/**
	 * Gives a non-owning pointer to reset to default widget on existing custom property widget if it exists.
	 *
	 * @return	Pointer to reset to default widget if custom property widget already exists, nullptr otherwise
	 */
	virtual FDetailWidgetDecl* CustomResetToDefaultWidget() = 0;
};
