// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class SWidget;
class ITableRow;

/** Defines a set of objects and a common base class between them for a root object customization */
struct FDetailsObjectSet
{
	/** List of objects that represent the root in the details panel */
	TArray<const UObject*> RootObjects;

	/** The most common base class between all root objects */
	const UClass* CommonBaseClass;
};

/** 
 * Interface for any class that lays out details for a specific class
 */
class IDetailRootObjectCustomization : public TSharedFromThis<IDetailRootObjectCustomization>
{
public:
	enum EExpansionArrowUsage
	{
		Custom,
		Default,
		None,
	};

	virtual ~IDetailRootObjectCustomization() = default;

	/**
	 * Called when the details panel wants to display an object header widget for a given object
	 *
	 * @param	InRootObjects	The object set whose header is being customized
	 */
	virtual TSharedPtr<SWidget> CustomizeObjectHeader(const FDetailsObjectSet& InRootObjectSet)
	{ 
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return CustomizeObjectHeader(InRootObjectSet.RootObjects[0]);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** 
	 * Called when the details panel wants to display an object header widget for a given object
	 *
	 * @param	InRootObjects	The object set whose header is being customized
	 * @param	InTableRow		The ITableRow object (table views to talk to their rows) that will use the current IDetailRootObjectCustomization element.  This may be null if the customization is not being shown in a table view
 	 */
	virtual TSharedPtr<SWidget> CustomizeObjectHeader(const FDetailsObjectSet& InRootObjectSet, const TSharedPtr<ITableRow>& InTableRow)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return CustomizeObjectHeader(InRootObjectSet.RootObjects[0], InTableRow);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Whether or not the objects and all of its children should be visible in the details panel
	 */
	virtual bool AreObjectsVisible(const FDetailsObjectSet& InRootObjectSet) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return IsObjectVisible(InRootObjectSet.RootObjects[0]);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Whether or not the object should have a header displayed or just show the children directly
	 *
	 * @return true if the this customization should be displayed or false to just show the children directly
	 */
	virtual bool ShouldDisplayHeader(const FDetailsObjectSet& InRootObjectSet) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ShouldDisplayHeader(InRootObjectSet.RootObjects[0]);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(4.25, "Please use the CustomizeObjectHeader version which takes in an array of root objects")
	virtual TSharedPtr<SWidget> CustomizeObjectHeader(const UObject* InRootObject)
	{
		return nullptr;
	}

	UE_DEPRECATED(4.25, "Please use the CustomizeObjectHeader version which takes in an array of root objects")
	virtual TSharedPtr<SWidget> CustomizeObjectHeader(const UObject* InRootObject, const TSharedPtr<ITableRow>& InTableRow)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return CustomizeObjectHeader(InRootObject);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(4.25, "Please use the AreObjectsVisible instead")
	virtual bool IsObjectVisible(const UObject* InRootObject) const { return true; }

	UE_DEPRECATED(4.25, "Please use the ShouldDisplayHeader version which takes in an array of objects instead")
	virtual bool ShouldDisplayHeader(const UObject* InRootObject) const
	{
		return true;
	}

	/**
	 * Gets the setup for expansion arrows in this customization
	 */
	virtual EExpansionArrowUsage GetExpansionArrowUsage() const
	{ 
		// Note: Returns none for backwards compatibility
		return EExpansionArrowUsage::None; 
	}
};
