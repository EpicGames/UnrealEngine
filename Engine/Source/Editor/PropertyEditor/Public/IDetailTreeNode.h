// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/SlateEnums.h"
#include "Framework/Commands/UIAction.h"
#include "Textures/SlateIcon.h"

class IPropertyHandle;
class SWidget;

enum class EDetailNodeType
{
	/** Node represents a category */
	Category,
	/** Node represents an item such as a property or widget */
	Item,
	/** Node represents an advanced dropdown */
	Advanced,
	/** Represents a top level object node if a view supports multiple root objects */
	Object,
};

/** Layout data for node's content widgets. */
struct FNodeWidgetLayoutData
{
	FNodeWidgetLayoutData()
	{
	}

	FNodeWidgetLayoutData(EHorizontalAlignment InHorizontalAlignment, EVerticalAlignment InVerticalAlignment, TOptional<float> InMinWidth, TOptional<float> InMaxWidth)
		: HorizontalAlignment(InHorizontalAlignment)
		, VerticalAlignment(InVerticalAlignment)
		, MinWidth(InMinWidth)
		, MaxWidth(InMaxWidth)
	{
	}

	/** The horizontal alignment requested by the widget. */
	EHorizontalAlignment HorizontalAlignment;

	/** The vertical alignment requested by the widget. */
	EVerticalAlignment VerticalAlignment;

	/** An optional minimum width requested by the widget. */
	TOptional<float> MinWidth;

	/** An optional maximum width requested by the widget. */
	TOptional<float> MaxWidth;
};

/* Defines a custom menu action which can be performed on a detail tree node. */
struct FNodeWidgetActionsCustomMenuData
{
	FNodeWidgetActionsCustomMenuData(const FUIAction& InAction, const FText& InName, const FText& InTooltip, const FSlateIcon& InSlateIcon)
		: Action(InAction)
		, Name(InName)
		, Tooltip(InTooltip)
		, SlateIcon(InSlateIcon)
	{
	}

	/* The action to be performed. */
	const FUIAction Action;

	/* The name to display for the menu item. */
	const FText Name;

	/* The tooltip to the display for the menu item. */
	const FText Tooltip;

	/* The icon to display for the menu item. */
	const FSlateIcon SlateIcon;
};

/** Defines actions which can be performed on node widgets. */
struct FNodeWidgetActions
{
	/** Action for copying data on this node */
	FUIAction CopyMenuAction;

	/** Action for pasting data on this node */
	FUIAction PasteMenuAction;

	/** Custom Actions on this node */
	TArray<FNodeWidgetActionsCustomMenuData> CustomMenuItems;
};

/** The widget contents of the node.  Any of these can be null depending on how the row was generated */
struct FNodeWidgets
{
	/** Widget for the name column */
	TSharedPtr<SWidget> NameWidget;

	/** Layout data for the widget in the name column. */
	FNodeWidgetLayoutData NameWidgetLayoutData;

	/** Widget for the value column*/
	TSharedPtr<SWidget> ValueWidget;

	/** Layout data for the widget in the value column. */
	FNodeWidgetLayoutData ValueWidgetLayoutData;

	/** Widget that spans the entire row.  Mutually exclusive with name/value widget */
	TSharedPtr<SWidget> WholeRowWidget;

	/** Layout data for the whole row widget. */
	FNodeWidgetLayoutData WholeRowWidgetLayoutData;

	/** Edit condition widget. */
	TSharedPtr<SWidget> EditConditionWidget;

	/** The actions which can be performed on the node widgets. */
	FNodeWidgetActions Actions;
};

class IDetailTreeNode
{
public:
	virtual ~IDetailTreeNode() = default;

	/** 
	 * @return Get the details view that this node is contained in.
	 */
	virtual TSharedPtr<class IDetailsView> GetNodeDetailsViewSharedPtr() const = 0;

	UE_DEPRECATED(5.5, "Use GetNodeDetailsViewSharedPtr() instead.")
	class IDetailsView* GetNodeDetailsView() const
	{
		return GetNodeDetailsViewSharedPtr().Get();
	}

	/**
	 * @return The type of this node.  Should be used to determine any external styling to apply to the generated r ow
	 */
	virtual EDetailNodeType GetNodeType() const = 0;

	/** 
	 * Creates a handle to the property on this row if the row represents a property. Only compatible with item node types that are properties
	 * 
	 * @return The property handle for the row or null if the node doesn't have a property 
	 */
	virtual TSharedPtr<IPropertyHandle> CreatePropertyHandle() const = 0;

	/**
	 * Creates the slate widgets for this row. 
	 *
	 * @return the node widget structure with either name/value pair or a whole row widget
	 */
	virtual FNodeWidgets CreateNodeWidgets() const = 0;

	/**
	 * Gets the children of this tree node    
	 * Note: Customizations can determine the visibility of children.  This will only return visible children if bInIgnoreVisibility if false
	 *
	 * @param OutChildren			The generated children
	 * @param bInIgnoreVisibility	Whether to return children regardless of visibility
	 */
	virtual void GetChildren(TArray<TSharedRef<IDetailTreeNode>>& OutChildren, const bool& bInIgnoreVisibility = false) = 0;

	/**
	 * Gets an identifier name for this node.  This is not a name formatted for display purposes, but can be useful for storing
	 * UI state like if this row is expanded.
	 */
	virtual FName GetNodeName() const = 0;

	/**
	 * Get the property row that this node is represented by.
	 */
	virtual TSharedPtr<class IDetailPropertyRow> GetRow() const = 0;

	/**
	 * Gets the filter strings for this node in the tree.
	 */
	virtual void GetFilterStrings(TArray<FString>& OutFilterStrings) const = 0;

	/**
	 * Gets if this node should be initially collapsed by default.
	 */
	virtual bool GetInitiallyCollapsed() const = 0;
};
