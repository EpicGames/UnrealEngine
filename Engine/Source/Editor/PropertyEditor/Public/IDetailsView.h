// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DetailsDisplayManager.h"
#include "Widgets/SCompoundWidget.h"
#include "PropertyEditorDelegates.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Layout/IScrollableWidget.h"
#include "DetailsViewArgs.h"

class FDetailTreeNode;
class AActor;
class FNotifyHook;
class FPropertyPath;
class FTabManager;
class FUICommandList;
class IClassViewerFilter;
class IDetailKeyframeHandler;
class IDetailPropertyExtensionHandler;
class IDetailPropertyChildrenCustomizationHandler;
class IDetailRootObjectCustomization;
class IPropertyTypeIdentifier;
class FDetailsViewObjectFilter;
class FComplexPropertyNode;

typedef TArray<TSharedPtr<FComplexPropertyNode>> FRootPropertyNodeList;

DECLARE_DELEGATE_RetVal_OneParam(bool, FOnValidateDetailsViewPropertyNodes, const FRootPropertyNodeList&)

/**
 * Interface class for all detail views
 */
class IDetailsView : public SCompoundWidget, public IScrollableWidget
{
public:
	/** Sets the callback for when the property view changes */
	virtual void SetOnObjectArrayChanged(FOnObjectArrayChanged OnObjectArrayChangedDelegate) = 0;

	/** List of all selected objects we are inspecting */
	virtual const TArray< TWeakObjectPtr<UObject> >& GetSelectedObjects() const = 0;

	/** @return	Returns list of selected actors we are inspecting */
	virtual const TArray< TWeakObjectPtr<AActor> >& GetSelectedActors() const = 0;

	/** @return Returns information about the selected set of actors */
	virtual const struct FSelectedActorInfo& GetSelectedActorInfo() const = 0;

	/** @return Whether or not the details view is viewing a CDO */
	virtual bool HasClassDefaultObject() const = 0;

	/**
	 * Registers a custom detail layout delegate for a specific class in this instance of the details view only
	 *
	 * @param Class	The class the custom detail layout is for
	 * @param DetailLayoutDelegate	The delegate to call when querying for custom detail layouts for the classes properties
	 */
	virtual void RegisterInstancedCustomPropertyLayout(UStruct* Class, FOnGetDetailCustomizationInstance DetailLayoutDelegate) = 0;
	virtual void RegisterInstancedCustomPropertyTypeLayout(FName PropertyTypeName, FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate, TSharedPtr<IPropertyTypeIdentifier> Identifier = nullptr) = 0; 

	/**
	 * Unregisters a custom detail layout delegate for a specific class in this instance of the details view only
	 *
	 * @param Class	The class with the custom detail layout delegate to remove
	 */
	virtual void UnregisterInstancedCustomPropertyLayout(UStruct* Class) = 0;
	virtual void UnregisterInstancedCustomPropertyTypeLayout(FName PropertyTypeName, TSharedPtr<IPropertyTypeIdentifier> Identifier = nullptr) = 0;

	/**
	 * Registers a customization that will be used only if this details panel contains multiple top level objects.
	 * I.E it was created with bAllowMultipleTopLevelObjects = true.	This interface will be used to customize the header for each top level object in the details panel
	 *
	 * @param InRootObjectCustomization	If null is passed in, the customization will be removed
	 */
	virtual void SetRootObjectCustomizationInstance(TSharedPtr<IDetailRootObjectCustomization> InRootObjectCustomization) = 0;

	/**
	 * Sets the objects this details view is viewing
	 *
	 * @param InObjects		The list of objects to observe
	 * @param bForceRefresh	If true, doesn't check if new objects are being set
	 * @param bOverrideLock	If true, will set the objects even if the details view is locked
	 */
	virtual void SetObjects( const TArray<UObject*>& InObjects, bool bForceRefresh = false, bool bOverrideLock = false ) = 0;
	virtual void SetObjects( const TArray< TWeakObjectPtr< UObject > >& InObjects, bool bForceRefresh = false, bool bOverrideLock = false ) = 0;

	/**
	 * Sets a single object that details view is viewing
	 *
	 * @param InObject		The object to view
	 * @param bForceRefresh	If true, doesn't check if new objects are being set
	 */
	virtual void SetObject( UObject* InObject, bool bForceRefresh = false ) = 0;

	/** Removes all invalid objects being observed by this details panel */
	virtual void RemoveInvalidObjects() = 0;

	/** Set overrides that should be used when looking for packages that contain the given object (used when editing a transient copy of an object, but you need access to th real package) */
	virtual void SetObjectPackageOverrides(const TMap<TWeakObjectPtr<UObject>, TWeakObjectPtr<UPackage>>& InMapping) = 0;

	/**
	 * Returns true if the details view is locked and cant have its observed objects changed 
	 */
	virtual bool IsLocked() const = 0;

	/**
	 * @return true if the details view can be updated from editor selection
	 */
	virtual bool IsUpdatable() const = 0;

	/**
	 * @return True if there is any filter of properties active in this details panel
	 */
	virtual bool HasActiveSearch() const = 0;
	
	/** 
	 * Clears any search terms in the current filter
	 */
	virtual void ClearSearch() = 0;
	
	/** 
	 * Sets the active search for the current filter
	 */
	virtual void SetSearchText(const FText& InSearch) = 0;

	/**
	 * @return The number of visible top level objects. This value is affected by filtering. 
	 * Note: this value will always be 1 if this details panel was not created with bAllowMultipleTopLevelObjects=true
	 */
	virtual int32 GetNumVisibleTopLevelObjects() const = 0;

	/** @return The identifier for this details view, or NAME_None is this view is anonymous */
	virtual FName GetIdentifier() const = 0;

	/**
	 * Sets a delegate to call to determine if a specific property should be visible in this instance of the details view
	 */ 
	virtual void SetIsPropertyVisibleDelegate( FIsPropertyVisible InIsPropertyVisible ) = 0;
	virtual FIsPropertyVisible& GetIsPropertyVisibleDelegate() = 0;

	/**
	 * Sets a delegate to call to determine if a specific property should be read-only in this instance of the details view
	 */ 
	virtual void SetIsPropertyReadOnlyDelegate( FIsPropertyReadOnly InIsPropertyReadOnly ) = 0;
	virtual FIsPropertyReadOnly& GetIsPropertyReadOnlyDelegate() = 0;

	/**
	 * Sets a delegate to call to determine if a specific custom row should be visible in this instance of the details view
	 */
	virtual void SetIsCustomRowVisibleDelegate(FIsCustomRowVisible InIsCustomRowVisible) = 0;
	virtual FIsCustomRowVisible& GetIsCustomRowVisibleDelegate() = 0;

	/**
	 * Sets a delegate to call to determine if a specific custom row should be visible in this instance of the details view
	 */
	virtual void SetIsCustomRowReadOnlyDelegate(FIsCustomRowReadOnly InIsCustomRowVisible) = 0;
	virtual FIsCustomRowReadOnly& GetIsCustomRowReadOnlyDelegate() = 0;

	/**
	 * Sets a delegate to call to layout generic details not specific to an object being viewed
	 */ 
	virtual void SetGenericLayoutDetailsDelegate( FOnGetDetailCustomizationInstance OnGetGenericDetails ) = 0;
	virtual FOnGetDetailCustomizationInstance& GetGenericLayoutDetailsDelegate() = 0;

	/**
	 * Sets a delegate to call to determine if the properties  editing is enabled
	 */ 
	virtual void SetIsPropertyEditingEnabledDelegate( FIsPropertyEditingEnabled IsPropertyEditingEnabled ) = 0;
	virtual FIsPropertyEditingEnabled& GetIsPropertyEditingEnabledDelegate() = 0;

	virtual void SetKeyframeHandler( TSharedPtr<IDetailKeyframeHandler> InKeyframeHandler ) = 0;
	virtual TSharedPtr<IDetailKeyframeHandler> GetKeyframeHandler() const = 0;

	virtual void SetExtensionHandler(TSharedPtr<IDetailPropertyExtensionHandler> InExtensionHandler) = 0;
	virtual TSharedPtr<IDetailPropertyExtensionHandler> GetExtensionHandler() const = 0;

	virtual void SetChildrenCustomizationHandler(TSharedPtr<IDetailPropertyChildrenCustomizationHandler> InExtensionHandler) = 0;
	virtual TSharedPtr<IDetailPropertyChildrenCustomizationHandler> GetChildrenCustomizationHandler() const = 0;

	/**
	 * @return true if property editing is enabled (based on the FIsPropertyEditingEnabled delegate)
	 */ 
	virtual bool IsPropertyEditingEnabled() const = 0;

	/**
	 * A delegate which is called after properties have been edited and PostEditChange has been called on all objects.
	 * This can be used to safely make changes to data that the details panel is observing instead of during PostEditChange (which is
	 * unsafe)
	 */
	virtual FOnFinishedChangingProperties& OnFinishedChangingProperties() const = 0;

	/** 
	 * Sets the visible state of the filter box/property grid area
	 */
	virtual void HideFilterArea(bool bIsVisible) = 0;

	/**
	 * Returns a list of all the properties displayed (via full path), order in list corresponds to draw order:
	 */
	virtual TArray< FPropertyPath > GetPropertiesInOrderDisplayed() const = 0;
	
	/**
	 * Returns the row number in the underlying TableView for each property being displayed. Note this won't include
	 * any properties that are apart of a collapsed section of the property tree because those properties don't have
	 * row numbers. If you need a list of all properties independent of expansion state, use GetPropertiesInOrderDisplayed
	 */
	virtual TArray<TPair<int32, FPropertyPath>> GetPropertyRowNumbers() const  = 0;

	/**
	 * Returns the boundary (in paint space) of a detail tree node if it's currently visible Otherwise return an invalid boundary
	 */
	virtual FSlateRect GetPaintSpacePropertyBounds(const TSharedRef<FDetailTreeNode>& InDetailTreeNode, bool IncludeChildren = true) const {return FSlateRect();}
	
	/**
	 * Returns the boundary (in tick space) of a detail tree node if it's currently visible Otherwise return an invalid boundary
	 */
	virtual FSlateRect GetTickSpacePropertyBounds(const TSharedRef<FDetailTreeNode>& InDetailTreeNode, bool IncludeChildren = true) const {return FSlateRect();}

	/**
	 * Returns true if the provided detail tree node is part of a collapsed/hidden subsection of the details panel
	 */
	virtual bool IsAncestorCollapsed(const TSharedRef<IDetailTreeNode>& Node) const { return false; }
	
	/**
	 * returns the number of rows in the underlying TableView
	 */
	virtual int32 CountRows() const  = 0;

	/**
	 * Creates a box around the treenode corresponding to Property and scrolls the treenode into view
	 */
	virtual void HighlightProperty(const FPropertyPath& Property) = 0;

	/**
	 * Scrolls the treenode for the specific property into view
	 */
	virtual void ScrollPropertyIntoView(const FPropertyPath& Property, bool bExpandProperty = false) = 0;

	/**
	 * Forces all advanced property sections to be in expanded state:
	 */
	virtual void ShowAllAdvancedProperties() = 0;

	/**
	 * Sets the expansion state all root nodes and optionally all of their children
	 *
	 * @param bExpand			The new expansion state
	 * @param bRecurse			Whether or not to apply the expansion change to any children
	 */
	virtual void SetRootExpansionStates(const bool bExpand, const bool bRecurse) = 0;
	
	/**
	 * Refreshes the visibility of root objects in this details view. 
	 * Note: This method has no effect if the details panel is viewing a single top-level object set only
	 */
	virtual void RefreshRootObjectVisibility() = 0;
	/**
	 * Assigns delegate called when view is filtered, useful for updating external control logic:
	 */
	virtual void SetOnDisplayedPropertiesChanged(FOnDisplayedPropertiesChanged InOnDisplayedPropertiesChangedDelegate) = 0;
	virtual FOnDisplayedPropertiesChanged& GetOnDisplayedPropertiesChanged() = 0;

	/**
	 * Disables or enables customization of the details view:
	 */
	virtual void SetDisableCustomDetailLayouts(bool bInDisableCustomDetailLayouts) = 0;

	/**
	 * Sets the set of properties that are considered differing, used when filtering out identical properties
	 */
	virtual void UpdatePropertyAllowList(const TSet<FPropertyPath>& InAllowedProperties) = 0;

	/** Returns the name area widget used to display object naming functionality so it can be placed in a custom location.  Note FDetailsViewArgs.bCustomNameAreaLocation must be true */
	virtual TSharedPtr<SWidget> GetNameAreaWidget() = 0;

	/** Optionally add custom tools into the NameArea */
	virtual void SetNameAreaCustomContent(TSharedRef<SWidget>& InCustomContent) = 0;

	/** Returns the search area widget used to display search and view options so it can be placed in a custom location.  Note FDetailsViewArgs.bCustomFilterAreaLocation must be true */
	virtual TSharedPtr<SWidget> GetFilterAreaWidget() = 0;

	/** Returns the command list of the hosting toolkit (can be nullptr if the widget that contains the details panel didn't route a command list in) */
	virtual TSharedPtr<FUICommandList> GetHostCommandList() const = 0;

	/** Returns the tab manager of the hosting toolkit (can be nullptr if the details panel is not hosted within a tab) */
	virtual TSharedPtr<FTabManager> GetHostTabManager() const = 0;

	/** Sets the tab manager of the hosting toolkit (can be nullptr if the details panel is not hosted within a tab) */
	virtual void SetHostTabManager(TSharedPtr<FTabManager> InTabManager) = 0;

	/** Request the details view to be refreshed (new widgets generated) with the current set of objects on the next Tick */
	virtual void RequestForceRefresh() = 0;

	/** Causes the details view to be refreshed (new widgets generated) with the current set of objects */
	virtual void ForceRefresh() = 0;

	/** Invalidates any cached state without necessarily doing a complete rebuild. */
	virtual void InvalidateCachedState()
	{
		ForceRefresh();
	}
	
	/** Sets an optional object filter to use for more complex handling of what a details panel is viewing. */
	virtual void SetObjectFilter(TSharedPtr<FDetailsViewObjectFilter> InFilter) = 0;

	/** Sets the custom filter(s) to be used when selecting values for class properties in this view. */
	virtual void SetClassViewerFilters(const TArray<TSharedRef<IClassViewerFilter>>& InFilters) = 0;

	/** Allows other systems to add a custom filter in the details panel */
	virtual void SetCustomFilterDelegate(FSimpleDelegate InDelegate) = 0;

	virtual void SetCustomFilterLabel(FText InText) = 0;

		/* Use this function to set a callback for SDetailsView that will skip the EnsureDataIsValid call in Tick.
	 * This is useful if your implementation doesn't need to validate nodes every tick or needs to perform some other form of validation. */
	virtual void SetCustomValidatePropertyNodesFunction(FOnValidateDetailsViewPropertyNodes InCustomValidatePropertyNodesFunction) = 0;

	/** Gets the minimum width of the detail panel's right column. */
	virtual float GetRightColumnMinWidth() const = 0;
	/** Sets the minimum width of the detail panel's right column. */
	virtual void SetRightColumnMinWidth(float InMinWidth) = 0;

	/**
	* Updates @code FDetailsViewStyleKey& StyleKey @endcode for the current IDetailsView state
	*/
	virtual void UpdateStyleKey() = 0;

	/**
	* Gets the @code FDetailsViewStyleKey& @endcode which provides a Key to the current style for a Details View
	*/
	virtual const FDetailsViewStyleKey& GetStyleKey() = 0;
	
	/**
   	* Returns a bool indicating whether the given @code FDetailsViewStyleKey @endcode is the default Details View Style 
   	*/
   	virtual bool IsDefaultStyle() const = 0;


	// IScrollableWidget interface
	virtual FVector2D GetScrollDistance() = 0;
	virtual FVector2D GetScrollDistanceRemaining() = 0;
	virtual TSharedRef<SWidget> GetScrollWidget() = 0;
	// End of IScrollableWidget interface

	/**
	 * Resets the selected sections to the default 'All' section.
	 *
	 * @return whether the operation succeeded.
	 */
	virtual bool ResetToDefaultSection() = 0;

};
