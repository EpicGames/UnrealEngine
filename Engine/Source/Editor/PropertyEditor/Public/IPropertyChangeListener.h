// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FPropertyListenerSettings
{
	/** Flags a property must have to be observed for changes */
	uint64 RequiredPropertyFlags;
	
	/** Flags that a property must not have to be observed for changes */
	uint64 DisallowedPropertyFlags;

	/** Whether or not to ignore object properties */
	bool bIgnoreObjectProperties;

	/** Whether or not to ignore arrays (not the array elements but the array property itself */
	bool bIgnoreArrayProperties;
};

/**
 * Interface to a property change listener that broadcasts notifications when a property changes
 */
class IPropertyChangeListener
{
public:
	/** Delegate called when a property changes */
	DECLARE_MULTICAST_DELEGATE_TwoParams( FOnPropertyChanged, const TArray<UObject*>&, const class IPropertyHandle&  );

	/**
	 * Sets the object that should be listened to for changes
	 *
	 * @param Object						The object to listen to
	 * @param InPropertyListenerSettings	Settings for which properties should be listened to
	 */
	virtual void SetObject( UObject& Object, const FPropertyListenerSettings& InPropertyListenerSettings ) = 0;

	/**
	 * Scans properties for changes
	 *
	 * @param bRecacheNewValues	If true recaches new values found for the next scan
	 * @return true if any changes were found
	 */
	virtual bool ScanForChanges( bool bRecacheNewValues = true ) = 0;
	
	/**
	 * Triggers all property changed delegates to fire
	 */
	virtual void TriggerAllPropertiesChangedDelegate() = 0;

	/**
	 * @return A delegate to call when a property changes
	 */
	virtual FOnPropertyChanged& GetOnPropertyChangedDelegate() = 0;
};
