// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PropertyEditorModule.h"

class IDetailTreeNode;
class FComplexPropertyNode;
class FStructOnScope;
class IStructureDataProvider;


DECLARE_DELEGATE_RetVal_OneParam(bool, FOnValidatePropertyRowGeneratorNodes, const FRootPropertyNodeList&)

struct FPropertyRowGeneratorArgs
{
	FPropertyRowGeneratorArgs()
		: NotifyHook(nullptr)
		, DefaultsOnlyVisibility(EEditDefaultsOnlyNodeVisibility::Show)
		, bAllowMultipleTopLevelObjects(false)
		, bShouldShowHiddenProperties(false)
		, bAllowEditingClassDefaultObjects(true)
	{}

	/** Notify hook to call when properties are changed */
	FNotifyHook* NotifyHook;

	/** Controls how CPF_DisableEditOnInstance nodes will be treated */
	EEditDefaultsOnlyNodeVisibility DefaultsOnlyVisibility;

	/** Whether the root node should contain multiple objects. */
	bool bAllowMultipleTopLevelObjects;

	/** Whether the generator should generate hidden properties. */
	bool bShouldShowHiddenProperties;

	/** Whether the generator should allow editing CDOs. */
	bool bAllowEditingClassDefaultObjects;
};

class IPropertyRowGenerator
{
public:
	virtual ~IPropertyRowGenerator() = default;

	/**
	 * Sets the objects that should be used to generate rows
	 *
	 * @param InObjects	The list of objects to generate rows from.  Note unless FPropertyRowGeneratorArgs.bAllowMultipleTopLevelObjects is set to true, the properties used will be the common base class of all passed in objects
	 */
	virtual void SetObjects(const TArray<UObject*>& InObjects) = 0;

	/**
	 * Sets the structure that should be used to generate rows
	 *
	 * @param InStruct The structure used to generate rows from.
	 */
	virtual void SetStructure(const TSharedPtr<FStructOnScope>& InStruct) = 0;
	virtual void SetStructure(const TSharedPtr<IStructureDataProvider>& InStruct) = 0;

	/**
	 * Get the list of objects that were used to generate detail tree nodes
	 * @return A list of weak pointers to this generator's objects.
	 */
	virtual const TArray<TWeakObjectPtr<UObject>>& GetSelectedObjects() const = 0;

	/**
	 * Delegate called when rows have been refreshed.  This delegate should always be bound to something because once this is called, none of the rows previously generated can be trusted
	 */
	DECLARE_EVENT(IPropertyRowGenerator, FOnRowsRefreshed);
	virtual FOnRowsRefreshed& OnRowsRefreshed() = 0;

	/** 
	 * @return The list of root tree nodes that have been generated.  Note: There will only be one root node unless  FPropertyRowGeneratorArgs.bAllowMultipleTopLevelObjects was set to true when the generator was created
	 */
	virtual const TArray<TSharedRef<IDetailTreeNode>>& GetRootTreeNodes() const = 0;

	/**
	 * Finds a tree node by property handle.  
	 * 
	 * @return The found tree node or null if the node cannot be found
	 */
	virtual TSharedPtr<IDetailTreeNode> FindTreeNode(TSharedPtr<IPropertyHandle> PropertyHandle) const = 0;

	/**
	* Finds tree nodes by property handles.
	* 
	* @return The found tree nodes (in the same order, a pointer will be null if the node cannot be found)
	*/
	virtual TArray<TSharedPtr<IDetailTreeNode>> FindTreeNodes(const TArray<TSharedPtr<IPropertyHandle>>& PropertyHandles) const = 0;

	/**
	 * Registers a custom detail layout delegate for a specific class in this instance of the generator only
	 *
	 * @param Class	The class the custom detail layout is for
	 * @param DetailLayoutDelegate	The delegate to call when querying for custom detail layouts for the classes properties
	 */
	virtual void RegisterInstancedCustomPropertyLayout(UStruct* Class, FOnGetDetailCustomizationInstance DetailLayoutDelegate) = 0;
	virtual void RegisterInstancedCustomPropertyTypeLayout(FName PropertyTypeName, FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate, TSharedPtr<IPropertyTypeIdentifier> Identifier = nullptr) = 0;

	/**
	 * Unregisters a custom detail layout delegate for a specific class in this instance of the generator only
	 *
	 * @param Class	The class with the custom detail layout delegate to remove
	 */
	virtual void UnregisterInstancedCustomPropertyLayout(UStruct* Class) = 0;
	virtual void UnregisterInstancedCustomPropertyTypeLayout(FName PropertyTypeName, TSharedPtr<IPropertyTypeIdentifier> Identifier = nullptr) = 0;

	virtual FOnFinishedChangingProperties& OnFinishedChangingProperties() = 0;
	virtual FOnNotifyStartedChangingProperties& OnNotifyStartedChangingProperties() = 0;

	/* Use this function to set a callback on FPropertyRowGenerator that will override the ValidatePropertyNodes function.
	 * This is useful if your implementation doesn't need to validate nodes every tick or needs to perform some other form of validation. */
	virtual void SetCustomValidatePropertyNodesFunction(FOnValidatePropertyRowGeneratorNodes InCustomValidatePropertyNodesFunction) = 0;

	/* Use this function to set property paths to generate FPropertyNodes. This improves the performance for cases where UPropertyView is only showing a few properties of the object by not generating all other FPropertyNodes 
	*  An example of a nested property path: RootPropertyName[0]->NestedPropertyName[0]->PropertyName
	*/
	virtual void SetPropertyGenerationAllowListPaths(const TSet<FString>& InPropertyGenerationAllowListPaths) = 0;

	/* Invalidates internal cached state.  Common use of this API is to synchronize the viewed object with changes made by external code. */
	virtual void InvalidateCachedState() = 0;

	/** Apply filters to generated row tree nodes */
	virtual void FilterNodes(const TArray<FString>& InFilterStrings) = 0;
};
