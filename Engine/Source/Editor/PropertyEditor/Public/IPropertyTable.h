// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Framework/Views/ITypedTableView.h"
#include "IPropertyTableUtilities.h"
#include "UObject/WeakFieldPtr.h"

class FPropertyPath;
class IPropertyTableCell;
class IPropertyTableColumn;
class IPropertyTableRow;

namespace EPropertyTableSelectionUnit
{
	enum Type
	{
		None = 0x00,
		Row  = 0x01,
		Cell = 0x03
	};
}


namespace EPropertyTableOrientation
{
	enum Type
	{
		AlignPropertiesInColumns, // Items and all their properties are in rows
		AlignPropertiesInRows, // Items and all their properties are in columns
	};
}

class IDataSource
{
public:

	virtual TWeakObjectPtr< UObject > AsUObject() const = 0;
	virtual TSharedPtr< class FPropertyPath > AsPropertyPath() const = 0;

	virtual bool IsValid() const = 0;
};

class IPropertyTable : public IPropertyTableUtilities
{
public: 

	virtual void Tick() = 0;

	virtual void ForceRefresh() = 0;
	virtual void RequestRefresh() = 0;

	virtual void AddColumn( const TWeakObjectPtr< class UObject >& Object ) = 0;
	virtual void AddColumn( const TWeakFieldPtr< class FProperty >& Property ) = 0;
	virtual void AddColumn( const TSharedRef< class FPropertyPath >& PropertyPath ) = 0;
	virtual void AddColumn( const TSharedRef< class IPropertyTableColumn >& Column ) = 0;
	virtual void RemoveColumn( const TSharedRef< class IPropertyTableColumn >& Column ) = 0;

	virtual void AddRow( const TWeakObjectPtr< UObject >& Object ) = 0;
	virtual void AddRow( const TWeakFieldPtr< FProperty >& Property ) = 0;
	virtual void AddRow( const TSharedRef< FPropertyPath >& PropertyPath ) = 0;
	virtual void AddRow( const TSharedRef< class IPropertyTableRow >& Row ) = 0;
	virtual void RemoveRow( const TSharedRef< class IPropertyTableRow >& Row ) = 0;

	virtual const EPropertyTableOrientation::Type GetOrientation() const = 0;
	virtual void SetOrientation( EPropertyTableOrientation::Type InOrientation ) = 0;

	virtual bool GetIsUserAllowedToChangeRoot() = 0;
	virtual void SetIsUserAllowedToChangeRoot( bool InAllowUserToChangeRoot ) = 0;

	virtual void SetRootPath( const TSharedPtr< FPropertyPath >& Path ) = 0;
	virtual TSharedRef< FPropertyPath> GetRootPath() const = 0;
	virtual TArray< struct FPropertyInfo > GetPossibleExtensionsForPath( const TSharedRef< FPropertyPath >& Path ) const = 0;

	virtual void GetSelectedTableObjects( TArray< TWeakObjectPtr< UObject > >& OutSelectedObjects) const = 0;

	virtual void SetObjects( const TArray< TWeakObjectPtr< UObject > >& Objects ) = 0;
	virtual void SetObjects( const TArray< UObject* >& Objects ) = 0;

	virtual TSharedRef< class FObjectPropertyNode > GetObjectPropertyNode( const TSharedRef< class IPropertyTableColumn >& Column, const TSharedRef< class IPropertyTableRow >& Row ) = 0;
	virtual TSharedRef< class FObjectPropertyNode > GetObjectPropertyNode( const TWeakObjectPtr< UObject >& Object ) = 0;

	virtual bool GetShowRowHeader() const = 0;
	virtual void SetShowRowHeader( const bool ShowRowHeader ) = 0;

	virtual bool GetShowObjectName() const = 0;
	virtual void SetShowObjectName( const bool ShowObjectName ) = 0;

	virtual const TArray< TSharedRef< class IPropertyTableColumn > >& GetColumns() = 0;

	virtual void SelectCellRange( const TSharedRef< class IPropertyTableCell >& StartingCell, const TSharedRef< class IPropertyTableCell >& EndingCell ) = 0;

	virtual TArray< TSharedRef< class IPropertyTableRow > >& GetRows() = 0;
	virtual const TSet< TSharedRef< class IPropertyTableRow > >& GetSelectedRows() = 0;
	virtual void SetSelectedRows( const TSet< TSharedRef< class IPropertyTableRow > >& InSelectedRows ) = 0;

	virtual const TSet< TSharedRef< class IPropertyTableCell > >& GetSelectedCells() = 0;
	virtual void SetSelectedCells( const TSet< TSharedRef< class IPropertyTableCell > >& InSelectedCells ) = 0;

	virtual float GetItemHeight() const = 0;
	virtual void SetItemHeight( float NewItemHeight ) = 0;

	virtual TSharedPtr< class IPropertyTableCell > GetLastClickedCell() const = 0;
	virtual void SetLastClickedCell( const TSharedPtr< class IPropertyTableCell >& Cell ) = 0;

	virtual TSharedPtr< class IPropertyTableCell > GetCurrentCell() const = 0;
	virtual void SetCurrentCell( const TSharedPtr< class IPropertyTableCell >& Cell ) = 0;

	virtual TSharedPtr< class IPropertyTableColumn > GetCurrentColumn() const = 0;
	virtual void SetCurrentColumn( const TSharedPtr< class IPropertyTableColumn >& Column ) = 0;

	virtual TSharedPtr< class IPropertyTableRow > GetCurrentRow() const = 0;
	virtual void SetCurrentRow( const TSharedPtr< class IPropertyTableRow >& Row ) = 0;

	virtual TSharedPtr< class IPropertyTableCell > GetFirstCellInSelection() = 0;
	virtual TSharedPtr< class IPropertyTableCell > GetLastCellInSelection() = 0;

	virtual TSharedPtr< class IPropertyTableCell > GetNextCellInRow( const TSharedRef< class IPropertyTableCell >& Cell ) = 0;
	virtual TSharedPtr< class IPropertyTableCell > GetPreviousCellInRow( const TSharedRef< class IPropertyTableCell >& Cell ) = 0;

	virtual TSharedPtr< class IPropertyTableCell > GetNextCellInColumn( const TSharedRef< class IPropertyTableCell >& Cell ) = 0;
	virtual TSharedPtr< class IPropertyTableCell > GetPreviousCellInColumn( const TSharedRef< class IPropertyTableCell >& Cell ) = 0;

	virtual TSharedPtr< class IPropertyTableCell > GetFirstCellInRow( const TSharedRef< class IPropertyTableRow >& Row ) = 0;
	virtual TSharedPtr< class IPropertyTableCell > GetLastCellInRow( const TSharedRef< class IPropertyTableRow >& Row ) = 0;

	virtual TSharedPtr< class IPropertyTableCell > GetFirstCellInColumn( const TSharedRef< class IPropertyTableColumn >& Column ) = 0;
	virtual TSharedPtr< class IPropertyTableCell > GetLastCellInColumn( const TSharedRef< class IPropertyTableColumn >& Column ) = 0;

	virtual TSharedPtr< class IPropertyTableCell > GetFirstCellInTable() = 0;
	virtual TSharedPtr< class IPropertyTableCell > GetLastCellInTable() = 0;

	virtual EPropertyTableSelectionUnit::Type GetSelectionUnit() const = 0;
	virtual void SetSelectionUnit( const EPropertyTableSelectionUnit::Type Unit ) = 0;

	virtual ESelectionMode::Type GetSelectionMode() const = 0;
	virtual void SetSelectionMode( const ESelectionMode::Type Mode ) = 0;

	virtual EColumnSortPriority::Type GetColumnSortPriority(const TSharedRef< class IPropertyTableColumn > Column) const = 0;
	virtual EColumnSortMode::Type GetColumnSortMode( const TSharedRef< class IPropertyTableColumn > Column ) const = 0;
	virtual void SortByColumnWithId( const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type SortMode ) = 0;
	virtual void SortByColumn( const TSharedRef< class IPropertyTableColumn >& Column, EColumnSortMode::Type SortMode, const EColumnSortPriority::Type SortPriority) = 0;

	virtual void PasteTextAtCell( const FString& Text, const TSharedRef< class IPropertyTableCell >& Cell ) = 0;

	DECLARE_EVENT( IPropertyTable, FSelectionChanged );
	virtual FSelectionChanged* OnSelectionChanged() = 0;

	DECLARE_EVENT( IPropertyTable, FColumnsChanged );
	virtual FColumnsChanged* OnColumnsChanged() = 0;

	DECLARE_EVENT( IPropertyTable, FRowsChanged );
	virtual FRowsChanged* OnRowsChanged() = 0;

	DECLARE_EVENT( IPropertyTable, FRootPathChanged );
	virtual FRootPathChanged* OnRootPathChanged() = 0;
};

