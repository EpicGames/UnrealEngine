// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PropertyHandle.h"

class IPropertyTableCell
{
public:
	virtual ~IPropertyTableCell() = default;

	virtual void Refresh() = 0;

	virtual bool IsReadOnly() const = 0;
	virtual bool IsBound() const = 0;
	virtual bool InEditMode() const = 0;
	virtual bool IsValid() const = 0;
	virtual bool PassesPermissionList() const { return true; }

	virtual FString GetValueAsString() const = 0;
	virtual FText GetValueAsText() const = 0;
	virtual void SetValueFromString( const FString& InString ) = 0;

	virtual TWeakObjectPtr< UObject > GetObject() const = 0;
	virtual TSharedPtr< class FPropertyNode > GetNode() const = 0;
	virtual TSharedRef< class IPropertyTableColumn > GetColumn() const = 0;
	virtual TSharedRef< class IPropertyTableRow > GetRow() const = 0;
	virtual TSharedRef< class IPropertyTable > GetTable() const = 0;
	virtual TSharedPtr< class IPropertyHandle > GetPropertyHandle() const = 0;

	virtual void EnterEditMode() = 0;
	virtual void ExitEditMode() = 0;

	DECLARE_EVENT( IPropertyTableCell, FEnteredEditModeEvent );
	virtual FEnteredEditModeEvent& OnEnteredEditMode() = 0;

	DECLARE_EVENT( IPropertyTableCell, FExitedEditModeEvent );
	virtual FExitedEditModeEvent& OnExitedEditMode() = 0;
};
