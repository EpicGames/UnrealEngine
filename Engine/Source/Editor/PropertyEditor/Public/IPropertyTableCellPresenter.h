// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateBrush.h"

class IPropertyTableCellPresenter
{
public:
	virtual ~IPropertyTableCellPresenter() = default;

	virtual TSharedRef< class SWidget > ConstructDisplayWidget() = 0;

	virtual const FSlateBrush* GetEditModeCellBrush() { return nullptr; }

	virtual bool RequiresDropDown() = 0;

	virtual TSharedRef< class SWidget > ConstructEditModeCellWidget() = 0;

	virtual TSharedRef< class SWidget > ConstructEditModeDropDownWidget() = 0;

	virtual TSharedRef< class SWidget > WidgetToFocusOnEdit() = 0;

	virtual FString GetValueAsString() = 0;

	virtual FText GetValueAsText() = 0;

	virtual bool HasReadOnlyEditMode() = 0;
};
