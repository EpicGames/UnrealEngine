// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/Views/SHeaderRow.h"

namespace EPropertyTableColumnSizeMode
{
	enum Type
	{
		Fill,
		Fixed,
	};
}

class IPropertyTableColumn
{
public:

	virtual FName GetId() const = 0;

	virtual FText GetDisplayName() const = 0;

	virtual TSharedRef< class IDataSource > GetDataSource() const = 0;

	virtual TSharedRef< class FPropertyPath > GetPartialPath() const = 0;

	virtual TSharedRef< class IPropertyTableCell > GetCell( const TSharedRef< class IPropertyTableRow >& Row ) = 0;

	virtual void RemoveCellsForRow( const TSharedRef< class IPropertyTableRow >& Row ) = 0;

	virtual TSharedRef< class IPropertyTable > GetTable() const = 0;

	virtual bool CanSelectCells() const = 0;

	virtual EPropertyTableColumnSizeMode::Type GetSizeMode() const = 0;

	virtual void SetSizeMode(EPropertyTableColumnSizeMode::Type InSizeMode) = 0;

	virtual float GetWidth() const = 0;
	virtual void SetWidth( float InWidth ) = 0;

	virtual bool IsHidden() const = 0;
	virtual void SetHidden( bool IsHidden ) = 0;

	virtual bool IsFrozen() const = 0;
	virtual void SetFrozen( bool IsFrozen ) = 0;

	virtual bool CanSortBy() const = 0;
	virtual void Sort( TArray< TSharedRef< class IPropertyTableRow > >& Rows, const EColumnSortMode::Type PrimarySortMode, const TSharedPtr<IPropertyTableColumn>& SecondarySortColumn, const EColumnSortMode::Type SecondarySortMode ) = 0;

	virtual TSharedPtr<struct FCompareRowByColumnBase> GetPropertySorter(const FProperty* Property, EColumnSortMode::Type SortMode) = 0;

	/**
	 * Tick the property column each frame
	 */
	virtual void Tick() = 0;

	DECLARE_EVENT_OneParam( IPropertyTableColumn, FFrozenStateChanged, const TSharedRef< IPropertyTableColumn >& );
	virtual FFrozenStateChanged* OnFrozenStateChanged() = 0;
};
