// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IPropertyTableColumn.h"
#include "IPropertyTableUtilities.h"

class IPropertyTableCell;
class IPropertyTableCellPresenter;

class IPropertyTableCustomColumn
{
public:
	virtual ~IPropertyTableCustomColumn() = default;

	virtual bool Supports( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities ) const = 0;

	virtual TSharedPtr< SWidget > CreateColumnLabel( const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style ) const = 0;

	virtual TSharedPtr< IPropertyTableCellPresenter > CreateCellPresenter( const TSharedRef< IPropertyTableCell >& Cell, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style ) const = 0;
};
