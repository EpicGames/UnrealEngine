// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class IPropertyTableRow
{
public:

	virtual TSharedRef< class IDataSource > GetDataSource() const = 0;
	
	virtual bool HasChildren() const = 0;

	virtual void GetChildRows( TArray< TSharedRef< class IPropertyTableRow > >& OutChildren ) const = 0;

	virtual TSharedRef< class IPropertyTable > GetTable() const = 0;

	virtual bool HasCells() const = 0;

	virtual TSharedRef< class FPropertyPath > GetPartialPath() const = 0;

	virtual void Tick() = 0;

	virtual void Refresh() = 0;

	DECLARE_EVENT( IPropertyTableRow, FRefreshed );
	virtual FRefreshed* OnRefresh() = 0;
};
