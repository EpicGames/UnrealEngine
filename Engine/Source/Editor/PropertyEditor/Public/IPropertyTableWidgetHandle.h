// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class IPropertyTableWidgetHandle
{
public: 

	virtual void RequestRefresh() = 0;
	virtual TSharedRef<class SWidget> GetWidget() = 0;

};
