// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/AppStyle.h"

class FAssetThumbnailPool;
class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;
class IPropertyTypeCustomizationUtils;
class IPropertyUtilities;

/**
 * Base class for property type customizations. The property type customizations are used for Struct and instanced Object properties.
 *
 * The CustomizeHeader() is used to customize the property row, and the CustomizeChildren() is called to add property rows under the header row.
 * If CustomizeHeader() does not populate the row, the child rows will be added inline at the level where the header row would have gone.
 *
 * Note for Object properties:
 *  - The customization is looked up based on the instanced Object type, or if multiple objects are edited, the common base class.
 *  - The PropertyHandle points to the property value (object pointer), the first child of PropertyHandle is the instanced object,
 *    and the instanced object properties are child of that.
 */
class IPropertyTypeCustomization
	: public TSharedFromThis<IPropertyTypeCustomization>
{
public:
	virtual ~IPropertyTypeCustomization() = default;

	/**
	 * Called when the header of the property (the row in the details panel where the property is shown)
	 * If nothing is added to the row, the header is not displayed
	 *
	 * @param PropertyHandle			Handle to the property being customized
	 * @param HeaderRow					A row that widgets can be added to
	 * @param StructCustomizationUtils	Utilities for customization
	 */
	virtual void CustomizeHeader( TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils ) = 0;

	/**
	 * Called when the children of the property should be customized or extra rows added
	 *
	 * @param PropertyHandle			Handle to the property being customized
	 * @param StructBuilder				A builder for adding children
	 * @param StructCustomizationUtils	Utilities for customization
	 */
	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils ) = 0;

	
	/**
	 * @return whether or not this customisation should be inlined when used as a key within a row
	 */
	virtual bool ShouldInlineKey() const
	{
		return false;
	}
};


/**
 * Utilities for property type customization
 */
class IPropertyTypeCustomizationUtils
{
public:
	virtual ~IPropertyTypeCustomizationUtils()= default;

	/**
	 * @return the font used for properties and details
	 */ 
	static FSlateFontInfo GetRegularFont() { return FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont") ); }

	/**
	 * @return the bold font used for properties and details
	 */ 
	static FSlateFontInfo GetBoldFont() { return FAppStyle::GetFontStyle( TEXT("PropertyWindow.BoldFont") ); }
	
	/**
	 * Gets the thumbnail pool that should be used for rendering thumbnails in the struct                  
	 */
	virtual TSharedPtr<FAssetThumbnailPool> GetThumbnailPool() const = 0;

	/**
	*	@return the utilities various widgets need access to certain features of PropertyDetails
	*/
	virtual TSharedPtr<IPropertyUtilities> GetPropertyUtilities() const { return nullptr; }
};


/** Deprecated IStructCustomizationUtils interface */
typedef IPropertyTypeCustomizationUtils IStructCustomizationUtils;

/** Deprecated IStructCustomization interface */
class IStructCustomization : public IPropertyTypeCustomization
{
public:
	virtual void CustomizeHeader( TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils )
	{
		CustomizeStructHeader( PropertyHandle, HeaderRow, CustomizationUtils );
	}

	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils ) 
	{
		CustomizeStructChildren( PropertyHandle, ChildBuilder, CustomizationUtils );
	}

	/**
	 * Called when the header of the struct (usually where the name of the struct and information about the struct as a whole is added)
	 * If nothing is added to the row, the header is not displayed
	 *
	 * @param StructPropertyHandle		Handle to the struct property
	 * @param HeaderRow					A row that widgets can be added to
	 * @param StructCustomizationUtils	Utilities for struct customization
	 */
	virtual void CustomizeStructHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IStructCustomizationUtils& StructCustomizationUtils ) = 0;

	/**
	 * Called when the children of the struct should be customized
	 *
	 * @param StructPropertyHandle		Handle to the struct property
	 * @param StructBuilder				A builder for customizing the struct children
	 * @param StructCustomizationUtils	Utilities for struct customization
	 */
	virtual void CustomizeStructChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IStructCustomizationUtils& StructCustomizationUtils ) = 0;
};
