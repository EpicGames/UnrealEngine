// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "AssetThumbnail.h"

struct FPropertyChangedEvent;
class FEditConditionParser;

/**
 * Settings for property editor widgets that call up to the base container for the widgets
 * without knowing information about that container
 */
class IPropertyUtilities
{
public:
	virtual ~IPropertyUtilities()= default;
	virtual class FNotifyHook* GetNotifyHook() const = 0;
	virtual bool AreFavoritesEnabled() const = 0;
	virtual void ToggleFavorite( const TSharedRef< class FPropertyEditor >& PropertyEditor ) const = 0;
	virtual void CreateColorPickerWindow( const TSharedRef< class FPropertyEditor >& PropertyEditor, bool bUseAlpha ) const = 0;
	virtual void EnqueueDeferredAction( FSimpleDelegate DeferredAction ) = 0;
	virtual bool IsPropertyEditingEnabled() const = 0;

	/**
	 * Force a rebuild of the view, recreating and updating all widgets.
	 * @note This may run immediately; consider RequestForceRefresh unless you need to ensure invalid references (eg, to deleted objects) are removed.
	 */
	virtual void ForceRefresh() = 0;

	/**
	 * Request a refresh of the view on the next tick, ideally without triggering a full rebuild of all widgets.
	 * @note This may run immediately or perform a rebuild depending on the implementation.
	 */
	virtual void RequestRefresh() = 0;

	/**
	 * Request a rebuild of the view on the next tick, recreating and updating all widgets.
	 * @note This may run immediately depending on the implementation.
	 */
	virtual void RequestForceRefresh() = 0;
	
	virtual TSharedPtr<class FAssetThumbnailPool> GetThumbnailPool() const = 0;
	virtual void NotifyStartedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) = 0;
	virtual void NotifyFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) = 0;
	virtual bool HasClassDefaultObject() const = 0;
	virtual const TArray<TWeakObjectPtr<UObject>>& GetSelectedObjects() const = 0;
	/** If a customization standalone widget is used, the value should be update only once, when its window is closed */
	virtual bool DontUpdateValueWhileEditing() const = 0;
	virtual const TArray<TSharedRef<class IClassViewerFilter>>& GetClassViewerFilters() const = 0;
};
