// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Fonts/SlateFontInfo.h"
#include "PropertyEditorModule.h"

class FNotifyHook;

/**
 * Init params for a single property
 */
struct FSinglePropertyParams
{
	/** Override for the property name that will be displayed instead of the property name */
	FText NameOverride;

	/** Font to use instead of the default property font */
	FSlateFontInfo Font;

	/** Notify hook interface to use for some property change events */
	FNotifyHook* NotifyHook;

	/** Whether or not to show the name */
	EPropertyNamePlacement::Type NamePlacement;

	/** Whether to hide an asset thumbnail, if available */
	bool bHideAssetThumbnail;

	/** Whether to hide the 'reset to default' button */
	bool bHideResetToDefault;

	FSinglePropertyParams()
		: NameOverride(FText::GetEmpty())
		, Font()
		, NotifyHook( NULL )
		, NamePlacement( EPropertyNamePlacement::Left )
		, bHideAssetThumbnail( false )
		, bHideResetToDefault( false )
	{
	}
};


/**
 * Represents a single property not in a property tree or details view for a single object
 * Structs and Array properties cannot be used with this method
 */
class ISinglePropertyView : public SCompoundWidget
{
public:
	/** Sets the object to view/edit on the widget */
	virtual void SetObject( UObject* InObject ) = 0;

	/** Sets the struct to view/edit on the widget */
	virtual void SetStruct( const TSharedPtr<class IStructureDataProvider>& InStruct) = 0;

	/** Sets a delegate called when the property value changes */
	virtual void SetOnPropertyValueChanged( const FSimpleDelegate& InOnPropertyValueChanged ) = 0;

	/** Whether or not this widget has a valid property */	
	virtual bool HasValidProperty() const = 0;

	virtual TSharedPtr<class IPropertyHandle> GetPropertyHandle() const = 0;
};
