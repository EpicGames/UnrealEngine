// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "UObject/StructOnScope.h"

//-----------------------------------------------------------------------------
//	IStructureDataProvider - Used to provide struct data for a property node.
//
// The IsPropertyIndirection() and GetValueBaseAddress() allow a struct provider to expose data to be edited that
// is not supported by the property system. When IsPropertyIndirection() returns true, GetValueBaseAddress() is called
// relative to the parent property node to retrieve pointer to the data to be edited.
// 
// In the following example, a struct provider is added via ParentProperty property handle. In this case "ParentProperty"
// is the parent property node, and the value passed into GetValueBaseAddress() of the provider is the value of the parent property.
//
//		TSharedPtr<IPropertyHandle> ParentProperty = ...; // E.g. handle to structure being customized
//		...
//		TSharedRef<FMyStructProvider> NewStructProvider = MakeShared<FMyStructProvider>(ParentProperty);
//		TArray<TSharedPtr<IPropertyHandle>> ChildProperties = ParentProperty->AddChildStructure(NewStructProvider);
// 
// The GetValueBaseAddress() may get called on foreign objects too (e.g. object templates), and if property indirection is used
// it should not rely on other values cached in the provider.
// 
//		class FMyStructProvider : public IStructureDataProvider
//		{
//		public:
//			...
//
//			virtual bool IsPropertyIndirection() const override { return true; }
//
//			virtual uint8* GetValueBaseAddress(uint8* ParentValueAddress, const UStruct* ExpectedType) const override
//			{
//				if (ParentValueAddress)
//				{
//					// "ParentValueAddress" is pointer to a value of the type defined in "ParentProperty".
//					FMyStruct& Value = *reinterpret_cast<FMyStruct*>(ParentValueAddress);
//					// The return value should match the ExpectedType. The expected type should be the same as the GetBaseStructure() returned by this provider.
//					return (Value.GetScriptStruct() == ExpectedType) ? Value.GetMemory() : nullptr;
//				}
//				return nullptr;
//			}
//		};
// 
// If IsPropertyIndirection() is false, there structures are expected to be standalone, and GetInstances() is used instead to retried the instance values.
// When indirection is not used and the struct provider is part of an another instance, the provider should return only one instance, since there is no to associate the provided value
// with an foreign object.
//
// ExpectedType parameter: This is the type that e.g. the UI is currently initialized for (cached via GetBaseStructure()).
// It is possible that the provided data changes due to some property change callbacks, and the ExpectedType does not match the provided data type.
// In that case empty data should be returned so that the UI does not try to access incompatible data.
//
// Multiple instances are supported when the provider is the root node (e.g. on Structure details view).
// 
//-----------------------------------------------------------------------------
class IStructureDataProvider
{
public:
	virtual ~IStructureDataProvider() = default;

	/** @return true if we have any data instances. */
	virtual bool IsValid() const = 0;
	
	/** @return the most common struct of the instance data. This struct will be used to build the UI for the instances. */
	virtual const UStruct* GetBaseStructure() const = 0;

	
	/**
	 * Returns instances to edit. Each provided struct should be compatible with ExpectedType.
	 * @param OutInstances returned instances
	 * @param ExpectedBaseStructure the base type of struct that is expected to be returned (see the documentation above about ExpectedType) 
	 */
	virtual void GetInstances(TArray<TSharedPtr<FStructOnScope>>& OutInstances, const UStruct* ExpectedBaseStructure) const = 0;

	/** @return true, if the provider supports handling indirections using GetValueBaseAddress(). */
	virtual bool IsPropertyIndirection() const { return false; }

	/**
	 * Returns base address of provided struct based on parent property nodes value.
	 * @param ParentValueAddress Value of the parent 
	 * @param ExpectedBaseStructure the base type of struct that is expected to be returned (see the documentation above about ExpectedType)
	 * @return value base address based on parent base address.
	 */
	virtual uint8* GetValueBaseAddress(uint8* ParentValueAddress, const UStruct* ExpectedBaseStructure) const { return ParentValueAddress; }

	UE_DEPRECATED(5.4, "Please use and override GetInstances() with ExpectedType instead.")
	virtual void GetInstances(TArray<TSharedPtr<FStructOnScope>>& OutInstances) const final
	{
		return GetInstances(OutInstances, nullptr);
	}
};


//-----------------------------------------------------------------------------
//	FStructOnScopeStructureDataProvider - Implementation of standalone struct that provides 
//  one base structure and one or more instances (allowing multi selection edit)
//-----------------------------------------------------------------------------
class FStructOnScopeStructureDataProvider : public IStructureDataProvider
{
public:
	FStructOnScopeStructureDataProvider() = default;
	explicit FStructOnScopeStructureDataProvider(const TSharedPtr<FStructOnScope>& InStructData)
		: StructDataInstances( {InStructData} )
	{
	}
	
	explicit FStructOnScopeStructureDataProvider(const TArray<TSharedPtr<FStructOnScope>>& InStructDataInstances)
		: StructDataInstances(InStructDataInstances)
	{
	}
	
	void SetStructData(const TSharedPtr<FStructOnScope>& InStructData)
	{
		StructDataInstances.Reset(1);
		StructDataInstances.Add(InStructData);
	}
	void SetStructData(const TArray<TSharedPtr<FStructOnScope>>& InStructData)
	{
		StructDataInstances.Reset(InStructData.Num());
		StructDataInstances = InStructData;
	}

	
	virtual bool IsValid() const override
	{
		const UStruct* BaseStructure = GetBaseStructure();
		for (const TSharedPtr<FStructOnScope>& StructInstance : StructDataInstances)
		{
			if (StructInstance.IsValid() && StructInstance->IsValid())
			{
				const UStruct* Struct = StructInstance->GetStruct();
				if (Struct->IsChildOf(BaseStructure))
				{
					return true;
				}
			}
		}

		return false;
	};

	virtual const UStruct* GetBaseStructure() const override
	{
		return FindBaseStructure(StructDataInstances);
	}
	
	virtual void GetInstances(TArray<TSharedPtr<FStructOnScope>>& OutInstances, const UStruct* ExpectedBaseStructure) const override
	{
		if (ExpectedBaseStructure != nullptr)
		{
			for (const TSharedPtr<FStructOnScope> &StructData : StructDataInstances)
			{
				if (StructData.IsValid()) 
				{
					const UStruct* Struct = StructData->GetStruct();
					if (Struct && Struct->IsChildOf(ExpectedBaseStructure))
					{
						OutInstances.Add(StructData);				
					}
				}
			}
		}
	}

	template<typename ContainerType>
	static const UStruct* FindBaseStructure(const TArray<TSharedPtr<ContainerType>>& StructDataInstances)
	{
		const UStruct* BaseStructure = nullptr;

		const int32 NumInstances = StructDataInstances.Num();
		if (NumInstances > 0)
		{
			int32 InstanceIndex = 0;

			// find first valid instance
			while ( !(StructDataInstances[InstanceIndex].IsValid() && StructDataInstances[InstanceIndex]->IsValid()) )
			{
				++InstanceIndex;
				if (InstanceIndex == NumInstances)
				{
					// no valid instances found
					return nullptr;
				}
			}

			BaseStructure = StructDataInstances[InstanceIndex]->GetStruct();

			// Iterate from there, if any instance left
			for (int32 i = InstanceIndex + 1; i < NumInstances; i++)
			{
				const TSharedPtr<ContainerType>& StructData = StructDataInstances[i];
				if (StructData.IsValid() && StructData->IsValid())
				{
					const UStruct* ScriptStruct = StructData->GetStruct();
					// check if the base structure is an ancestor of the script struct
					while (BaseStructure && ScriptStruct && !ScriptStruct->IsChildOf(BaseStructure))
					{
						// if not, go a level up in the base structure hierarchy
						BaseStructure = Cast<UStruct>(BaseStructure->GetSuperStruct());
					}
				}
			}
		}

		return BaseStructure;
	}

protected:
	TArray<TSharedPtr<FStructOnScope>> StructDataInstances;
};
