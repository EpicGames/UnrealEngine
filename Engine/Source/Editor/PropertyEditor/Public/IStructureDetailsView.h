// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "UObject/StructOnScope.h"
#include "PropertyEditorDelegates.h"

class IDetailsView;

/**
 * Interface class for all detail views
 */
class IStructureDetailsView
{
public:

	/**
	 * Get an interface the underlying details view.
	 *
	 * @return Details view interface.
	 */
	virtual IDetailsView* GetDetailsView() = 0;

	/**
	 * Get this view's Slate widget.
	 *
	 * @return The widget for this view.
	 */
	virtual TSharedPtr<class SWidget> GetWidget() = 0;

	/**
	 * Set the structure data to be displayed in this view.
	 *
	 * @param StructData The structure data to display.
	 */
	virtual void SetStructureData(TSharedPtr<class FStructOnScope> StructData) = 0;

	/**
	 * Set the structure data to be displayed in this view.
	 *
	 * @param StructProvider Provider of the the structure data to display.
	 */
	virtual void SetStructureProvider(TSharedPtr<class IStructureDataProvider> StructProvider) = 0;

	/**
	 * Set the CustomName used for display purposes.
	 *
	 * @param Text The FText to use for setting the Name
	 */
	virtual void SetCustomName(const FText& Text) = 0;

	/*
	* Get Const access to the StructureProvider.
	* Not pure virtual for backwards compatibility reasons.
	*/
	virtual TSharedPtr<const class IStructureDataProvider> GetStructureProvider() const
	{
		return nullptr;
	}

public:

	/**
	 * Get a delegate that is executed when the view has finished changing properties.
	 *
	 * @return The delegate.
	 */
	virtual FOnFinishedChangingProperties& GetOnFinishedChangingPropertiesDelegate() = 0;
};
