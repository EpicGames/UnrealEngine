// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "IDetailCustomNodeBuilder.h"
#include "DetailWidgetRow.h"
#include "Materials/MaterialInterface.h"

#define UE_API PROPERTYEDITOR_API

class FMaterialItemView;
class FMaterialListBuilder;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class IMaterialListBuilder;
class UActorComponent;

/**
 * Delegate called when we need to get new materials for the list
 */
DECLARE_DELEGATE_OneParam(FOnGetMaterials, IMaterialListBuilder&);

/**
 * Delegate called when a user changes the material
 */
DECLARE_DELEGATE_FourParams( FOnMaterialChanged, UMaterialInterface*, UMaterialInterface*, int32, bool );

DECLARE_DELEGATE_RetVal_TwoParams( TSharedRef<SWidget>, FOnGenerateWidgetsForMaterial, UMaterialInterface*, int32 );

DECLARE_DELEGATE_TwoParams( FOnResetMaterialToDefaultClicked, UMaterialInterface*, int32 );

DECLARE_DELEGATE_RetVal(bool, FOnMaterialListDirty);

DECLARE_DELEGATE_RetVal(bool, FOnCanCopyMaterialList);
DECLARE_DELEGATE(FOnCopyMaterialList);
DECLARE_DELEGATE(FOnPasteMaterialList);

DECLARE_DELEGATE_RetVal_OneParam(bool, FOnCanCopyMaterialItem, int32);
DECLARE_DELEGATE_OneParam(FOnCopyMaterialItem, int32);
DECLARE_DELEGATE_OneParam(FOnPasteMaterialItem, int32);

struct FMaterialListDelegates
{
	FMaterialListDelegates()
		: OnGetMaterials()
		, OnMaterialChanged()
		, OnGenerateCustomNameWidgets()
		, OnGenerateCustomMaterialWidgets()
		, OnResetMaterialToDefaultClicked()
		, OnGenerateExtraBottomWidget()
	{}

	/** Delegate called to populate the list with materials */
	FOnGetMaterials OnGetMaterials;
	/** Delegate called when a user changes the material */
	FOnMaterialChanged OnMaterialChanged;
	/** Delegate called to generate custom widgets under the name of in the left column of a details panel*/
	FOnGenerateWidgetsForMaterial OnGenerateCustomNameWidgets;
	/** Delegate called to generate custom widgets under each material */
	FOnGenerateWidgetsForMaterial OnGenerateCustomMaterialWidgets;
	/** Delegate called when a material list item should be reset to default */
	FOnResetMaterialToDefaultClicked OnResetMaterialToDefaultClicked;
	/** Delegate called to generate extra widget under the material row */
	FOnGenerateWidgetsForMaterial OnGenerateExtraBottomWidget;
	/** Delegate called when we tick the material list to know if the list is dirty*/
	FOnMaterialListDirty OnMaterialListDirty;

	/** Delegate called Copying a material list */
	FOnCopyMaterialList OnCopyMaterialList;
	/** Delegate called to know if we can copy a material list */
	FOnCanCopyMaterialList OnCanCopyMaterialList;
	/** Delegate called Pasting a material list */
	FOnPasteMaterialList OnPasteMaterialList;
	/** Delegate called Pasting an optionally tagged text snippet */
	TWeakPtr<FOnPasteFromText> OnPasteFromText;

	/** Delegate called Copying a material item */
	FOnCopyMaterialItem OnCopyMaterialItem;
	/** Delegate called to know if we can copy a material item */
	FOnCanCopyMaterialItem OnCanCopyMaterialItem;
	/** Delegate called Pasting a material item */
	FOnPasteMaterialItem OnPasteMaterialItem;
};

/**
 * Builds up a list of unique materials while creating some information about the materials
 */
class IMaterialListBuilder
{
public:

	/** Virtual destructor. */
	virtual ~IMaterialListBuilder()= default;

	/** 
	 * Adds a new material to the list
	 * 
	 * @param SlotIndex The slot (usually mesh element index) where the material is located on the component.
	 * @param Material The material being used.
	 * @param bCanBeReplced Whether or not the material can be replaced by a user.
	 * @param InCurrentComponent The current component using the current material
	 */
	virtual void AddMaterial( uint32 SlotIndex, UMaterialInterface* Material, bool bCanBeReplaced, UActorComponent* InCurrentComponent = nullptr, FName SlotName = FName()) = 0;
};

/**
 * A Material item in a material list slot
 */
struct FMaterialListItem
{
	/** Material being used */
	TWeakObjectPtr<UMaterialInterface> Material;

	/** Slot on a component where this material is at (mesh element) */
	int32 SlotIndex;

	/** Whether or not this material can be replaced by a new material */
	bool bCanBeReplaced;

	/** The current component using the current material */
	TWeakObjectPtr<UActorComponent> CurrentComponent;

	/** User-specified name for the slot on a component where this material is at (mesh element). May be empty. */
	FName SlotName;

	FMaterialListItem( UMaterialInterface* InMaterial = NULL, uint32 InSlotIndex = 0, bool bInCanBeReplaced = false, UActorComponent* InCurrentComponent = nullptr, FName InSlotName = FName())
		: Material( InMaterial )
		, SlotIndex( InSlotIndex )
		, bCanBeReplaced( bInCanBeReplaced )
		, CurrentComponent(InCurrentComponent)
		, SlotName(InSlotName)
	{}

	friend uint32 GetTypeHash( const FMaterialListItem& InItem )
	{
		return HashCombine(HashCombine(GetTypeHash(InItem.Material), GetTypeHash(InItem.SlotIndex)), GetTypeHash(InItem.SlotName));
	}

	bool operator==( const FMaterialListItem& Other ) const
	{
		return Material == Other.Material && SlotIndex == Other.SlotIndex && SlotName == Other.SlotName;
	}

	bool operator!=( const FMaterialListItem& Other ) const
	{
		return !(*this == Other);
	}
};

/**
 * A view of a single item in an FMaterialList
 */
class FMaterialItemView : public TSharedFromThis<FMaterialItemView>
{
public:
	/** Virtual destructor */
	virtual ~FMaterialItemView() = default;
	
	/**
	* Creates a new instance of this class
	*
	* @param Material				The material to view
	* @param InOnMaterialChanged	Delegate for when the material changes
	*/
	static UE_API TSharedRef<FMaterialItemView> Create(
		const FMaterialListItem& Material, 
		FOnMaterialChanged InOnMaterialChanged,
		FOnGenerateWidgetsForMaterial InOnGenerateNameWidgetsForMaterial, 
		FOnGenerateWidgetsForMaterial InOnGenerateWidgetsForMaterial, 
		FOnResetMaterialToDefaultClicked InOnResetToDefaultClicked,
		FOnGenerateWidgetsForMaterial InOnGenerateExtraBottomWidget,
		int32 InMultipleMaterialCount,
		bool bShowUsedTextures);

	/**
	 * Create a Name Content for Material Item Widget
	 */
	UE_API virtual TSharedRef<SWidget> CreateNameContent();

	/**
	 * Create a Value Content for Material Item Widget
	 */
	UE_API virtual TSharedRef<SWidget> CreateValueContent(IDetailLayoutBuilder& InDetailBuilder, const TArray<FAssetData>& OwnerAssetDataArray, UActorComponent* InActorComponent);

	/**
	 * Get Material item in a material list slot
	 */
	const FMaterialListItem& GetMaterialListItem() const { return MaterialItem; }

	/**
	 * Replace material in slot
	 *
	 * @param NewMaterial			The new material apply to slot
	 * @param bReplaceAll			Whether it should be applied to all slots
	 */
	UE_API void ReplaceMaterial( UMaterialInterface* NewMaterial, bool bReplaceAll = false );

	/**
	 * Called to get the visibility of the reset to base button
	 */
	UE_API bool GetResetToBaseVisibility() const;

	/**
	 * Called when reset to base is clicked
	 */
	UE_API void OnResetToBaseClicked();

private:
	UE_API FMaterialItemView(const FMaterialListItem& InMaterial, 
					FOnMaterialChanged& InOnMaterialChanged, 
					FOnGenerateWidgetsForMaterial& InOnGenerateNameWidgets, 
					FOnGenerateWidgetsForMaterial& InOnGenerateMaterialWidgets, 
					FOnResetMaterialToDefaultClicked& InOnResetToDefaultClicked,
					FOnGenerateWidgetsForMaterial& OnGenerateExtraBottomWidget,
					int32 InMultipleMaterialCount,
					bool bInShowUsedTextures);

	/**
	 * Handler when the material asset has been changed
	 */
	UE_API void OnSetObject( const FAssetData& AssetData );

	/**
	 * @return Whether or not the textures menu is enabled
	 */
	UE_API bool IsTexturesMenuEnabled() const;

	/**
	 * Generate list of the textures for material selection list
	 */
	UE_API TSharedRef<SWidget> OnGetTexturesMenuForMaterial();

	/**
	 * Create button for browse to nanite override material
	 */
	UE_API TSharedRef<SWidget> MakeBrowseNaniteOverrideMaterialButton() const;

	/**
	 * @return The tool tip for the textures menuuuuu
	 */
	UE_API FText GetTexturesMenuToolTipText() const;

	/**
	 * On Get object path handler
	 */
	UE_API FString OnGetObjectPath() const;

	/**
	 * Finds the asset in the content browser
	 */
	UE_API void GoToAssetInContentBrowser( TWeakObjectPtr<UObject> Object );

	/**
	 * Get Property Row Extenssion Arguments.
	 * The extension arguments is needed to add button or custom widget to property row.
	 *
	 * @param InDetailBuilder The builder for laying custom details.
	 * @param InCurrentComponent The current component using the current material.
	 * @return Generate Global Row Extension Args
	 */
	UE_API struct FOnGenerateGlobalRowExtensionArgs GetGlobalRowExtensionArgs(IDetailLayoutBuilder& InDetailBuilder, UActorComponent* InCurrentComponent) const;

	/**
	 * Get row extension widget
	 *
	 * @param InDetailBuilder The builder for laying custom details.
	 * @param InCurrentComponent The current component using the current material.
	 * @return Pointer to widget
	 */
	UE_API TSharedPtr<SWidget> GetGlobalRowExtensionWidget(IDetailLayoutBuilder& InDetailBuilder, UActorComponent* InCurrentComponent) const;

private:
	/** The Material item in a material list slot */
	FMaterialListItem MaterialItem;

	/** Material Changed delegate */
	FOnMaterialChanged OnMaterialChanged;

	/** Generate custom name delegate */
	FOnGenerateWidgetsForMaterial OnGenerateCustomNameWidgets;

	/** Generate custom material widget delegate */
	FOnGenerateWidgetsForMaterial OnGenerateCustomMaterialWidgets;

	/** Reset to default delegate */
	FOnResetMaterialToDefaultClicked OnResetToDefaultClicked;

	/** Generate custom widget under the material row delegate */
	FOnGenerateWidgetsForMaterial OnGenerateExtraBottomWidget;

	/** Number of material count */
	int32 MultipleMaterialCount;

	/** Whether it should show textures */
	bool bShowUsedTextures;
};

/**
 * Custom Node Builder for Material List
 */
class FMaterialList
	: public IDetailCustomNodeBuilder
	, public TSharedFromThis<FMaterialList>
{
public:
	/** Add Extra widgets for bottom material value field. */
	DECLARE_MULTICAST_DELEGATE_FourParams(FOnAddMaterialItemViewExtraBottomWidget,
									const TSharedRef<FMaterialItemView>& /* InMaterialItemView */,
									UActorComponent* /* InCurrentComponent */,
									IDetailLayoutBuilder& /*InDetailBuilder*/,
									TArray<TSharedPtr<SWidget>>& /* OutExtensions */);
	PROPERTYEDITOR_API static FOnAddMaterialItemViewExtraBottomWidget OnAddMaterialItemViewExtraBottomWidget;
	
	PROPERTYEDITOR_API FMaterialList( IDetailLayoutBuilder& InDetailLayoutBuilder, FMaterialListDelegates& MaterialListDelegates, const TArray<FAssetData>& InOwnerAssetDataArray, bool bInAllowCollapse = false, bool bInShowUsedTextures = true);

private:
	/**
	 * Called when a user expands all materials in a slot.
	 *
	 * @param SlotIndex The index of the slot being expanded.
	 */
	void OnDisplayMaterialsForElement( int32 SlotIndex );

	/**
	 * Called when a user hides all materials in a slot.
	 *
	 * @param SlotIndex The index of the slot being hidden.
	 */
	void OnHideMaterialsForElement( int32 SlotIndex );

	/** IDetailCustomNodeBuilder interface */
	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRebuildChildren  ) override { OnRebuildChildren = InOnRebuildChildren; } 
	virtual bool RequiresTick() const override { return true; }
	virtual void Tick( float DeltaTime ) override;
	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override;
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override;
	virtual FName GetName() const override { return "MaterialList"; }
	virtual bool InitiallyCollapsed() const override { return bAllowCollpase; }

	/**
	 * Adds a new material item to the list
	 *
	 * @param Row					The row to add the item to
	 * @param CurrentSlot			The slot id of the material
	 * @param Item					The material item to add
	 * @param bDisplayLink			If a link to the material should be displayed instead of the actual item (for multiple materials)
	 * @param InCurrentComponent	The current component using the current material
	 */
	void AddMaterialItem(FDetailWidgetRow& Row, int32 CurrentSlot, const FMaterialListItem& Item, bool bDisplayLink, UActorComponent* InCurrentComponent = nullptr);

private:
	bool OnCanCopyMaterialList() const;
	void OnCopyMaterialList();
	void OnPasteMaterialList();

	/** Handle pasting an optionally tagged text snippet */
	void OnPasteMaterialListFromText(const FString& InTag, const FString& InText, const TOptional<FGuid>& InOperationId);

	bool OnCanCopyMaterialItem(int32 CurrentSlot) const;
	void OnCopyMaterialItem(int32 CurrentSlot);
	void OnPasteMaterialItem(int32 CurrentSlot);

	/** Delegates for the material list */
	FMaterialListDelegates MaterialListDelegates;

	/** Called to rebuild the children of the detail tree */
	FSimpleDelegate OnRebuildChildren;

	/** Parent detail layout this list is in */
	IDetailLayoutBuilder& DetailLayoutBuilder;

	/** Set of all unique displayed materials */
	TArray< FMaterialListItem > DisplayedMaterials;

	/** Set of all materials currently in view (may be less than DisplayedMaterials) */
	TArray< TSharedRef<FMaterialItemView> > ViewedMaterials;

	/** Set of all expanded slots */
	TSet<uint32> ExpandedSlots;

	/** Material list builder used to generate materials */
	TSharedRef<FMaterialListBuilder> MaterialListBuilder;

	/** Allow Collapse of material header row. Right now if you allow collapse, it will initially collapse. */
	bool bAllowCollpase;
	/** Whether or not to use the used textures menu for each material entry */
	bool bShowUsedTextures;
	/** The mesh asset that owns these materials */
	TArray<FAssetData> OwnerAssetDataArray;
};

#undef UE_API
