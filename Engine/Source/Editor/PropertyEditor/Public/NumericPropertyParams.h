// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "UObject/UnrealType.h"

template<typename NumericType>
struct TNumericPropertyParams
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(FString, FMetaDataGetter, FName);

	TNumericPropertyParams(const FProperty* Property, FMetaDataGetter MetaDataGetter)	
	{
		FString UIMinString;
		FString UIMaxString;
		FString SliderExponentString;
		FString LinearDeltaSensitivityString;
		FString DeltaString;
		FString ClampMinString;
		FString ClampMaxString;
		FString ForcedUnits;
		FString WheelStepString;

		if (!MetaDataGetter.IsBound() && Property != nullptr)
		{
			MetaDataGetter = FMetaDataGetter::CreateLambda([Property](const FName& Key) 
			{ 
				return Property->GetMetaData(Key); 
			});
		}

		if (MetaDataGetter.IsBound())
		{
			UIMinString = MetaDataGetter.Execute("UIMin");
			UIMaxString = MetaDataGetter.Execute("UIMax");
			SliderExponentString = MetaDataGetter.Execute("SliderExponent");
			LinearDeltaSensitivityString = MetaDataGetter.Execute("LinearDeltaSensitivity");
			DeltaString = MetaDataGetter.Execute("Delta");
			ClampMinString = MetaDataGetter.Execute("ClampMin");
			ClampMaxString = MetaDataGetter.Execute("ClampMax");
			ForcedUnits = MetaDataGetter.Execute("ForceUnits");
			WheelStepString = MetaDataGetter.Execute("WheelStep");
		}

		// If no UIMin/Max was specified then use the clamp string
		const FString& ActualUIMinString = UIMinString.Len() ? UIMinString : ClampMinString;
		const FString& ActualUIMaxString = UIMaxString.Len() ? UIMaxString : ClampMaxString;

		NumericType ClampMin = TNumericLimits<NumericType>::Lowest();
		NumericType ClampMax = TNumericLimits<NumericType>::Max();

		if (!ClampMinString.IsEmpty())
		{
			TTypeFromString<NumericType>::FromString(ClampMin, *ClampMinString);
		}

		if (!ClampMaxString.IsEmpty())
		{
			TTypeFromString<NumericType>::FromString(ClampMax, *ClampMaxString);
		}

		NumericType UIMin = TNumericLimits<NumericType>::Lowest();
		NumericType UIMax = TNumericLimits<NumericType>::Max();
		TTypeFromString<NumericType>::FromString(UIMin, *ActualUIMinString);
		TTypeFromString<NumericType>::FromString(UIMax, *ActualUIMaxString);

		if (ClampMin >= ClampMax && (ClampMinString.Len() || ClampMaxString.Len()))
		{
			UE_LOGF(LogTemp, Warning, "Clamp Min (%ls) >= Clamp Max (%ls) for Ranged Numeric property %ls", *ClampMinString, *ClampMaxString, Property ? *Property->GetPathName() : TEXT(""));
		}

		const NumericType ActualUIMin = FMath::Max(UIMin, ClampMin);
		const NumericType ActualUIMax = FMath::Min(UIMax, ClampMax);

		MinValue = ClampMinString.Len() ? ClampMin : TOptional<NumericType>();
		MaxValue = ClampMaxString.Len() ? ClampMax : TOptional<NumericType>();
		MinSliderValue = (ActualUIMinString.Len()) ? ActualUIMin : TOptional<NumericType>();
		MaxSliderValue = (ActualUIMaxString.Len()) ? ActualUIMax : TOptional<NumericType>();

		if ((ActualUIMin >= ActualUIMax) && (MinSliderValue.IsSet() && MaxSliderValue.IsSet()))
		{
			UE_LOGF(LogTemp, Warning, "UI Min (%ls) >= UI Max (%ls) for Ranged Numeric property %ls", *ActualUIMinString, *ActualUIMaxString, Property ? *Property->GetPathName() : TEXT(""));
		}
		
		SliderExponent = NumericType(1);
		if (SliderExponentString.Len())
		{
			TTypeFromString<NumericType>::FromString(SliderExponent, *SliderExponentString);
		}

		Delta = NumericType(0);
		if (DeltaString.Len())
		{
			TTypeFromString<NumericType>::FromString(Delta, *DeltaString);
		}

		LinearDeltaSensitivity = 0;
		if (LinearDeltaSensitivityString.Len())
		{
			TTypeFromString<int32>::FromString(LinearDeltaSensitivity, *LinearDeltaSensitivityString);
		}
		// LinearDeltaSensitivity only works in SSpinBox if delta is provided, so add it in if it wasn't.
		Delta = (LinearDeltaSensitivity != 0 && Delta == NumericType(0)) ? NumericType(1) : Delta;

		NumericType WheelStepValue = NumericType(0);

		if (!WheelStepString.IsEmpty())
		{
			TTypeFromString<NumericType>::FromString(WheelStepValue, *WheelStepString);
			WheelStep = WheelStepString.Len() ? WheelStepValue : TOptional<NumericType>();
		}
	}

	TAttribute<int32> GetLinearDeltaSensitivityAttribute() const
	{
		return LinearDeltaSensitivity != 0 ? TAttribute<int32>(LinearDeltaSensitivity) : TAttribute<int32>();
	}

	TOptional<NumericType> MinValue;
	TOptional<NumericType> MaxValue;
	TOptional<NumericType> MinSliderValue;
	TOptional<NumericType> MaxSliderValue;
	NumericType SliderExponent;
	NumericType Delta;
	int32 LinearDeltaSensitivity;
	TOptional<NumericType> WheelStep;
};
