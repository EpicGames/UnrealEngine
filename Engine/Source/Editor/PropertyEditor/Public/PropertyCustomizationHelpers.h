// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Misc/Attribute.h"
#include "Layout/Margin.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/SlateDelegates.h"
#include "Materials/MaterialInterface.h"
#include "PropertyHandle.h"
#include "IDetailCustomNodeBuilder.h"
#include "DetailWidgetRow.h"
#include "SResetToDefaultMenu.h"
#include "ActorPickerMode.h"
#include "SceneDepthPickerMode.h"
#include "IDetailPropertyRow.h"
#include "ClassViewerModule.h"

class AActor;
class FAssetThumbnailPool;
class FPropertyEditor;
class IClassViewerFilter;
class IDetailCategoryBuilder;
class IDetailChildrenBuilder;
class IDetailGroup;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SPropertyEditorAsset;
class SPropertyEditorClass;
class SPropertyEditorStruct;
class SToolTip;
class UActorComponent;
class UFactory;
class FPropertyNode;
struct FAssetData;
struct FSceneOutlinerFilters;

DECLARE_DELEGATE_OneParam(FOnAssetSelected, const FAssetData& /*AssetData*/);
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnShouldSetAsset, const FAssetData& /*AssetData*/);
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnShouldFilterAsset, const FAssetData& /*AssetData*/);
DECLARE_DELEGATE_OneParam(FOnComponentSelected, const UActorComponent* /*ActorComponent*/);
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnShouldFilterComponent, const UActorComponent* /*ActorComponent*/);
DECLARE_DELEGATE_OneParam( FOnGetActorFilters, TSharedPtr<FSceneOutlinerFilters>& );
DECLARE_DELEGATE_ThreeParams(FOnGetPropertyComboBoxStrings, TArray< TSharedPtr<FString> >&, TArray<TSharedPtr<SToolTip>>&, TArray<bool>&);
DECLARE_DELEGATE_RetVal(FString, FOnGetPropertyComboBoxValue);
DECLARE_DELEGATE_OneParam(FOnPropertyComboBoxValueSelected, const FString&);
DECLARE_DELEGATE_ThreeParams(FOnInstancedPropertyIteration, IDetailCategoryBuilder&, IDetailGroup*, TSharedRef<IPropertyHandle>&);
DECLARE_DELEGATE_RetVal(bool, FOnIsEnabled);
DECLARE_DELEGATE_TwoParams(FOnSetOptional, FProperty*, const UClass*);

/** Collects advanced arguments for MakePropertyComboBox */
struct FPropertyComboBoxArgs
{
	/** If set, the combo box will bind to a specific property. If this is null, the following 3 delegates must be set */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** Delegate that is called to generate the list of possible strings inside the combo box list. If not set it will generate using the property handle */
	FOnGetPropertyComboBoxStrings OnGetStrings;

	/** Delegate that is called to get the current string value to display as the combo box label. If not set it will generate using the property handle */
	FOnGetPropertyComboBoxValue OnGetValue;

	/** Delegate called when a string is selected. If not set it will modify what is bound to the property handle */
	FOnPropertyComboBoxValueSelected OnValueSelected;

	/** If number of items in combo box is >= this, it will show a search box to allow filtering. -1 means to never show it */
	int32 ShowSearchForItemCount = 20;

	/** Font to use for text display. If not set it will use the default property editor font */
	FSlateFontInfo Font;

	/** Default constructor, the caller will need to fill in values manually */
	FPropertyComboBoxArgs()
	{}

	/** Constructor using original function arguments */
	FPropertyComboBoxArgs(const TSharedPtr<IPropertyHandle>& InPropertyHandle,
		FOnGetPropertyComboBoxStrings InOnGetStrings = FOnGetPropertyComboBoxStrings(),
		FOnGetPropertyComboBoxValue InOnGetValue = FOnGetPropertyComboBoxValue(),
		FOnPropertyComboBoxValueSelected InOnValueSelected = FOnPropertyComboBoxValueSelected())
		: PropertyHandle(InPropertyHandle)
		, OnGetStrings(InOnGetStrings)
		, OnGetValue(InOnGetValue)
		, OnValueSelected(InOnValueSelected)
	{}
};

struct FPropertyFunctionCallArgs
{
	TWeakObjectPtr<UFunction> Function;

	TOptional<FText> LabelOverride;

	TOptional<FText> ToolTipTextOverride;

	using FOnExecute = TDelegate<FReply(TWeakObjectPtr<UFunction>)>;
	FOnExecute OnExecute;

	using FOnCanExecute = TDelegate<bool(TWeakObjectPtr<UFunction>)>;
	FOnCanExecute OnCanExecute;

	FTextBuilder* SearchText = nullptr;

	FPropertyFunctionCallArgs(
		UFunction* InFunction,
		const FOnExecute& InOnExecute,
		const FOnCanExecute& InOnCanExecute = {},
		const TOptional<FText>& InLabelOverride = {},
		const TOptional<FText>& InToolTipTextOverride = {},
		FTextBuilder* InSearchText = nullptr)
		: Function(InFunction)
		, LabelOverride(InLabelOverride)
		, ToolTipTextOverride(InToolTipTextOverride)
		, OnExecute(InOnExecute)
		, OnCanExecute(InOnCanExecute)
		, SearchText(InSearchText)
	{
	}
};

/** The callbacks, if specified, are used when invoking function calls. */
struct FPropertyFunctionCallDelegates
{
	using FOnGetExecutionContext = TDelegate<TArray<TWeakObjectPtr<UObject>>(TWeakObjectPtr<UFunction>)>;
	FOnGetExecutionContext OnGetExecutionContext;

	using FOnExecute = TDelegate<FReply(TWeakObjectPtr<UFunction>)>;
	FOnExecute OnExecute;

	using FOnCanExecute = TDelegate<bool(TWeakObjectPtr<UFunction>)>;
	FOnCanExecute OnCanExecute;

	FPropertyFunctionCallDelegates(
		const FOnExecute& InOnExecute,
		const FOnCanExecute& InOnCanExecute = {})
		: OnExecute(InOnExecute)
		, OnCanExecute(InOnCanExecute)
	{
	}

	FPropertyFunctionCallDelegates(const FOnGetExecutionContext& InOnGetExecutionContext)
		: OnGetExecutionContext(InOnGetExecutionContext)
	{
	}
};

namespace PropertyCustomizationHelpers
{
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeCustomButton(const FSlateBrush* IconBrush, FSimpleDelegate OnClicked, TAttribute<FText> ToolTipText = FText(), TAttribute<bool> IsEnabled = true, TAttribute<EVisibility> IsVisible = EVisibility::Visible);
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeResetButton(FSimpleDelegate OnResetClicked, TAttribute<FText> OptionalToolTipText = FText(), TAttribute<bool> IsEnabled = true);
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeAddButton( FSimpleDelegate OnAddClicked, TAttribute<FText> OptionalToolTipText = FText(), TAttribute<bool> IsEnabled = true );
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeRemoveButton( FSimpleDelegate OnRemoveClicked, TAttribute<FText> OptionalToolTipText = FText(), TAttribute<bool> IsEnabled = true );
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeEditButton( FSimpleDelegate OnEditClicked, TAttribute<FText> OptionalToolTipText = FText(), TAttribute<bool> IsEnabled = true );
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeEmptyButton( FSimpleDelegate OnEmptyClicked, TAttribute<FText> OptionalToolTipText = FText(), TAttribute<bool> IsEnabled = true );
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeInsertDeleteDuplicateButton( FExecuteAction OnInsertClicked, FExecuteAction OnDeleteClicked, FExecuteAction OnDuplicateClicked );
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeDeleteButton( FSimpleDelegate OnDeleteClicked, TAttribute<FText> OptionalToolTipText = FText(), TAttribute<bool> IsEnabled = true );
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeClearButton( FSimpleDelegate OnClearClicked, TAttribute<FText> OptionalToolTipText = FText(), TAttribute<bool> IsEnabled = true );
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeVisibilityButton(FOnClicked OnVisibilityClicked, TAttribute<FText> OptionalToolTipText = FText(), TAttribute<bool> VisibilityDelegate = true);
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeNewBlueprintButton( FSimpleDelegate OnNewBlueprintClicked, TAttribute<FText> OptionalToolTipText = FText(), TAttribute<bool> IsEnabled = true );
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeUseSelectedButton( FSimpleDelegate OnUseSelectedClicked, TAttribute<FText> OptionalToolTipText = FText(), TAttribute<bool> IsEnabled = true, const bool IsActor = false );
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeBrowseButton( FSimpleDelegate OnFindClicked, TAttribute<FText> OptionalToolTipText = FText(), TAttribute<bool> IsEnabled = true, const bool IsActor = false, const TAttribute<const FSlateBrush*>& InIconOverride = TAttribute<const FSlateBrush*>());
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeAssetPickerAnchorButton( FOnGetAllowedClasses OnGetAllowedClasses, FOnAssetSelected OnAssetSelectedFromPicker, const TSharedPtr<IPropertyHandle>& PropertyHandle = TSharedPtr<IPropertyHandle>());
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeAssetPickerWithMenu( const FAssetData& InitialObject, const bool AllowClear, const TArray<const UClass*>& AllowedClasses, const TArray<UFactory*>& NewAssetFactories, FOnShouldFilterAsset OnShouldFilterAsset, FOnAssetSelected OnSet, FSimpleDelegate OnClose, const TSharedPtr<IPropertyHandle>& PropertyHandle = TSharedPtr<IPropertyHandle>(), const TArray<FAssetData>& OwnerAssetArray = TArray<FAssetData>());
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeAssetPickerWithMenu( const FAssetData& InitialObject, const bool AllowClear, const TArray<const UClass*>& AllowedClasses, const TArray<const UClass*>& DisallowedClasses, const TArray<UFactory*>& NewAssetFactories, FOnShouldFilterAsset OnShouldFilterAsset, FOnAssetSelected OnSet, FSimpleDelegate OnClose, const TSharedPtr<IPropertyHandle>& PropertyHandle = TSharedPtr<IPropertyHandle>(), const TArray<FAssetData>& OwnerAssetArray = TArray<FAssetData>());
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeAssetPickerWithMenu( const FAssetData& InitialObject, const bool AllowClear, const bool AllowCopyPaste, const TArray<const UClass*>& AllowedClasses, const TArray<UFactory*>& NewAssetFactories, FOnShouldFilterAsset OnShouldFilterAsset, FOnAssetSelected OnSet, FSimpleDelegate OnClose, const TSharedPtr<IPropertyHandle>& PropertyHandle = TSharedPtr<IPropertyHandle>(), const TArray<FAssetData>& OwnerAssetArray = TArray<FAssetData>());
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeAssetPickerWithMenu( const FAssetData& InitialObject, const bool AllowClear, const bool AllowCopyPaste, const TArray<const UClass*>& AllowedClasses, const TArray<const UClass*>& DisallowedClasses, const TArray<UFactory*>& NewAssetFactories, FOnShouldFilterAsset OnShouldFilterAsset, FOnAssetSelected OnSet, FSimpleDelegate OnClose, const TSharedPtr<IPropertyHandle>& PropertyHandle = TSharedPtr<IPropertyHandle>(), const TArray<FAssetData>& OwnerAssetArray = TArray<FAssetData>());
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeActorPickerAnchorButton( FOnGetActorFilters OnGetActorFilters, FOnActorSelected OnActorSelectedFromPicker );
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeActorPickerWithMenu( AActor* const InitialActor, const bool AllowClear, FOnShouldFilterActor ActorFilter, FOnActorSelected OnSet, FSimpleDelegate OnClose, FSimpleDelegate OnUseSelected );
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeActorPickerWithMenu(AActor* const InitialActor, const bool AllowClear, const bool AllowPickingLevelInstanceContent, FOnShouldFilterActor ActorFilter, FOnActorSelected OnSet, FSimpleDelegate OnClose, FSimpleDelegate OnUseSelected, bool bDisplayUseSelected=true, bool bShowTransient=false);
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeComponentPickerWithMenu( UActorComponent* const InitialComponent, const bool AllowClear, FOnShouldFilterActor ActorFilter, FOnShouldFilterComponent ComponentFilter, FOnComponentSelected OnSet, FSimpleDelegate OnClose, UWorld* PickerWorld = nullptr, bool bShowTransient = false);
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeInteractiveActorPicker(FOnGetAllowedClasses OnGetAllowedClasses, FOnShouldFilterActor OnShouldFilterActor, FOnActorSelected OnActorSelectedFromPicker);
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeSceneDepthPicker(FOnSceneDepthLocationSelected OnSceneDepthLocationSelected);
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeEditConfigHierarchyButton(FSimpleDelegate OnEditConfigClicked, TAttribute<FText> OptionalToolTipText = FText(), TAttribute<bool> IsEnabled = true);
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeDocumentationButton(const TSharedRef<FPropertyEditor>& InPropertyEditor);
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeSaveButton(FSimpleDelegate OnSaveClicked, TAttribute<FText> OptionalToolTipText = FText(), TAttribute<bool> IsEnabled = true);
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeSetOptionalButton(FOnSetOptional OnSetOptional, TAttribute<FText> OptionalToolTipText = FText(), TAttribute<bool> IsEnabled = true);
	PROPERTYEDITOR_API TSharedRef<SWidget> MakePickOptionalButton(FOnSetOptional OnSetOptional, FSimpleDelegate OnClearOptional, const TSharedRef<FPropertyNode>& PropertyNode, TAttribute<FText> OptionalToolTipText = FText(), TAttribute<bool> IsEnabled = true);
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeClearOptionalButton(FSimpleDelegate OnClearOptional, TAttribute<FText> OptionalToolTipText = FText(), TAttribute<bool> IsEnabled = true);
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeFunctionCallButton(const FPropertyFunctionCallArgs& InArgs);
	PROPERTYEDITOR_API void AddFunctionCallWidgets(IDetailGroup& RootGroup, const TArrayView<UFunction*>& InCallInEditorFunctions, const FPropertyFunctionCallDelegates& InArgs);
	PROPERTYEDITOR_API void AddFunctionCallWidgets(IDetailLayoutBuilder& DetailBuilder, const TArrayView<UFunction*>& InCallInEditorFunctions, const FPropertyFunctionCallDelegates& InArgs);
	PROPERTYEDITOR_API void AddCallInEditorFunctionCallWidgetsForClass(IDetailGroup& RootGroup, const UClass* Class, const FPropertyFunctionCallDelegates& InArgs);
	PROPERTYEDITOR_API void AddCallInEditorFunctionCallWidgetsForClass(IDetailLayoutBuilder& DetailBuilder, const UClass* Class, const FPropertyFunctionCallDelegates& InArgs);

	/** @return the FBoolProperty edit condition property if one exists. */
	PROPERTYEDITOR_API FBoolProperty* GetEditConditionProperty(const FProperty* InProperty, bool& bNegate);

	/** Returns a list of factories which can be used to create new assets, based on the supplied class */
	PROPERTYEDITOR_API TArray<UFactory*> GetNewAssetFactoriesForClasses(const TArray<const UClass*>& Classes);

	/** Returns a list of factories which can be used to create new assets, based on the supplied classes and respecting the disallowed set */
	PROPERTYEDITOR_API TArray<UFactory*> GetNewAssetFactoriesForClasses(const TArray<const UClass*>& Classes, const TArray<const UClass*>& DisallowedClasses);

	/**
	 * Build a combo button that you bind to a Name/String/Enum property or display using general delegates, using an arguments structure
	 *
	 * @param InArgs Options used to create combo box
	 */
	PROPERTYEDITOR_API TSharedRef<SWidget> MakePropertyComboBox(const FPropertyComboBoxArgs& InArgs);

	/** 
	 * Build a combo button that you bind to a Name/String/Enum property or display using general delegates
	 * 
	 * @param InPropertyHandle	If set, will bind to a specific property. If this is null, all 3 delegates must be set
	 * @param OnGetStrings		Delegate that will generate the list of possible strings. If not set will generate using property handle
	 * @param OnGetValue		Delegate that is called to get the current string value to display. If not set will generate using property handle
	 * @param OnValueSelected	Delegate called when a string is selected. If not set will set the property handle
	 */
	PROPERTYEDITOR_API TSharedRef<SWidget> MakePropertyComboBox(const TSharedPtr<IPropertyHandle>& InPropertyHandle, FOnGetPropertyComboBoxStrings OnGetStrings = FOnGetPropertyComboBoxStrings(), FOnGetPropertyComboBoxValue OnGetValue = FOnGetPropertyComboBoxValue(), FOnPropertyComboBoxValueSelected OnValueSelected = FOnPropertyComboBoxValueSelected());

	/**
	 * Loops through all of an instanced object property's child properties and call AddRowDelegate on properties that needs to be added to the UI to let us customize it.
	 *
	 * @param ExistingGroup  A map used internally to determine the existing group categories. Should be empty.
	 * @param BaseCategory   The category that will be used as the root of the instanced object properties.
	 * @param BaseProperty   The instanced class property.
	 * @param AddRowDelegate The delegate that will be called on each child property.
	 */
	PROPERTYEDITOR_API void MakeInstancedPropertyCustomUI(TMap<FName, IDetailGroup*>& ExistingGroup, IDetailCategoryBuilder& BaseCategory, TSharedRef<IPropertyHandle>& BaseProperty, FOnInstancedPropertyIteration AddRowDelegate);
	
	/**
	 * Parse and load the given metadata string into a list of allowed classes.
	 * The metadata string is likely from something like AllowedClasses or DisallowedClasses.
	 */
	PROPERTYEDITOR_API TArray<const UClass*> GetClassesFromMetadataString(const FString& MetadataString);

	/**
	 * Parse and load the given metadata string into a list of allowed structs.
	 * The metadata string is likely from something like AllowedClasses or DisallowedClasses.
	 */
	PROPERTYEDITOR_API TArray<const UScriptStruct*> GetStructsFromMetadataString(const FString& MetadataString);
	
	/**
	 *
	 */
	PROPERTYEDITOR_API void GetCallInEditorFunctionsForClass(const UClass* InClass, TArray<UFunction*>& OutCallInEditorFunctions, EFieldIterationFlags InIterationFlags = EFieldIterationFlags::IncludeSuper);

	/**
	 *	@param InFunctionFilter	A filter to select candidate UFunctions
	 */
	PROPERTYEDITOR_API void GetCallInEditorFunctionsForClass(const UClass* InClass, const TFunctionRef<bool(const UFunction*)>& InFunctionFilter, TArray<UFunction*>& OutCallInEditorFunctions, EFieldIterationFlags InIterationFlags = EFieldIterationFlags::IncludeSuper);

	/*
	*	Makes a class picker widget for the given instanced editinline UObject property handle. Shares code with SPropertyEditorInline, but doesn't create a combo button, just the class picker.
	*/
	PROPERTYEDITOR_API TSharedRef<SWidget> MakeEditInlineObjectClassPicker(TSharedRef<IPropertyHandle> PropertyHandle, FOnClassPicked OnClassPicked, TSharedPtr<IClassViewerFilter> AdditionalClassFilter=nullptr);

	/*
	* Creates a new instance of the given object class inside the given property handle, mimicking what the SPropertyEditorEditInline widget does on class selection
	*/
	PROPERTYEDITOR_API void CreateNewInstanceOfEditInlineObjectClass(TSharedRef<IPropertyHandle> PropertyHandle, UClass* Class, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags);

}


/** Delegate used to get a generic object */
DECLARE_DELEGATE_RetVal( const UObject*, FOnGetObject );

/** Delegate used to set a generic object */
DECLARE_DELEGATE_OneParam( FOnSetObject, const FAssetData& );


/**
 * Simulates an object property field 
 * Can be used when a property should act like a FObjectProperty but it isn't one
 */
class SObjectPropertyEntryBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SObjectPropertyEntryBox )
		: _AllowedClass( UObject::StaticClass() )
		, _AllowClear( true )
		, _DisplayUseSelected( true )
		, _DisplayBrowse( true )
		, _EnableContentPicker(true)
		, _DisplayCompactSize(false)
		, _DisplayThumbnail(true)
	{}
		/** The path to the object */
		SLATE_ATTRIBUTE( FString, ObjectPath )
		/** Optional property handle that can be used instead of the object path */
		SLATE_ARGUMENT( TSharedPtr<IPropertyHandle>, PropertyHandle )
		/** Optional, array of the objects path, in case the property handle is not valid we will use this one to pass additional object to the picker config*/
		SLATE_ARGUMENT(TArray<FAssetData>, OwnerAssetDataArray)
		/** Thumbnail pool */
		SLATE_ARGUMENT( TSharedPtr<FAssetThumbnailPool>, ThumbnailPool )
		/** Class that is allowed in the asset picker */
		SLATE_ARGUMENT( const UClass*, AllowedClass )
		/** Optional list of factories which may be used to create new assets */
		SLATE_ARGUMENT( TOptional<TArray<UFactory*>>, NewAssetFactories )
		/** Called to check if an asset should be set */
		SLATE_EVENT(FOnShouldSetAsset, OnShouldSetAsset)
		/** Called when the object value changes */
		SLATE_EVENT(FOnSetObject, OnObjectChanged)
		/** Called to check if an asset is valid to use */
		SLATE_EVENT(FOnShouldFilterAsset, OnShouldFilterAsset)
		/** Called to check if the asset should be enabled. */
		SLATE_EVENT(FOnIsEnabled, OnIsEnabled)
		/** Whether the asset can be 'None' */
		SLATE_ARGUMENT(bool, AllowClear)
		/** Whether the asset can be created from the asset picker directly */
		SLATE_ARGUMENT(bool, AllowCreate)
		/** Whether to show the 'Use Selected' button */
		SLATE_ARGUMENT(bool, DisplayUseSelected)
		/** Whether to show the 'Browse' button */
		SLATE_ARGUMENT(bool, DisplayBrowse)
		/** Optional delegate called when the 'Browse' button is clicked. Used to override the default editor behavior */
		SLATE_EVENT(FSimpleDelegate, OnBrowseOverride)
		/** Optional override for the tooltip of the 'Browse' button */
		SLATE_ATTRIBUTE(FText, OnBrowseToolTipOverride)
		/** Optional override for the icon of the 'Browse' button */
		SLATE_ATTRIBUTE(const FSlateBrush*, OnBrowseIconOverride)
		/** Whether to enable the content Picker */
		SLATE_ARGUMENT(bool, EnableContentPicker)
		/** Whether or not to display a smaller, compact size for the asset thumbnail */ 
		SLATE_ARGUMENT(bool, DisplayCompactSize)
		/** Whether or not to display the asset thumbnail */ 
		SLATE_ARGUMENT(bool, DisplayThumbnail)
		/** A custom content slot for widgets */ 
		SLATE_NAMED_SLOT(FArguments, CustomContentSlot)
		/** A custom slot added to the row of buttons */
		SLATE_NAMED_SLOT(FArguments, CustomButtonSlot)
		SLATE_ATTRIBUTE(FIntPoint, ThumbnailSizeOverride)
		/** Called to check if an actor is valid to use */
		SLATE_EVENT(FOnShouldFilterActor, OnShouldFilterActor)
		/** When this is true, the drop target will only get recognized when entering while drag & dropping. */
		SLATE_ATTRIBUTE(bool, bOnlyRecognizeOnDragEnter)
	SLATE_END_ARGS()

	PROPERTYEDITOR_API void Construct( const FArguments& InArgs );

	PROPERTYEDITOR_API void GetDesiredWidth(float& OutMinDesiredWidth, float &OutMaxDesiredWidth);

	PROPERTYEDITOR_API void OpenEntryBox();

private:
	/**
	 * Delegate function called when an object is changed
	 */
	void OnSetObject(const FAssetData& InObject);

	/** @return the object path for the object we are viewing */
	FString OnGetObjectPath() const;

	bool IsEnabled() const;

private:
	/** Delegate to call to determine whether the asset should be set */
	FOnShouldSetAsset OnShouldSetAsset;
	/** Delegate to call to determine whether the actor should be allowed */
	FOnShouldFilterActor OnShouldFilterActor;
	/** Delegate to call when the object changes */
	FOnSetObject OnObjectChanged;
	/** Delegate to call to check if this widget should be enabled. */
	FOnIsEnabled OnIsEnabled;
	/** Path to the object */
	TAttribute<FString> ObjectPath;
	/** Handle to a property we modify (if any)*/
	TSharedPtr<IPropertyHandle> PropertyHandle;
	/** The widget used to edit the object 'property' */
	TSharedPtr<SPropertyEditorAsset> PropertyEditorAsset;
};


/** Delegate used to set a class */
DECLARE_DELEGATE_OneParam( FOnSetClass, const UClass* );


/**
 * Simulates a class property field 
 * Can be used when a property should act like a FClassProperty but it isn't one
 */
class SClassPropertyEntryBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SClassPropertyEntryBox)
		: _MetaClass(UObject::StaticClass())
		, _RequiredInterface(nullptr)
		, _AllowAbstract(false)
		, _IsBlueprintBaseOnly(false)
		, _AllowNone(true)
		, _HideViewOptions(false)
		, _ShowDisplayNames(false)
		, _ShowTreeView(false)
	{}
		/** The meta class that the selected class must be a child-of (required) */
		SLATE_ARGUMENT(const UClass*, MetaClass)
		/** An interface that the selected class must implement (optional) */
		SLATE_ARGUMENT(const UClass*, RequiredInterface)
		/** Allowed class that the selected class must be a child-of. (optional) */
		SLATE_ARGUMENT(TArray<const UClass*>, AllowedClasses)
		/** Classes that the selected class cannot be a child-of. (optional) */
		SLATE_ARGUMENT(TArray<const UClass*>, DisallowedClasses)
		/** Whether or not abstract classes are allowed (optional) */
		SLATE_ARGUMENT(bool, AllowAbstract)
		/** Should only base blueprints be displayed? (optional) */
		SLATE_ARGUMENT(bool, IsBlueprintBaseOnly)
		/** Should we be able to select "None" as a class? (optional) */
		SLATE_ARGUMENT(bool, AllowNone)
		/** Show the View Options part of the class picker dialog*/
		SLATE_ARGUMENT(bool, HideViewOptions)
		/** true to show class display names rather than their native names, false otherwise */
		SLATE_ARGUMENT(bool, ShowDisplayNames)
		/** Show the class picker as a tree view rather than a list*/
		SLATE_ARGUMENT(bool, ShowTreeView)
		/** Attribute used to get the currently selected class (required) */
		SLATE_ATTRIBUTE(const UClass*, SelectedClass)
		/** Delegate used to set the currently selected class (required) */
		SLATE_EVENT(FOnSetClass, OnSetClass)
		/** Custom class filter(s) to be applied on the derived classes of the Metaclass (may be empty)*/
		SLATE_ARGUMENT(TArray<TSharedRef<IClassViewerFilter>>, ClassViewerFilters)	
	SLATE_END_ARGS()

	PROPERTYEDITOR_API void Construct(const FArguments& InArgs);

private:
	/** The widget used to edit the class 'property' */
	TSharedPtr<SPropertyEditorClass> PropertyEditorClass;
};


/** Delegate used to set a struct */
DECLARE_DELEGATE_OneParam(FOnSetStruct, const UScriptStruct*);


/**
 * Simulates a struct type property field 
 * Can be used when a property should act like a struct type but it isn't one
 */
class SStructPropertyEntryBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SStructPropertyEntryBox)
		: _MetaStruct(nullptr)
		, _AllowNone(true)
		, _HideViewOptions(false)
		, _ShowDisplayNames(false)
		, _ShowTreeView(false)
	{}
		/** The meta class that the selected struct must be a child-of (optional) */
		SLATE_ARGUMENT(const UScriptStruct*, MetaStruct)
		/** Should we be able to select "None" as a struct? (optional) */
		SLATE_ARGUMENT(bool, AllowNone)
		/** Show the View Options part of the struct picker dialog*/
		SLATE_ARGUMENT(bool, HideViewOptions)
		/** true to show struct display names rather than their native names, false otherwise */
		SLATE_ARGUMENT(bool, ShowDisplayNames)
		/** Show the struct picker as a tree view rather than a list*/
		SLATE_ARGUMENT(bool, ShowTreeView)
		/** Attribute used to get the currently selected struct (required) */
		SLATE_ATTRIBUTE(const UScriptStruct*, SelectedStruct)
		/** Delegate used to set the currently selected struct (required) */
		SLATE_EVENT(FOnSetStruct, OnSetStruct)
	SLATE_END_ARGS()

	PROPERTYEDITOR_API void Construct(const FArguments& InArgs);

private:
	/** The widget used to edit the struct 'property' */
	TSharedPtr<SPropertyEditorStruct> PropertyEditorStruct;
};


/**
 * Represents a widget that can display a FProperty 
 * With the ability to customize the look of the property                 
 */
class SProperty
	: public SCompoundWidget
{
public:
	DECLARE_DELEGATE( FOnPropertyValueChanged );

	SLATE_BEGIN_ARGS( SProperty )
		: _ShouldDisplayName( true )
		, _DisplayResetToDefault( true )
		{}
		/** The display name to use in the default property widget */
		SLATE_ATTRIBUTE( FText, DisplayName )
		/** Whether or not to display the property name */
		SLATE_ARGUMENT( bool, ShouldDisplayName )
		/** The widget to display for this property instead of the default */
		SLATE_DEFAULT_SLOT( FArguments, CustomWidget )
		/** Whether or not to display the default reset to default button.  Note this value has no affect if overriding the widget */
		SLATE_ARGUMENT( bool, DisplayResetToDefault )
	SLATE_END_ARGS()

	virtual ~SProperty()= default;
	PROPERTYEDITOR_API void Construct( const FArguments& InArgs, TSharedPtr<IPropertyHandle> InPropertyHandle );

	/**
	 * Resets the property to default                   
	 */
	PROPERTYEDITOR_API virtual void ResetToDefault();

	/**
	 * @return Whether or not the reset to default option should be visible                    
	 */
	PROPERTYEDITOR_API virtual bool ShouldShowResetToDefault() const;

	/**
	 * @return a label suitable for displaying in a reset to default menu
	 */
	PROPERTYEDITOR_API virtual FText GetResetToDefaultLabel() const;

	/**
	 * Returns whether or not this property is valid.  Sometimes property widgets are created even when their FProperty is not exposed to the user.  In that case the property is invalid
	 * Properties can also become invalid if selection changes in the detail view and this value is stored somewhere.
	 * @return Whether or not the property is valid.                   
	 */
	PROPERTYEDITOR_API virtual bool IsValidProperty() const;

protected:
	/** The handle being accessed by this widget */
	TSharedPtr<IPropertyHandle> PropertyHandle;
};


DECLARE_DELEGATE_ThreeParams( FOnGenerateArrayElementWidget, TSharedRef<IPropertyHandle>, int32, IDetailChildrenBuilder& );

class FDetailArrayBuilder
	: public IDetailCustomNodeBuilder
{
public:

	FDetailArrayBuilder(TSharedRef<IPropertyHandle> InBaseProperty, bool InGenerateHeader = true, bool InDisplayResetToDefault = true, bool InDisplayElementNum = true)
		: ArrayProperty( InBaseProperty->AsArray() )
		, BaseProperty( InBaseProperty )
		, bGenerateHeader( InGenerateHeader)
		, bDisplayResetToDefault(InDisplayResetToDefault)
		, bDisplayElementNum(InDisplayElementNum)
	{
		check( ArrayProperty.IsValid() );
		
		// Delegate for when the number of children in the array changes
		FSimpleDelegate OnNumChildrenChanged = FSimpleDelegate::CreateRaw( this, &FDetailArrayBuilder::OnNumChildrenChanged );
		OnNumElementsChangedHandle = ArrayProperty->SetOnNumElementsChanged( OnNumChildrenChanged );

		BaseProperty->MarkHiddenByCustomization();
	}
	
	~FDetailArrayBuilder()
	{
		ArrayProperty->UnregisterOnNumElementsChanged(OnNumElementsChangedHandle);
	}

	void SetDisplayName( const FText& InDisplayName )
	{
		DisplayName = InDisplayName;
	}

	void OnGenerateArrayElementWidget( FOnGenerateArrayElementWidget InOnGenerateArrayElementWidget )
	{
		OnGenerateArrayElementWidgetDelegate = InOnGenerateArrayElementWidget;
	}

	virtual bool RequiresTick() const override { return false; }

	virtual void Tick( float DeltaTime ) override {}

	virtual FName GetName() const override
	{
		return BaseProperty->GetProperty()->GetFName();
	}
	
	virtual bool InitiallyCollapsed() const override { return false; }

	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override
	{
		if (bGenerateHeader)
		{
			TSharedPtr<SHorizontalBox> ContentHorizontalBox;
			SAssignNew(ContentHorizontalBox, SHorizontalBox);
			if (bDisplayElementNum)
			{
				ContentHorizontalBox->AddSlot()
				[
					BaseProperty->CreatePropertyValueWidget()
				];
			}

			FUIAction CopyAction;
			FUIAction PasteAction;
			BaseProperty->CreateDefaultPropertyCopyPasteActions(CopyAction, PasteAction);

			NodeRow
			.FilterString(!DisplayName.IsEmpty() ? DisplayName : BaseProperty->GetPropertyDisplayName())
			.NameContent()
			[
				BaseProperty->CreatePropertyNameWidget(DisplayName, FText::GetEmpty())
			]
			.ValueContent()
			[
				ContentHorizontalBox.ToSharedRef()
			]
			.CopyAction(CopyAction)
			.PasteAction(PasteAction);

			if (bDisplayResetToDefault)
			{
				TSharedPtr<SResetToDefaultMenu> ResetToDefaultMenu;
				ContentHorizontalBox->AddSlot()
				.AutoWidth()
				.Padding(FMargin(2.0f, 0.0f, 0.0f, 0.0f))
				[
					SAssignNew(ResetToDefaultMenu, SResetToDefaultMenu)
				];
				ResetToDefaultMenu->AddProperty(BaseProperty);
			}
		}
	}

	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override
	{
		uint32 NumChildren = 0;
		ArrayProperty->GetNumElements( NumChildren );

		for( uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex )
		{
			TSharedRef<IPropertyHandle> ElementHandle = ArrayProperty->GetElement( ChildIndex );

			OnGenerateArrayElementWidgetDelegate.Execute( ElementHandle, ChildIndex, ChildrenBuilder );
		}
	}

	virtual void RefreshChildren()
	{
		OnRebuildChildren.ExecuteIfBound();
	}

	virtual TSharedPtr<IPropertyHandle> GetPropertyHandle() const
	{
		return BaseProperty;
	}

protected:

	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRebuildChildren  ) override { OnRebuildChildren = InOnRebuildChildren; } 
	
	void OnNumChildrenChanged()
	{
		OnRebuildChildren.ExecuteIfBound();
	}

	bool GetShouldGenerateHeader() const {return bGenerateHeader;}
	bool GetShouldDisplayResetToDefault() const {return bDisplayResetToDefault;}
	bool GetShouldDisplayElementNum() const {return bDisplayElementNum;}

private:

	FText DisplayName;
	FOnGenerateArrayElementWidget OnGenerateArrayElementWidgetDelegate;
	TSharedPtr<IPropertyHandleArray> ArrayProperty;
	TSharedRef<IPropertyHandle> BaseProperty;
	FSimpleDelegate OnRebuildChildren;
	bool bGenerateHeader;
	bool bDisplayResetToDefault;
	bool bDisplayElementNum;
	FDelegateHandle OnNumElementsChangedHandle;
};

/**
 * Helper class to create a material slot name widget for material lists
 */
class SMaterialSlotWidget : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SMaterialSlotWidget)
		: _DeleteMaterialSlotVisibility(EVisibility::Visible)
	{}
		SLATE_ATTRIBUTE(FText, MaterialName)
		SLATE_ATTRIBUTE(bool, IsMaterialSlotNameReadOnly)
		SLATE_EVENT(FOnTextChanged, OnMaterialNameChanged)
		SLATE_EVENT(FOnTextCommitted, OnMaterialNameCommitted)
		SLATE_ATTRIBUTE(bool, CanDeleteMaterialSlot)
		SLATE_EVENT(FSimpleDelegate, OnDeleteMaterialSlot)
		SLATE_ATTRIBUTE(EVisibility, DeleteMaterialSlotVisibility)
	SLATE_END_ARGS()

	PROPERTYEDITOR_API void Construct(const FArguments& InArgs, int32 SlotIndex, bool bIsMaterialUsed);
};

//////////////////////////////////////////////////////////////////////////
//
// SECTION LIST

/**
* Delegate called when we need to get new sections for the list
*/
DECLARE_DELEGATE_OneParam(FOnGetSections, class ISectionListBuilder&);

/**
* Delegate called when a user changes the Section
*/
DECLARE_DELEGATE_FourParams(FOnSectionChanged, int32, int32, int32, FName);

DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<SWidget>, FOnGenerateWidgetsForSection, int32, int32);

DECLARE_DELEGATE_TwoParams(FOnResetSectionToDefaultClicked, int32, int32);

DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FOnGenerateLODComboBox, int32);

DECLARE_DELEGATE_RetVal(bool, FOnCanCopySectionList);
DECLARE_DELEGATE(FOnCopySectionList);
DECLARE_DELEGATE(FOnPasteSectionList);

DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnCanCopySectionItem, int32, int32);
DECLARE_DELEGATE_TwoParams(FOnCopySectionItem, int32, int32);
DECLARE_DELEGATE_TwoParams(FOnPasteSectionItem, int32, int32);
DECLARE_DELEGATE_ThreeParams(FOnEnableSectionItem, int32, int32, bool);

struct FSectionListDelegates
{
	FSectionListDelegates()
		: OnGetSections()
		, OnSectionChanged()
		, OnGenerateCustomNameWidgets()
		, OnGenerateCustomSectionWidgets()
		, OnResetSectionToDefaultClicked()
	{}

	/** Delegate called to populate the list with Sections */
	FOnGetSections OnGetSections;
	/** Delegate called when a user changes the Section */
	FOnSectionChanged OnSectionChanged;
	/** Delegate called to generate custom widgets under the name of in the left column of a details panel*/
	FOnGenerateWidgetsForSection OnGenerateCustomNameWidgets;
	/** Delegate called to generate custom widgets under each Section */
	FOnGenerateWidgetsForSection OnGenerateCustomSectionWidgets;
	/** Delegate called when a Section list item should be reset to default */
	FOnResetSectionToDefaultClicked OnResetSectionToDefaultClicked;

	/** Delegate called Copying a section list */
	FOnCopySectionList OnCopySectionList;
	/** Delegate called to know if we can copy a section list */
	FOnCanCopySectionList OnCanCopySectionList;
	/** Delegate called Pasting a section list */
	FOnPasteSectionList OnPasteSectionList;

	/** Delegate called Copying a section item */
	FOnCopySectionItem OnCopySectionItem;
	/** Delegate called To know if we can copy a section item */
	FOnCanCopySectionItem OnCanCopySectionItem;
	/** Delegate called Pasting a section item */
	FOnPasteSectionItem OnPasteSectionItem;
	/** Delegate called when enabling/disabling a section item */
	FOnEnableSectionItem OnEnableSectionItem;
};

/**
* Builds up a list of unique Sections while creating some information about the Sections
*/
class ISectionListBuilder
{
public:
	virtual ~ISectionListBuilder() = default;
	/**
	* Adds a new Section to the list
	*
	* @param SlotIndex		The slot (usually mesh element index) where the Section is located on the component
	* @param Section		The Section being used
	* @param bCanBeReplced	Whether or not the Section can be replaced by a user
	*/
	virtual void AddSection(int32 LodIndex, int32 SectionIndex, FName InMaterialSlotName, int32 InMaterialSlotIndex, FName InOriginalMaterialSlotName, const TMap<int32, FName> &InAvailableMaterialSlotName, const UMaterialInterface* Material, bool IsSectionUsingCloth, bool bIsChunkSection, int32 DefaultMaterialIndex) = 0;
};


/**
* A Section item in a Section list slot
*/
struct FSectionListItem
{
	/** LodIndex of the Section*/
	int32 LodIndex;
	/** Section index */
	int32 SectionIndex;

	/* Is this section is using cloth */
	bool IsSectionUsingCloth;

	/* Size of the preview material thumbnail */
	int32 ThumbnailSize;

	/** Material being readonly view in the list */
	TWeakObjectPtr<UMaterialInterface> Material;

	/* Material Slot Name */
	FName MaterialSlotName;
	int32 MaterialSlotIndex;
	FName OriginalMaterialSlotName;

	/* Available material slot name*/
	TMap<int32, FName> AvailableMaterialSlotName;

	bool bIsChunkSection;
	int32 DefaultMaterialIndex;


	FSectionListItem(int32 InLodIndex, int32 InSectionIndex, FName InMaterialSlotName, int32 InMaterialSlotIndex, FName InOriginalMaterialSlotName, const TMap<int32, FName> &InAvailableMaterialSlotName, const UMaterialInterface* InMaterial, bool InIsSectionUsingCloth, int32 InThumbnailSize, bool InIsChunkSection, int32 InDefaultMaterialIndex)
		: LodIndex(InLodIndex)
		, SectionIndex(InSectionIndex)
		, IsSectionUsingCloth(InIsSectionUsingCloth)
		, ThumbnailSize(InThumbnailSize)
		, Material(const_cast<UMaterialInterface*>(InMaterial))
		, MaterialSlotName(InMaterialSlotName)
		, MaterialSlotIndex(InMaterialSlotIndex)
		, OriginalMaterialSlotName(InOriginalMaterialSlotName)
		, AvailableMaterialSlotName(InAvailableMaterialSlotName)
		, bIsChunkSection(InIsChunkSection)
		, DefaultMaterialIndex(InDefaultMaterialIndex)
	{}

	bool operator==(const FSectionListItem& Other) const
	{
		bool IsSectionItemEqual = LodIndex == Other.LodIndex
			&& SectionIndex == Other.SectionIndex
			&& MaterialSlotIndex == Other.MaterialSlotIndex
			&& MaterialSlotName == Other.MaterialSlotName
			&& Material == Other.Material
			&& AvailableMaterialSlotName.Num() == Other.AvailableMaterialSlotName.Num()
			&& IsSectionUsingCloth == Other.IsSectionUsingCloth
			&& bIsChunkSection == Other.bIsChunkSection
			&& DefaultMaterialIndex == Other.DefaultMaterialIndex;
		if (IsSectionItemEqual)
		{
			for (auto Kvp : AvailableMaterialSlotName)
			{
				if (!Other.AvailableMaterialSlotName.Contains(Kvp.Key))
				{
					IsSectionItemEqual = false;
					break;
				}
				FName OtherName = *(Other.AvailableMaterialSlotName.Find(Kvp.Key));
				if (Kvp.Value != OtherName)
				{
					IsSectionItemEqual = false;
					break;
				}
			}
		}
		return IsSectionItemEqual;
	}

	bool operator!=(const FSectionListItem& Other) const
	{
		return !(*this == Other);
	}
};


class FSectionList : public IDetailCustomNodeBuilder, public TSharedFromThis<FSectionList>
{
public:
	PROPERTYEDITOR_API FSectionList(IDetailLayoutBuilder& InDetailLayoutBuilder, FSectionListDelegates& SectionListDelegates, bool bInInitiallyCollapsed, int32 InThumbnailSize, int32 InSectionsLodIndex, FName InSectionListName);

	/**
	* @return true if Sections are being displayed
	*/
	bool IsDisplayingSections() const { return true; }
private:
	/**
	* Called when a user expands all materials in a slot
	*/
	void OnDisplaySectionsForLod(int32 LodIndex);

	/**
	* Called when a user hides all materials in a slot
	*/
	void OnHideSectionsForLod(int32 LodIndex);

	/** IDetailCustomNodeBuilder interface */
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRebuildChildren) override { OnRebuildChildren = InOnRebuildChildren; }
	virtual bool RequiresTick() const override { return true; }
	virtual void Tick(float DeltaTime) override;
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	virtual FName GetName() const override { return SectionListName; }
	virtual bool InitiallyCollapsed() const override
	{
		return bInitiallyCollapsed;
	}

	/**
	* Adds a new Section item to the list
	*
	* @param Row			The row to add the item to
	* @param LodIndex		The Lod of the section
	* @param Item			The Section item to add
	* @param bDisplayLink	If a link to the Section should be displayed instead of the actual item (for multiple Sections)
	*/
	void AddSectionItem(class FDetailWidgetRow& Row, int32 LodIndex, const struct FSectionListItem& Item, bool bDisplayLink);

private:
	bool OnCanCopySectionList() const;
	void OnCopySectionList();
	void OnPasteSectionList();

	bool OnCanCopySectionItem(int32 LODIndex, int32 SectionIndex) const;
	void OnCopySectionItem(int32 LODIndex, int32 SectionIndex);
	void OnPasteSectionItem(int32 LODIndex, int32 SectionIndex);
	void OnEnableSectionItem(int32 LodIndex, int32 SectionIndex, bool bEnable);

	/** Delegates for the Section list */
	FSectionListDelegates SectionListDelegates;
	/** Called to rebuild the children of the detail tree */
	FSimpleDelegate OnRebuildChildren;
	/** Parent detail layout this list is in */
	IDetailLayoutBuilder& DetailLayoutBuilder;
	/** Set of all unique displayed Sections */
	TArray< FSectionListItem > DisplayedSections;
	/** Set of all Sections currently in view (may be less than DisplayedSections) */
	TArray< TSharedRef<class FSectionItemView> > ViewedSections;
	/** Set of all expanded slots */
	TSet<uint32> ExpandedSlots;
	/** Section list builder used to generate Sections */
	TSharedRef<class FSectionListBuilder> SectionListBuilder;

	/** Set the initial state of the collapse. */
	bool bInitiallyCollapsed;

	FName SectionListName;

	int32 ThumbnailSize;
	int32 SectionsLodIndex;
};
