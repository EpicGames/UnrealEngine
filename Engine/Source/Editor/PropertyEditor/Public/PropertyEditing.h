// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()

#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "IDetailCustomNodeBuilder.h"
#include "IDetailPropertyRow.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IPropertyTypeCustomization.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailCustomization.h"
#include "IDetailsView.h"
#include "IDetailRootObjectCustomization.h"
