// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"

namespace PropertyEditorPolicy
{
	class IArchetypePolicy
	{
	public:
		virtual UObject* GetArchetypeForObject(const UObject* Object) const = 0;
	};

	PROPERTYEDITOR_API UObject* GetArchetype(const UObject* Object);
	PROPERTYEDITOR_API void RegisterArchetypePolicy(IArchetypePolicy* Policy);
	PROPERTYEDITOR_API void UnregisterArchetypePolicy(IArchetypePolicy* Policy);
}