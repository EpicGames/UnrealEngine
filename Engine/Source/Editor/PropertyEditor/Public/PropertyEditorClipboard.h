// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Templates/Function.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/NameTypes.h"

#define UE_API PROPERTYEDITOR_API

/**
 * Specializes clipboard handling to allow tagged entries,
 * where you can partition the clipboard contents and look them up by tag (name).
 * The clipboard is persisted using both the Platform implementation, and as a stored virtual clipboard.
 */
class FPropertyEditorClipboard
{
public:
	/** Perform a copy. */
	static UE_API void ClipboardCopy(const TCHAR* Str);

	/** Perform a copy, optionally specifying the tag. */
	static UE_API void ClipboardCopy(const TCHAR* Str, FName Tag);

	/** Perform a copy, supplying a function that maps the result to multiple tags. */
	static UE_API void ClipboardCopy(TUniqueFunction<void(TMap<FName, FString>&)>&& TagMappingFunc);

	/** Append the given values to the current clipboard entry. If none exists, it will create it. */
	static UE_API void ClipboardAppend(TUniqueFunction<void(TMap<FName, FString>&)>&& TagMappingFunc);

	/** Gets the clipboard contents. */
	static UE_API bool ClipboardPaste(FString& Dest);
	
	/**
	 * Will return true if the tag is specified and exists. 
	 * If the tag doesn't exist, Dest will be populated with the untagged clipboard contents.  
	 */
	static UE_API bool ClipboardPaste(FString& Dest, FName Tag);

private:
	static UE_API TSharedRef<FPropertyEditorClipboard> Get();

	/** Item per-tag, cleared when a new copy operation occurs. */
	TMap<FName, FString> ClipboardContents;
};

#undef UE_API
