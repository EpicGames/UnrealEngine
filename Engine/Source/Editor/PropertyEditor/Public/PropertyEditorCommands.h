// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Templates/SharedPointer.h"

class FUICommandInfo;

class FPropertyEditorCommands : public TCommands<FPropertyEditorCommands>
{
public:
	FPropertyEditorCommands();

	//~ Begin TCommands
	virtual void RegisterCommands() override;
	//~ End TCommands

	TSharedPtr<FUICommandInfo> CollapseAll;
	TSharedPtr<FUICommandInfo> ExpandAll;
};