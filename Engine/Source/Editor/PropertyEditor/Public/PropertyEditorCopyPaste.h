// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "PropertyHandle.h"

/** Utilities related to copy/paste. */
namespace UE::PropertyEditor
{
	/** Gets the property path from the provided PropertyHandle (if valid). */
	[[nodiscard]] PROPERTYEDITOR_API FString GetPropertyPath(const TSharedPtr<IPropertyHandle>& InPropertyHandle);

	/** Returns true if the Tag is empty, or the specified property matches it. */
	[[nodiscard]] PROPERTYEDITOR_API bool TagMatchesProperty(const FString& InTag, const TSharedPtr<IPropertyHandle>& InPropertyHandle);
}

/** Subscribe to and parse the string contents. Note that the caller is responsible for providing valid UE text,
 * and the subscriber is responsible for applying it if applicable.  */
typedef TMulticastDelegate<void(const FString& InTag, const FString& InText, const TOptional<FGuid>& InOperationId)> FOnPasteFromText;

/** Subscribe to and copy the property value to text. Optionally required when there is custom behavior needed on copy to match FOnPasteFromText,
 * otherwise the default property handle copy will be used. Note that the caller is responsible for providing a valid property handle, and the
 * subscriber is responsible for copying it to text if applicable. OutPropertyText should be left empty if the subscriber does not handle
 * the given property.
 */
typedef TMulticastDelegate<void(TSharedPtr<IPropertyHandle> InPropertyHandle, FString& OutPropertyText, const TOptional<FGuid>& InOperationId)> FOnCopyToText;

