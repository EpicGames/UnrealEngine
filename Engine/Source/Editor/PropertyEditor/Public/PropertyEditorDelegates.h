// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Framework/Commands/UIAction.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Templates/ValueOrError.h"
#include "Textures/SlateIcon.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FName;
class FProperty;
class FPropertyNode;
class FPropertyPath;
class IDetailCustomization;
class IDetailTreeNode;
class IPropertyHandle;
class IPropertyTreeRow;
class IPropertyTypeCustomization;
class IPropertyTypeIdentifier;
class SHeaderRow;
class SWidget;
class UObject;
struct FPropertyChangedEvent;

struct FPropertyAndParent
{
	explicit FPropertyAndParent(const TSharedRef<IPropertyHandle>& InPropertyHandle);
	explicit FPropertyAndParent(const TSharedRef<FPropertyNode>& InPropertyNode);

	/** The property always exists */
	const FProperty& Property;

	/** The array index of this property if applicable, otherwise INDEX_NONE. */
	int32 ArrayIndex;

	/** The entire chain of parent properties, all the way to the property root. ParentProperties[0] is the immediate parent.*/
	TArray< const FProperty* > ParentProperties;

	/**
	 * Array indices in parent properties where applicable, otherwise INDEX_NONE.
	 * The size of this array is always equal to the size of ParentProperties.
	 */
	TArray<int32> ParentArrayIndices;

	/** The objects for these properties */
	TArray< TWeakObjectPtr< UObject > > Objects;

private:
	void Initialize(const TSharedRef<FPropertyNode>& InPropertyNode);
};

/** Delegate called to see if a property should be visible */
DECLARE_DELEGATE_RetVal_OneParam(bool, FIsPropertyVisible, const FPropertyAndParent&);

/** Delegate called to see if a property should be read-only */
DECLARE_DELEGATE_RetVal_OneParam(bool, FIsPropertyReadOnly, const FPropertyAndParent&);

/** Delegate called to determine if a custom row should be visible. */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FIsCustomRowVisible, FName /*InRowName*/, FName /*InParentName*/);

/** Delegate called to determine if a custom row should be read-only. */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FIsCustomRowReadOnly, FName /*InRowName*/, FName /*InParentName*/);

/** Delegate called to get a detail layout for a specific object class */
DECLARE_DELEGATE_RetVal(TSharedRef<IDetailCustomization>, FOnGetDetailCustomizationInstance);

/** Delegate called to get a property layout for a specific property type */
DECLARE_DELEGATE_RetVal(TSharedRef<IPropertyTypeCustomization>, FOnGetPropertyTypeCustomizationInstance);

/** Notification for when a property view changes */
DECLARE_DELEGATE_TwoParams(FOnObjectArrayChanged, const FString&, const TArray<UObject*>&);

/** Notification for when displayed properties changes (for instance, because the user has filtered some properties */
DECLARE_DELEGATE(FOnDisplayedPropertiesChanged);

/** Notification for when a property selection changes. */
DECLARE_DELEGATE_OneParam(FOnPropertySelectionChanged, FProperty*)

/** Notification for when a property is double clicked by the user*/
DECLARE_DELEGATE_OneParam(FOnPropertyDoubleClicked, FProperty*)

/** Notification for when a property is clicked by the user*/
DECLARE_DELEGATE_OneParam(FOnPropertyClicked, const TSharedPtr<FPropertyPath >&)

/** */
DECLARE_DELEGATE_OneParam(FConstructExternalColumnHeaders, const TSharedRef< SHeaderRow >&)

DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef< SWidget >, FConstructExternalColumnCell, const FName& /*ColumnName*/, const TSharedRef< IPropertyTreeRow >& /*RowWidget*/)

/** Delegate called to see if a property editing is enabled */
DECLARE_DELEGATE_RetVal(bool, FIsPropertyEditingEnabled);

/** Delegate to be called before a container item changes */
using FCanPropertyItemChangeRetType = TValueOrError<bool, FText>;
DECLARE_DELEGATE_RetVal_TwoParams(FCanPropertyItemChangeRetType, FCanPropertyItemChange, EPropertyChangeType::Type, int32 /* Index */);

/** Delegate to be called when a change is blocked on a property */
DECLARE_MULTICAST_DELEGATE_TwoParams(FPropertyChangeBlocked, const FProperty*, FText /* ErrorMessage */);

/**
 * A delegate that returns the name of a type that is to be used to look up the layout callback instead of the
 * property's type. Return None to not provide an override.
 */
DECLARE_DELEGATE_RetVal_OneParam(FName, FPropertyHandleLayoutOverride, const IPropertyHandle& Handle)

/**
 * A delegate which is called after properties have been edited and PostEditChange has been called on all objects.
 * This can be used to safely make changes to data that the details panel is observing instead of during PostEditChange (which is
 * unsafe)
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnFinishedChangingProperties, const FPropertyChangedEvent&);

/**
 * A delegate which is called before properties have been edited and PreEditChange has been called on all objects.
 * This can be used to identify what precisely changed.
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnNotifyStartedChangingProperties, const FPropertyChangedEvent&);

/**
 * A property row extension arguments is displayed at the end of a property row, either inline or as a button.
 */
struct FOnGenerateGlobalRowExtensionArgs
{
	/** The detail row's property handle. */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** The detail row's owner tree node. */
	TWeakPtr<IDetailTreeNode> OwnerTreeNode;

	/** Owner object for the property extension */
	UObject* OwnerObject = nullptr;

	/** Path of the exposed property */
	FString PropertyPath = TEXT("");

	/** Exposed property */
	FProperty* Property = nullptr;
};

/**
 * A property row extension button is displayed at the end of a property row, either inline as a button,
 * or in a dropdown when not all buttons can fit.
 */
struct FPropertyRowExtensionButton
{
	/** The icon to display for the button. */
	TAttribute<FSlateIcon> Icon;
	/** The label to display for the button when shown in the dropdown. */
	TAttribute<FText> Label;
	/** The tooltip to display for the button. */
	TAttribute<FText> ToolTip;
	/** The UIAction to use for the button - this includes on execute, can execute and visibility handlers. */
	FUIAction UIAction;
};

/**
 * Delegate called to get add an extension to a property row's name column.
 * To use, bind an handler to the delegate that adds an extension to the out array parameter.
 * When called, EWidgetPosition indicates the position for which the delegate is gathering extensions.
 * ie. The favorite system is implemented by adding the star widget when the delegate is called with the left position.
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGenerateGlobalRowExtension, const FOnGenerateGlobalRowExtensionArgs& /*InArgs*/, TArray<FPropertyRowExtensionButton>& /*OutExtensions*/);

/**
 * Callback executed to query the custom layout of details
 */
struct FDetailLayoutCallback
{
	/** Delegate to call to query custom layout of details */
	FOnGetDetailCustomizationInstance DetailLayoutDelegate;
	/** The order of this class in the map of callbacks to send (callbacks sent in the order they are received) */
	int32 Order;
};

struct FPropertyTypeLayoutCallback
{
	FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate;

	TSharedPtr<IPropertyTypeIdentifier> PropertyTypeIdentifier;

	bool IsValid() const { return PropertyTypeLayoutDelegate.IsBound(); }

	TSharedRef<IPropertyTypeCustomization> GetCustomizationInstance() const;
};


struct FPropertyTypeLayoutCallbackList
{
	/** The base callback is a registered callback with a null identifier */
	FPropertyTypeLayoutCallback BaseCallback;

	/** List of registered callbacks with a non null identifier */
	TArray< FPropertyTypeLayoutCallback > IdentifierList;

	void Add(const FPropertyTypeLayoutCallback& NewCallback);

	void Remove(const TSharedPtr<IPropertyTypeIdentifier>& InIdentifier);

	const FPropertyTypeLayoutCallback& Find(const IPropertyHandle& PropertyHandle) const;
};

/** This is a multimap as there many be more than one customization per property type */
typedef TMap< FName, FPropertyTypeLayoutCallbackList > FCustomPropertyTypeLayoutMap;