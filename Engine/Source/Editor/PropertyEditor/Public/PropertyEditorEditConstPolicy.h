// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"

class FProperty;
class FEditPropertyChain;

namespace PropertyEditorPolicy
{
	class IEditConstPolicy
	{
	public:
		virtual bool CanEditProperty(const FEditPropertyChain& PropertyChain, const UObject* Object) const = 0;
		virtual bool CanEditProperty(const FProperty* Property, const UObject* Object) const = 0;
	};

	PROPERTYEDITOR_API bool IsPropertyEditConst(const FEditPropertyChain& PropertyChain, UObject* Object);
	PROPERTYEDITOR_API bool IsPropertyEditConst(const FProperty* Property, UObject* Object);
	PROPERTYEDITOR_API void RegisterEditConstPolicy(IEditConstPolicy* Policy);
	PROPERTYEDITOR_API void UnregisterEditConstPolicy(IEditConstPolicy* Policy);
}