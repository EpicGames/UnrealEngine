// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IDetailsView.h"
#include "IPropertyTypeCustomization.h"
#include "HAL/IConsoleManager.h" // TAutoConsoleVariable
#include "PropertyEditorDelegates.h"

#include "Modules/ModuleInterface.h"
#include "UObject/StructOnScope.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"

#include "Misc/Optional.h"

class FAssetEditorToolkit;
class FNotifyHook;
class IPropertyHandle;
class IPropertyTable;
class IPropertyTableCellPresenter;
class ISinglePropertyView;
class SDetailsView;
class SPropertyTreeViewImpl;
class SSingleProperty;
class UToolMenu;

namespace UE::PropertyEditor
{
	inline const FName RowContextMenuName = TEXT("PropertyEditor.RowContextMenu");
}

/**
 * The location of a property name relative to its editor widget                   
 */
namespace EPropertyNamePlacement
{
	enum Type
	{
		/** Do not show the property name */
		Hidden,
		/** Show the property name to the left of the widget */
		Left,
		/** Show the property name to the right of the widget */
		Right,
		/** Inside the property editor edit box (unused for properties that dont have edit boxes ) */
		Inside,
	};
}


/**
 * Potential results from accessing the values of properties                   
 */
namespace FPropertyAccess
{
	enum Result
	{
		/** Multiple values were found so the value could not be read */
		MultipleValues,
		/** Failed to set or get the value */
		Fail,
		/** Successfully set the got the value */
		Success,
	};
}



class IPropertyHandle;
class IPropertyTableCell;
class SPropertyTreeViewImpl;
class SWindow;
class IPropertyTableCellPresenter;
class IPropertyTypeCustomization;
class IDetailsView;
class IToolkitHost;
class FAssetEditorToolkit;

/**
 * Base class for adding an extra data to identify a custom property type
 */
class IPropertyTypeIdentifier
{
public:
	virtual ~IPropertyTypeIdentifier() = default;

	/**
	 * Called to identify if a property type is customized
	 *
	 * @param IPropertyHandle	Handle to the property being tested
	 */
	virtual bool IsPropertyTypeCustomized( const IPropertyHandle& PropertyHandle ) const = 0;
};

typedef TMap< TWeakObjectPtr<UStruct>, FDetailLayoutCallback > FCustomDetailLayoutMap;
typedef TMap< FName, FDetailLayoutCallback > FCustomDetailLayoutNameMap;



/** Struct used to control the visibility of properties in a Structure Detail View */
struct FStructureDetailsViewArgs
{
	FStructureDetailsViewArgs()
		: bShowObjects(false)
		, bShowAssets(true)
		, bShowClasses(true)
		, bShowInterfaces(false)
	{
	}

	/** True if we should show general object properties in the details view */
	bool bShowObjects : 1;

	/** True if we should show asset properties in the details view */
	bool bShowAssets : 1;

	/** True if we should show class properties in the details view */
	bool bShowClasses : 1;

	/** True if we should show interface properties in the details view */
	bool bShowInterfaces : 1;
};

/** 
 * A property section is a group of categories with a name, eg. "Rendering" might contain "Materials" and "Lighting".
 * Categories may belong to zero or more sections. 
 */
class FPropertySection
{
public:

	/**
	 * @param InName		The internal name of this section.
	 * @param InDisplayName	The localizable display name to show to the user.
	 * @param InOrder		The order in which this section should be displayed. Lower values are displayed first.
	 */
	FPropertySection(FName InName, FText InDisplayName, const int32 InOrder = 0) : 
		Name(InName),
		DisplayName(InDisplayName),
		Order(InOrder)
	{
	}

	FPropertySection(const FPropertySection&) = default;
	virtual ~FPropertySection() = default;

	/** Add a category to this section. */
	virtual void AddCategory(FName CategoryName);

	/** Remove a category from this section. */
	virtual void RemoveCategory(FName CategoryName);

	/** Does this section add the given category? */
	virtual bool HasAddedCategory(FName CategoryName) const;

	/** Does this section remove the given category? */
	virtual bool HasRemovedCategory(FName CategoryName) const;

	/** Get the internal name of this property section. */
	virtual FName GetName() const { return Name; }

	/** Get the display name of this section. */
	virtual FText GetDisplayName() const { return DisplayName; }

	/** Get the (sort) order of this section, where the default is 0. */
	virtual int32 GetOrder() const { return Order; }
	
private:

	/** The internal name to use for this section. */
	FName Name;

	/** The display name to use for this section. */
	FText DisplayName;

	/** The order in which this section should be displayed. Lower values are displayed first. */
	int32 Order = 0;
	
	/** The set of categories that are added to this section. */
	TSet<FName> AddedCategories;

	/** 
	 * The set of categories that are removed from this section. 
	 * This exists to allow users to prevent sections from being crowded when inheriting.
	 */
	TSet<FName> RemovedCategories;
};

/** A mapping of categories to section names for a given class. */
class FClassSectionMapping
{
public:
	FClassSectionMapping(FName ClassName);
	FClassSectionMapping(const FClassSectionMapping&) = default;

	/**
	 * Find or add a section of the given name.
	 */
	TSharedPtr<FPropertySection> FindSection(FName SectionName) const;

	/**
	 * Find or add a section of the given name.
	 * Optionally, you can specify an order for the section to be displayed in. Lower values are displayed first.
	 * Note that the default "All" section will always be displayed last.
	 */
	TSharedRef<FPropertySection> FindOrAddSection(FName SectionName, FText DisplayName, const int32 Order = 0);

	/** 
	 * Remove a section of the given name.
	 */
	void RemoveSection(FName SectionName);

	/** 
	 * Get the sections that the given category belongs to and append them to OutSections. 
	 * @param CategoryName	The category name to search for.
	 * @param OutSections	The array to append any found sections. The array will not be cleared.
	 * @return				true if any sections were found, false otherwise.
	 */
	bool GetSectionsForCategory(FName CategoryName, TArray<TSharedPtr<FPropertySection>>& OutSections) const;

private:

	friend class FPropertyEditorModule;

	FName ClassName;

	/** The sections defined for this class. */
	TMap<FName, TSharedPtr<FPropertySection>> DefinedSections;
};


struct FRegisterCustomClassLayoutParams
{
	/* Optional order to register this class layout with. Registration order is used when not specified. Lower values are added first */
	TOptional<int32> OptionalOrder;
};

class FPropertyEditorModule : public IModuleInterface
{
public:
	/**
	 * Called right after the module has been loaded                   
	 */
	virtual void StartupModule();

	/**
	 * Called by the module manager right before this module is unloaded
	 */
	virtual void ShutdownModule();

	/**
	 * Refreshes property windows with a new list of objects to view
	 * 
	 * @param NewObjectList	The list of objects each property window should view
	 */
	virtual void UpdatePropertyViews( const TArray<UObject*>& NewObjectList );

	/**
	 * Replaces objects being viewed by open property views with new objects
	 *
	 * @param OldToNewObjectMap	A mapping between object to replace and its replacement
	 */
	virtual void ReplaceViewedObjects( const TMap<UObject*, UObject*>& OldToNewObjectMap );

	/**
	 * Removes deleted objects from property views that are observing them
	 *
	 * @param DeletedObjects	The objects to delete
	 */
	virtual void RemoveDeletedObjects( TArray<UObject*>& DeletedObjects );

	/**
	 * Returns true if there is an unlocked detail view
	 */
	virtual bool HasUnlockedDetailViews() const;

	/**
	 * Registers a custom detail layout delegate for a specific class
	 *
	 * @param ClassName	The name of the class that the custom detail layout is for
	 * @param DetailLayoutDelegate	The delegate to call when querying for custom detail layouts for the classes properties
	 */
	virtual void RegisterCustomClassLayout( FName ClassName, FOnGetDetailCustomizationInstance DetailLayoutDelegate, FRegisterCustomClassLayoutParams Params = FRegisterCustomClassLayoutParams());

	/**
	 * Unregisters a custom detail layout delegate for a specific class name
	 *
	 * @param ClassName	The class name with the custom detail layout delegate to remove
	 */
	virtual void UnregisterCustomClassLayout( FName ClassName );

	/**
	 * Registers a property type customization
	 * A property type is a specific FProperty type, a struct, instanced Object property, or enum type
	 *
	 * @param PropertyTypeName		The name of the property type to customize.  For structs and enums this is the name of the struct class or enum	(not StructProperty or ByteProperty) 
	 * @param PropertyTypeLayoutDelegate	The delegate to call when querying for a custom layout of the property type
	 * @param Identifier			An identifier to use to differentiate between two customizations on the same type
	 */
	virtual void RegisterCustomPropertyTypeLayout( FName PropertyTypeName, FOnGetPropertyTypeCustomizationInstance PropertyTypeLayoutDelegate, TSharedPtr<IPropertyTypeIdentifier> Identifier = nullptr);

	/**
	 * Unregisters a custom detail layout for a property type
	 *
	 * @param PropertyTypeName 	The name of the property type that was registered
	 * @param Identifier 		An identifier to use to differentiate between two customizations on the same type
	 */
	virtual void UnregisterCustomPropertyTypeLayout( FName PropertyTypeName, TSharedPtr<IPropertyTypeIdentifier> InIdentifier = nullptr);

	/**
	 * Registers a property layout override callback
	 *
	 * @param PropertyTypeName Name of the property type (e.g. struct FName) to override
	 * @param Delegate The delegate to call when we're looking for a property handle's layout override
	 */
	virtual FDelegateHandle RegisterPropertyHandleLayoutOverride(FName PropertyTypeName, const FPropertyHandleLayoutOverride& Delegate);

	/**
	 * Unregisters a property layout override callback
	 *
	 * @param DelegateHandle The handle returned by RegisterPropertyHandleLayoutOverride
	 */
	virtual void UnregisterPropertyHandleLayoutOverride(FDelegateHandle DelegateHandle);

	/**
	 * Find an existing section or create a section for a class.
	 * 
	 * @param ClassName		The class to add a section mapping for.
	 * @param SectionName	The section to find or create.
	 * @param DisplayName	The display name to use for the section. If the section already exists for this class, the display name will not be replaced.
	 * @param Order			The order in which this section should be displayed. Lower values are displayed first. The default "All" section will always be displayed last.
	 * @return				A new section, or the existing one. 
	 */
	virtual TSharedRef<FPropertySection> FindOrCreateSection(FName ClassName, FName SectionName, FText DisplayName, const int32 Order = 0);

	/** 
	 * Find the section that the given category in the given struct should be a part of. 
	 * @param Struct		The struct to start searching from. Note: all super-structs of the given struct will also be searched.
	 * @param CategoryName	The category to search for.
	 */
	virtual TArray<TSharedPtr<FPropertySection>> FindSectionsForCategory(const UStruct* Struct, FName CategoryName) const;

	/** 
	 * Get all registered sections for the given struct (including the default section). 
	 * @param Struct		The struct to fetch sections for.
	 * @param OutSections	Sections will be appended to this parameter. The array will not be cleared beforehand.
	 */
	virtual void GetAllSections(const UStruct* Struct, TArray<TSharedPtr<FPropertySection>>& OutSections) const;

	/**
	 * Remove a given section from the given class.
	 * @param ClassName		The class to remove the section from.
	 * @param SectionName	The section to remove.
	 */
	virtual void RemoveSection(FName ClassName, FName SectionName);

	/**
	 * Customization modules should call this when that module has been unloaded, loaded, etc...
	 * so the property module can clean up its data.  Needed to support dynamic reloading of modules
	 */
	virtual void NotifyCustomizationModuleChanged();

	UE_EXPERIMENTAL(5.8, "This function may be removed and replaced with some other signaling mechanism.")
	virtual void RequestRefreshOfAllDetailsPanels();

	/**
	 * Creates a new detail view
	 *
 	 * @param DetailsViewArgs		The struct containing all the user definable details view arguments
	 * @return The new detail view
	 */
	virtual TSharedRef<class IDetailsView> CreateDetailView( const struct FDetailsViewArgs& DetailsViewArgs );

	/**
	 * Find an existing detail view
	 *
 	 * @param ViewIdentifier	The name of the details view to find
	 * @return The existing detail view, or null if it wasn't found
	 */
	virtual TSharedPtr<class IDetailsView> FindDetailView( const FName ViewIdentifier ) const;

	/**
	 *  Convenience method for creating a new floating details window (a details view with its own top level window)
	 *
	 * @param InObjects			The objects to create the detail view for.
	 * @param bIsLockable		True if the property view can be locked.
	 * @return The new details view window.
	 */
	virtual TSharedRef<SWindow> CreateFloatingDetailsView( const TArray< UObject* >& InObjects, bool bIsLockable );

	/**
	 * Creates a standalone widget for a single object property
	 *
	 * @param InObject			The object to view
	 * @param InPropertyName	The name of the property to display
	 * @param InitParams		Optional init params for a single property
	 * @return The new property if valid or null
	 */
	virtual TSharedPtr<class ISinglePropertyView> CreateSingleProperty( UObject* InObject, FName InPropertyName, const struct FSinglePropertyParams& InitParams );

	/**
	 * Creates a standalone widget for a single struct property
	 *
	 * @param InStruct			The struct containing the property to view
	 * @param InPropertyName	The name of the property to display
	 * @param InitParams		Optional init params for a single property
	 * @return The new property if valid or null
	 */
	virtual TSharedPtr<class ISinglePropertyView> CreateSingleProperty(const TSharedPtr<class IStructureDataProvider>& InStruct, FName InPropertyName, const struct FSinglePropertyParams& InitParams);

	virtual TSharedRef<class IStructureDetailsView> CreateStructureDetailView(const struct FDetailsViewArgs& DetailsViewArgs, const FStructureDetailsViewArgs& StructureDetailsViewArgs, TSharedPtr<class FStructOnScope> StructData, const FText& CustomName = FText::GetEmpty());
	virtual TSharedRef<class IStructureDetailsView> CreateStructureProviderDetailView(const FDetailsViewArgs& DetailsViewArgs, const FStructureDetailsViewArgs& StructureDetailsViewArgs, TSharedPtr<IStructureDataProvider> StructProvider, const FText& CustomName = FText::GetEmpty());

	virtual TSharedRef<class IPropertyRowGenerator> CreatePropertyRowGenerator(const struct FPropertyRowGeneratorArgs& InArgs);

	/**
	 * Creates a property change listener that notifies users via a  delegate when a property on an object changes
	 *
	 * @return The new property change listener
	 */
	virtual TSharedRef<class IPropertyChangeListener> CreatePropertyChangeListener();

	virtual TSharedRef< class IPropertyTable > CreatePropertyTable();

	virtual TSharedRef< SWidget > CreatePropertyTableWidget( const TSharedRef< class IPropertyTable >& PropertyTable );

	virtual TSharedRef< SWidget > CreatePropertyTableWidget( const TSharedRef< class IPropertyTable >& PropertyTable, const TArray< TSharedRef< class IPropertyTableCustomColumn > >& Customizations );
	virtual TSharedRef< class IPropertyTableWidgetHandle > CreatePropertyTableWidgetHandle( const TSharedRef< IPropertyTable >& PropertyTable );
	virtual TSharedRef< class IPropertyTableWidgetHandle > CreatePropertyTableWidgetHandle( const TSharedRef< IPropertyTable >& PropertyTable, const TArray< TSharedRef< class IPropertyTableCustomColumn > >& Customizations );

	virtual TSharedRef< IPropertyTableCellPresenter > CreateTextPropertyCellPresenter( const TSharedRef< class FPropertyNode >& InPropertyNode, const TSharedRef< class IPropertyTableUtilities >& InPropertyUtilities, 
		const FSlateFontInfo* InFontPtr = NULL, const TSharedPtr< IPropertyTableCell >& InCell = nullptr);

	/**
	 * Register a floating struct so that the details panel may use it as a property
	 *
	 * @param StructClass		The struct to register
	 * @return The struct property that may may be associated with the details panel
 	 */
	virtual FStructProperty* RegisterStructProperty(const UStruct* StructClass);

	/**
	 *
	 */
	virtual TSharedRef< FAssetEditorToolkit > CreatePropertyEditorToolkit(const TSharedPtr< class IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit );
	virtual TSharedRef< FAssetEditorToolkit > CreatePropertyEditorToolkit(const TSharedPtr< IToolkitHost >& InitToolkitHost, const TArray< UObject* >& ObjectsToEdit );
	virtual TSharedRef< FAssetEditorToolkit > CreatePropertyEditorToolkit(const TSharedPtr< IToolkitHost >& InitToolkitHost, const TArray< TWeakObjectPtr< UObject > >& ObjectsToEdit );

	FPropertyTypeLayoutCallback GetPropertyTypeCustomization(const FProperty* InProperty,const IPropertyHandle& PropertyHandle, const FCustomPropertyTypeLayoutMap& InstancedPropertyTypeLayoutMap);
	FPropertyTypeLayoutCallback FindPropertyTypeLayoutCallback(FName PropertyTypeName, const IPropertyHandle& PropertyHandle, const FCustomPropertyTypeLayoutMap& InstancedPropertyTypeLayoutMapp);
	PROPERTYEDITOR_API bool IsCustomizedStruct(const UStruct* Struct, const FCustomPropertyTypeLayoutMap& InstancePropertyTypeLayoutMap) const;

	DECLARE_EVENT(PropertyEditorModule, FPropertyEditorOpenedEvent);
	virtual FPropertyEditorOpenedEvent& OnPropertyEditorOpened() { return PropertyEditorOpened; }

	DECLARE_EVENT_OneParam(PropertyEditorModule, FOnPropertyMatrixAssetsSaved, const TArray<UObject*>&);
	virtual FOnPropertyMatrixAssetsSaved& OnPropertyMatrixAssetsSaved() { return PropertyMatrixAssetsSaved; }

	const FCustomDetailLayoutNameMap& GetClassNameToDetailLayoutNameMap() const { return ClassNameToDetailLayoutNameMap; }

	/** Get the global row extension generators. */
	FOnGenerateGlobalRowExtension& GetGlobalRowExtensionDelegate() { return OnGenerateGlobalRowExtension; }

	const bool GetCanUsePropertyMatrix() const { return bCanUsePropertyMatrixOverride; } 
	void SetCanUsePropertyMatrix(const bool bInCanUsePropertyMatrix) 
	{ 
		bCanUsePropertyMatrixOverride = bInCanUsePropertyMatrix; 
	} 

	UE_EXPERIMENTAL(5.8, "This CVar may be removed.")
	PROPERTYEDITOR_API static TAutoConsoleVariable<bool> CVarAllowPropertyMatrixForNonActors;
private:

	/**
	 * Creates and returns a property view widget for embedding property views in other widgets
	 * NOTE: At this time these MUST not be referenced by the caller of CreatePropertyView when the property module unloads
	 * 
	 * @param	InObject						The UObject that the property view should observe(Optional)
	 * @param	bAllowFavorites					Whether the property view should save favorites
	 * @param	bIsLockable						Whether or not the property view is lockable
	 * @param	bAllowSearch					Whether or not the property window allows searching it
	 * @param	InNotifyHook					Notify hook to call on some property change events
	 * @param	ColumnWidth						The width of the name column
	 * @param	OnPropertySelectionChanged		Delegate for notifying when the property selection has changed.
	 * @return	The newly created SPropertyTreeViewImpl widget
	 */
	virtual TSharedRef<SPropertyTreeViewImpl> CreatePropertyView( UObject* InObject, bool bAllowFavorites, bool bIsLockable, bool bHiddenPropertyVisibility, bool bAllowSearch, bool ShowTopLevelNodes, FNotifyHook* InNotifyHook, float InNameColumnWidth, FOnPropertySelectionChanged OnPropertySelectionChanged, FOnPropertyClicked OnPropertyMiddleClicked, FConstructExternalColumnHeaders ConstructExternalColumnHeaders, FConstructExternalColumnCell ConstructExternalColumnCell );

	virtual TSharedRef<class IStructureDetailsView> CreateStructureDetailView(const struct FDetailsViewArgs& DetailsViewArgs, const FStructureDetailsViewArgs& StructureDetailsViewArgs, const FText& CustomName = FText::GetEmpty());

	TSharedPtr<FAssetThumbnailPool> GetThumbnailPool();

	void GetAllSectionsHelper(const UStruct* Struct, TArray<TSharedPtr<FPropertySection>>& OutSections, TSet<const UStruct*>& ProcessedStructs) const;
	void FindSectionsForCategoryHelper(const UStruct* Struct, FName CategoryName, TArray<TSharedPtr<FPropertySection>>& OutSections, TSet<const UStruct*>& SearchedStructs) const;

	TSharedPtr<class ISinglePropertyView> CreateSinglePropertyImpl(UObject* InObject, const TSharedPtr<IStructureDataProvider>& InStruct, FName InPropertyName, const struct FSinglePropertyParams& InitParams);
	void CompactSinglePropertyViewArray();

	/** Register Menu extension points */
	void RegisterMenus();

	static void PopulateRowContextMenu(UToolMenu* InToolMenu);

private:
	/** All created detail views */
	TArray< TWeakPtr<class SDetailsView> > AllDetailViews;
	/** All created single property views */
	TArray< TWeakPtr<class SSingleProperty> > AllSinglePropertyViews;
	/** A mapping of class names to detail layout delegates, called when querying for custom detail layouts */
	FCustomDetailLayoutNameMap ClassNameToDetailLayoutNameMap;
	/** A mapping of property names to property type layout delegates, called when querying for custom property layouts */
	FCustomPropertyTypeLayoutMap GlobalPropertyTypeToLayoutMap;
	/** Registered list of override callbacks. First one returning non-None is going to be applied (if any). */
	TMultiMap<FName, FPropertyHandleLayoutOverride> PropertyHandleLayoutOverrides;
	/** A mapping of class names to section mappings. */
	TMap<FName, TSharedPtr<FClassSectionMapping>> ClassSectionMappings;
	/** Event to be called when a property editor is opened */
	FPropertyEditorOpenedEvent PropertyEditorOpened;
	/** Event to be called when assets are saved in bulk from the property matrix editor. */
	FOnPropertyMatrixAssetsSaved PropertyMatrixAssetsSaved;
	/** Mapping of registered floating UStructs to their struct proxy so they show correctly in the details panel */
	TMap<FName, FStructProperty*> RegisteredStructToProxyMap;
	/** Shared thumbnail pool used by property row generators */
	TSharedPtr<class FAssetThumbnailPool> GlobalThumbnailPool;
	/** Container for ScopeOnStruct FStructProperty objects */
	UStruct* StructOnScopePropertyOwner;
	/** Delegate called to extend the name column widget on a property row. */
	FOnGenerateGlobalRowExtension OnGenerateGlobalRowExtension;
	/** Override for if the Property Matrix is availabe */
	bool bCanUsePropertyMatrixOverride = true;
};
