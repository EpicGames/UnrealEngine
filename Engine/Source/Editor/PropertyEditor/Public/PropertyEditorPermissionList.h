// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Deprecated in UE 5.3
UE_DEPRECATED_HEADER(5.3, "Header has been moved. #include PropertyPermissionList.h instead of PropertyEditorPermissionList.h.")

#include "PropertyPermissionList.h"