// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

namespace PropertyEditorUtils
{
	/**
	 * Calculates the possible drop-down options for the specified property path
	 * @param	InOutContainers		The container objects to resolve the property path against
	 * @param	InOutPropertyPath	The property path
	 * @param	InOutOptions		The resulting options
	 */
	PROPERTYEDITOR_API void GetPropertyOptions(TArray<UObject*>& InOutContainers, FString& InOutPropertyPath, TArray<TSharedPtr<FString>>& InOutOptions);

	/**
	 * Calculates the possible drop-down options for the specified property path
	 * The resulting option 
	 * @param	InOutContainers		The container objects to resolve the property path against
	 * @param	InOutPropertyPath	The property path
	 * @param	InOutOptions		The resulting options
	 */
	PROPERTYEDITOR_API void GetPropertyOptions(
		TArray<UObject*>& InOutContainers,
		FString& InOutPropertyPath,
		TArray<FString>& OutOptions,
		TArray<FText>* OutDisplayNames);

	/**
	 * Get all allowed and disallowed classes according to meta data
	 * @param	ObjectList			The list of object that owns the property
	 * @param	Property			The FProperty that contains the meta data
	 * @param	AllowedClasses		The list of allowed classes
	 * @param	DisallowedClasses	The list of disallowed classes
	 * @param	bExactClass			Whether subclasses allowed or not
	 * @param	ObjectClass			The class to add to the disallowed classes if there is no allowed classes
	 */
	PROPERTYEDITOR_API void GetAllowedAndDisallowedClasses(const TArray<UObject*>& ObjectList, const FProperty& Property, TArray<const UClass*>& AllowedClasses, TArray<const UClass*>& DisallowedClasses, bool bExactClass, const UClass* ObjectClass = UObject::StaticClass());

	/** 
	 * Metadata key for specifying that the property editor's class picker should include the root Object 
	 * type (/Script/CoreUObject.Object) in the displayed list of classes. By default Object is filtered out.
	 * This metadata key has no value, behavior is determined by its presence or absence, e.g.:
	 * 
	 * UPROPERTY(meta=(ShowObjectRootClass))
	 * TSoftClassPtr<UObject> Value;
	 */
	extern const FLazyName ShowObjectRootClassKey;
}
