// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/UnrealType.h"
#include "Widgets/SWidget.h"
#include "PropertyEditorModule.h"
#include "UObject/PropertyPortFlags.h"

struct FAssetData;
class FPropertyRestriction;
class FResetToDefaultOverride;
class IPropertyHandleArray;
class IPropertyHandleMap;
class IPropertyHandleSet;
class IPropertyHandleOptional;
class IPropertyHandleStruct;
class IStructureDataProvider;

namespace EPropertyValueSetFlags
{
	typedef uint32 Type;

	/** Normal way to call set value (make a transaction, call posteditchange) */
	inline const Type DefaultFlags = 0;
	/** No transaction will be created when setting this value (no undo/redo) */
	inline const Type NotTransactable = 1 << 0;
	/** When PostEditChange is called mark the change as interactive (e.g, user is spinning a value in a spin box) */
	inline const Type InteractiveChange = 1 << 1;
	/** If the property being imported to is an instanced object, create a new object rather than simply setting the value literally */
	inline const Type InstanceObjects = 1 << 2;
	/** If the property is being set due to being reset to default */
	inline const Type ResetToDefault = 1 << 3;

};

/**
 * A handle to a property which is used to read and write the value without needing to handle Pre/PostEditChange, transactions, package modification
 * A handle also is used to identify the property in detail customization interfaces
 */
class IPropertyHandle : public TSharedFromThis<IPropertyHandle>
{
public:
	virtual ~IPropertyHandle()= default;

	/**
	 * @return Whether or not the handle points to a valid property node. This can be true but GetProperty may still return null
	 */
	virtual bool IsValidHandle() const = 0;

	/**
	 * @return Whether or not the handle points to the same property node. Will return true if both point to an invalid node.
	 */
	virtual bool IsSamePropertyNode(TSharedPtr<IPropertyHandle> OtherHandle) const = 0;
	
	/**
	 * @return Whether or not the property is edit const (can't be changed)
	 */
	virtual bool IsEditConst() const = 0;

	/**
	* @return The opposite of IsEditConst
	*/
	virtual bool IsEditable() const = 0;

	/**
	 * @return Whether or not the property is expanded
	 */
	virtual bool IsExpanded() const = 0;

	/**
	 * Sets the expanded flag for this property.
	 */
	virtual void SetExpanded(bool bExpanded) = 0;

	/**
	 * Gets the class of the property being edited
	 */
	virtual const FFieldClass* GetPropertyClass() const = 0;

	/**
	 * Gets the property being edited
	 */
	virtual FProperty* GetProperty() const = 0;

	/**
	 * Helper to fetch a PropertyPath 
	 */
	virtual FStringView GetPropertyPath() const = 0;

	/**
	 * Helper to fetch a PropertyPath 
	 */
	virtual TSharedPtr<FPropertyPath> CreateFPropertyPath() const = 0;

	/**
	 * Helper to fetch the ArrayIndex
	 */
	virtual int32 GetArrayIndex() const = 0;

	/**
	 * Indicates that children of this node should be rebuilt next tick.  Some topology changes will require this
	 */
	virtual void RequestRebuildChildren() = 0;

	/**
	 * Gets the property we should use to read meta-data
	 */
	virtual FProperty* GetMetaDataProperty() const = 0;

	/**
	 * Determines if the property has any metadata associated with the key
	 * 
	 * @param Key The key to lookup in the metadata
	 * @return true if there is a (possibly blank) value associated with this key
	 */
	virtual bool HasMetaData(const FName& Key) const = 0;

	/**
	 * Find the metadata value associated with the key
	 * 
	 * @param Key The key to lookup in the metadata
	 * @return The value associated with the key
	 */
	virtual const FString& GetMetaData(const FName& Key) const = 0;

	/**
	 * Find the metadata value associated with the key and return bool
	 *
	 * @param Key The key to lookup in the metadata
	 * @return return true if the value was true (case insensitive)
	 */
	virtual bool GetBoolMetaData(const FName& Key) const = 0;

	/**
	 * Find the metadata value associated with the key and return int32
	 *
	 * @param Key The key to lookup in the metadata
	 * @return the int value stored in the metadata.
	 */
	virtual int32 GetIntMetaData(const FName& Key) const = 0;

	/**
	 * Find the metadata value associated with the key and return float
	 *
	 * @param Key The key to lookup in the metadata
	 * @return the float value stored in the metadata.
	 */
	virtual float GetFloatMetaData(const FName& Key) const = 0;

	/**
	 * Find the metadata value associated with the key and return double
	 *
	 * @param Key The key to lookup in the metadata
	 * @return the double value stored in the metadata.
	 */
	virtual double GetDoubleMetaData(const FName& Key) const = 0;

	/**
	 * Find the metadata value associated with the key and return UClass*
	 *
	 * @param Key The key to lookup in the metadata
	 * @return the UClass value stored in the metadata.
	 */
	virtual UClass* GetClassMetaData(const FName& Key) const = 0;

	/** Set metadata value for 'Key' to 'Value' on this property instance (as opposed to the class) */
	virtual void SetInstanceMetaData(const FName& Key, const FString& Value) = 0;

	/**
	 * Get metadata value for 'Key' for this property instance (as opposed to the class)
	 * 
	 * @return Pointer to metadata value; nullptr if Key not found
	 */
	virtual const FString* GetInstanceMetaData(const FName& Key) const = 0;

	/**
	 * Get metadata map for this property instance (as opposed to the class)
	 * 
	 * @return Map ptr containing metadata pairs; nullptr if property node is invalid
	 */
	virtual const TMap<FName, FString>* GetInstanceMetaDataMap() const = 0;

	/**
	 * Gets the property tool tip text.
	 */
	virtual FText GetToolTipText() const = 0;
	
	/**
	 * Sets the tooltip shown for this property
	 *
	 * @param ToolTip	The tool tip to show
	 */
	virtual void SetToolTipText(const FText& ToolTip) = 0;

	/**
	 * @return True if this property has custom documentation, false otherwise
	 */
	virtual bool HasDocumentation() = 0;

	/**
	 * @return The custom documentation link for this property
	 */
	virtual FString GetDocumentationLink() = 0;

	/**
	 * @return The custom documentation except name for this property
	 */
	virtual FString GetDocumentationExcerptName() = 0;

	/**
	* Calculates the memory address for the data associated with this item's value.
	*
	* @param Base The location to use as the starting point for the calculation; typically the address of an object.
	*
	* @return A pointer to a FProperty value or UObject.
	*/
	virtual uint8* GetValueBaseAddress( uint8* Base ) const = 0;

	/**
	 * Gets the value formatted as a string.
	 *
	 * @param OutValue		String where the value is stored.  Remains unchanged if the value could not be set
	 * @param PortFlags		Property flags to determine how the string is retrieved. Defaults to PPF_PropertyWindow
	 * @return The result of attempting to get the value
	 */
	virtual FPropertyAccess::Result GetValueAsFormattedString( FString& OutValue, EPropertyPortFlags PortFlags = PPF_PropertyWindow ) const = 0;

	/**
	 * Gets the value formatted as a string, possibly using an alternate form more suitable for display in the UI
	 *
	 * @param OutValue		String where the value is stored.  Remains unchanged if the value could not be set
	 * @param PortFlags		Property flags to determine how the string is retrieved. Defaults to PPF_PropertyWindow
	 * @return The result of attempting to get the value
	 */
	virtual FPropertyAccess::Result GetValueAsDisplayString( FString& OutValue, EPropertyPortFlags PortFlags = PPF_PropertyWindow ) const = 0;

	/**
	 * Gets the value formatted as a string, as Text.
	 *
	 * @param OutValue	Text where the value is stored.  Remains unchanged if the value could not be set
	 * @return The result of attempting to get the value
	 */
	virtual FPropertyAccess::Result GetValueAsFormattedText( FText& OutValue ) const = 0;

	/**
	 * Gets the value formatted as a string, as Text, possibly using an alternate form more suitable for display in the UI
	 *
	 * @param OutValue	Text where the value is stored.  Remains unchanged if the value could not be set
	 * @return The result of attempting to get the value
	 */
	virtual FPropertyAccess::Result GetValueAsDisplayText( FText& OutValue ) const = 0;

	/**
	 * Sets the value formatted as a string.
	 *
	 * @param OutValue	String where the value is stored.  Is unchanged if the value could not be set
	 * @return The result of attempting to set the value
	 */
	virtual FPropertyAccess::Result SetValueFromFormattedString( const FString& InValue,  EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	
	/**
	 * Sets a delegate to call when the value of the property is changed
	 *
	 * @param InOnPropertyValueChanged	The delegate to call
	 */
	virtual void SetOnPropertyValueChanged( const FSimpleDelegate& InOnPropertyValueChanged ) = 0;
	
	/**
	 * Sets a delegate to call when the value of the property is changed including the property changed event as parameter
	 * 
	 * @param InOnPropertyValueChanged The delegate to call. Needs to be of type void with a const FPropertyChangedEvent& parameter
	 */
	virtual void SetOnPropertyValueChangedWithData( const TDelegate<void(const FPropertyChangedEvent&)>& InOnPropertyValueChanged ) = 0;

	/**
	 * Sets a delegate to call when the value of the property of a child is changed
	 * 
	 * @param InOnChildPropertyValueChanged	The delegate to call
	 */
	virtual void SetOnChildPropertyValueChanged( const FSimpleDelegate& InOnChildPropertyValueChanged ) = 0;
	
	/**
	 * Sets a delegate to call when the value of the property of a child is changed including the property changed event as parameter
	 * 
	 * @param InOnPropertyValueChanged The delegate to call. Needs to be of type void with a const FPropertyChangedEvent& parameter
	 */
	virtual void SetOnChildPropertyValueChangedWithData( const TDelegate<void(const FPropertyChangedEvent&)>& InOnPropertyValueChanged ) = 0;

	/**
	 * Sets a delegate to call when the value of the property is about to be changed
	 *
	 * @param InOnPropertyValuePreChange	The delegate to call
	 */
	virtual void SetOnPropertyValuePreChange(const FSimpleDelegate& InOnPropertyValuePreChange) = 0;

	/**
	 * Sets a delegate to call when the value of the property of a child is about to be changed
	 *
	 * @param InOnChildPropertyValuePreChange	The delegate to call
	 */
	virtual void SetOnChildPropertyValuePreChange(const FSimpleDelegate& InOnChildPropertyValuePreChange) = 0;

	/**
	 * Sets a delegate to call when the property is reset to default
	 *
	 * @param InOnPropertyResetToDefault	The delegate to call
	 */
	virtual void SetOnPropertyResetToDefault(const FSimpleDelegate& InOnPropertyResetToDefault) = 0;

	/**
	 * Gets the typed value of a property.  
	 * If the property does not support the value type FPropertyAccess::Fail is returned
	 *
	 * @param OutValue	The value that will be set if successful
	 * @return The result of attempting to get the value
	 */
	virtual FPropertyAccess::Result GetValue( float& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( double& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( bool& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( int8& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( int16& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( int32& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( int64& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( uint8& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( uint16& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( uint32& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( uint64& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( FString& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( FText& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( FName& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( FVector& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( FVector2D& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( FVector4& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( FQuat& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( FRotator& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( UObject*& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( const UObject*& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValue( FAssetData& OutValue ) const = 0;
	virtual FPropertyAccess::Result GetValueData( void*& OutAddress ) const = 0;
	virtual FPropertyAccess::Result GetValue(FProperty*& OutValue) const = 0;
	virtual FPropertyAccess::Result GetValue(const FProperty*& OutValue) const = 0;
	/**
	 * Sets the typed value of a property.  
	 * If the property does not support the value type FPropertyAccess::Fail is returned
	 *
	 * @param InValue	The value to set
	 * @return The result of attempting to set the value
	 */
	virtual FPropertyAccess::Result SetValue( const float& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const double& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const bool& InValue,  EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const int8& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const int16& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const int32& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const int64& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const uint8& InValue,  EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const uint16& InValue,  EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const uint32& InValue,  EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const uint64& InValue,  EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const FString& InValue,  EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const FText& InValue,  EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const FName& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const FVector& InValue,  EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const FVector2D& InValue,  EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const FVector4& InValue,  EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const FQuat& InValue,  EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const FRotator& InValue,  EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( UObject* const& InValue,  EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const UObject* const& InValue,  EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const FAssetData& InValue,  EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( const TCHAR* InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;
	virtual FPropertyAccess::Result SetValue( FProperty* const& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) = 0;
	virtual FPropertyAccess::Result SetValue( const FProperty* const& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags) = 0;

	/**
	 * Called to manually notify root objects that this property is about to change
	 * This does not need to be called when SetValue functions are used since it will be called automatically
	 */
	virtual void NotifyPreChange() = 0;

	/**
	 * Called to manually notify root objects that this property has changed
	 * This does not need to be called when SetValue functions are used since it will be called automatically
	 */
	virtual void NotifyPostChange(EPropertyChangeType::Type ChangeType) = 0;

	/**
	 * Called to manually notify root objects that this property has finished changing
	 * This does not need to be called when SetValue functions are used since it will be called automatically
	 */
	virtual void NotifyFinishedChangingProperties() = 0;

	/**
	 * Sets the object value from the current editor selection
	 * Will fail if this handle isn't an object property
	 */
	virtual FPropertyAccess::Result SetObjectValueFromSelection() = 0;

	/**
	 * Gets the number of objects that this handle is editing
	 */
	virtual int32 GetNumPerObjectValues() const = 0;

	/**
	 * Sets a unique value for each object this handle is editing
	 *
	 * @param PerObjectValues	The per object values as a formatted string.  There must be one entry per object or the return value is FPropertyAccess::Fail
	 */
	virtual FPropertyAccess::Result SetPerObjectValues( const TArray<FString>& PerObjectValues, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;

	/**
	 * Gets a unique value for each object this handle is editing
	 */
	virtual FPropertyAccess::Result GetPerObjectValues( TArray<FString>& OutPerObjectValues ) const = 0;

	/**
	 * Sets a value on the specified object that this handle is editing
	 *
	 * @param ObjectIndex		The index of the object to set the value of
	 * @param ObjectValue		The value to set on the given object
	 */
	virtual FPropertyAccess::Result SetPerObjectValue( const int32 ObjectIndex, const FString& ObjectValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags ) = 0;

	/**
	 * Gets a value for the specified object that this handle is editing
	 *
	 * @param ObjectIndex		The index of the object to get the value of
	 * @param OutObjectValue	Filled with the value for this object
	 */
	virtual FPropertyAccess::Result GetPerObjectValue( const int32 ObjectIndex, FString& OutObjectValue ) const = 0;

	/**
	 * @return The index of this element in an array if it is in one.  INDEX_NONE otherwise                                                              
	 */
	virtual int32 GetIndexInArray() const = 0;

	/**
	 * Gets a child handle of this handle.  Useful for accessing properties in structs.  
	 * Array elements cannot be accessed in this way  
	 *
	 * @param ChildName The name of the child
	 * @param bRecurse	Whether or not to recurse into children of children and so on. If false will only search all immediate children
	 * @return The property handle for the child if it exists
	 */
	virtual TSharedPtr<IPropertyHandle> GetChildHandle( FName ChildName, bool bRecurse = true ) const = 0;

	/**
	 * Gets a child handle of this handle.  Useful for accessing properties in structs.  
	 *
	 * @param The index of the child
	 * @return The property handle for the child if it exists
	 */
	virtual TSharedPtr<IPropertyHandle> GetChildHandle( uint32 Index ) const = 0;
	
	/**
	 * @return a handle to the parent property 
	 * This parent handle may not contain a valid FProperty if the parent is the uobject.
	 */
	virtual TSharedPtr<IPropertyHandle> GetParentHandle() const = 0;

	/**
	 * @return The property handle to the key element for this value if this is a map element
	 */
	virtual TSharedPtr<IPropertyHandle> GetKeyHandle() const = 0;

	/**
	 * @return The number of children the property handle has
	 */
	virtual FPropertyAccess::Result GetNumChildren( uint32& OutNumChildren ) const = 0;

	/**
	 * @return Get the number objects that contain this property and are being observed in the property editor
	 */
	virtual uint32 GetNumOuterObjects() const = 0;
	
	/**
	 * Get the objects that contain this property 
	 *
	 * @param OuterObjects	An array that will be populated with the outer objects 
	 */
	virtual void GetOuterObjects( TArray<UObject*>& OuterObjects ) const = 0;

	/**
	 * Get the structures that contain this property 
	 *
	 * @param OutStructs	An array that will be populated with the outer structures 
	 */
	virtual void GetOuterStructs( TArray<TSharedPtr<FStructOnScope>>& OutStructs ) const = 0;

	/**
	 * Get the shared base class of the objects that contain this property.
	 *
	 * @return The shared base class of the outer objects, or null if none are selected.
	 */
	virtual const UClass* GetOuterBaseClass() const = 0;

	/**
	 * Set the outer objects for this property
	 *
	 * @param OuterObjects An array that contains the new outer objects
	 */
	virtual void ReplaceOuterObjects( const TArray<UObject*>& OuterObjects ) = 0;


	/**
	 * Get the packages that contain this property
	 *
	 * @param OuterPackages	An array that will be populated with the outer packages
	 */
	virtual void GetOuterPackages(TArray<UPackage*>& OuterPackages) const = 0;

	/**
	 * Enumerate the raw data of this property.  (Each pointer can be cast to the property data type)
	 *
	 * @param InRawDataCallback		The function to call for each data
	 */ 
	typedef TFunctionRef<bool(void* /*RawData*/, const int32 /*DataIndex*/, const int32 /*NumDatas*/)> EnumerateRawDataFuncRef; /** Return true to continue enumeration */
	typedef TFunctionRef<bool(const void* /*RawData*/, const int32 /*DataIndex*/, const int32 /*NumDatas*/)> EnumerateConstRawDataFuncRef; /** Return true to continue enumeration */
	virtual void EnumerateRawData( const EnumerateRawDataFuncRef& InRawDataCallback ) = 0;
	virtual void EnumerateConstRawData( const EnumerateConstRawDataFuncRef& InRawDataCallback ) const = 0;

	/**
	 * Accesses the raw data of this property.  (Each pointer can be cast to the property data type)
	 *
	 * @param RawData	An array of raw data.  The elements in this array are the raw data for this property on each of the objects in the  property editor
	 */ 
	virtual void AccessRawData( TArray<void*>& RawData ) = 0;
	virtual void AccessRawData( TArray<const void*>& RawData ) const = 0;

	/**
	 * Returns this handle as an array if possible
	 *
	 * @return the handle as an array if it is an array (static or dynamic)
	 */
	virtual TSharedPtr<IPropertyHandleArray> AsArray() = 0;

	/**
	 * @return This handle as a set if possible
	 */
	virtual TSharedPtr<IPropertyHandleSet> AsSet() = 0;

	/**
	 * @return This handle as a map if possible
	 */
	virtual TSharedPtr<IPropertyHandleMap> AsMap() = 0;

	/**
	 * @return This handle as an optional if possible
	 */
	virtual TSharedPtr<IPropertyHandleOptional> AsOptional() = 0;

	/**
	 * @return This handle as struct if possible
	 */
	virtual TSharedPtr<IPropertyHandleStruct> AsStruct() = 0;

	/**
	 * @return The display name of the property
	 */
	virtual FText GetPropertyDisplayName() const = 0;

	/** 
	* Allows the handle to override the node's display name
	*/
	virtual void SetPropertyDisplayName(FText InDisplayName) = 0;
	
	/**
	 * Resets the value to its default
	 */
	virtual void ResetToDefault() = 0;

	/**
	 * @return Whether or not the value differs from its default                   
	 */
	virtual bool DiffersFromDefault() const = 0;

	/**
	 * @return A label suitable for displaying the reset to default value
	 */
	virtual FText GetResetToDefaultLabel() const = 0;

	/**
	 * Generates a list of possible enum/class options for the property
	 */
	virtual bool GeneratePossibleValues(TArray< TSharedPtr<FString> >& OutOptionStrings, TArray< FText >& OutToolTips, TArray<bool>& OutRestrictedItems) = 0;

	/**
	* Generates a list of possible enum/class options for the property
	* DisplayNames is optional
	*/
	virtual bool GeneratePossibleValues(TArray<FString>& OutOptionStrings, TArray< FText >& OutToolTips, TArray<bool>& OutRestrictedItems, TArray<FText>* OutDisplayNames) = 0;

	/**
	 * Marks this property has hidden by customizaton (will not show up in the default place)
	 */
	virtual void MarkHiddenByCustomization() = 0;

	/**
	 * Marks this property has having a custom reset to default (reset to default will not show up in the default place)
	 */
	virtual void MarkResetToDefaultCustomized(bool bCustomized = true) = 0;

	/**
	 * Marks this property as not having a custom reset to default (useful when a widget customizing reset to default goes away)
	 */
	virtual void ClearResetToDefaultCustomized() = 0;

	/**
	* @return true if the property is mark as a favorite
	*/
	virtual bool IsFavorite() const = 0;

	/**
	 * @return True if this property's UI is customized                                                              
	 */
	virtual bool IsCustomized() const = 0;

	/**
	 * @return True if this property's reset to default UI is customized (but not necessarialy the property UI itself)
	 */
	virtual bool IsResetToDefaultCustomized() const = 0;

	/**
	 * Generates a path from the parent UObject class to this property
	 *
	 * @return The path to this property
	 */
	virtual FString GeneratePathToProperty() const = 0;

	/**
	 * Creates a name widget for this property
	 * @param NameOverride				The name override to use instead of the property name
	 * @param ToolTipOverride			The tooltip override to use instead of the property name
	 * @param bDisplayResetToDefault	Whether or not to display the reset to default button
	 * @param bDisplayText				Whether or not to display the text name of the property
	 * @param bDisplayThumbnail			Whether or not to display the thumbnail for the property (if any)
	 * @return the name widget for this property
	 */
	UE_DEPRECATED(5.0, "CreatePropertyNameWidget no longer supports bDisplayResetToDefault, bDisplayText or bDisplayThumbnail.")
	virtual TSharedRef<SWidget> CreatePropertyNameWidget( const FText& NameOverride, const FText& ToolTipOverride, bool bDisplayResetToDefault, bool bDisplayText = true, bool bDisplayThumbnail = true ) const = 0;

	/**
	 * Creates a name widget for this property
	 * @param NameOverride				The name override to use instead of the property name
	 * @param ToolTipOverride			The tooltip override to use instead of the property name
	 */
	virtual TSharedRef<SWidget> CreatePropertyNameWidget(const FText& NameOverride = FText::GetEmpty(), const FText& ToolTipOverride = FText::GetEmpty()) const = 0;
	
	/**
	 * Creates a value widget for this property
	 * 
	 * @param bDisplayDefaultPropertyButtons	If the value widget should include the property buttons.
	 *
	 * @return the value widget for this property
	 */
	virtual TSharedRef<SWidget> CreatePropertyValueWidget( bool bDisplayDefaultPropertyButtons = true ) const = 0;

	/**
	 * Creates a value widget for this property using customization, if available.
	 * Note that this is only the value widget for the header/main row for properties with child rows (ie: structs)
	 * 
	 * @param DetailsView						The details view to create the value widget for. Used to retrieve per details view customizations.
	 * 
	 * @return the value widget for this property
	 */
	virtual TSharedRef<SWidget> CreatePropertyValueWidgetWithCustomization( const IDetailsView* DetailsView ) = 0;

	/**
	 * Creates the default buttons which appear next to value widgets.  This is useful when creating customizations
	 * which don't use CreatePropertyValueWidget but you still want array item behaviors and reset to default capabilities.
	 */
	virtual TSharedRef<SWidget> CreateDefaultPropertyButtonWidgets() const = 0;

	/**
	 * Creates the default copy and paste actions for a single property.  This is useful when creating customizations which use
	 * custom node builders since they can't implement copy/paste by default because the contents of the rows is not known,
	 * but frequently you're using the handles on each row to generate this UI.
	 */
	virtual void CreateDefaultPropertyCopyPasteActions(FUIAction& OutCopyAction, FUIAction& OutPasteAction) const = 0;

	/**
	 * Adds a restriction to the possible values for this property.
	 * @param Restriction	The restriction being added to this property.
	 */
	virtual void AddRestriction( TSharedRef<const FPropertyRestriction> Restriction ) = 0;

	/**
	* Tests if a value is restricted for this property
	* @param Value			The value to test for restriction.
	* @return				True if this value is restricted.
	*/
	virtual bool IsRestricted(const FString& Value) const = 0;

	/**
	* Tests if a value is restricted for this property.
	* @param Value			The value to test for restriction.
	 * @param OutReasons	Outputs an array of the reasons why this value is restricted.
	* @return				True if this value is restricted.
	*/
	virtual bool IsRestricted(const FString& Value, TArray<FText>& OutReasons) const = 0;

	/**
	 * Generates a consistent tooltip describing this restriction for use in the editor.
	 * @param Value			The value to test for restriction and generate the tooltip from.
	 * @param OutTooltip	The tooltip describing why this value is restricted.
	 * @return				True if this value is restricted.
	 */
	virtual bool GenerateRestrictionToolTip(const FString& Value, FText& OutTooltip) const = 0;

	/**
	* Tests if a value is disabled for this property
	* @param Value			The value to test whether it is disabled.
	* @return				True if this value is disabled.
	*/
	virtual bool IsDisabled(const FString& Value) const = 0;

	/**
	* Tests if a value is disabled for this property.
	* @param Value			The value to test whether it is disabled.
	* @param OutReasons	Outputs an array of the reasons why this value is disabled.
	* @return				True if this value is disabled.
	*/
	virtual bool IsDisabled(const FString& Value, TArray<FText>& OutReasons) const = 0;

	/**
	* Tests if a value is hidden for this property
	* @param Value			The value to test whether it is hidden.
	* @return				True if this value is hidden.
	*/
	virtual bool IsHidden(const FString& Value) const = 0;

	/**
	* Tests if a value is hidden for this property.
	* @param Value			The value to test whether it is hidden.
	* @param OutReasons	Outputs an array of the reasons why this value is hidden.
	* @return				True if this value is hidden.
	*/
	virtual bool IsHidden(const FString& Value, TArray<FText>& OutReasons) const = 0;

	 /** 
	  * Sets whether or not data validation should occur for this property and all of its children. It is generally unsafe to set this value unless you know what you are doing.  Data validation done by the details panel ensures changes to properties out from under the details panel are known
	  * This should only ever be set for extremely large arrays or other costly validation checks where validation is handled by the customizer
	  */
	virtual void SetIgnoreValidation(bool bInIgnore) = 0;
	
	/**
	 * Adds a child structure
	 * 
	 * @param ChildStructure	The structure to add
	 * @return An array of interfaces to the properties that were added
	 */
	virtual TArray<TSharedPtr<IPropertyHandle>> AddChildStructure( TSharedRef<FStructOnScope> ChildStructure ) = 0;
	virtual TArray<TSharedPtr<IPropertyHandle>> AddChildStructure( TSharedRef<IStructureDataProvider> ChildStructure ) = 0;

	/** Removes all children of this handle */
	virtual void RemoveChildren() = 0;

	/**
	 * Returns whether or not the property can be set to default
	 * 
	 * @return If this property can be reset to default
	 */
	virtual bool CanResetToDefault() const = 0;
	
	/**
	 * Sets an override for this property's reset to default behavior
	 */
	virtual void ExecuteCustomResetToDefault(const FResetToDefaultOverride& OnCustomResetToDefault) = 0;

	/**
	 * Gets the category FName that a property is in at the default location defined by the class the property is in
	 * It does not handle the property being moved to another category during customization
	 */
	virtual FName GetDefaultCategoryName() const = 0;

	/**
	* Gets the category display name that a property is in at the default location defined by the class the property is in
	* It does not handle the property being moved to another category during customization
	*/
	virtual FText GetDefaultCategoryText() const = 0;

	virtual bool IsCategoryHandle() const = 0;
};

class FPropertyHandleItemAddResult
{
public:
	FPropertyHandleItemAddResult() = default;
	FPropertyHandleItemAddResult(int32 InIndex, FPropertyAccess::Result InResult)
		: Index(InIndex)
		, PropertyAccessResult(InResult)
	{}

	int32 GetIndex() const { return Index; }
	FPropertyAccess::Result GetAccessResult() const { return PropertyAccessResult; }

	operator FPropertyAccess::Result() const { return PropertyAccessResult; }
	
private:
	int32 Index = INDEX_NONE;
	FPropertyAccess::Result PropertyAccessResult = FPropertyAccess::Fail;
};

/**
 * A handle to an array property which allows you to manipulate the array                                                              
 */
class IPropertyHandleArray
{
public:
	virtual ~IPropertyHandleArray()= default;
	
	/**
	 * Adds an item to the end of the array
	 * @return Whether or not this was successful
	 */
	virtual FPropertyHandleItemAddResult AddItem() = 0;

	/**
	 * Empty the array
	 * @return Whether or not this was successful
	 */
	virtual FPropertyAccess::Result EmptyArray() = 0;

	/**
	 * Inserts an item into the array at the specified index
	 * @return Whether or not this was successful
	 */
	virtual FPropertyAccess::Result Insert( int32 Index ) = 0;

	/**
	 * Duplicates the item at the specified index in the array.
	 * @return Whether or not this was successful
	 */
	virtual FPropertyAccess::Result DuplicateItem( int32 Index ) = 0;

	/**
	 * Deletes the item at the specified index of the array
	 * @return Whether or not this was successful
	 */
	virtual FPropertyAccess::Result DeleteItem( int32 Index ) = 0;

	/**
	 * Swaps two items
	 * @return Whether or not this was successful
	 */
	virtual FPropertyAccess::Result SwapItems( int32 FirstIndex, int32 SecondIndex ) = 0;

	/**
	 * @return The number of elements in the array
	 */
	virtual FPropertyAccess::Result GetNumElements( uint32& OutNumItems ) const = 0;

	/**
	 * @return a handle to the element at the specified index
	 */
	virtual TSharedRef<IPropertyHandle> GetElement( int32 Index ) const = 0;

	/**
	* Moves an element from OriginalIndex to NewIndex
	* @return Whether or not this was successful
	*/
	virtual FPropertyAccess::Result MoveElementTo(int32 OriginalIndex, int32 NewIndex) = 0;

	/**
	 * Sets a delegate to call before an item in a container is being mutated
	 */
	virtual void SetOnCanItemChange(const FCanPropertyItemChange& InDelegate) = 0;

	/**
	 * Sets a delegate to call when a property change has been disallowed
	 */
	virtual FDelegateHandle SetOnChangeBlocked(const FPropertyChangeBlocked::FDelegate& InDelegate) = 0;

	/**
	 * Unregisters a delegate that is bound to OnChangeBlocked
	 */
	virtual void UnregisterOnChangeBlocked(FDelegateHandle Handle) = 0;

	/**
	 * Sets a delegate to call when the number of elements changes
	 */
	virtual FDelegateHandle SetOnNumElementsChanged( const FSimpleDelegate& InOnNumElementsChanged ) = 0;

	/**
	 * Unregisters a delegate that is called when the number of elements changes
	 */
	virtual void UnregisterOnNumElementsChanged(FDelegateHandle Handle) = 0;
};

/**
 * A handle to a property which allows you to manipulate a Set
 */
class IPropertyHandleSet
{
public:
	virtual ~IPropertyHandleSet()= default;

	/**
	 * @return True if the set contains an element with a default value, false otherwise
	 */
	virtual bool HasDefaultElement() = 0;

	/**
	 * Adds an item to the set.
	 * @return Whether or not this was successful
	 */
	virtual FPropertyHandleItemAddResult AddItem() = 0;

	/**
	 * Empties the set
	 * @return Whether or not this was successful
	 */
	virtual FPropertyAccess::Result Empty() = 0;

	/**
	* Deletes the item in the set with the specified internal index
	* @return Whether or not this was successful
	*/
	virtual FPropertyAccess::Result DeleteItem(int32 Index) = 0;

	/**
	 * @return The number of elements in the set
	 */
	virtual FPropertyAccess::Result GetNumElements(uint32& OutNumElements) = 0;

	/**
	 * @return a handle to the element at the specified index
	 */
	virtual TSharedRef<IPropertyHandle> GetElement(int32 Index) const = 0;


	/**
	 * Sets a delegate to call when the number of elements changes
	 */
	virtual FDelegateHandle SetOnNumElementsChanged(const FSimpleDelegate& InOnNumElementsChanged) = 0;

	/**
	 * Unregisters a delegate that is called when the number of elements changes
	 */
	virtual void UnregisterOnNumElementsChanged(FDelegateHandle Handle) = 0;
};

/**
 * A handle to a property which allows you to manipulate a Map
 */
class IPropertyHandleMap
{
public:
	virtual ~IPropertyHandleMap() = default;

	/**
	 * @return True if the map contains a key with a default value, false otherwise
	 */
	virtual bool HasDefaultKey() = 0;

	/**
	 * Adds an item to the map.
	 * @return Whether or not this was successful
	 */
	virtual FPropertyHandleItemAddResult AddItem() = 0;

	/**
	 * Empties the map
	 * @return Whether or not this was successful
	 */
	virtual FPropertyAccess::Result Empty() = 0;

	/**
	 * Deletes the item in the map with the specified internal index
	 * @return Whether or not this was successful
	 */
	virtual FPropertyAccess::Result DeleteItem(int32 Index) = 0;

	/**
	 * @return The number of elements in the map
	 */
	virtual FPropertyAccess::Result GetNumElements(uint32& OutNumElements) = 0;

	/**
	 * @return a handle to the element at the specified index
	 */
	virtual TSharedRef<IPropertyHandle> GetElement(int32 Index) const = 0;

	/**
	 * Sets a delegate to call when the number of elements changes
	 */
	virtual FDelegateHandle SetOnNumElementsChanged(const FSimpleDelegate& InOnNumElementsChanged) = 0;
	
	/**
	 * Unregisters a delegate that is called when the number of elements changes
	 */
	virtual void UnregisterOnNumElementsChanged(FDelegateHandle Handle) = 0;
};

/**
 * A handle to a property which allows you to access a Struct's Data
 */
class IPropertyHandleStruct
{
public:
	virtual ~IPropertyHandleStruct() = default;

	virtual TSharedPtr<FStructOnScope> GetStructData() const = 0;
};

/**
 * A handle to an optional property which allows you to manipulate the optional
 */
class IPropertyHandleOptional
{
public:
	virtual ~IPropertyHandleOptional() = default;

	/**
	 * Get the item
	 * @param OutValue	The value of the optional if it is set.
	 */
	virtual FPropertyAccess::Result GetOptionalValue(FProperty*& OutValue) = 0;

	/**
	 * Set the item
	 * @param NewValue	The value to set the optional to. Pass in nullptr to default-initialize the value.
	 * @return			Whether or not this was successful
	 */
	virtual FPropertyAccess::Result SetOptionalValue(FProperty* NewProperty, const UClass* NewObjectClass) = 0;

	/**
	 * Clear the item
	 * @return Whether or not this was successful
	 */
	virtual FPropertyAccess::Result ClearOptionalValue() = 0;
};
