// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakFieldPtr.h"

namespace UE { class FPropertyPathName; }

struct FPropertyInfo
{
public:

	FPropertyInfo()
		: Property()
		, ArrayIndex(INDEX_NONE)
	{}

	FPropertyInfo(TWeakFieldPtr< FProperty > InProperty, int32 InArrayIndex = INDEX_NONE)
		: Property(InProperty)
		, ArrayIndex(InArrayIndex)
	{}

	bool operator==( const FPropertyInfo& Other ) const 
	{
		return Property == Other.Property && ArrayIndex == Other.ArrayIndex;
	}

	bool operator!=( const FPropertyInfo& Other ) const 
	{
		return !(*this == Other);
	}

	TWeakFieldPtr<FProperty> Property;
	int32 ArrayIndex;
};


class FPropertyPath
{
public:

	static TSharedRef< FPropertyPath > CreateEmpty()
	{
		return MakeShareable( new FPropertyPath() );
	}

	static TSharedRef< FPropertyPath > Create( const TWeakFieldPtr<FProperty> Property )
	{
		TSharedRef< FPropertyPath > NewPath = MakeShareable( new FPropertyPath() );

		FPropertyInfo NewPropInfo;
		NewPropInfo.Property = Property;
		NewPropInfo.ArrayIndex = INDEX_NONE;

		NewPath->Properties.Add( NewPropInfo );

		return NewPath;
	}

	bool IsValid() const 
	{ 
		return !Properties.IsEmpty(); 
	}

	int32 GetNumProperties() const 
	{ 
		return Properties.Num(); 
	}

	const FPropertyInfo& GetPropertyInfo( int32 index ) const
	{
		return Properties[ index ];
	}

	FPropertyInfo& GetPropertyInfoRef( int32 index )
	{
		return Properties[ index ];
	}

	const FPropertyInfo& GetLeafMostProperty() const
	{
		return Properties[ Properties.Num() - 1 ];
	}

	const FPropertyInfo& GetRootProperty() const
	{
		return Properties[ 0 ];
	}

	TSharedRef< FPropertyPath > ExtendPath( const FPropertyInfo& NewLeaf ) const
	{
		TSharedRef< FPropertyPath > NewPath = MakeShareable( new FPropertyPath() );

		NewPath->Properties = Properties;
		NewPath->Properties.Add( NewLeaf );
		
		return NewPath;
	}

	TSharedRef< FPropertyPath > ExtendPath( const TSharedRef< FPropertyPath >& Extension ) const
	{
		TSharedRef< FPropertyPath > NewPath = MakeShareable( new FPropertyPath() );

		NewPath->Properties = Properties;

		for (int Index = Extension->GetNumProperties() - 1; Index >= 0 ; Index--)
		{
			NewPath->Properties.Add( Extension->GetPropertyInfo( Index ) );
		}

		return NewPath;
	}

	TSharedRef< FPropertyPath > TrimPath( const int32 AmountToTrim ) const
	{
		TSharedRef< FPropertyPath > NewPath = MakeShareable( new FPropertyPath() );

		NewPath->Properties = Properties;

		for (int Count = 0; Count < AmountToTrim; Count++)
		{
			NewPath->Properties.Pop();
		}

		return NewPath;
	}

	TSharedRef< FPropertyPath > TrimRoot( const int32 AmountToTrim ) const
	{
		TSharedRef< FPropertyPath > NewPath = MakeShareable( new FPropertyPath() );

		for (int Count = AmountToTrim; Count < Properties.Num(); Count++)
		{
			NewPath->Properties.Add( Properties[ Count ] );
		}

		return NewPath;
	}

	FString ToString(const TCHAR* Separator = TEXT("->")) const
	{
		FString NewDisplayName;
		bool FirstAddition = true;
		for (int PropertyIndex = 0; PropertyIndex < Properties.Num(); PropertyIndex++)
		{
			const FPropertyInfo& PropInfo = Properties[ PropertyIndex ];
			if ( !(PropInfo.Property->IsA( FArrayProperty::StaticClass() ) && PropertyIndex != Properties.Num() - 1 ) )
			{
				if ( !FirstAddition )
				{
					NewDisplayName += Separator;
				}

				NewDisplayName += PropInfo.Property->GetFName().ToString();

				if ( PropInfo.ArrayIndex != INDEX_NONE )
				{
					NewDisplayName += FString::Printf( TEXT( "[%d]" ), PropInfo.ArrayIndex );
				}

				FirstAddition = false;
			}
		}

		return NewDisplayName;
	}

	/**
	 * Add another property to be associated with this path
	 */
	void AddProperty( const FPropertyInfo& InProperty )
	{
		Properties.Add( InProperty );
	}


	static bool AreEqual( const TSharedRef< FPropertyPath >& PathLhs, const TSharedRef< FPropertyPath >& PathRhs )
	{
		bool Result = false;

		if ( PathLhs->GetNumProperties() == PathRhs->GetNumProperties() )
		{
			bool FoundMatch = true;
			for( int Index = PathLhs->GetNumProperties() - 1; Index >= 0; --Index )
			{
				const FPropertyInfo& LhsPropInfo = PathLhs->GetPropertyInfo( Index );
				const FPropertyInfo& RhsPropInfo = PathRhs->GetPropertyInfo( Index );

				if ( LhsPropInfo.Property != RhsPropInfo.Property ||
					LhsPropInfo.ArrayIndex != RhsPropInfo.ArrayIndex )
				{
					FoundMatch = false;
					break;
				}
			}

			if ( FoundMatch )
			{
				Result = true;
			}
		}

		return Result;
	}

	/** Converts this to an array of property path names that are split between objects in the path. */
	PROPERTYEDITOR_API TArray< UE::FPropertyPathName > ToPropertyPathName() const;

private:

	TArray< FPropertyInfo > Properties;
};

FORCEINLINE bool operator==( const FPropertyPath& LHS, const FPropertyPath& RHS )
{
	if( LHS.GetNumProperties() != RHS.GetNumProperties() )
	{
		return false;
	}

	for( int32 i = 0, end = RHS.GetNumProperties(); i != end; ++i )
	{
		if( LHS.GetPropertyInfo(i) != RHS.GetPropertyInfo(i) )
		{
			return false;
		}
	}

	return true;
}

FORCEINLINE bool operator!=(const FPropertyPath& LHS, const FPropertyPath& RHS)
{
	return !(LHS == RHS);
}

FORCEINLINE uint32 GetTypeHash(FPropertyPath const& Path)
{
	//lets concat the name and just hash the string
	uint32 Ret = 0;
	for( int32 i = 0, end = Path.GetNumProperties(); i != end; ++i )
	{
		if( Path.GetPropertyInfo(i).Property.IsValid() )
		{
			Ret = Ret ^ GetTypeHash(Path.GetPropertyInfo(i).Property->GetFName());
		}
	}
	return Ret;
}

