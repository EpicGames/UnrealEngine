// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Ticker.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "Misc/NamePermissionList.h"

#define UE_API PROPERTYEDITOR_API

class FDetailTreeNode;

DECLARE_LOG_CATEGORY_EXTERN(LogPropertyEditorPermissionList, Log, All);

/** Struct, OwnerName */
DECLARE_MULTICAST_DELEGATE_TwoParams(FPermissionListUpdated, TSoftObjectPtr<const UStruct>, FName);
DECLARE_DELEGATE_RetVal_OneParam(bool, FIsEnabledForPackage, const UPackage*);
DECLARE_DELEGATE_RetVal_OneParam(TOptional<bool>, FCustomDetailTreeNodePermissionDelegate, TSharedRef<FDetailTreeNode> DetailTreeNode);

/**
 * A hierarchical set of rules that can be used to PermissionList all properties of specific Structs without
 * having to manually add every single property in those Structs. These rules are applied in order from
 * the base Struct to the leaf Struct. UseExistingPermissionList has dual-functionality to alternatively
 * inherit the parent Struct's rule if no PermissionList is manually defined.
 * 
 * For example, if you have:
 * class A - (UseExistingPermissionList "MyProp")						PermissionList = "MyProp"
 * class B : public class A - (AllowListAllProperties)					PermissionList = "MyProp","PropA1","PropA2"
 * class C : public class B - (UseExistingPermissionList "AnotherProp")	PermissionList = "MyProp","PropA1","PropA2","AnotherProp"
 * class D : public class B - (UseExistingPermissionList)				PermissionList = "MyProp","PropA1","PropA2","PropD1","PropD2"
 * Note that because class C manually defines a PermissionList, it does not inherit the AllowListAllProperties rule from class B, while
 * class D does not define a PermissionList, so it does inherit the rule, causing all of class D's properties to also get added to the AllowList.
 */
enum class EPropertyPermissionListRules : uint8
{
	// If no PermissionList is manually defined for this Struct, AllowList all properties from this Struct and its subclasses
	AllowListAllProperties,
	// If a PermissionList is manually defined for this Struct, AllowList all properties from this Struct's subclasses.
	// If this functionality is needed without any properties to AllowList, a fake property must be added to AllowList instead.
	AllowListAllSubclassProperties,
	// If a PermissionList is manually defined for this struct, PermissionList those properties. Otherwise, use the parent Struct's rule.
	UseExistingPermissionList
};

struct FPropertyPermissionListEntry
{
	FNamePermissionList PermissionList;
	// When the permission list does not contain any AllowList, DenyList or DenyListAll it needs an AdditionalOwnerName to be tracked here so stays alive without being removed
	TArray<FName> AdditionalOwnerNames;
	EPropertyPermissionListRules Rules = EPropertyPermissionListRules::UseExistingPermissionList;
};

class FPropertyPermissionList
{
public:
	/** Add a set of rules for a specific base UStruct to determine which properties are visible in all details panels */
	UE_DEPRECATED(5.5, "Call AddPermissionList with additional required arguments instead.")
	UE_API void AddPermissionList(TSoftObjectPtr<const UStruct> Struct, const FNamePermissionList& PermissionList, const EPropertyPermissionListRules Rules = EPropertyPermissionListRules::UseExistingPermissionList);
	/** Add a set of rules for a specific base UStruct to determine which properties are visible in all details panels */
	UE_API void AddPermissionList(TSoftObjectPtr<const UStruct> Struct, const FNamePermissionList& PermissionList, const EPropertyPermissionListRules Rules, const TConstArrayView<FName> InAdditionalOwnerNames);
	/** Remove a set of rules for a specific base UStruct to determine which properties are visible in all details panels */
	UE_API void RemovePermissionList(TSoftObjectPtr<const UStruct> Struct);
	/** Remove all rules */
	UE_API void ClearPermissionList();

	/** Unregister an owner from all permission lists currently stored */
	UE_API void UnregisterOwner(const FName Owner);

	/** Add a specific property to a UStruct's AllowList */
	UE_API void AddToAllowList(TSoftObjectPtr<const UStruct> Struct, const FName PropertyName, const FName Owner);
	/** Add a list of properties to a UStruct's AllowList */
	UE_API void AddToAllowList(TSoftObjectPtr<const UStruct> Struct, const TArray<FName>& PropertyNames, const FName Owner);
	/** Remove a specific property from a UStruct's AllowList */
	UE_API void RemoveFromAllowList(TSoftObjectPtr<const UStruct> Struct, const FName PropertyName, const FName Owner);
	/** Add a specific property to a UStruct's DenyList */
	UE_API void AddToDenyList(TSoftObjectPtr<const UStruct> Struct, const FName PropertyName, const FName Owner);
	/** Remove a specific property from a UStruct's DenyList */
	UE_API void RemoveFromDenyList(TSoftObjectPtr<const UStruct> Struct, const FName PropertyName, const FName Owner);
	/** Registers a delegate that is capable of handling more advanced use cases, such as details panel rows whose identifiers are created dynamically. */
	UE_API void RegisterCustomDetailsTreeNodePermissionDelegate(TSoftObjectPtr<const UStruct> Struct, FCustomDetailTreeNodePermissionDelegate InDelegate);
	UE_API void UnregisterCustomDetailsTreeNodePermissionDelegate(TSoftObjectPtr<const UStruct> Struct);

	/** When the PermissionList or DenyList for any struct was added to or removed from. */
	FPermissionListUpdated PermissionListUpdatedDelegate;

	/** When the entire PermissionList is enabled or disabled */
	FSimpleMulticastDelegate PermissionListEnabledDelegate;

	/** Controls whether DoesPropertyPassFilter always returns true or performs property-based filtering. */
	bool IsEnabled() const { return EnablePermissionList.IsBound() && EnablePermissionList.Execute(); }
	/** 
	 * Turn on or off the property PermissionList. DoesPropertyPassFilter will always return true if disabled.
	 * Optionally the change can be announced through a broadcast, which is typically recommended, but may not be needed
	 * if the permission list is only temporary enabled, e.g. during saving.
	 * 
	 * Under the hood, it will be a delegate that will always return bEnabled. If a delegate is passed in instead, that
	 * delegate is called to return dynamically whether the filter is enabled or not.
	 */
	UE_API void SetEnabled(bool bEnable, bool bAnnounce = true);
	UE_API void SetEnabled(const FDelegateEnablePathFilter& bEnable, bool bAnnounce = true);

	/** Checks if the provided struct has any entries in the allow, deny and deny all lists and returns true in that case. */
	UE_API bool HasFiltering(const UStruct* ObjectStruct) const;

	/**
	 * Checks if a property passes the PermissionList/DenyList filtering specified by PropertyPermissionLists
	 * This should be relatively fast as it maintains a flattened cache of all inherited PermissionLists for every UStruct (which is generated lazily).
	 */
	UE_API bool DoesPropertyPassFilter(const UStruct* ObjectStruct, FName PropertyName) const;
	/** Checks if a given detail tree node passes the filters. This is more flexible than the above function but should only be considered for advanced use cases,
	 * such as dynamically generated rows from customizations that don't fit the structure of simply checking against an FName. */
	UE_API bool DoesDetailTreeNodePassFilter(const UStruct* ObjectStruct, TSharedRef<FDetailTreeNode> DetailTreeNode);

	/** */
	UE_API bool HasSpecificList(const UStruct* ObjectStruct) const;

	/** Check whether a property exists on the PermissionList for a specific Struct - this will return false if the property is AllowListed on a parent Struct */
	UE_API bool IsSpecificPropertyAllowListed(const UStruct* ObjectStruct, FName PropertyName) const;
	/** Check whether a property exists on the DenyList for a specific Struct - this will return false if the property is DenyListed on a parent Struct */
	UE_API bool IsSpecificPropertyDenyListed(const UStruct* ObjectStruct, FName PropertyName) const;

	/** Gets a read-only reference to the original, un-flattened PermissionList. */
	const TMap<TSoftObjectPtr<const UStruct>, FPropertyPermissionListEntry>& GetRawPermissionList() const { return RawPropertyPermissionList; }

	/** Clear CachedPropertyPermissionList to cause the PermissionListed property list to be regenerated next time it's queried */
	UE_API void ClearCache();

	/** Gets a read-only reference to the cached permission list for the given struct, or null if the filtering is disabled. A non-null value will remain valid to read, but the info in it may become stale if this struct is re-cached because its raw rules change. */
	UE_API TSharedPtr<const FNamePermissionList> GetCachedPermissionListForStructIfEnabled(const UStruct* Struct) const;

	/** If true, PermissionListUpdatedDelegate will not broadcast when modifying the permission lists */
	bool bSuppressUpdateDelegate = false;

	const FDelegateEnablePathFilter& DelegateEnablePathFilter() { return EnablePermissionList; }

protected:
	UE_API FPropertyPermissionList();
	UE_API ~FPropertyPermissionList();

private:
	bool Tick(float DeltaTime);
	void OnObjectPostCDOCompiled(UObject* CDO, const FObjectPostCDOCompiledContext& Context);

	/** Reset the cache and queue a broadcast. You must hold a write lock on PermissionListRW before calling this function! */
	void ClearCacheAndQueueBroadcast_NoLock(TSoftObjectPtr<const UStruct> ObjectStruct = nullptr, FName OwnerName = NAME_None);

	/** Clear the given class (and any derived classes) from the cache. You must hold a write lock on PermissionListRW before calling this function! */
	void ClearCacheForClass_NoLock(const UClass* Class);

	/** Delegate to return whether DoesPropertyPassFilter should perform its PermissionList check or always return true */
	FDelegateEnablePathFilter EnablePermissionList;

	/** Stores assigned PermissionLists from AddPermissionList(), which are later flattened and stored in CachedPropertyPermissionList. */
	TMap<TSoftObjectPtr<const UStruct>, FPropertyPermissionListEntry> RawPropertyPermissionList;
	TMap<TSoftObjectPtr<const UStruct>, FCustomDetailTreeNodePermissionDelegate> CustomDetailTreeNodePermissionDelegates;;	

	/** Handle for our tick function */
	FTSTicker::FDelegateHandle OnTickHandle;
	
	struct FPermissionListUpdate
	{
		TSoftObjectPtr<const UStruct> ObjectStruct;
		FName OwnerName;
	};
	friend bool operator==(const FPermissionListUpdate& A, const FPermissionListUpdate& B);
	friend uint32 GetTypeHash(const FPermissionListUpdate& PermisisonList);
	TSet< FPermissionListUpdate > PendingUpdates;

	struct FCachedPermissionList
	{
		FCachedPermissionList(FNamePermissionList&& InPermissionList, EPropertyPermissionListRules InRules)
			: PermissionList(MakeShared<FNamePermissionList>(MoveTemp(InPermissionList)))
			, Rules(InRules)
		{
		}

		TSharedRef<const FNamePermissionList> PermissionList;
		EPropertyPermissionListRules Rules = EPropertyPermissionListRules::UseExistingPermissionList;
	};
	/** Lazily-constructed combined cache of both the flattened class PermissionList and struct PermissionList */
	mutable TMap<TWeakObjectPtr<const UStruct>, TSharedRef<const FCachedPermissionList>> CachedPropertyPermissionList;

	/** Protects RawPropertyPermissionList, CachedPropertyPermissionList, and PendingUpdates */
	mutable FRWLock PermissionListRW;

	/** Get or create the cached PermissionList for a specific UStruct */
	TSharedRef<const FCachedPermissionList> GetCachedPermissionListEntryForStruct(const UStruct* Struct) const;
};

class FPropertyEditorPermissionList : public FPropertyPermissionList
{
public:
	static UE_API FPropertyEditorPermissionList& Get();

	/** Whether the Details View should show special menu entries to add/remove items in the PermissionList */
	bool ShouldShowMenuEntries() const { return bShouldShowMenuEntries; }
	/** Turn on or off menu entries to modify the PermissionList from a Details View */
	void SetShouldShowMenuEntries(bool bShow) { bShouldShowMenuEntries = bShow; }
	/** Set the delegate that is called in IsEnabledForPackage */
	void SetIsEnabledForPackageDelegate(const FIsEnabledForPackage& InIsEnabledForPackage ) { IsEnabledForPackageDelegate = InIsEnabledForPackage; }
	/** A special call to disable the filter for a specific blueprint, if the IsEnabledForPackageDelegate is set and returns false */
	UE_API bool IsEnabledForPackage(const UPackage* Package) const;

private:
	FPropertyEditorPermissionList() = default;
	~FPropertyEditorPermissionList() = default;

	FIsEnabledForPackage IsEnabledForPackageDelegate;

	/** Whether SDetailSingleItemRow should add menu items to add/remove properties to/from the PermissionList */
	bool bShouldShowMenuEntries = false;
};

class FHiddenPropertyPermissionList : public FPropertyPermissionList
{
public:
	static UE_API FHiddenPropertyPermissionList& Get();
private:
	FHiddenPropertyPermissionList() = default;
	~FHiddenPropertyPermissionList() = default;
};

class FEnumValuePermissionList
{
public:
	/** Add a set of rules for a specific UEnum to determine which values are visible in all details panels */
	UE_API void AddPermissionList(TSoftObjectPtr<const UEnum> Enum, const FNamePermissionList& PermissionList);
	/** Remove a set of rules for a specific UEnum to determine which values are visible in all details panels */
	UE_API void RemovePermissionList(TSoftObjectPtr<const UEnum> Enum);
	/** Remove all rules */
	UE_API void ClearPermissionList();

	/** Check whether the specified value is permitted for the provided UEnum */
	UE_API bool DoesEnumValuePassFilter(const UEnum* Enum, FName ValueName) const;

	/** Checks if the provided enum has any entries in the permission list and returns true in that case. */
	UE_API bool HasFiltering(const UEnum* Enum) const;

	/** When false, DoesEnumValuePassFilter will always return true. */
	bool IsEnabled() const { return bEnablePermissionList; }
	/** Turn on or off the enum PermissionList. When off, DoesEnumValuePassFilter will always return true. */
	void SetEnabled(bool bEnable) { bEnablePermissionList = bEnable; }

	/** Gets a read-only copy of the original, un-flattened PermissionList. */
	const TMap<TSoftObjectPtr<const UEnum>, FNamePermissionList>& GetRawPermissionList() const { return PermissionList; }

	static UE_API FEnumValuePermissionList& Get();

private:
	TMap<TSoftObjectPtr<const UEnum>, FNamePermissionList> PermissionList;

	bool bEnablePermissionList = false;
};

#undef UE_API
