// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Internationalization/Text.h"

#define UE_API PROPERTYEDITOR_API

class IClassViewerFilter;

class FPropertyRestriction
{
public:

	FPropertyRestriction(const FText& InReason)
		: Reason(InReason)
	{
	}

	const FText& GetReason() const { return Reason; }

	UE_API bool IsValueHidden(const FString& Value) const;
	UE_API bool IsValueDisabled(const FString& Value) const;
	UE_API void AddHiddenValue(FString Value);
	UE_API void AddDisabledValue(FString Value);
	UE_API void AddClassFilter(TSharedRef<IClassViewerFilter> ClassFilter);

	UE_API void RemoveHiddenValue(FString Value);
	UE_API void RemoveDisabledValue(FString Value);
	UE_API void RemoveClassFilter(TSharedRef<IClassViewerFilter> ClassFilter);
	UE_API void RemoveAll();

	TArray<FString>::TConstIterator GetHiddenValuesIterator() const 
	{
		return HiddenValues.CreateConstIterator();
	}

	TArray<FString>::TConstIterator GetDisabledValuesIterator() const
	{
		return DisabledValues.CreateConstIterator();
	}
	
	TArray<TSharedRef<IClassViewerFilter>>::TConstIterator GeClassViewFilterIterator() const
	{
		return ClassViewFilter.CreateConstIterator();
	}

private:
	TArray<FString> HiddenValues;
	TArray<FString> DisabledValues;
	TArray<TSharedRef<IClassViewerFilter>> ClassViewFilter;
	FText Reason;
};

#undef UE_API
