// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "AsyncDetailViewDiff.h"
#include "DiffUtils.h"
#include "Templates/SharedPointer.h"
#include "IDetailsView.h"

#define UE_API PROPERTYEDITOR_API

// Splitter that allows you to provide an FAsyncDetailViewDiff to connect like-properties between two or more details panels
class SDetailsSplitter : public SCompoundWidget
{
public:
	// Callback that's called before highlighting a row. If true, the row is skipped.
	DECLARE_DELEGATE_RetVal_OneParam(bool, FShouldIgnoreRow, const TWeakPtr<FDetailTreeNode>&)
	DECLARE_DELEGATE_RetVal_OneParam(FLinearColor, FRowHighlightColor, const TUniquePtr<FAsyncDetailViewDiff::DiffNodeType>&)
	DECLARE_DELEGATE_RetVal_OneParam(bool, FShouldHighlightRow, const TUniquePtr<FAsyncDetailViewDiff::DiffNodeType>&)

	class FSlot : public TSlotBase<FSlot>
	{
	public:
		FSlot() = default;
		
		SLATE_SLOT_BEGIN_ARGS(FSlot, TSlotBase<FSlot>)
			/** When the RuleSize is set to FractionOfParent, the size of the slot is the Value percentage of its parent size. */
			SLATE_ATTRIBUTE(float, Value)
			SLATE_ARGUMENT(TSharedPtr<IDetailsView>, DetailsView)
			SLATE_ATTRIBUTE(bool, IsReadonly) // default true
			SLATE_ATTRIBUTE(TSharedPtr<FAsyncDetailViewDiff>, DifferencesWithRightPanel)
			SLATE_EVENT(FShouldIgnoreRow, ShouldIgnoreRow) // default false
		SLATE_SLOT_END_ARGS()
	};
	struct FPanel
	{
		TSharedPtr<IDetailsView> DetailsView;
		TAttribute<bool> IsReadonly;
		TAttribute<TSharedPtr<FAsyncDetailViewDiff>> DiffRight;
		FShouldIgnoreRow ShouldIgnoreRow;
	};
	static UE_API FSlot::FSlotArguments Slot();
	
	SLATE_BEGIN_ARGS(SDetailsSplitter)
	{}
		SLATE_EVENT(FRowHighlightColor, RowHighlightColor) // default cyan: FLinearColor(0.f, 1.f, 1.f, .7f)
		SLATE_EVENT(FShouldHighlightRow, ShouldHighlightRow) // default: highlight when node difference result is 'not identical'
		SLATE_SLOT_ARGUMENT(FSlot, Slots)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,
	                      FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle,
	                      bool bParentEnabled) const override;

	UE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	
	UE_API void AddSlot(const FSlot::FSlotArguments& SlotArgs, int32 Index = INDEX_NONE);
	UE_API FPanel& GetPanel(int32 Index);

	UE_API void SetRowHighlightColorDelegate(const FRowHighlightColor& Delegate);
	// generate a highlight color delegate that utilizes the merge results to determine the color
	UE_API void HighlightFromMergeResults(const TMap<FString, TMap<FPropertySoftPath, ETreeDiffResult>>& Highlights);

private:
	
	enum class EPropertyCopyDirection
	{
		Copy_None,
		CopyLeftToRight,
		CopyRightToLeft,
	};
	
	struct FCopyPropertyButton
	{
		TWeakPtr<FDetailTreeNode> SourceDetailsNode;
		TWeakPtr<FDetailTreeNode> DestinationDetailsNode;
		ETreeDiffResult DiffResult = ETreeDiffResult::Invalid;
		EPropertyCopyDirection CopyDirection = EPropertyCopyDirection::Copy_None;
	};
	
	UE_API void PaintPropertyConnector(FSlateWindowElementList& OutDrawElements, int32 LayerId, const FSlateRect& LeftPropertyRect, 
		const FSlateRect& RightPropertyRect, const FLinearColor& FillColor, const FLinearColor& OutlineColor) const;
	
	UE_API void PaintCopyPropertyButton(FSlateWindowElementList& OutDrawElements, int32 LayerId, const TUniquePtr<FAsyncDetailViewDiff::DiffNodeType>& DiffNode,
		const FSlateRect& LeftPropertyRect, const FSlateRect& RightPropertyRect, EPropertyCopyDirection CopyDirection) const;
	
	TSharedPtr<SSplitter> Splitter;
	TArray<FPanel> Panels;
	FCopyPropertyButton HoveredCopyButton;
	FRowHighlightColor GetRowHighlightColor;
	FShouldHighlightRow GetShouldHighlightRow;
};

#undef UE_API
