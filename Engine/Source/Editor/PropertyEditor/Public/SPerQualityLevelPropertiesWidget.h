// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/SlateDelegates.h"

#define UE_API PROPERTYEDITOR_API

class FMenuBuilder;

class SOverridePropertiesWidget : public SCompoundWidget
{
public:
	typedef typename TSlateDelegates<FName>::FOnGenerateWidget FOnGenerateWidget;

	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnOverrideAction, FName);

	SLATE_BEGIN_ARGS(SOverridePropertiesWidget)
		: _OnGenerateWidget()
	{}

	SLATE_EVENT(FOnGenerateWidget, OnGenerateWidget)
		SLATE_EVENT(FOnOverrideAction, OnAddEntry)
		SLATE_EVENT(FOnOverrideAction, OnRemoveEntry)

		SLATE_ATTRIBUTE(TArray<FName>, EntryNames)

		SLATE_END_ARGS()

	/**
	* Construct this widget
	*
	* @param	InArgs	The declaration data for this widget
	*/
	UE_API virtual void Construct(const typename SOverridePropertiesWidget::FArguments& InArgs);

protected:
	virtual void ConstructChildren() = 0;

	UE_API TSharedRef<SWidget> MakeOverrideWidget(FName InName, FText InDisplayText, const TArray<FName>& InEntries, FMenuBuilder& InAddMenuBuilder);

	UE_API void AddEntry(FName EntryName);

	UE_API FReply RemoveEntry(FName EntryName);

	UE_API void AddEntryToMenu(const FName& EntryName, const FTextFormat Format, FMenuBuilder& AddEntryMenuBuilder);

	FOnGenerateWidget OnGenerateWidget;
	FOnOverrideAction OnAddEntry;
	FOnOverrideAction OnRemoveEntry;
	TAttribute<TArray<FName>> EntryNames;
	int32 LastEntryNames;
	bool bAddedMenuItem;

	FString ToolTip;
};

/**
* SPerQualityLevelPropertiesWidget
*/
class SPerQualityLevelPropertiesWidget : public SOverridePropertiesWidget
{
public:

	UE_API virtual void Construct(const typename SPerQualityLevelPropertiesWidget::FArguments& InArgs);

protected:
	UE_API virtual void ConstructChildren() override;

};




#undef UE_API
