// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API PROPERTYEDITOR_API

struct FGeometry;

/** 
 * Delegate used to get the text to display when resetting our value
 * This will only be used when we are not using a property handle.
 */
DECLARE_DELEGATE_RetVal(FText, FOnGetResetToDefaultText);

/**
 * Displays a reset to default menu for a set of properties
 * Will hide itself automatically when all of the properties being observed are already at their default values
 */
class SResetToDefaultMenu : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SResetToDefaultMenu )
		: _VisibilityWhenDefault( EVisibility::Hidden )
		, _DiffersFromDefault( false )
	{}
		/** 
		 * The visibility of this widget when the value is default 
		 * and the reset to default option does need to be shown
		 */
		SLATE_ARGUMENT( EVisibility, VisibilityWhenDefault )

		/** 
		 * Attribute used to determine whether we are different from the default. Only used
		 * when we do not have a valid property handle
		 */
		SLATE_ATTRIBUTE( bool, DiffersFromDefault )

		/** Delegate fired when we reset to default - only used when we don't have a property handle */
		SLATE_ARGUMENT(FSimpleDelegate, OnResetToDefault)

		/** Delegate used to get the text to display when resetting our value - only used when we don't have a property handle */
		SLATE_ARGUMENT(FOnGetResetToDefaultText, OnGetResetToDefaultText)
	SLATE_END_ARGS()

	UE_API void Construct( const FArguments& InArgs );

	/**
	 * Adds a new property to the menu that is displayed when a user clicks the reset to default button
	 *
	 * @param InProperty	The property handle to display
	 */
	UE_API void AddProperty( TSharedRef<class IPropertyHandle> InProperty );

private:
	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	/**
	 * @return The visibility of the reset to default widget
	 */
	UE_API EVisibility GetResetToDefaultVisibility() const;

	/**
	 * Called when the menu is open to regenerate the default menu content
	 */
	UE_API TSharedRef<class SWidget> OnGenerateResetToDefaultMenuContent();

	/**
	 * Resets property to default, only used when our property handle is not valid.
	 */
	UE_API void ResetToDefault();

	/**
	 * Resets all properties to default
	 */
	UE_API void ResetAllToDefault();
private:

	/** Properties that should be displayed in the menu */
	TArray< TSharedRef<class IPropertyHandle> > Properties;

	/** The visibility to use when the properties are already the default */
	EVisibility VisibilityWhenDefault;

	/** Whether or this menu should be visible */
	bool bShouldBeVisible;

	/** Attribute used to determine whether we are different from the default */
	TAttribute<bool> DiffersFromDefault;

	/** Delegate fired when we reset to default - only used when we don't have a property handle */
	FSimpleDelegate OnResetToDefault;

	/** Delegate used to get the text to display when resetting our value - only used when we don't have a property handle */
	FOnGetResetToDefaultText OnGetResetToDefaultText;
};

#undef UE_API
