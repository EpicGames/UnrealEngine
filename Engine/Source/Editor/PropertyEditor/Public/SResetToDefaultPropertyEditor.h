// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IDetailPropertyRow.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API PROPERTYEDITOR_API

// Forward decl
class FResetToDefaultOverride;
struct FGeometry;

/** Widget showing the reset to default value button */
class SResetToDefaultPropertyEditor : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SResetToDefaultPropertyEditor )
		: _NonVisibleState( EVisibility::Hidden )
		{}
		SLATE_ARGUMENT( EVisibility, NonVisibleState )
		SLATE_ARGUMENT( TOptional<FResetToDefaultOverride>, CustomResetToDefault )
	SLATE_END_ARGS()

	UE_API ~SResetToDefaultPropertyEditor();

	UE_API void Construct( const FArguments& InArgs, const TSharedPtr< class IPropertyHandle>& InPropertyHandle );

private:
	UE_API FText GetResetToolTip() const;

	UE_API EVisibility GetDiffersFromDefaultAsVisibility() const;

	UE_API void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime );

	UE_API FReply OnResetClicked();

	UE_API void UpdateDiffersFromDefaultState();

private:
	TOptional<FResetToDefaultOverride> OptionalCustomResetToDefault;

	TSharedPtr< class IPropertyHandle > PropertyHandle;

	EVisibility NonVisibleState;

	bool bValueDiffersFromDefault;
};

#undef UE_API
