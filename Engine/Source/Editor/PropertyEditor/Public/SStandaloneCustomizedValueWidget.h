// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailWidgetRow.h"
#include "IPropertyTypeCustomization.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API PROPERTYEDITOR_API

class FAssetThumbnailPool;
class FDetailCategoryImpl;

/**
 * Generates the header widget for a customized struct or other type.
 * This widget is generally used in the property editor to display a struct as a single row, like with an FColor.
 * Properties passed in that do not have a header customization will return a null widget.
 */
class SStandaloneCustomizedValueWidget : public SCompoundWidget, public IPropertyTypeCustomizationUtils
{
public:
	SLATE_BEGIN_ARGS( SStandaloneCustomizedValueWidget )
	{}
		/** Optional Parent Detail Category, useful to access Thumbnail Pool. */
		SLATE_ARGUMENT( TSharedPtr<FDetailCategoryImpl>, ParentCategory)
	SLATE_END_ARGS()
	
	UE_API void Construct(const FArguments& InArgs,
		TSharedPtr<IPropertyTypeCustomization> InCustomizationInterface, TSharedRef<IPropertyHandle> InPropertyHandle);

	UE_API virtual TSharedPtr<FAssetThumbnailPool> GetThumbnailPool() const override;

private:
	TWeakPtr<FDetailCategoryImpl> ParentCategory;
	TSharedPtr<IPropertyTypeCustomization> CustomizationInterface;
	TSharedPtr<IPropertyHandle> PropertyHandle;
	TSharedPtr<FDetailWidgetRow> CustomPropertyWidget;
};

#undef UE_API
