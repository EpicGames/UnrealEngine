//  Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "DetailsDisplayManager.h"
#include "ToolElementRegistry.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Input/SComboButton.h"
#include "Templates/SharedPointer.h"

class FDetailsDisplayManager;

DECLARE_DELEGATE_OneParam(FGetIsRowHoveredOver, bool)

class FCategoryMenuComboButtonBuilder : public FToolElementRegistrationArgs
{
public:

	/**
	 * the delegate to get the menu content
	 */
	FOnGetContent OnGetContent;

	/**
	 * The constructor, which takes a @code TSharedRef<FDetailsDisplayManager> @endcode to initialize
	 * the Details Display Manager
	 */
	FCategoryMenuComboButtonBuilder( TSharedRef<FDetailsDisplayManager> InDetailsDisplayManager );

	/**
	 * Set the OnGetContent for the menu that this button is responsible for
	 */
	FCategoryMenuComboButtonBuilder& Set_OnGetContent(FOnGetContent InOnGetContent);

	/**
	 * Set the IsVisible delegate for the menu that this button is responsible for
	 */
	FCategoryMenuComboButtonBuilder& Bind_IsVisible(TAttribute<EVisibility> InIsVisible);

	/** Implements the generation of the Category Menu button SWidget */
	virtual TSharedPtr<SWidget> GenerateWidget() override;

	/**
	 * Converts this into the SWidget it builds
	 */
	virtual ~FCategoryMenuComboButtonBuilder() override;

	/**
	 * Converts this into the SWidget it builds
	 */
    TSharedRef<SWidget> operator*();

private:
	/**
	 * The @code DetailsDisplayManager @endcode which provides an API to manage some of the characteristics of the
	 * details display
	 */
	TSharedRef<FDetailsDisplayManager> DisplayManager;

	/**
	 * An attribute indicating whether the current row is hovered over
	 */
	TAttribute<EVisibility> IsRowHoveredOver;

	/**
	 * the attribute which tells whether this menu button is visible
	 */
	TAttribute<EVisibility> IsVisible;
};
