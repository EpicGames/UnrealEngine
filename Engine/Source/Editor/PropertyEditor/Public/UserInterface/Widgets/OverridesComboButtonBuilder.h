//  Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "DetailsDisplayManager.h"
#include "PropertyUpdatedWidgetBuilder.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Input/SComboButton.h"
#include "Templates/SharedPointer.h"

enum class EOverriddenState : uint8;
class FDetailsDisplayManager;


DECLARE_DELEGATE_OneParam(FGetIsRowHoveredOver, bool)

/**
 * A Display builder for the overrides combo button 
 */
class FOverridesComboButtonBuilder : public FPropertyUpdatedWidgetBuilder
{
public:

	/**
	 * the delegate to get the menu content
	 */
	FOnGetContent OnGetContent;

	/**
	 * The constructor, which takes a @code TSharedRef<FDetailsDisplayManager> @endcode to initialize
	 * the Details Display Manager
	 *
	 * @param InDetailsDisplayManager the FDetailsDisplayManager which manages the details display
	 * @param bInIsCategoryOverridesComboButton if true, this FOverridesComboButtonBuilder is for a Category rather
	 * than a property row   
	 */
	PROPERTYEDITOR_API FOverridesComboButtonBuilder(
		TSharedRef<FDetailsDisplayManager> InDetailsDisplayManager,
		bool bInIsCategoryOverridesComboButton,
		TWeakObjectPtr<UObject> InObject );

	/**
	 * Set the OnGetContent for the menu that this button is responsible for
	 */
	PROPERTYEDITOR_API FOverridesComboButtonBuilder& Set_OnGetContent(FOnGetContent InOnGetContent);

	/**
	 * Sets the FEditPropertyChain for this FOverridesComboButtonBuilder
	 */
	PROPERTYEDITOR_API void SetEditPropertyChain(TSharedRef<FEditPropertyChain>& InEditPropertyChain);

	/**
	 * Gets the FEditPropertyChain for this FOverridesComboButtonBuilder
	 */
	PROPERTYEDITOR_API TSharedPtr<FEditPropertyChain> GetEditPropertyChain() const;

	/**
	 * Implements the generation of the Category Menu button SWidget
	 */
	virtual TSharedPtr<SWidget> GenerateWidget() override;

	/**
	 * Converts this into the SWidget it builds
	 */
	PROPERTYEDITOR_API virtual ~FOverridesComboButtonBuilder() override;

	/**
	 * Converts this into the SWidget it builds
	 */
	TSharedRef<SWidget> operator*();

private:

	/**
	 * The @code DetailsDisplayManager @endcode which provides an API to manage some of the characteristics of the
	 * details display
	 */
	TSharedRef<FDetailsDisplayManager> DisplayManager;

	/**
	 * if true, this is an overrides combo button for a Category 
	 */
	bool bIsCategoryOverridesComboButton;

	/**
	 * The UObject that will be queried for its override state
	 */
	TWeakObjectPtr<UObject> Object;
	
	/**
    * If this is a ComboButton builder for a Property, this will point to the FEditPropertyChain for that Property, else it will be nullptr
    * 
    */
	TSharedPtr<FEditPropertyChain> EditPropertyChain;

	/**
	 * if true this is a builder for an Overrides combo button for a Category, else it is a builder
	 * for Properties within a Category 
	 */
	bool bIsCategory = true;

};
