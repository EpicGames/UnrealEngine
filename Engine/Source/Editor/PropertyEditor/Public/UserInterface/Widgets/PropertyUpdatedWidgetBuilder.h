// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolElementRegistry.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"

class FPropertyUpdatedWidgetBuilder : public FToolElementRegistrationArgs
{
public:

	DECLARE_DELEGATE(FResetToDefault);

	/**
	 * default constructor
	 */
	PROPERTYEDITOR_API FPropertyUpdatedWidgetBuilder();

	/**
	 * Set the IsVisible delegate for the menu that this button is responsible for
	 */
	PROPERTYEDITOR_API FPropertyUpdatedWidgetBuilder& Bind_IsVisible(TAttribute<EVisibility> InIsVisible);

	/**
	* Sets the ResetToDefault delegate
	*
	* @param InResetToDefault the FResetToDefault delegate to set for this widget builder
	*/
	PROPERTYEDITOR_API FPropertyUpdatedWidgetBuilder& Set_ResetToDefault(FResetToDefault InResetToDefault);

	/**
	 * returns the ResetToDefault delegate
	 */
	PROPERTYEDITOR_API FResetToDefault Get_ResetToDefault();

	/**
	 *  binds the IsRowHovered Attribute
	 *
	 * @param bInIsRowHoveredAttr the attribute which will determine if the row for this property is hovered
	 */
	PROPERTYEDITOR_API void Bind_IsRowHovered(TAttribute<bool> InIsRowHoveredAttr);

	/**
	 * If true, the row that this is associated with is currently hovered over
	 */
	TAttribute<bool> IsRowHoveredAttr;
protected:
	
	/**
	 * the attribute which tells whether this menu button is visible
	 */
	TAttribute<EVisibility> IsVisible;

	/**
	 * A delegate for resetting the active property row
	 */
	FResetToDefault ResetToDefault;

};
