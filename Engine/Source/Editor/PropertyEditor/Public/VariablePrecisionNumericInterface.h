// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Input/NumericTypeInterface.h"

//
// FVariablePrecisionNumericInterface
// 
// allow more precision as the numbers get closer to zero
// 
struct FVariablePrecisionNumericInterface : public TDefaultNumericTypeInterface<float>
{
	FVariablePrecisionNumericInterface() {}

	/** Convert the type to/from a string */
	virtual FString ToString(const float& Value) const override
	{
		// 1000
		// 100.1
		// 10.12
		// 1.123

		int32 MaxDigits = 3;
		if ((Value / 1000.f) >= 1.f)
			MaxDigits = 0;
		else if ((Value / 100.f) >= 1.f)
			MaxDigits = 1;
		else if ((Value / 10.f) >= 1.f)
			MaxDigits = 2;

		const FNumberFormattingOptions NumberFormattingOptions = FNumberFormattingOptions()
			.SetUseGrouping(false)
			.SetMinimumFractionalDigits( MaxDigits )
			.SetMaximumFractionalDigits( MaxDigits );
		return FastDecimalFormat::NumberToString(Value, ExpressionParser::GetLocalizedNumberFormattingRules(), NumberFormattingOptions);
	}
};